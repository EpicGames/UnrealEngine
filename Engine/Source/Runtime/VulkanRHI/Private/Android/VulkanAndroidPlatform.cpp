// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanAndroidPlatform.h"
#include "../VulkanRayTracing.h"
#include "../VulkanPipeline.h"
#include "../VulkanRenderpass.h"
#include <dlfcn.h>
#include "Android/AndroidWindow.h"
#include "Android/AndroidPlatformFramePacer.h"
#include "Math/UnrealMathUtility.h"
#include "Android/AndroidApplication.h"
#include "Android/AndroidPlatformMisc.h"
#include "Android/AndroidJavaEnv.h"
#include "Android/AndroidJNI.h"
#include "Android/AndroidPlatformMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "../VulkanExtensions.h"
#include <android/sharedmem_jni.h>
#include <sys/mman.h>
#include "ProfilingDebugging/ScopedTimers.h"
#include "Android/AndroidDynamicRHI.h"
#include "PSOMetrics.h"
#include "../VulkanContext.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"

#if USE_ANDROID_VULKAN_SWAPPY
#undef VK_NO_PROTOTYPES
#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"
#include "swappy/swappyVk.h"
#include "EngineGlobals.h"

namespace AndroidVulkan
{
	void VKSwappyPostWaitCallback(void*, int64_t cpu_time_ns, int64_t gpu_time_ns)
	{
		const double GPUTimeInSeconds = (double)gpu_time_ns / 1000000000.0;
		uint64 Cycles = GPUTimeInSeconds / FPlatformTime::GetSecondsPerCycle64();
		GetDynamicRHI<FVulkanDynamicRHI>()->ExternalGPUTime = Cycles;
	}

	void SetSwappyPostWaitCallback()
	{
		UE_CALL_ONCE([]
		{
			SwappyTracer Tracer = { 0 };
			Tracer.postWait = VKSwappyPostWaitCallback;
			SwappyVk_injectTracer(&Tracer);
		});

		int32 FrameTimeFenceInMillis = FAndroidPlatformRHIFramePacer::CVarSwappyGPUFrameTimeFence.GetValueOnAnyThread();

		SwappyVk_setFenceTimeoutNS(FrameTimeFenceInMillis * 1000000); // millis to ns (ms * 1000000)
	}
};

#endif // #if USE_ANDROID_VULKAN_SWAPPY

// From VulklanSwapChain.cpp
extern int32 GPrintVulkanVsyncDebug;

int32 GVulkanExtensionFramePacer = 1;
static FAutoConsoleVariableRef CVarVulkanExtensionFramePacer(
	TEXT("r.Vulkan.ExtensionFramePacer"),
	GVulkanExtensionFramePacer,
	TEXT("Whether to enable the google extension Framepacer for Vulkan (when available on device)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVulkanSupportsBCTextureFormats(
	TEXT("r.Vulkan.SupportsBCTextureFormats"),
	0,
	TEXT("Whether or not BC Texture formats are supported\n")
	TEXT("  0 = unsupported\n")
	TEXT("  1 = supported."),
	ECVF_SetByDeviceProfile
);
#if PLATFORM_ANDROID_X64 	

static TAutoConsoleVariable<int32> CVarVulkanHasUnifiedMemory(
	TEXT("android.Vulkan.HasUnifiedMemory"),
	1,
	TEXT("Whether or not device has unified memory\n")
	TEXT("  0 = doesn't have\n")
	TEXT("  1 = has."),
	ECVF_SetByDeviceProfile
);
#endif

FAutoConsoleVariableRef FVulkanAndroidPlatform::CVarVulkanRequiresWaitingForFrameCompletionEvent(
	TEXT("r.Vulkan.RequiresWaitingForFrameCompletionEvent"),
	FVulkanAndroidPlatform::bRequiresWaitingForFrameCompletionEvent,
	TEXT("Whether Android VulkanRHI should wait for frame completion event.")
	TEXT("  0 = don't wait (default)\n")
	TEXT("  1 = wait"),
	ECVF_RenderThreadSafe
);

// Vulkan function pointers
#define DEFINE_VK_ENTRYPOINTS(Type,Func) Type VulkanDynamicAPI::Func = NULL;
ENUM_VK_ENTRYPOINTS_ALL(DEFINE_VK_ENTRYPOINTS)

#define VULKAN_MALI_LAYER_NAME "VK_LAYER_ARM_AGA"


#if USE_ANDROID_VULKAN_SWAPPY
bool FVulkanAndroidPlatform::bSwappyEnabledAtRHIInit = false;
#endif
void* FVulkanAndroidPlatform::VulkanLib = nullptr;
bool FVulkanAndroidPlatform::bAttemptedLoad = false;

#if VULKAN_SUPPORTS_GOOGLE_DISPLAY_TIMING
bool FVulkanAndroidPlatform::bHasGoogleDisplayTiming = false;
TUniquePtr<class FGDTimingFramePacer> FVulkanAndroidPlatform::GDTimingFramePacer;
#endif

TUniquePtr<struct FAndroidVulkanFramePacer> FVulkanAndroidPlatform::FramePacer;
int32 FVulkanAndroidPlatform::CachedFramePace = 0;
int32 FVulkanAndroidPlatform::CachedRefreshRate = 60;
int32 FVulkanAndroidPlatform::CachedSyncInterval = 1;
int32 FVulkanAndroidPlatform::SuccessfulRefreshRateFrames = 1;
int32 FVulkanAndroidPlatform::UnsuccessfulRefreshRateFrames = 0;
TArray<TArray<ANSICHAR>> FVulkanAndroidPlatform::DebugVulkanDeviceLayers;
TArray<TArray<ANSICHAR>> FVulkanAndroidPlatform::DebugVulkanInstanceLayers;
TArray<TArray<ANSICHAR>> FVulkanAndroidPlatform::SwappyRequiredExtensions;
int32 FVulkanAndroidPlatform::AFBCWorkaroundOption = 0;
int32 FVulkanAndroidPlatform::ASTCWorkaroundOption = 0;
bool FVulkanAndroidPlatform::bRequiresDepthStencilFullWrite = false;
bool FVulkanAndroidPlatform::bRequiresMergedRenderPassDepthStencilLayout = false;
bool FVulkanAndroidPlatform::bRequiresWaitingForFrameCompletionEvent = false;

#define CHECK_VK_ENTRYPOINTS(Type,Func) if (VulkanDynamicAPI::Func == NULL) { bFoundAllEntryPoints = false; UE_LOGF(LogRHI, Warning, "Failed to find entry point for %ls", TEXT(#Func)); }


#if VULKAN_SUPPORTS_GOOGLE_DISPLAY_TIMING
FGDTimingFramePacer::FGDTimingFramePacer(VkDevice InDevice, VkSwapchainKHR InSwapChain)
	: Device(InDevice)
	, SwapChain(InSwapChain)
{
	FMemory::Memzero(PresentTime);

	ZeroVulkanStruct(PresentTimesInfo, VK_STRUCTURE_TYPE_PRESENT_TIMES_INFO_GOOGLE);
	PresentTimesInfo.swapchainCount = 1;
	PresentTimesInfo.pTimes = &PresentTime;
}

// Used as a safety measure to prevent scheduling too far ahead in case of an error
static constexpr uint64 GMaxAheadSchedulingTimeNanosec = 500000000llu; // 0.5 sec.

static uint64 TimeNanoseconds()
{
#if PLATFORM_ANDROID || PLATFORM_LINUX
	struct timespec ts;
	clock_gettime(CLOCK_MONOTONIC, &ts);
	return ts.tv_sec * 1000000000ull + ts.tv_nsec;
#else
#error VK_GOOGLE_display_timing requires TimeNanoseconds() implementation for this platform
#endif
}

void FGDTimingFramePacer::ScheduleNextFrame(uint32 InPresentID, int32 InFramePace, int32 InRefreshRate)
{
	UpdateSyncDuration(InFramePace, InRefreshRate);
	if (SyncDuration == 0)
	{
		if (GPrintVulkanVsyncDebug != 0)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT(" -- SyncDuration == 0"));
		}
		return;
	}

	const uint64 CpuPresentTime = TimeNanoseconds();

	PresentTime.presentID = InPresentID; // Still need to pass ID for proper history values

	PollPastFrameInfo();
	if (!LastKnownFrameInfo.bValid)
	{
		if (GPrintVulkanVsyncDebug != 0)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT(" -- LastKnownFrameInfo not valid"));
		}
		return;
	}

	const uint64 CpuTargetPresentTimeMin = CalculateMinPresentTime(CpuPresentTime);
	const uint64 CpuTargetPresentTimeMax = CalculateMaxPresentTime(CpuPresentTime);
	const uint64 GpuTargetPresentTime = (PredictLastScheduledFramePresentTime(InPresentID) + SyncDuration);

	const uint64 TargetPresentTime = CalculateNearestVsTime(LastKnownFrameInfo.ActualPresentTime, FMath::Clamp(GpuTargetPresentTime, CpuTargetPresentTimeMin, CpuTargetPresentTimeMax));
	LastScheduledPresentTime = TargetPresentTime;

	PresentTime.desiredPresentTime = (TargetPresentTime - HalfRefreshDuration);

	if (GPrintVulkanVsyncDebug != 0)
	{
		double cpuPMin = CpuTargetPresentTimeMin / 1000000000.0;
		double cpuPMax = CpuTargetPresentTimeMax / 1000000000.0;
		double gpuP = GpuTargetPresentTime / 1000000000.0;
		double desP = PresentTime.desiredPresentTime / 1000000000.0;
		double lastP = LastKnownFrameInfo.ActualPresentTime / 1000000000.0;
		double cpuDelta = 0.0;
		double cpuNow = CpuPresentTime / 1000000000.0;
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT(" -- ID: %u, desired %.3f, pred-gpu %.3f, pred-cpu-min %.3f, pred-cpu-max %.3f, last: %.3f, cpu-gpu-delta: %.3f, now-cpu %.3f"), PresentTime.presentID, desP, gpuP, cpuPMin, cpuPMax, lastP, cpuDelta, cpuNow);
	}
}

void FGDTimingFramePacer::UpdateSyncDuration(int32 InFramePace, int32 InRefreshRate)
{
	if (FramePace == InFramePace)
	{
		return;
	}
	
	// It's possible we have requested a change in native refresh rate that has yet to take effect. However if we base the schedule for the next
	// frame on our intend native refresh rate, the exact number of vsyncs the extension has to wait is irrelevant and should never present earler
	// than intended.
	RefreshDuration = InRefreshRate > 0 ? FMath::DivideAndRoundNearest(1000000000ull, (uint64)InRefreshRate) : 0;
	ensure(RefreshDuration > 0);
	if (RefreshDuration == 0)
	{
		RefreshDuration = 16666667;
	}
	HalfRefreshDuration = (RefreshDuration / 2);


	FramePace = InFramePace;
	SyncDuration = InFramePace > 0 ? FMath::DivideAndRoundNearest(1000000000ull, (uint64)FramePace) : 0;

	if (SyncDuration > 0)
	{
		SyncDuration = (FMath::Max((SyncDuration + HalfRefreshDuration) / RefreshDuration, 1llu) * RefreshDuration);
	}
}

uint64 FGDTimingFramePacer::PredictLastScheduledFramePresentTime(uint32 CurrentPresentID) const
{
	const uint32 PredictFrameCount = (CurrentPresentID - LastKnownFrameInfo.PresentID - 1);
	// Use RefreshDuration for predicted frames and not SyncDuration for most optimistic prediction of future frames after last known (possible hitchy) frame.
	// Second parameter will be always >= than LastScheduledPresentTime if use SyncDuration.
	// It is possible that GPU will recover after hitch without any changes to a normal schedule but pessimistic planning will prevent this from happening.
	return FMath::Max(LastScheduledPresentTime, LastKnownFrameInfo.ActualPresentTime + (RefreshDuration * PredictFrameCount));
}

uint64 FGDTimingFramePacer::CalculateMinPresentTime(uint64 CpuPresentTime) const
{
	// Do not use delta on Android because already using CLOCK_MONOTONIC for CPU time which is also used in the extension.
	// Using delta will mostly work fine but there were problems in other projects. If GPU load changes quickly because
	// of the delta filter lag its value may be too high for current frame and cause pessimistic planning and stuttering.
	// Need additional time for testing to improve filtering.
	// Adding HalfRefreshDuration to produce round-up (ceil) in the final CalculateNearestVsTime()
	return (CpuPresentTime + HalfRefreshDuration);
}

uint64 FGDTimingFramePacer::CalculateMaxPresentTime(uint64 CpuPresentTime) const
{
	return (CpuPresentTime + GMaxAheadSchedulingTimeNanosec);
}

uint64 FGDTimingFramePacer::CalculateNearestVsTime(uint64 ActualPresentTime, uint64 TargetTime) const
{
	if (TargetTime > ActualPresentTime)
	{
		return (ActualPresentTime + ((TargetTime - ActualPresentTime) + HalfRefreshDuration) / RefreshDuration * RefreshDuration);
	}
	return ActualPresentTime;
}

void FGDTimingFramePacer::PollPastFrameInfo()
{
	for (;;)
	{
		// MUST call once with nullptr to get the count, or the API won't return any results at all.
		uint32 Count = 0;
		VkResult Result = VulkanDynamicAPI::vkGetPastPresentationTimingGOOGLE(Device, SwapChain, &Count, nullptr);
		checkf(Result == VK_SUCCESS, TEXT("vkGetPastPresentationTimingGOOGLE failed: %i"), Result);

		if (Count == 0)
		{
			break;
		}

		Count = 1;
		VkPastPresentationTimingGOOGLE PastPresentationTiming;
		Result = VulkanDynamicAPI::vkGetPastPresentationTimingGOOGLE(Device, SwapChain, &Count, &PastPresentationTiming);
		checkf(Result == VK_SUCCESS || Result == VK_INCOMPLETE, TEXT("vkGetPastPresentationTimingGOOGLE failed: %i"), Result);

		// If desiredPresentTime was too large for some reason driver may ignore this value to prevent long wait
		// Reset LastScheduledPresentTime in that case to be able to schedule on proper time
		if (PastPresentationTiming.actualPresentTime < PastPresentationTiming.desiredPresentTime)
		{
			UE_LOGF(LogVulkanRHI, Warning, "PastPresentationTiming actualPresentTime is less than desiredPresentTime! Resetting LastScheduledPresentTime...");
			LastScheduledPresentTime = 0;
		}

		LastKnownFrameInfo.PresentID = PastPresentationTiming.presentID;
		LastKnownFrameInfo.ActualPresentTime = PastPresentationTiming.actualPresentTime;
		LastKnownFrameInfo.bValid = true;
	}
}
#endif //VULKAN_SUPPORTS_GOOGLE_DISPLAY_TIMING

bool FVulkanAndroidPlatform::LoadVulkanLibrary()
{
	if (bAttemptedLoad)
	{
		return (VulkanLib != nullptr);
	}
	bAttemptedLoad = true;

	// try to load libvulkan.so
	VulkanLib = dlopen("libvulkan.so", RTLD_NOW | RTLD_LOCAL);

	if (VulkanLib == nullptr)
	{
		return false;
	}

	bool bFoundAllEntryPoints = true;

#define GET_VK_ENTRYPOINTS(Type,Func) VulkanDynamicAPI::Func = (Type)dlsym(VulkanLib, #Func);

	ENUM_VK_ENTRYPOINTS_BASE(GET_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_BASE(CHECK_VK_ENTRYPOINTS);
	if (!bFoundAllEntryPoints)
	{
		dlclose(VulkanLib);
		VulkanLib = nullptr;
		return false;
	}

	ENUM_VK_ENTRYPOINTS_OPTIONAL_BASE(GET_VK_ENTRYPOINTS);
#if UE_BUILD_DEBUG
	ENUM_VK_ENTRYPOINTS_OPTIONAL_BASE(CHECK_VK_ENTRYPOINTS);
#endif

#undef GET_VK_ENTRYPOINTS

	// Init frame pacer
	FramePacer = MakeUnique<FAndroidVulkanFramePacer>();
	FPlatformRHIFramePacer::Init(FramePacer.Get());

	return true;
}

bool FVulkanAndroidPlatform::LoadVulkanInstanceFunctions(VkInstance inInstance)
{
	bool bFoundAllEntryPoints = true;

#define GETINSTANCE_VK_ENTRYPOINTS(Type, Func) VulkanDynamicAPI::Func = (Type)VulkanDynamicAPI::vkGetInstanceProcAddr(inInstance, #Func);

	ENUM_VK_ENTRYPOINTS_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_INSTANCE(CHECK_VK_ENTRYPOINTS);

	ENUM_VK_ENTRYPOINTS_SURFACE_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_SURFACE_INSTANCE(CHECK_VK_ENTRYPOINTS);

	ENUM_VK_ENTRYPOINTS_PLATFORM_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_PLATFORM_INSTANCE(CHECK_VK_ENTRYPOINTS);

	const bool bFoundRayTracingEntries = FVulkanRayTracingPlatform::CheckVulkanInstanceFunctions(inInstance);
	if (!bFoundRayTracingEntries)
	{
		UE_LOGF(LogVulkanRHI, Warning, "Vulkan RHI ray tracing is enabled, but failed to load instance functions.");
	}

	if (!bFoundAllEntryPoints)
	{
		return false;
	}

	ENUM_VK_ENTRYPOINTS_OPTIONAL_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_OPTIONAL_PLATFORM_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
#if UE_BUILD_DEBUG
	ENUM_VK_ENTRYPOINTS_OPTIONAL_INSTANCE(CHECK_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_OPTIONAL_PLATFORM_INSTANCE(CHECK_VK_ENTRYPOINTS);
#endif

#undef GETINSTANCE_VK_ENTRYPOINTS

	return true;
}

void FVulkanAndroidPlatform::FreeVulkanLibrary()
{
	if (VulkanLib != nullptr)
	{
#define CLEAR_VK_ENTRYPOINTS(Type,Func) VulkanDynamicAPI::Func = nullptr;
		ENUM_VK_ENTRYPOINTS_ALL(CLEAR_VK_ENTRYPOINTS);

		dlclose(VulkanLib);
		VulkanLib = nullptr;
	}
	bAttemptedLoad = false;
}

#undef CHECK_VK_ENTRYPOINTS

void FVulkanAndroidPlatform::InitDevice(FVulkanDevice* InDevice)
{
#if USE_ANDROID_VULKAN_SWAPPY
	if (FVulkanAndroidPlatform::bSwappyEnabledAtRHIInit)
	{
		FVulkanQueue* GfxQueue = InDevice->GetGraphicsQueue();
		check(GfxQueue);
		SwappyVk_setQueueFamilyIndex(InDevice->GetHandle(), GfxQueue->GetHandle(), GfxQueue->GetFamilyIndex());
	}
#endif
}

void* FVulkanAndroidPlatform::GetHardwareWindowHandle()
{
	// don't use cached window handle coming from VulkanViewport, as it could be gone by now
	void* WindowHandle = FAndroidWindow::GetHardwareWindow_EventThread();
	if (WindowHandle == nullptr)
	{
		// Sleep if the hardware window isn't currently available.
		// The Window may not exist if the activity is pausing/resuming, in which case we make this thread wait
		FPlatformMisc::LowLevelOutputDebugString(TEXT("Waiting for Native window in FVulkanAndroidPlatform::CreateSurface"));
		WindowHandle = FAndroidWindow::WaitForHardwareWindow();

		if (WindowHandle == nullptr)
		{
			FPlatformMisc::LowLevelOutputDebugString(TEXT("Aborting FVulkanAndroidPlatform::CreateSurface, FAndroidWindow::WaitForHardwareWindow() returned null"));
		}
	}

	return WindowHandle;
}

bool FVulkanAndroidPlatform::SupportsBCTextureFormats()
{
	return (CVarVulkanSupportsBCTextureFormats.GetValueOnAnyThread() == 1);
}

void FVulkanAndroidPlatform::CreateSurface(FVulkanPlatformWindowContext& WindowContext, VkInstance Instance, VkSurfaceKHR* OutSurface)
{
	ANativeWindow* Handle = FAndroidMisc::UseNewWindowBehavior() ? WindowContext.GetANativeWindow() : (ANativeWindow*)GetHardwareWindowHandle();
	check(Handle);

	VkAndroidSurfaceCreateInfoKHR SurfaceCreateInfo;
	ZeroVulkanStruct(SurfaceCreateInfo, VK_STRUCTURE_TYPE_ANDROID_SURFACE_CREATE_INFO_KHR);
	SurfaceCreateInfo.window = Handle;
	VERIFYVULKANRESULT(VulkanDynamicAPI::vkCreateAndroidSurfaceKHR(Instance, &SurfaceCreateInfo, VULKAN_CPU_ALLOCATOR, OutSurface));
}


void FVulkanAndroidPlatform::GetInstanceExtensions(FVulkanInstanceExtensionArray& OutExtensions)
{
	OutExtensions.Add(MakeUnique<FVulkanInstanceExtension>(VK_KHR_ANDROID_SURFACE_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));

	// VK_GOOGLE_display_timing (as instance extension?)
	if (FAndroidPlatformRHIFramePacer::CVarAllowFrameTimestamps.GetValueOnAnyThread() != 0)
	{
		OutExtensions.Add(
			MakeUnique<FVulkanInstanceExtension>(VK_GOOGLE_DISPLAY_TIMING_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));
	}
}

void FVulkanAndroidPlatform::GetInstanceLayers(TArray<const ANSICHAR*>& OutLayers)
{
#if !UE_BUILD_SHIPPING
	if (DebugVulkanInstanceLayers.IsEmpty())
	{
		TArray<FString> LayerNames;
		GConfig->GetArray(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("DebugVulkanInstanceLayers"), LayerNames, GEngineIni);

		if (!LayerNames.IsEmpty())
		{
			uint32 Index = 0;
			for (auto& LayerName : LayerNames)
			{
				TArray<ANSICHAR> LayerNameANSI{ TCHAR_TO_ANSI(*LayerName), LayerName.Len() + 1 };
				DebugVulkanInstanceLayers.Add(LayerNameANSI);
			}
		}
	}

	for (const TArray<ANSICHAR>& LayerName : DebugVulkanInstanceLayers)
	{
		OutLayers.Add(LayerName.GetData());
	}
#endif
}

static int32 GVulkanUseASTCDecodeMode = 1;
static FAutoConsoleVariableRef CVarVulkanUseASTCDecodeMode(
	TEXT("r.Vulkan.UseASTCDecodeMode"),
	GVulkanUseASTCDecodeMode,
	TEXT("Whether to use VK_EXT_astc_decode_mode extension\n"),
	ECVF_ReadOnly
);

void FVulkanAndroidPlatform::GetDeviceExtensions(FVulkanDevice* Device, FVulkanDeviceExtensionArray& OutExtensions)
{
	OutExtensions.Add(MakeUnique<FVulkanDeviceExtension>(Device, VK_KHR_ANDROID_SURFACE_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));
	if (FAndroidPlatformRHIFramePacer::CVarAllowFrameTimestamps.GetValueOnAnyThread() != 0)
	{
		OutExtensions.Add(MakeUnique<FVulkanDeviceExtension>(
			Device, VK_GOOGLE_DISPLAY_TIMING_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));
	}
	if (GVulkanUseASTCDecodeMode)
	{
		OutExtensions.Add(MakeUnique<FVulkanDeviceExtension>(Device, VK_EXT_ASTC_DECODE_MODE_EXTENSION_NAME, VULKAN_SUPPORTS_ASTC_DECODE_MODE, VULKAN_EXTENSION_NOT_PROMOTED, DEVICE_EXT_FLAG_SETTER(HasEXTASTCDecodeMode)));
	}
	OutExtensions.Add(MakeUnique<FVulkanDeviceExtension>(Device, VK_EXT_TEXTURE_COMPRESSION_ASTC_HDR_EXTENSION_NAME, VULKAN_SUPPORTS_TEXTURE_COMPRESSION_ASTC_HDR, VK_API_VERSION_1_3, DEVICE_EXT_FLAG_SETTER(HasEXTTextureCompressionASTCHDR)));
	OutExtensions.Add(MakeUnique<FVulkanDeviceExtension>(Device, VK_EXT_QUEUE_FAMILY_FOREIGN_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED));
	OutExtensions.Add(MakeUnique<FVulkanDeviceExtension>(Device, VK_ANDROID_EXTERNAL_MEMORY_ANDROID_HARDWARE_BUFFER_EXTENSION_NAME, VULKAN_SUPPORTS_ANDROID_EXTERNAL_MEMORY_ANDROID_HARDWARE_BUFFER, VULKAN_EXTENSION_NOT_PROMOTED, DEVICE_EXT_FLAG_SETTER(HasANDROIDExternalMemoryHardwareBuffer)));

#if !UE_BUILD_SHIPPING
	// Layer name as extension
	OutExtensions.Add(MakeUnique<FVulkanDeviceExtension>(Device, VULKAN_MALI_LAYER_NAME, VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));
#endif

#if USE_ANDROID_VULKAN_SWAPPY
	if (FVulkanAndroidPlatform::bSwappyEnabledAtRHIInit)
	{
		// make sure any extensions swappy requires are included
		for (const TArray<ANSICHAR>& SwappyRequiredExtension : SwappyRequiredExtensions)
		{
			if (FVulkanExtensionBase::FindExtension(OutExtensions, SwappyRequiredExtension.GetData()) != INDEX_NONE)
			{
				continue;
			}

			OutExtensions.Add(MakeUnique<FVulkanDeviceExtension>(Device, SwappyRequiredExtension.GetData(), VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));
		}
	}
#endif

}

void FVulkanAndroidPlatform::GetDeviceLayers(TArray<const ANSICHAR*>& OutLayers)
{
#if !UE_BUILD_SHIPPING
	if (DebugVulkanDeviceLayers.IsEmpty())
	{
		TArray<FString> LayerNames;
		GConfig->GetArray(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("DebugVulkanDeviceLayers"), LayerNames, GEngineIni);

		if (!LayerNames.IsEmpty())
		{
			uint32 Index = 0;
			for (auto& LayerName : LayerNames)
			{
				TArray<ANSICHAR> LayerNameANSI{ TCHAR_TO_ANSI(*LayerName), LayerName.Len() + 1 };
				DebugVulkanDeviceLayers.Add(LayerNameANSI);
			}
		}
	}

	for (auto& LayerName : DebugVulkanDeviceLayers)
	{
		OutLayers.Add(LayerName.GetData());
	}
#endif
}

void FVulkanAndroidPlatform::NotifyFoundDeviceLayersAndExtensions(VkPhysicalDevice PhysicalDevice, const TArray<const ANSICHAR*>& Layers, const TArray<const ANSICHAR*>& Extensions)
{
#if USE_ANDROID_VULKAN_SWAPPY
	if (FVulkanAndroidPlatform::bSwappyEnabledAtRHIInit)
	{
		// Determine extensions required by Swappy
		// We need to pass in vkEnumerateDeviceExtensionProperties directly so we cannot use the Extensions array as passed in.
		uint32 PropertyCount = 0;
		VERIFYVULKANRESULT(VulkanRHI::vkEnumerateDeviceExtensionProperties(PhysicalDevice, nullptr, &PropertyCount, nullptr));
		if (PropertyCount > 0)
		{
			TArray<VkExtensionProperties> Properties;
			Properties.AddZeroed(PropertyCount);
			VERIFYVULKANRESULT(VulkanRHI::vkEnumerateDeviceExtensionProperties(PhysicalDevice, nullptr, &PropertyCount, Properties.GetData()));
			check(PropertyCount == Properties.Num());

			uint32 SwappyRequiredExtensionCount = 0;
			SwappyVk_determineDeviceExtensions(PhysicalDevice, PropertyCount, Properties.GetData(), &SwappyRequiredExtensionCount, nullptr);

			if (SwappyRequiredExtensionCount > 0)
			{
				UE_LOGF(LogVulkanRHI, Log, "Swappy requires %d extensions:", SwappyRequiredExtensionCount);

				SwappyRequiredExtensions.Empty(SwappyRequiredExtensionCount);
				SwappyRequiredExtensions.AddDefaulted(SwappyRequiredExtensionCount);
				// SwappyVk_determineDeviceExtensions API requires an array of pointers to char that it can fill in.
				TArray<ANSICHAR*> SwappyRequiredExtensionPtrs;
				SwappyRequiredExtensionPtrs.Empty(SwappyRequiredExtensionCount);
				for (int32 i = 0; i < SwappyRequiredExtensionCount; i++)
				{
					SwappyRequiredExtensions[i].AddZeroed(VK_MAX_EXTENSION_NAME_SIZE + 1);
					SwappyRequiredExtensionPtrs.Add(SwappyRequiredExtensions[i].GetData());
				}

				SwappyVk_determineDeviceExtensions(PhysicalDevice, PropertyCount, Properties.GetData(), &SwappyRequiredExtensionCount, SwappyRequiredExtensionPtrs.GetData());

				check(SwappyRequiredExtensionCount == SwappyRequiredExtensions.Num());

				for (int32 i = 0; i < SwappyRequiredExtensionCount; i++)
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("  %s\n"), ANSI_TO_TCHAR(SwappyRequiredExtensions[i].GetData()));
				}
			}
			else
			{
				UE_LOGF(LogVulkanRHI, Log, "Swappy didn't ask for any extensions");
			}
		}
		else
		{
			UE_LOGF(LogVulkanRHI, Log, "No extensions available for Swappy");
		}
	}
#endif


#if VULKAN_SUPPORTS_GOOGLE_DISPLAY_TIMING
	FVulkanAndroidPlatform::bHasGoogleDisplayTiming = Extensions.ContainsByPredicate([](const ANSICHAR* Key)
		{
			return Key && !FCStringAnsi::Strcmp(Key, VK_GOOGLE_DISPLAY_TIMING_EXTENSION_NAME);
		});
	UE_LOGF(LogVulkanRHI, Log, "bHasGoogleDisplayTiming = %d", FVulkanAndroidPlatform::bHasGoogleDisplayTiming);
#endif
}

bool FVulkanAndroidPlatform::SupportsTimestampRenderQueries()
{
	static const auto CVarAndroidSupportsTimestampQueries = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Android.SupportsTimestampQueries"));

	// standalone devices have newer drivers where timestamp render queries work.
	return CVarAndroidSupportsTimestampQueries != nullptr &&
		CVarAndroidSupportsTimestampQueries->GetBool();
}

bool FVulkanAndroidPlatform::SupportsDynamicResolution()
{
	// separating render timestamp queries from dynres availability

#if USE_ANDROID_VULKAN_SWAPPY
	
	static const auto CVarAndroidSupportsDynamicResolution = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Android.SupportsDynamicResolution"));
	
	return FVulkanAndroidPlatform::bSwappyEnabledAtRHIInit &&
		CVarAndroidSupportsDynamicResolution != nullptr &&
		CVarAndroidSupportsDynamicResolution->GetBool(); // is supported

#else // USE_ANDROID_VULKAN_SWAPPY

	// defaulted to the previous code
	return SupportsTimestampRenderQueries();

#endif
}

extern bool IsInAndroidEventThread();

namespace AndroidVulkan
{
#if USE_ANDROID_STANDALONE
	static std::atomic<bool> bWaitingForWindowCreated{false};
#endif

	static void OnLostWindow(TOptional<FAndroidWindow::FNativeAccessor> WindowContainer)
	{
		check(IsInAndroidEventThread());
		check(FAndroidMisc::UseNewWindowBehavior());
		UE_LOGF(LogVulkanRHI, Log, "OnLostWindow: Event thread");
		FVulkanPlatformWindowContext WindowContext(MoveTemp(WindowContainer));
		FGraphEventRef OnComplete = FGraphEvent::CreateGraphEvent();

		ENQUEUE_RENDER_COMMAND(OnAndroidLostWindow)([OnComplete, &WindowContext](FRHICommandListImmediate& RHICmdList) mutable
		{
				UE_LOGF(LogVulkanRHI, Log, "OnLostWindow: RT");
				RHICmdList.EnqueueLambda([&WindowContext](FRHICommandListImmediate& ExecutingCmdList) mutable
					{
						check(IsInRHIThread() || IsRunningRHIInSeparateThread());
						UE_LOGF(LogVulkanRHI, Log, "OnLostWindow: RHI");

						// cannot just throw out this command buffer (needs to be submitted or other checks fail)
						FVulkanCommandListContext& Context = FVulkanCommandListContext::Get(ExecutingCmdList);
						Context.FlushCommands(EVulkanFlushFlags::WaitForSubmission);

						bool bFound = false;
						TArray<FVulkanViewport*> Viewports = GVulkanRHI->GetViewports();
						for (FVulkanViewport* Viewport : Viewports)
						{
							if (Viewport->GetWindowHandle() == WindowContext.GetWindowHandle())
							{
								UE_LOGF(LogVulkanRHI, Log, "OnLostWindow: Destroying viewport swapchain (window %p -> null)", WindowContext.GetANativeWindow() );
								Viewport->DestroySwapchain(nullptr);
								UE_LOGF(LogVulkanRHI, Log, "OnLostWindow: Destroying viewport swapchain done.");
								bFound = true;
								break;
							}
						}
						check(bFound);
						UE_LOGF(LogVulkanRHI, Log, "OnLostWindow: RHI done");
					});

				// Swapchain creation pushes some commands - flush the command buffers now to begin with a fresh state
				RHICmdList.RHIThreadFence(true);
				RHICmdList.SubmitAndBlockUntilGPUIdle();

				UE_LOGF(LogVulkanRHI, Log, "OnLostWindow:RT done");
				OnComplete->DispatchSubsequents();
			});

		UE_LOGF(LogVulkanRHI, Log, "OnLostWindow: wait RT");
		OnComplete->Wait();
		UE_LOGF(LogVulkanRHI, Log, "OnLostWindow: done");
	}

	static void OnFoundWindow(TOptional<FAndroidWindow::FNativeAccessor> WindowContainer)
	{
		check(IsInAndroidEventThread());
		check(FAndroidMisc::UseNewWindowBehavior());

		FVulkanPlatformWindowContext WindowContext(MoveTemp(WindowContainer));

		UE_LOGF(LogVulkanRHI, Log, "OnFoundWindow:");

#if USE_ANDROID_STANDALONE
		bWaitingForWindowCreated.store(true);
#endif
		FGraphEventRef OnComplete = FGraphEvent::CreateGraphEvent();
		ENQUEUE_RENDER_COMMAND(OnAndroidFoundWindow)([OnComplete, &WindowContext](FRHICommandListImmediate& RHICmdList) mutable
			{
				UE_LOGF(LogVulkanRHI, Log, "OnFoundWindow: RT");
				RHICmdList.EnqueueLambda([&WindowContext](FRHICommandListImmediate& ExecutingCmdList) mutable
					{
						check(IsInRHIThread() || IsRunningRHIInSeparateThread());
						UE_LOGF(LogVulkanRHI, Log, "OnFoundWindow: RHI");

						FVulkanDevice* Device = FVulkanDynamicRHI::Get().GetDevice();

						// cannot just throw out this command buffer (needs to be submitted or other checks fail)
						FVulkanCommandListContext& Context = FVulkanCommandListContext::Get(ExecutingCmdList);
						Context.FlushCommands(EVulkanFlushFlags::WaitForSubmission);

						bool bFound = false;
						TArray<FVulkanViewport*> Viewports = GVulkanRHI->GetViewports();
						for (FVulkanViewport* Viewport : Viewports)
						{
							if (Viewport->GetWindowHandle() == WindowContext.GetWindowHandle())
							{
								UE_LOGF(LogVulkanRHI, Log, "OnFoundWindow: RHI RecreateSwapchain (window %p)", WindowContext.GetANativeWindow());
								Viewport->RecreateSwapchain(Context, WindowContext);
								UE_LOGF(LogVulkanRHI, Log, "OnFoundWindow: RHI RecreateSwapchain done (window %p)", WindowContext.GetANativeWindow());
								bFound = true;
								break;
							}
						}
						check(bFound);
						UE_LOGF(LogVulkanRHI, Log, "OnFoundWindow: RHI done");
					});
				// Swapchain creation pushes some commands - flush the command buffers now to begin with a fresh state
				RHICmdList.RHIThreadFence(true);
				RHICmdList.SubmitAndBlockUntilGPUIdle();
				UE_LOGF(LogVulkanRHI, Log, "OnFoundWindow: RT done");
				OnComplete->DispatchSubsequents();
			});
		UE_LOGF(LogVulkanRHI, Log, "OnFoundWindow: wait RT");
		OnComplete->Wait();
		UE_LOGF(LogVulkanRHI, Log, "OnFoundWindow: done");
	}
}

#if USE_ANDROID_STANDALONE
bool FVulkanAndroidPlatform::IsWaitingForWindowCreated()
{
	return AndroidVulkan::bWaitingForWindowCreated.load();
}
#endif

// Old window behavior window recreate function.
// Moved to platform specific code from VulkanRHI.cpp.
void FVulkanAndroidPlatform::RecreateSwapChain(void* NewNativeWindow)
{
	check(!FAndroidMisc::UseNewWindowBehavior());

	if (NewNativeWindow)
	{
		if (IsInGameThread())
		{
			FlushRenderingCommands();
		}

		TArray<FVulkanViewport*> Viewports = FVulkanDynamicRHI::Get().GetViewports();
		ENQUEUE_RENDER_COMMAND(VulkanRecreateSwapChain)(
			[Viewports = MoveTemp(Viewports), NewNativeWindow](FRHICommandListImmediate& RHICmdList) mutable
		{
			UE_LOGF(LogVulkanRHI, Log, "Recreate swapchain ... ");
			RHICmdList.EnqueueLambda(TEXT("RecreateSwapchain"), [Viewports = MoveTemp(Viewports), NewNativeWindow](FRHICommandListImmediate& ExecutingCmdList)
			{
				for (FVulkanViewport* Viewport : Viewports)
				{
					FVulkanPlatformWindowContext WindowContext(NewNativeWindow);
					Viewport->RecreateSwapchain(FVulkanCommandListContext::Get(ExecutingCmdList), WindowContext);
				}
			});

			RHICmdList.SubmitAndBlockUntilGPUIdle();
		});

		if (IsInGameThread())
		{
			FlushRenderingCommands();
		}
	}
}

// Old window behavior window destroy function.
// Moved to platform specific code from VulkanRHI.cpp.
void FVulkanAndroidPlatform::DestroySwapChain()
{
	check(!FAndroidMisc::UseNewWindowBehavior());

	if (IsInGameThread())
	{
		FlushRenderingCommands();
	}

	TArray<FVulkanViewport*> Viewports = FVulkanDynamicRHI::Get().GetViewports();
	ENQUEUE_RENDER_COMMAND(VulkanDestroySwapChain)(
		[Viewports = MoveTemp(Viewports)](FRHICommandListImmediate& RHICmdList) mutable
		{
			UE_LOGF(LogVulkanRHI, Log, "Destroy swapchain ... ");

			RHICmdList.EnqueueLambda(TEXT("FVulkanDynamicRHI::DestroySwapChain"), [Viewports = MoveTemp(Viewports)](FRHICommandListBase& ExecutingCmdList)
				{
					for (FVulkanViewport* Viewport : Viewports)
					{
						Viewport->DestroySwapchain(nullptr);
					}
				});
			RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
		});

	if (IsInGameThread())
	{
		FlushRenderingCommands();
	}
}


void FVulkanAndroidPlatform::OverridePlatformHandlers(bool bInit)
{
	if (bInit)
	{
		if (FAndroidMisc::UseNewWindowBehavior())
		{
			UE::FAndroidPlatformDynamicRHI::SetRHIOnReleaseWindowCallback(AndroidVulkan::OnLostWindow);
			UE::FAndroidPlatformDynamicRHI::SetRHIOnReInitWindowCallback(AndroidVulkan::OnFoundWindow);
#if USE_ANDROID_STANDALONE
			UE::FAndroidPlatformDynamicRHI::SetRHIOnWindowCreatedCallback(
				[]()
				{
					// Enqueue on the render thread so this executes AFTER the viewport resize
					// render command (triggered by OnWindowSizeChanged) has completed.
					// Unblocking immediately here would race with the resize, allowing a frame
					// to present with the old swapchain size.
					ENQUEUE_RENDER_COMMAND(OnVulkanWindowCreated)([](FRHICommandListImmediate& RHICmdList)
					{
						RHICmdList.EnqueueLambda([](FRHICommandListImmediate&)
						{
							AndroidVulkan::bWaitingForWindowCreated.store(false);
							UE_LOGF(LogVulkanRHI, Display, "OnWindowCreatedProcessed: rendering unblocked");
						});
					});
				});
#endif
		}
		else
		{
			FPlatformMisc::SetOnReInitWindowCallback(FVulkanAndroidPlatform::RecreateSwapChain);
			FPlatformMisc::SetOnReleaseWindowCallback(FVulkanAndroidPlatform::DestroySwapChain);
		}
		FPlatformMisc::SetOnPauseCallback(FVulkanDynamicRHI::SavePipelineCache);
	}
	else
	{
		FPlatformMisc::SetOnReInitWindowCallback(nullptr);
		FPlatformMisc::SetOnReleaseWindowCallback(nullptr);
		UE::FAndroidPlatformDynamicRHI::SetRHIOnReleaseWindowCallback(nullptr);
		UE::FAndroidPlatformDynamicRHI::SetRHIOnReInitWindowCallback(nullptr);
		UE::FAndroidPlatformDynamicRHI::SetRHIOnWindowCreatedCallback(nullptr);

		FPlatformMisc::SetOnPauseCallback(nullptr);
	}
}
#if PLATFORM_ANDROID_X64

bool FVulkanAndroidPlatform::HasUnifiedMemory()
{
	return CVarVulkanHasUnifiedMemory.GetValueOnAnyThread();
}
#endif

void FVulkanAndroidPlatform::SetupRequiresDepthStencilFullWriteWorkaround(const FVulkanDevice& Device)
{
	bRequiresDepthStencilFullWrite = false;
	if (Device.GetVendorId() == EGpuVendorId::Arm)
	{
		const uint32_t DriverVersion = Device.GetDeviceProperties().driverVersion;
		if (DriverVersion >= VK_MAKE_API_VERSION(0, 16, 0, 0) && DriverVersion < VK_MAKE_API_VERSION(0, 24, 0, 0))
		{
			bRequiresDepthStencilFullWrite = true;
			UE_LOGF(LogRHI, Display, "Enabling workaround that changes partial depth/stencil attachment stores into full depth/stencil stores.");
		}
	}
}

void FVulkanAndroidPlatform::SetupRequiresMergedDepthStencilLayoutWorkaround(const FVulkanDevice& Device)
{
	bRequiresMergedRenderPassDepthStencilLayout = false;
	if (Device.GetVendorId() == EGpuVendorId::Arm)
	{
		const uint32_t DriverVersion = Device.GetDeviceProperties().driverVersion;
		if (DriverVersion < VK_MAKE_API_VERSION(0, 54, 1, 0))
		{
			bRequiresMergedRenderPassDepthStencilLayout = true;
			UE_LOGF(LogRHI, Display, "Enabling workaround that avoids usage of dedicated stencil layout via VkAttachmentDescriptionStencilLayout.");
		}
	}
}

bool FVulkanAndroidPlatform::FramePace(FVulkanDevice& Device, void* WindowHandle, VkSwapchainKHR Swapchain, uint32 PresentID, VkPresentInfoKHR& Info)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_VulkanAndroid_FramePace);
	
	bool bVsyncMultiple = (CachedSyncInterval != 0);
	int32 CurrentFramePace = FAndroidPlatformRHIFramePacer::GetFramePace();

#if USE_ANDROID_VULKAN_SWAPPY
	if (FVulkanAndroidPlatform::bSwappyEnabledAtRHIInit && CurrentFramePace != 0)
	{
		// cache refresh rate and sync interval
		if (CurrentFramePace != CachedFramePace)
		{
			CachedFramePace = CurrentFramePace;
			FramePacer->SupportsFramePaceInternal(CurrentFramePace, CachedRefreshRate, CachedSyncInterval);
			SwappyVk_resetFramePacing(Swapchain);

			if (CachedSyncInterval != 0)
			{
				// Multiple of sync interval, use swappy directly
				SwappyVk_setSwapIntervalNS(Device.GetHandle(), Swapchain, (1000000000L) / (int64)CurrentFramePace);
				bVsyncMultiple = true;
			}
			else
			{
				// Unsupported frame rate. Set to higher refresh rate and use CPU frame pacer to limit to desired frame pace
				SwappyVk_setSwapIntervalNS(Device.GetHandle(), Swapchain, (1000000000L) / (int64)CachedRefreshRate);
				// indicate that the RHI should perform CPU frame pacing to handle the requested frame rate
				bVsyncMultiple = false;
			}
		}

		return bVsyncMultiple;
	}
#endif

#if VULKAN_SUPPORTS_GOOGLE_DISPLAY_TIMING
	if (GVulkanExtensionFramePacer && bHasGoogleDisplayTiming)
	{
		check(GDTimingFramePacer);
		GDTimingFramePacer->ScheduleNextFrame(PresentID, CurrentFramePace, CachedRefreshRate);
		Info.pNext = GDTimingFramePacer->GetPresentTimesInfo();
	}
#else
	{}
#endif
	return bVsyncMultiple; 
}

VkResult FVulkanAndroidPlatform::Present(VkQueue Queue, VkPresentInfoKHR& PresentInfo)
{
#if USE_ANDROID_VULKAN_SWAPPY
	if (FVulkanAndroidPlatform::bSwappyEnabledAtRHIInit)
	{
		return SwappyVk_queuePresent(Queue, &PresentInfo);
	}
	else
#endif
	{
		return VulkanRHI::vkQueuePresentKHR(Queue, &PresentInfo);
	}
}

VkResult FVulkanAndroidPlatform::CreateSwapchainKHR(FVulkanPlatformWindowContext& WindowContext, VkPhysicalDevice PhysicalDevice, VkDevice Device, const VkSwapchainCreateInfoKHR* CreateInfo, const VkAllocationCallbacks* Allocator, VkSwapchainKHR* Swapchain)
{
	VkResult Result = VulkanRHI::vkCreateSwapchainKHR(Device, CreateInfo, Allocator, Swapchain);

	if (Result == VK_SUCCESS)
	{
#if USE_ANDROID_VULKAN_SWAPPY
		if (FVulkanAndroidPlatform::bSwappyEnabledAtRHIInit)
		{
			JNIEnv* Env = FAndroidApplication::GetJavaEnv();
			if (ensure(Env))
			{
				ANativeWindow* WindowHandle = FAndroidMisc::UseNewWindowBehavior() ? WindowContext.GetANativeWindow() : (ANativeWindow*)GetHardwareWindowHandle();
				check(WindowHandle);
								
				uint64_t RefreshDuration; // in nanoseconds
				SwappyVk_initAndGetRefreshCycleDuration(Env, FJavaWrapper::GameActivityThis, PhysicalDevice, Device, *Swapchain, &RefreshDuration);
				SwappyVk_setWindow(Device, *Swapchain, WindowHandle);	
				SwappyVk_setAutoSwapInterval(false);
				AndroidVulkan::SetSwappyPostWaitCallback();
				UE_LOGF(LogVulkanRHI, Log, "SwappyVk_initAndGetRefreshCycleDuration: %llu", RefreshDuration);
			}

			GVulkanExtensionFramePacer = 0;
		}
		else
#endif
#if VULKAN_SUPPORTS_GOOGLE_DISPLAY_TIMING
		if (GVulkanExtensionFramePacer && FVulkanAndroidPlatform::bHasGoogleDisplayTiming)
		{
			GDTimingFramePacer = MakeUnique<FGDTimingFramePacer>(Device, *Swapchain);
		}
#else
		{}
#endif
	}
	return Result;
}

void FVulkanAndroidPlatform::DestroySwapchainKHR(VkDevice Device, VkSwapchainKHR Swapchain, const VkAllocationCallbacks* Allocator)
{
#if USE_ANDROID_VULKAN_SWAPPY
	if (FVulkanAndroidPlatform::bSwappyEnabledAtRHIInit)
	{
		SwappyVk_destroySwapchain(Device, Swapchain);
		UE_LOGF(LogVulkanRHI, Log, "SwappyVk_destroySwapchain");
	}
#endif
		
	VulkanRHI::vkDestroySwapchainKHR(Device, Swapchain, Allocator);

	// reset frame pace, to force display refresh rate update after we create a new swapchain
	// see FVulkanAndroidPlatform::FramePace
	CachedFramePace = 0;
}


bool FAndroidVulkanFramePacer::SupportsFramePaceInternal(int32 QueryFramePace, int32& OutRefreshRate, int32& OutSyncInterval)
{
	TArray<int32> RefreshRates = FAndroidMisc::GetSupportedNativeDisplayRefreshRates();
	RefreshRates.Sort();

	FString DebugString = "FAndroidVulkanFramePacer -> Supported Refresh Rates:";
	for (int32 RefreshRate : RefreshRates)
	{
		DebugString += FString::Printf(TEXT(" %d"), RefreshRate);
	}
	UE_LOGF(LogRHI, Log, "%ls", *DebugString);

	for (int32 Rate : RefreshRates)
	{
		if ((Rate % QueryFramePace) == 0)
		{
			UE_LOGF(LogRHI, Log, "Supports %d using refresh rate %d and sync interval %d", QueryFramePace, Rate, Rate / QueryFramePace);
			OutRefreshRate = Rate;
			OutSyncInterval = Rate / QueryFramePace;
			return true;
		}
	}

	// check if we want to use CPU frame pacing at less than a multiple of supported refresh rate
	if (FAndroidPlatformRHIFramePacer::CVarSupportNonVSyncMultipleFrameRates.GetValueOnAnyThread() == 1)
	{
		for (int32 Rate : RefreshRates)
		{
			if (Rate > QueryFramePace)
			{
				UE_LOGF(LogRHI, Log, "Supports %d using refresh rate %d with CPU frame pacing", QueryFramePace, Rate);
				OutRefreshRate = Rate;
				OutSyncInterval = 0;
				return true;
			}
		}
	}

	OutRefreshRate = QueryFramePace;
	OutSyncInterval = 0;
	return false;
}

struct GraphicsPipelineCreateInfo
{
	VkPipelineCreateFlags PipelineCreateFlags;
	uint32_t StageCount;
	
	bool bHasVkPipelineVertexInputStateCreateInfo;
	bool bHasVkPipelineInputAssemblyStateCreateInfo;	
	bool bHasVkPipelineTessellationStateCreateInfo;
	bool bHasVkPipelineViewportStateCreateInfo;
	bool bHasVkPipelineRasterizationStateCreateInfo;
	bool bHasVkPipelineMultisampleStateCreateInfo;
	bool bHasVkPipelineDepthStencilStateCreateInfo;
	bool bHasVkPipelineColorBlendStateCreateInfo;
	bool bHasVkPipelineDynamicStateCreateInfo;

	uint32_t subpass;
};

#define COPY_TO_BUFFER(Dst, Src, Size) \
		Dst.Append((const char*)Src, Size); 

void CharArrayToBuffer(const TArray<const ANSICHAR*>& CharArray, TArray<char>& MemoryStream)
{
	uint32_t Count = CharArray.Num();
	COPY_TO_BUFFER(MemoryStream, &Count, sizeof(uint32_t));
	for (uint32_t Idx = 0; Idx < Count; ++Idx)
	{
		uint32_t StrLength = strlen(CharArray[Idx])+1;
		COPY_TO_BUFFER(MemoryStream, &StrLength, sizeof(uint32_t));
		COPY_TO_BUFFER(MemoryStream, CharArray[Idx], StrLength);
	}
}

void GetVKStructsFromPNext(const void* InNext, TMap<VkStructureType, const void*>& VkStructs, const TArray<VkStructureType>& ValidTypes)
{
	const VkBaseInStructure* Next = reinterpret_cast<const VkBaseInStructure*>(InNext);
	while (Next != nullptr)
	{
		if (!ValidTypes.Contains(Next->sType))
		{
			UE_LOGF(LogRHI, Warning, "GetVKStructsFromPNext: Unexpected type found when reading pNext->sType %d, Valid Types: ", (uint32_t)Next->sType);
		}

		VkStructs.FindOrAdd(Next->sType) = Next;
		Next = reinterpret_cast<const VkBaseInStructure*>(Next->pNext);
	}
}

void HandleGraphicsPipelineCreatePNext(const VkGraphicsPipelineCreateInfo* PipelineCreateInfo, TArray<char>& MemoryStream)
{
	TMap<VkStructureType, const void*> VkStructs;
	GetVKStructsFromPNext(PipelineCreateInfo->pNext, VkStructs, { VK_STRUCTURE_TYPE_PIPELINE_FRAGMENT_SHADING_RATE_STATE_CREATE_INFO_KHR });

	int32_t HandledCount = 0;

	// FSR Create Info
	bool bHasFSRCreateInfo = false;

	const void** Struct = VkStructs.Find(VK_STRUCTURE_TYPE_PIPELINE_FRAGMENT_SHADING_RATE_STATE_CREATE_INFO_KHR);
	bHasFSRCreateInfo = Struct != nullptr;
	COPY_TO_BUFFER(MemoryStream, &bHasFSRCreateInfo, sizeof(bool));

	if (bHasFSRCreateInfo)
	{
		VkPipelineFragmentShadingRateStateCreateInfoKHR* FSRCreateInfo = (VkPipelineFragmentShadingRateStateCreateInfoKHR*)*Struct;
		check(FSRCreateInfo->pNext == nullptr);
		FSRCreateInfo->pNext = nullptr;

		COPY_TO_BUFFER(MemoryStream, FSRCreateInfo, sizeof(VkPipelineFragmentShadingRateStateCreateInfoKHR));
		HandledCount++;
	} 

	check(HandledCount == VkStructs.Num());
}

void HandlePipelineShaderStagePNext(const VkPipelineShaderStageCreateInfo* CreateInfo, TArray<char>& MemoryStream)
{
	TMap<VkStructureType, const void*> VkStructs;
	GetVKStructsFromPNext(CreateInfo->pNext, VkStructs, { VK_STRUCTURE_TYPE_PIPELINE_SHADER_STAGE_REQUIRED_SUBGROUP_SIZE_CREATE_INFO });

	int32_t HandledCount = 0;

	// Subgroup Size Info
	bool bHasSubGroupSizeInfo = false;

	const void** Struct = VkStructs.Find(VK_STRUCTURE_TYPE_PIPELINE_SHADER_STAGE_REQUIRED_SUBGROUP_SIZE_CREATE_INFO);
	bHasSubGroupSizeInfo = Struct != nullptr;
	COPY_TO_BUFFER(MemoryStream, &bHasSubGroupSizeInfo, sizeof(bool));

	if (bHasSubGroupSizeInfo)
	{
		VkPipelineShaderStageRequiredSubgroupSizeCreateInfo* SubgroupSizeCreateInfo = (VkPipelineShaderStageRequiredSubgroupSizeCreateInfo*)*Struct;
		check(SubgroupSizeCreateInfo->pNext == nullptr);

		COPY_TO_BUFFER(MemoryStream, SubgroupSizeCreateInfo, sizeof(VkPipelineShaderStageRequiredSubgroupSizeCreateInfo));
		HandledCount++;
	}

	check(HandledCount == VkStructs.Num());
}

void HandleSubpassDescriptionPNext(VkSubpassDescription2* SubpassDescription, TArray<char>& MemoryStream)
{
	TMap<VkStructureType, const void*> VkStructs;
	GetVKStructsFromPNext(SubpassDescription->pNext, VkStructs, { VK_STRUCTURE_TYPE_FRAGMENT_SHADING_RATE_ATTACHMENT_INFO_KHR });

	int32_t HandledCount = 0;

	// FSR Create Info 
	bool bHasFSRAttachmentCreateInfo = false;

	const void** Struct = VkStructs.Find(VK_STRUCTURE_TYPE_FRAGMENT_SHADING_RATE_ATTACHMENT_INFO_KHR);
	bHasFSRAttachmentCreateInfo = Struct != nullptr;
	COPY_TO_BUFFER(MemoryStream, &bHasFSRAttachmentCreateInfo, sizeof(bool));

	if (bHasFSRAttachmentCreateInfo)
	{
		VkFragmentShadingRateAttachmentInfoKHR* FragmentShadingRateCreateInfo = (VkFragmentShadingRateAttachmentInfoKHR*)*Struct;

		check(FragmentShadingRateCreateInfo->pFragmentShadingRateAttachment->pNext == nullptr);
		COPY_TO_BUFFER(MemoryStream, FragmentShadingRateCreateInfo->pFragmentShadingRateAttachment, sizeof(VkAttachmentReference2));
		COPY_TO_BUFFER(MemoryStream, &FragmentShadingRateCreateInfo->shadingRateAttachmentTexelSize, sizeof(VkExtent2D));
		HandledCount++;
	}

	check(HandledCount == VkStructs.Num());
}

void HandleDepthStencilAttachmentPNext(const VkAttachmentReference2* Attachment, TArray<char>& MemoryStream)
{
	TMap<VkStructureType, const void*> VkStructs;
	GetVKStructsFromPNext(Attachment->pNext, VkStructs, { VK_STRUCTURE_TYPE_ATTACHMENT_REFERENCE_STENCIL_LAYOUT });

	int32_t HandledCount = 0;

	bool bHasStencilLayout = false;
	
	const void** Struct = VkStructs.Find(VK_STRUCTURE_TYPE_ATTACHMENT_REFERENCE_STENCIL_LAYOUT);
	bHasStencilLayout = Struct != nullptr;
	COPY_TO_BUFFER(MemoryStream, &bHasStencilLayout, sizeof(bool));

	if (bHasStencilLayout)
	{
		VkAttachmentReferenceStencilLayout* StencilLayout = (VkAttachmentReferenceStencilLayout*)*Struct;
		check(StencilLayout->pNext == nullptr);
		COPY_TO_BUFFER(MemoryStream, StencilLayout, sizeof(VkAttachmentReferenceStencilLayout));
		HandledCount++;
	}

	check(HandledCount == VkStructs.Num());
}

void HandleAttachmentDescriptionPNext(const VkAttachmentDescription2& Description, TArray<char>& MemoryStream)
{
	TMap<VkStructureType, const void*> VkStructs;
	GetVKStructsFromPNext(Description.pNext, VkStructs, { VK_STRUCTURE_TYPE_ATTACHMENT_DESCRIPTION_STENCIL_LAYOUT });

	int32_t HandledCount = 0;

	const void** Struct = VkStructs.Find(VK_STRUCTURE_TYPE_ATTACHMENT_DESCRIPTION_STENCIL_LAYOUT);
	const bool bHasStencilLayout = Struct != nullptr;
	COPY_TO_BUFFER(MemoryStream, &bHasStencilLayout, sizeof(bool));

	if (bHasStencilLayout)
	{
		VkAttachmentDescriptionStencilLayout* StencilLayout = (VkAttachmentDescriptionStencilLayout*)*Struct;
		check(StencilLayout->pNext == nullptr);
		COPY_TO_BUFFER(MemoryStream, StencilLayout, sizeof(VkAttachmentDescriptionStencilLayout));
		HandledCount++;
	}

	check(HandledCount == VkStructs.Num());
}

void PipelineToBinary(FVulkanDevice* Device, const VkGraphicsPipelineCreateInfo* PipelineInfo, const FGfxPipelineDesc* GfxEntry, const FVulkanRenderTargetLayout* RTLayout, TArray<char>& MemoryStream)
{
	static const unsigned int INITIAL_PSO_STREAM_SIZE = 64 * 1024;
	MemoryStream.Reserve(INITIAL_PSO_STREAM_SIZE);
	
	GraphicsPipelineCreateInfo pipelineCreateInfo;

	pipelineCreateInfo.PipelineCreateFlags = PipelineInfo->flags;
	pipelineCreateInfo.StageCount = PipelineInfo->stageCount; 

	pipelineCreateInfo.bHasVkPipelineVertexInputStateCreateInfo		= PipelineInfo->pVertexInputState != nullptr;
	pipelineCreateInfo.bHasVkPipelineInputAssemblyStateCreateInfo	= PipelineInfo->pInputAssemblyState != nullptr;
	pipelineCreateInfo.bHasVkPipelineTessellationStateCreateInfo	= PipelineInfo->pTessellationState != nullptr;
	pipelineCreateInfo.bHasVkPipelineViewportStateCreateInfo		= PipelineInfo->pViewportState != nullptr;
	pipelineCreateInfo.bHasVkPipelineRasterizationStateCreateInfo	= PipelineInfo->pRasterizationState != nullptr;
	pipelineCreateInfo.bHasVkPipelineMultisampleStateCreateInfo		= PipelineInfo->pMultisampleState != nullptr;
	pipelineCreateInfo.bHasVkPipelineDepthStencilStateCreateInfo	= PipelineInfo->pDepthStencilState != nullptr;
	pipelineCreateInfo.bHasVkPipelineColorBlendStateCreateInfo		= PipelineInfo->pColorBlendState != nullptr;
	pipelineCreateInfo.bHasVkPipelineDynamicStateCreateInfo			= PipelineInfo->pDynamicState != nullptr;

	pipelineCreateInfo.subpass = PipelineInfo->subpass;

	TArray<const ANSICHAR*> InstanceLayers = GVulkanRHI->GetInstanceLayers();
	CharArrayToBuffer(InstanceLayers, MemoryStream);
	TArray<const ANSICHAR*> InstanceExtensions = GVulkanRHI->GetInstanceExtensions();
	CharArrayToBuffer(InstanceExtensions, MemoryStream);
	TArray<const ANSICHAR*> DeviceExtensions = Device->GetDeviceExtensions();
	CharArrayToBuffer(DeviceExtensions, MemoryStream);
	COPY_TO_BUFFER(MemoryStream, &pipelineCreateInfo, sizeof(GraphicsPipelineCreateInfo));

	HandleGraphicsPipelineCreatePNext(PipelineInfo, MemoryStream);

	// VkPipelineShaderStageCreateInfo
	for (int32_t Idx = 0; Idx < PipelineInfo->stageCount; ++Idx)
	{
		VkPipelineShaderStageCreateInfo ShaderStage;
		FMemory::Memzero(ShaderStage);

		ShaderStage.sType = PipelineInfo->pStages[Idx].sType;
		ShaderStage.flags = PipelineInfo->pStages[Idx].flags;
		ShaderStage.stage = PipelineInfo->pStages[Idx].stage;

		HandlePipelineShaderStagePNext(&PipelineInfo->pStages[Idx], MemoryStream);

		COPY_TO_BUFFER(MemoryStream, &ShaderStage, sizeof(VkPipelineShaderStageCreateInfo));

		uint32_t NameLength = static_cast<uint32_t>(strlen(PipelineInfo->pStages[Idx].pName)+1);

		COPY_TO_BUFFER(MemoryStream, &NameLength, sizeof(uint32_t));
		COPY_TO_BUFFER(MemoryStream, PipelineInfo->pStages[Idx].pName, NameLength);

		const bool bHasSpecializationInfo = PipelineInfo->pStages[Idx].pSpecializationInfo != nullptr;
		COPY_TO_BUFFER(MemoryStream, &bHasSpecializationInfo, sizeof(bHasSpecializationInfo));
		if (bHasSpecializationInfo)
		{
			const VkSpecializationInfo SpecializationInfo = {
				.mapEntryCount = PipelineInfo->pStages[Idx].pSpecializationInfo->mapEntryCount,
				.pMapEntries = nullptr,
				.dataSize = PipelineInfo->pStages[Idx].pSpecializationInfo->dataSize,
				.pData = nullptr
			};
			COPY_TO_BUFFER(MemoryStream, &SpecializationInfo, sizeof(VkSpecializationInfo));
			COPY_TO_BUFFER(MemoryStream, PipelineInfo->pStages[Idx].pSpecializationInfo->pMapEntries, SpecializationInfo.mapEntryCount * sizeof(VkSpecializationMapEntry));
			COPY_TO_BUFFER(MemoryStream, PipelineInfo->pStages[Idx].pSpecializationInfo->pData, SpecializationInfo.dataSize);
		}
	}

	if (pipelineCreateInfo.bHasVkPipelineVertexInputStateCreateInfo)
	{
		const VkPipelineVertexInputStateCreateInfo* VertexInputState = PipelineInfo->pVertexInputState;
		check(VertexInputState->pNext == nullptr);

		VkPipelineVertexInputStateCreateInfo CopyVertexInputState;
		FMemory::Memzero(CopyVertexInputState);

		CopyVertexInputState.sType = VertexInputState->sType;
		CopyVertexInputState.flags = VertexInputState->flags;
		CopyVertexInputState.vertexBindingDescriptionCount = VertexInputState->vertexBindingDescriptionCount;
		CopyVertexInputState.vertexAttributeDescriptionCount = VertexInputState->vertexAttributeDescriptionCount;

		COPY_TO_BUFFER(MemoryStream, &CopyVertexInputState, sizeof(VkPipelineVertexInputStateCreateInfo));
		
		if(VertexInputState->vertexBindingDescriptionCount > 0)
		{
			uint32_t Length = sizeof(VkVertexInputBindingDescription) * VertexInputState->vertexBindingDescriptionCount;
			COPY_TO_BUFFER(MemoryStream, VertexInputState->pVertexBindingDescriptions, Length);
		}

		if(VertexInputState->vertexAttributeDescriptionCount > 0)
		{
			uint32_t Length = sizeof(VkVertexInputAttributeDescription) * VertexInputState->vertexAttributeDescriptionCount;
			COPY_TO_BUFFER(MemoryStream, VertexInputState->pVertexAttributeDescriptions, Length);
		}
	}

	if (pipelineCreateInfo.bHasVkPipelineInputAssemblyStateCreateInfo)
	{
		VkPipelineInputAssemblyStateCreateInfo InputAssemblyCreateInfo = *PipelineInfo->pInputAssemblyState;
		check(PipelineInfo->pInputAssemblyState->pNext == nullptr);

		InputAssemblyCreateInfo.pNext = nullptr;
		COPY_TO_BUFFER(MemoryStream, &InputAssemblyCreateInfo, sizeof(VkPipelineInputAssemblyStateCreateInfo));
	}

	if (pipelineCreateInfo.bHasVkPipelineTessellationStateCreateInfo)
	{
		VkPipelineTessellationStateCreateInfo TesselationCreateInfo = *PipelineInfo->pTessellationState;
		check(TesselationCreateInfo.pNext == nullptr);

		TesselationCreateInfo.pNext = nullptr;
		COPY_TO_BUFFER(MemoryStream, &TesselationCreateInfo, sizeof(VkPipelineTessellationStateCreateInfo));
	}

	if (pipelineCreateInfo.bHasVkPipelineViewportStateCreateInfo)
	{
		VkPipelineViewportStateCreateInfo ViewportState = *PipelineInfo->pViewportState;
		check(ViewportState.pNext == nullptr);

		ViewportState.pNext = nullptr;
		COPY_TO_BUFFER(MemoryStream, &ViewportState, sizeof(VkPipelineViewportStateCreateInfo));

		uint32_t ViewportCount = ViewportState.viewportCount && ViewportState.pViewports ? ViewportState.viewportCount : 0;
		COPY_TO_BUFFER(MemoryStream, &ViewportCount, sizeof(uint32_t));

		if (ViewportCount > 0)
		{
			COPY_TO_BUFFER(MemoryStream, ViewportState.pViewports, sizeof(VkViewport) * ViewportCount);
		}

		uint32_t ScissorCount = ViewportState.scissorCount && ViewportState.pScissors ? ViewportState.scissorCount : 0;
		COPY_TO_BUFFER(MemoryStream, &ScissorCount, sizeof(uint32_t));

		if (ScissorCount > 0)
		{
			COPY_TO_BUFFER(MemoryStream, ViewportState.pScissors, sizeof(VkRect2D) * ScissorCount);
		}
	}

	if (pipelineCreateInfo.bHasVkPipelineRasterizationStateCreateInfo)
	{
		VkPipelineRasterizationStateCreateInfo RasterizationState = *PipelineInfo->pRasterizationState;
		check(RasterizationState.pNext == nullptr);

		COPY_TO_BUFFER(MemoryStream, &RasterizationState, sizeof(VkPipelineRasterizationStateCreateInfo));
	}

	if (pipelineCreateInfo.bHasVkPipelineMultisampleStateCreateInfo)
	{
		VkPipelineMultisampleStateCreateInfo MultiSampleState = *PipelineInfo->pMultisampleState;
		check(MultiSampleState.pNext == nullptr);

		COPY_TO_BUFFER(MemoryStream, &MultiSampleState, sizeof(VkPipelineMultisampleStateCreateInfo));
	}

	if (pipelineCreateInfo.bHasVkPipelineDepthStencilStateCreateInfo)
	{
		VkPipelineDepthStencilStateCreateInfo DepthStencilState = *PipelineInfo->pDepthStencilState;
		check(DepthStencilState.pNext == nullptr);

		COPY_TO_BUFFER(MemoryStream, &DepthStencilState, sizeof(VkPipelineDepthStencilStateCreateInfo));
	}

	if (pipelineCreateInfo.bHasVkPipelineColorBlendStateCreateInfo)
	{
		VkPipelineColorBlendStateCreateInfo ColorBlendState = *PipelineInfo->pColorBlendState;
		check(ColorBlendState.pNext == nullptr);

		ColorBlendState.pAttachments = nullptr;

		COPY_TO_BUFFER(MemoryStream, &ColorBlendState, sizeof(VkPipelineColorBlendStateCreateInfo));

		if(ColorBlendState.attachmentCount > 0)
		{
			COPY_TO_BUFFER(MemoryStream, PipelineInfo->pColorBlendState->pAttachments, sizeof(VkPipelineColorBlendAttachmentState)* ColorBlendState.attachmentCount);
		}
	}

	if (pipelineCreateInfo.bHasVkPipelineDynamicStateCreateInfo)
	{
		VkPipelineDynamicStateCreateInfo DynamicState = *PipelineInfo->pDynamicState;
		check(DynamicState.pNext == nullptr);

		DynamicState.pDynamicStates = nullptr;

		COPY_TO_BUFFER(MemoryStream, &DynamicState, sizeof(VkPipelineDynamicStateCreateInfo));

		if (DynamicState.dynamicStateCount > 0)
		{
			COPY_TO_BUFFER(MemoryStream, PipelineInfo->pDynamicState->pDynamicStates, sizeof(VkDynamicState) * DynamicState.dynamicStateCount);
		}
	}

	VkPipelineLayoutCreateInfo PipelineLayout;
	FMemory::Memzero(PipelineLayout);

	PipelineLayout.sType = VK_STRUCTURE_TYPE_PIPELINE_LAYOUT_CREATE_INFO;
	PipelineLayout.setLayoutCount = GfxEntry->DescriptorSetLayoutBindings.Num();
	
	COPY_TO_BUFFER(MemoryStream, &PipelineLayout, sizeof(VkPipelineLayoutCreateInfo));

	for (uint32_t Idx = 0; Idx < GfxEntry->DescriptorSetLayoutBindings.Num(); ++Idx)
	{
		uint32_t SetBindingsCount = GfxEntry->DescriptorSetLayoutBindings[Idx].Num();
		COPY_TO_BUFFER(MemoryStream, &SetBindingsCount, sizeof(uint32_t));

		for (auto DescriptorSetBinding : GfxEntry->DescriptorSetLayoutBindings[Idx])
		{
			VkDescriptorSetLayoutBinding binding;
			binding.descriptorType = (VkDescriptorType)DescriptorSetBinding.DescriptorType;
			binding.binding = DescriptorSetBinding.Binding;
			binding.stageFlags = DescriptorSetBinding.StageFlags;
			binding.descriptorCount = 1;
			binding.pImmutableSamplers = 0;

			COPY_TO_BUFFER(MemoryStream, &binding, sizeof(VkDescriptorSetLayoutBinding));
		}
	}

	// Render pass
	bool bUseRenderPass2 = false;

	bUseRenderPass2 = Device->GetOptionalExtensions().HasKHRRenderPass2;

	COPY_TO_BUFFER(MemoryStream, &bUseRenderPass2, sizeof(bool));

#if VULKAN_SUPPORTS_RENDERPASS2
	if (bUseRenderPass2)
	{
		FVulkanRenderPassBuilder<FVulkanSubpassDescription<VkSubpassDescription2>, FVulkanSubpassDependency<VkSubpassDependency2>, FVulkanAttachmentReference<VkAttachmentReference2>, FVulkanAttachmentDescription<VkAttachmentDescription2>, FVulkanRenderPassCreateInfo<VkRenderPassCreateInfo2>> Creator(*Device);
	
		Creator.BuildCreateInfo(*RTLayout);

		FVulkanRenderPassCreateInfo<VkRenderPassCreateInfo2>& CreateInfo = Creator.GetCreateInfo();

		VkRenderPassCreateInfo2 RenderPassCreateInfo;
		FMemory::Memzero(RenderPassCreateInfo);

		RenderPassCreateInfo.sType = CreateInfo.sType;
		RenderPassCreateInfo.flags = CreateInfo.flags;
		RenderPassCreateInfo.attachmentCount = CreateInfo.attachmentCount;
		RenderPassCreateInfo.subpassCount = CreateInfo.subpassCount;
		RenderPassCreateInfo.dependencyCount = CreateInfo.dependencyCount;
		RenderPassCreateInfo.correlatedViewMaskCount = CreateInfo.correlatedViewMaskCount;

		COPY_TO_BUFFER(MemoryStream, &RenderPassCreateInfo, sizeof(VkRenderPassCreateInfo2));

		// Check for VK_STRUCTURE_TYPE_RENDER_PASS_FRAGMENT_DENSITY_MAP_CREATE_INFO_EXT
		bool bHasCreateInfoNext = RenderPassCreateInfo.pNext != nullptr;
		COPY_TO_BUFFER(MemoryStream, &bHasCreateInfoNext, sizeof(bool));

		if (bHasCreateInfoNext)
		{
			VkStructureType NextType = *(VkStructureType*)RenderPassCreateInfo.pNext;
			check(NextType == VK_STRUCTURE_TYPE_RENDER_PASS_FRAGMENT_DENSITY_MAP_CREATE_INFO_EXT);

			VkRenderPassFragmentDensityMapCreateInfoEXT* FragmentDensityMap = (VkRenderPassFragmentDensityMapCreateInfoEXT*)RenderPassCreateInfo.pNext;
			COPY_TO_BUFFER(MemoryStream, FragmentDensityMap, sizeof(VkRenderPassFragmentDensityMapCreateInfoEXT));

			check(FragmentDensityMap->pNext == nullptr);
		}

		if(RenderPassCreateInfo.attachmentCount > 0)
		{
			COPY_TO_BUFFER(MemoryStream, CreateInfo.pAttachments, sizeof(VkAttachmentDescription2) * RenderPassCreateInfo.attachmentCount);

			for (uint32_t AttachmentIndex = 0; AttachmentIndex < RenderPassCreateInfo.attachmentCount; ++AttachmentIndex)
			{
				HandleAttachmentDescriptionPNext(CreateInfo.pAttachments[AttachmentIndex], MemoryStream);
			}
		}

		if(RenderPassCreateInfo.dependencyCount > 0)
		{
			COPY_TO_BUFFER(MemoryStream, CreateInfo.pDependencies, sizeof(VkSubpassDependency2) * RenderPassCreateInfo.dependencyCount);
		}

		for (uint32_t Idx = 0; Idx < RenderPassCreateInfo.subpassCount; ++Idx)
		{
			VkSubpassDescription2 SubpassDescription = CreateInfo.pSubpasses[Idx];
			SubpassDescription.pColorAttachments = nullptr;
			SubpassDescription.pDepthStencilAttachment = nullptr;
			SubpassDescription.pPreserveAttachments = nullptr;
			SubpassDescription.pInputAttachments = nullptr;
			SubpassDescription.pResolveAttachments = nullptr;
			
			COPY_TO_BUFFER(MemoryStream, &SubpassDescription, sizeof(VkSubpassDescription2));

			HandleSubpassDescriptionPNext(&SubpassDescription, MemoryStream);

			if(SubpassDescription.colorAttachmentCount > 0)
			{
				for (uint32_t n = 0; n < SubpassDescription.colorAttachmentCount; ++n)
				{
					check(CreateInfo.pSubpasses[Idx].pColorAttachments[n].pNext == nullptr);
				}
				COPY_TO_BUFFER(MemoryStream, CreateInfo.pSubpasses[Idx].pColorAttachments, sizeof(VkAttachmentReference2) * SubpassDescription.colorAttachmentCount);
			}

			if(SubpassDescription.inputAttachmentCount > 0)
			{
				for (uint32_t n = 0; n < SubpassDescription.inputAttachmentCount; ++n)
				{
					check(CreateInfo.pSubpasses[Idx].pInputAttachments[n].pNext == nullptr);
				}
				COPY_TO_BUFFER(MemoryStream, CreateInfo.pSubpasses[Idx].pInputAttachments, sizeof(VkAttachmentReference2) * SubpassDescription.inputAttachmentCount);
			}

			bool bHasResolveAttachment = CreateInfo.pSubpasses[Idx].pResolveAttachments != nullptr;
			COPY_TO_BUFFER(MemoryStream, &bHasResolveAttachment, sizeof(bool));

			if (bHasResolveAttachment)
			{
				if(SubpassDescription.colorAttachmentCount > 0)
				{
					for (uint32_t n = 0; n < SubpassDescription.colorAttachmentCount; ++n)
					{
						check(CreateInfo.pSubpasses[Idx].pResolveAttachments[n].pNext == nullptr);
					}
					check(CreateInfo.pSubpasses[Idx].pResolveAttachments == nullptr);
					COPY_TO_BUFFER(MemoryStream, CreateInfo.pSubpasses[Idx].pResolveAttachments, sizeof(VkAttachmentReference2)* SubpassDescription.colorAttachmentCount);
				}
			}

			bool bHasDepthStencilAttachment = CreateInfo.pSubpasses[Idx].pDepthStencilAttachment != nullptr;
			COPY_TO_BUFFER(MemoryStream, &bHasDepthStencilAttachment, sizeof(bool));

			if (bHasDepthStencilAttachment)
			{
				HandleDepthStencilAttachmentPNext(CreateInfo.pSubpasses[Idx].pDepthStencilAttachment, MemoryStream);
				COPY_TO_BUFFER(MemoryStream, CreateInfo.pSubpasses[Idx].pDepthStencilAttachment, sizeof(VkAttachmentReference2));
			}
		}

		if(RenderPassCreateInfo.correlatedViewMaskCount > 0)
		{
			COPY_TO_BUFFER(MemoryStream, CreateInfo.pCorrelatedViewMasks, sizeof(uint32_t) * RenderPassCreateInfo.correlatedViewMaskCount);
		}
	}
	else
#endif
	{
		FVulkanRenderPassBuilder<FVulkanSubpassDescription<VkSubpassDescription>, FVulkanSubpassDependency<VkSubpassDependency>, FVulkanAttachmentReference<VkAttachmentReference>, FVulkanAttachmentDescription<VkAttachmentDescription>, FVulkanRenderPassCreateInfo<VkRenderPassCreateInfo>> Creator(*Device);
	
		Creator.BuildCreateInfo(*RTLayout);

		FVulkanRenderPassCreateInfo<VkRenderPassCreateInfo>& CreateInfo = Creator.GetCreateInfo();

		VkRenderPassCreateInfo RenderPassCreateInfo;
		FMemory::Memzero(RenderPassCreateInfo);

		RenderPassCreateInfo.sType = CreateInfo.sType;
		RenderPassCreateInfo.flags = CreateInfo.flags;
		RenderPassCreateInfo.attachmentCount = CreateInfo.attachmentCount;
		RenderPassCreateInfo.subpassCount = CreateInfo.subpassCount;
		RenderPassCreateInfo.dependencyCount = CreateInfo.dependencyCount;

		COPY_TO_BUFFER(MemoryStream, &RenderPassCreateInfo, sizeof(VkRenderPassCreateInfo));

		bool bHasCreateInfoNext = RenderPassCreateInfo.pNext != nullptr;
		COPY_TO_BUFFER(MemoryStream, &bHasCreateInfoNext, sizeof(bool));

		if (bHasCreateInfoNext)
		{
			VkStructureType NextType = *(VkStructureType*)RenderPassCreateInfo.pNext;
			check(NextType == VK_STRUCTURE_TYPE_RENDER_PASS_FRAGMENT_DENSITY_MAP_CREATE_INFO_EXT);

			VkRenderPassFragmentDensityMapCreateInfoEXT* FragmentDensityMap = (VkRenderPassFragmentDensityMapCreateInfoEXT*)RenderPassCreateInfo.pNext;
			COPY_TO_BUFFER(MemoryStream, FragmentDensityMap, sizeof(VkRenderPassFragmentDensityMapCreateInfoEXT));

			check(FragmentDensityMap->pNext == nullptr);
			// TODO: Support Multiview create info
		} 

		if(RenderPassCreateInfo.attachmentCount > 0)
		{
			COPY_TO_BUFFER(MemoryStream, CreateInfo.pAttachments, sizeof(VkAttachmentDescription) * RenderPassCreateInfo.attachmentCount);
		}

		if(RenderPassCreateInfo.dependencyCount > 0)
		{
			COPY_TO_BUFFER(MemoryStream, CreateInfo.pDependencies, sizeof(VkSubpassDependency) * RenderPassCreateInfo.dependencyCount);
		}

		for (uint32_t Idx = 0; Idx < RenderPassCreateInfo.subpassCount; ++Idx)
		{
			VkSubpassDescription SubpassDescription = CreateInfo.pSubpasses[Idx];
			SubpassDescription.pColorAttachments = nullptr;
			SubpassDescription.pDepthStencilAttachment = nullptr;
			SubpassDescription.pPreserveAttachments = nullptr;
			SubpassDescription.pInputAttachments = nullptr;
			SubpassDescription.pResolveAttachments = nullptr;
			
			COPY_TO_BUFFER(MemoryStream, &SubpassDescription, sizeof(VkSubpassDescription));

			if(SubpassDescription.colorAttachmentCount > 0)
			{
				COPY_TO_BUFFER(MemoryStream, CreateInfo.pSubpasses[Idx].pColorAttachments, sizeof(VkAttachmentReference) * SubpassDescription.colorAttachmentCount);
			}

			if(SubpassDescription.inputAttachmentCount > 0)
			{
				COPY_TO_BUFFER(MemoryStream, CreateInfo.pSubpasses[Idx].pInputAttachments, sizeof(VkAttachmentReference) * SubpassDescription.inputAttachmentCount);
			}

			bool bHasResolveAttachment = CreateInfo.pSubpasses[Idx].pResolveAttachments != nullptr;
			COPY_TO_BUFFER(MemoryStream, &bHasResolveAttachment, sizeof(bool));

			if (bHasResolveAttachment)
			{
				if(SubpassDescription.colorAttachmentCount > 0)
				{
					COPY_TO_BUFFER(MemoryStream, CreateInfo.pSubpasses[Idx].pResolveAttachments, sizeof(VkAttachmentReference)* SubpassDescription.colorAttachmentCount);
				}
			}

			bool bHasDepthStencilAttachment = CreateInfo.pSubpasses[Idx].pDepthStencilAttachment != nullptr;
			COPY_TO_BUFFER(MemoryStream, &bHasDepthStencilAttachment, sizeof(bool));

			if (bHasDepthStencilAttachment)
			{
				COPY_TO_BUFFER(MemoryStream, CreateInfo.pSubpasses[Idx].pDepthStencilAttachment, sizeof(VkAttachmentReference));
			}
		}
	}
}

#if UE_BUILD_SHIPPING
#define CHECK_JNI_EXCEPTIONS(env)  env->ExceptionClear();
#else
#define CHECK_JNI_EXCEPTIONS(env)  if (env->ExceptionCheck()) {env->ExceptionDescribe();env->ExceptionClear();}
#endif

namespace AndroidVulkanService
{
	std::atomic<bool> GRemoteCompileServicesStarted = false;
	std::atomic<bool> GRemoteCompileServicesActive = false;
	std::atomic<bool> bOneTimeErrorEncountered = false;
	std::atomic<int> TotalErrors = 0;
}

struct FVKRemoteProgramCompileJNI
{
	jclass PSOServiceAccessor = 0;
	jmethodID DispatchPSOCompile = 0;
	jmethodID DispatchPSOCompileShm = 0;
	jmethodID StartRemoteProgramLink = 0;
	jmethodID HaveServicesFailed = 0;
	jmethodID AreProgramServicesReady = 0;	
	jmethodID StopRemoteProgramLink = 0;
	jmethodID SetProgramInitDataRetainLimit = 0;
	jmethodID GetRetainedProgramInitDataPath = 0;
	jclass ProgramResponseClass = 0;
	jfieldID ProgramResponse_SuccessField = 0;
	jfieldID ProgramResponse_ErrorField = 0;
	jfieldID ProgramResponse_SHMOutputHandleField = 0;
	jfieldID ProgramResponse_CompiledBinaryField = 0;
	jfieldID ProgramResponse_CompilationDurationField = 0;
	bool bAllFound = false;

	void Init(JNIEnv* Env)
	{
		// class JNIProgramLinkResponse
		// {
		// 	boolean bCompileSuccess;
		// 	String ErrorMessage;
		// 	byte[] CompiledProgram;
		// };
		// JNIProgramLinkResponse AndroidThunkJava_OGLRemoteProgramLink(...):

		if (PSOServiceAccessor)
		{
			return;
		}

		check(PSOServiceAccessor == 0);
		PSOServiceAccessor = AndroidJavaEnv::FindJavaClassGlobalRef("com/epicgames/unreal/psoservices/PSOProgramServiceAccessor");
		CHECK_JNI_EXCEPTIONS(Env);
		if (PSOServiceAccessor)
		{
			DispatchPSOCompile = FJavaWrapper::FindStaticMethod(Env, PSOServiceAccessor, "AndroidThunkJava_VKPSOGFXCompile", "([BJ[B[B[B[BZ)Lcom/epicgames/unreal/psoservices/PSOProgramServiceAccessor$JNIProgramLinkResponse;", false);
			CHECK_JNI_EXCEPTIONS(Env);
			DispatchPSOCompileShm = FJavaWrapper::FindStaticMethod(Env, PSOServiceAccessor, "AndroidThunkJava_VKPSOGFXCompileShm", "([BJIJJJJZ)Lcom/epicgames/unreal/psoservices/PSOProgramServiceAccessor$JNIProgramLinkResponse;", false);
			CHECK_JNI_EXCEPTIONS(Env);
			StartRemoteProgramLink = FJavaWrapper::FindStaticMethod(Env, PSOServiceAccessor, "AndroidThunkJava_StartRemoteProgramLink", "(IZZ)Z", false);
			CHECK_JNI_EXCEPTIONS(Env);
			HaveServicesFailed = FJavaWrapper::FindStaticMethod(Env, PSOServiceAccessor, "AndroidThunkJava_HaveServicesFailed", "()Z", false);
			CHECK_JNI_EXCEPTIONS(Env);
			AreProgramServicesReady = FJavaWrapper::FindStaticMethod(Env, PSOServiceAccessor, "AndroidThunkJava_AreProgramServicesReady", "()Z", false);
			CHECK_JNI_EXCEPTIONS(Env);
			StopRemoteProgramLink = FJavaWrapper::FindStaticMethod(Env, PSOServiceAccessor, "AndroidThunkJava_StopRemoteProgramLink", "()V", false);
			CHECK_JNI_EXCEPTIONS(Env);
			SetProgramInitDataRetainLimit = FJavaWrapper::FindStaticMethod(Env, PSOServiceAccessor, "AndroidThunkJava_SetProgramInitDataRetainLimit", "(I)V", false);
			CHECK_JNI_EXCEPTIONS(Env);
			GetRetainedProgramInitDataPath = FJavaWrapper::FindStaticMethod(Env, PSOServiceAccessor, "AndroidThunkJava_GetRetainedProgramInitDataPath", "()Ljava/lang/String;", false);
			CHECK_JNI_EXCEPTIONS(Env);
			ProgramResponseClass = AndroidJavaEnv::FindJavaClassGlobalRef("com/epicgames/unreal/psoservices/PSOProgramServiceAccessor$JNIProgramLinkResponse");
			CHECK_JNI_EXCEPTIONS(Env);
			ProgramResponse_SuccessField = FJavaWrapper::FindField(Env, ProgramResponseClass, "bCompileSuccess", "Z", true);
			CHECK_JNI_EXCEPTIONS(Env);
			ProgramResponse_CompiledBinaryField = FJavaWrapper::FindField(Env, ProgramResponseClass, "CompiledProgram", "[B", true);
			CHECK_JNI_EXCEPTIONS(Env);
			ProgramResponse_ErrorField = FJavaWrapper::FindField(Env, ProgramResponseClass, "ErrorMessage", "Ljava/lang/String;", true);
			CHECK_JNI_EXCEPTIONS(Env);
			ProgramResponse_SHMOutputHandleField = FJavaWrapper::FindField(Env, ProgramResponseClass, "SHMOutputHandle", "I", true);
			CHECK_JNI_EXCEPTIONS(Env);
			ProgramResponse_CompilationDurationField = FJavaWrapper::FindField(Env, ProgramResponseClass, "CompilationDuration", "F", true);
			CHECK_JNI_EXCEPTIONS(Env);
		}

		bAllFound = PSOServiceAccessor && DispatchPSOCompile && DispatchPSOCompileShm && StartRemoteProgramLink && HaveServicesFailed && AreProgramServicesReady && StopRemoteProgramLink && SetProgramInitDataRetainLimit && GetRetainedProgramInitDataPath
			&& ProgramResponseClass && ProgramResponse_SuccessField && ProgramResponse_CompiledBinaryField && ProgramResponse_ErrorField && ProgramResponse_SHMOutputHandleField && ProgramResponse_CompilationDurationField;
		UE_CLOGF(!bAllFound, LogRHI, Fatal, "Failed to find JNI Vulkan remote program compiler.");
	}
}VKRemoteProgramCompileJNI;

static bool AreAndroidVulkanRemoteCompileServicesAvailable()
{
	static int RemoteCompileService = -1;
	if (RemoteCompileService == -1)
	{
		const FString* ConfigRulesDisableProgramCompileServices = FAndroidMisc::GetConfigRulesVariable(TEXT("DisableProgramCompileServices"));
		bool bConfigRulesDisableProgramCompileServices = ConfigRulesDisableProgramCompileServices && ConfigRulesDisableProgramCompileServices->Equals("true", ESearchCase::IgnoreCase);
		static const auto CVarProgramLRU = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Vulkan.EnablePipelineLRUCache"));
		static const auto CVarNumRemoteProgramCompileServices = IConsoleManager::Get().FindConsoleVariable(TEXT("Android.Vulkan.NumRemoteProgramCompileServices"));

		RemoteCompileService = !bConfigRulesDisableProgramCompileServices && VKRemoteProgramCompileJNI.bAllFound && (CVarProgramLRU->GetInt() != 0) && (CVarNumRemoteProgramCompileServices->GetInt() > 0);
		FGenericCrashContext::SetEngineData(TEXT("Android.PSOService"), RemoteCompileService == 0 ? TEXT("disabled") : TEXT("enabled"));

		UE_LOGF(LogRHI, Log, "External PSO compilers = %ls", RemoteCompileService == 0 ? TEXT("disabled") : TEXT("enabled"));
	}
	return RemoteCompileService;
}

bool FVulkanAndroidPlatform::AreRemoteCompileServicesActive()
{
	// The services could be stopped at any point elsewhere, the return value is not guaranteed to be correct.
	// it does not need to be exact as the PSO service will reject any new requests after service stop has been encountered.
	// any existing PSOservice jobs will complete as normal.
	if (AndroidVulkanService::GRemoteCompileServicesStarted && AreAndroidVulkanRemoteCompileServicesAvailable())
	{
		if (!AndroidVulkanService::GRemoteCompileServicesActive)
		{
			JNIEnv* Env = FAndroidApplication::GetJavaEnv();
			AndroidVulkanService::GRemoteCompileServicesActive = (bool)Env->CallStaticBooleanMethod(VKRemoteProgramCompileJNI.PSOServiceAccessor, VKRemoteProgramCompileJNI.AreProgramServicesReady);
			if (!AndroidVulkanService::GRemoteCompileServicesActive)
			{
				if ((bool)Env->CallStaticBooleanMethod(VKRemoteProgramCompileJNI.PSOServiceAccessor, VKRemoteProgramCompileJNI.HaveServicesFailed))
				{
					UE_LOGF(LogRHI, Error, "Remote compile services failed to start.");
					StopRemoteCompileServices();
				}
			}
			else
			{
				UE_LOGF(LogRHI, Log, "Remote compile services are active.");
			}
		}
		return AndroidVulkanService::GRemoteCompileServicesActive;
	}
	return false;
}


namespace AndroidVulkan
{
	namespace PSOCapture
	{
#if WITH_ADDITIONAL_CRASH_CONTEXTS
		int32 GPSOInitHistoryLimit = 0;
		static FAutoConsoleVariableRef CVarVulkanExtensionFramePacer(
			TEXT("android.Vulkan.RetainLimitForPSOCreateCrashReport"),
			GPSOInitHistoryLimit,
			TEXT("Retain the PSO creation source data from the external compiler services when > 0. If a crash occurs the data is added to the crash report.\n")
			TEXT(" 0 to disable (default)\n")
			TEXT(" >0 == amount of history to retain per service."),
			ECVF_RenderThreadSafe
		);

		static FString PSOCrashHistoryDir;

		static void OnCrashDumpPSOHistoryInfo(FCrashContextExtendedWriter& Writer)
		{
			if (GPSOInitHistoryLimit > 0 && FVulkanAndroidPlatform::AreRemoteCompileServicesActive())
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("OnCrashDumpPSOHistoryInfo"));
				IPlatformFile::GetPlatformPhysical().IterateDirectory(*PSOCrashHistoryDir, [&Writer](const TCHAR* Pathname, bool bIsDirectory)
					{
						if(!bIsDirectory)
						{
							FPlatformMisc::LowLevelOutputDebugStringf(TEXT("OnCrashDumpPSOHistoryInfo: %s -> %s"), Pathname, *FPaths::GetCleanFilename(Pathname));
							Writer.MoveFile(Pathname);
						}
						return true;
					});
			}
		}
#endif
		static void InitCrashDataCapture()
		{
#if WITH_ADDITIONAL_CRASH_CONTEXTS
			UE_LOGF(LogRHI, Log, "Retaining remote compile services history, limit = %d", AndroidVulkan::PSOCapture::GPSOInitHistoryLimit);

			if (AndroidVulkan::PSOCapture::GPSOInitHistoryLimit > 0)
			{
				check(IsInRenderingThread());
				check(PSOCrashHistoryDir.IsEmpty());
				JNIEnv* Env = FAndroidApplication::GetJavaEnv();
				Env->CallStaticVoidMethod(VKRemoteProgramCompileJNI.PSOServiceAccessor, VKRemoteProgramCompileJNI.SetProgramInitDataRetainLimit, (jint)AndroidVulkan::PSOCapture::GPSOInitHistoryLimit);
				PSOCrashHistoryDir = FJavaHelper::FStringFromLocalRef(Env, (jstring)Env->CallStaticObjectMethod(VKRemoteProgramCompileJNI.PSOServiceAccessor, VKRemoteProgramCompileJNI.GetRetainedProgramInitDataPath));
				CHECK_JNI_EXCEPTIONS(Env);
			}
#endif
		}
	}
}                                                                                                                                                                                                                                                                                                                                              

#if WITH_ADDITIONAL_CRASH_CONTEXTS
void FVulkanAndroidPlatform::OnCreatePipelineCrash(FCrashContextExtendedWriter& Writer, TConstArrayView<FVulkanShader*> Shaders, TStringView<TCHAR> ExtraInfo, bool bIsPrecache)
{
	if(bIsPrecache)
	{
		// we've crashed during PSO compile of a precache PSO, dump all the retained history.
		// just do this once.
		static std::atomic_bool bDoneOnce = false;
		if (bDoneOnce.exchange(true) == false)
		{
			AndroidVulkan::PSOCapture::OnCrashDumpPSOHistoryInfo(Writer);
		}
	}

	// Call generic version to write out shader hashes
	FVulkanGenericPlatform::OnCreatePipelineCrash(Writer, Shaders, ExtraInfo, bIsPrecache);
}
#endif //WITH_ADDITIONAL_CRASH_CONTEXTS

bool FVulkanAndroidPlatform::StartRemoteCompileServices(int NumServices)
{
	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	
	VKRemoteProgramCompileJNI.Init(Env);

	if (Env && AreAndroidVulkanRemoteCompileServicesAvailable() && !AndroidVulkanService::GRemoteCompileServicesStarted)
	{
		AndroidVulkanService::GRemoteCompileServicesStarted = (bool)Env->CallStaticBooleanMethod(VKRemoteProgramCompileJNI.PSOServiceAccessor, VKRemoteProgramCompileJNI.StartRemoteProgramLink, (jint)NumServices, /*bUseRobustEGLContext*/(jboolean)false, /*bUseVulkan*/(jboolean)true);

		AndroidVulkan::PSOCapture::InitCrashDataCapture();
	}

	return AndroidVulkanService::GRemoteCompileServicesStarted;
}

void FVulkanAndroidPlatform::StopRemoteCompileServices()
{
	bool bExpected = true;
	if(AndroidVulkanService::GRemoteCompileServicesStarted.compare_exchange_strong(bExpected, false) )
	{
		UE_LOGF(LogVulkanRHI, Log, "Stopping Remote Compile Services");
		AndroidVulkanService::GRemoteCompileServicesActive = false;
		JNIEnv* Env = FAndroidApplication::GetJavaEnv();

		if (Env && ensure(AreAndroidVulkanRemoteCompileServicesAvailable()))
		{
			Env->CallStaticVoidMethod(VKRemoteProgramCompileJNI.PSOServiceAccessor, VKRemoteProgramCompileJNI.StopRemoteProgramLink);
		}
	}
}

FString FVulkanAndroidPlatform::GetCompiledPSOCacheTopFolderPath()
{
#if USE_ANDROID_FILE
	extern FString GExternalFilePath;
	return GExternalFilePath / TEXT("VulkanProgramBinaryCache");
#else
	return FVulkanGenericPlatform::GetCompiledPSOCacheTopFolderPath();
#endif
}

VkPipelineCache FVulkanAndroidPlatform::PrecompilePSO(
		FVulkanDevice* Device, 
		const TArrayView<const uint8> ContextData,
		const TArrayView<const uint8> OptionalPSOCacheData, 
		FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType PSOCompileType,
		const VkGraphicsPipelineCreateInfo* PipelineInfo, 
		const FGfxPipelineDesc* GfxEntry, 
		const FVulkanRenderTargetLayout* RTLayout, 
		TArrayView<const uint32_t> VS, 
		TArrayView<const uint32_t> PS, 
		size_t& AfterSize,
		FString* FailureMessageOUT
	)
{
	if (!ensure(AreRemoteCompileServicesActive()))
	{
		return VK_NULL_HANDLE;
	}
	QUICK_SCOPE_CYCLE_COUNTER(STAT_VulkanAndroid_PrecompilePSO);

	auto SetFailureMessage = [FailureMessageOUT](FString&& Msg) { if (FailureMessageOUT) { *FailureMessageOUT = Msg; }};

	VkPipelineCache ReturnPipelineCache = VK_NULL_HANDLE;
	FPlatformDynamicRHI::FPSOServicePriInfo PriorityInfo(PSOCompileType);

// 	FScopedDurationTimeLogger Timer(TEXT("FVulkanAndroidPlatform::PrecompilePSO"));

	TArray<char> MemoryStream;
	PipelineToBinary(Device, PipelineInfo, GfxEntry, RTLayout, MemoryStream);

	bool bResult = false;
	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	FString ErrorMessage;

	if (Env && ensure(VKRemoteProgramCompileJNI.bAllFound))
	{
		// In this version we pass all data via shared buffer, offsets are still supplied via args
		auto ArrayByteSize = [](const auto& ArrayToSize) {return ArrayToSize.GetTypeSize() * ArrayToSize.Num(); };

		const uint64 VSSize = ArrayByteSize(VS);
		const uint64 PSSize = ArrayByteSize(PS);
		const uint64 PSOParamsSize = ArrayByteSize(MemoryStream);
		const uint64 PreSuppliedCacheSize = ArrayByteSize(OptionalPSOCacheData);
		const uint64 TotalOutputSize = VSSize + PSSize + PSOParamsSize + PreSuppliedCacheSize;

		auto ContextJavaArray = NewScopedJavaObject(Env, Env->NewByteArray(ContextData.Num()));
		Env->SetByteArrayRegion(*ContextJavaArray, 0, ContextData.Num(), reinterpret_cast<const jbyte*>(ContextData.GetData()));
		CHECK_JNI_EXCEPTIONS(Env);

		// create a shared mem region for external process to access.
		FScopedJavaObject<_jobject*> ProgramResponseObj;
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_VulkanAndroid_PrecompilePSO1);

			const uint64 TotalSHMSize = TotalOutputSize;
			const uint64 TotalSHMSizeAligned = Align(TotalSHMSize, FPlatformMemory::GetConstants().PageSize);
			int SharedMemFD = ASharedMemory_create("", TotalSHMSizeAligned);
			if (ensure(SharedMemFD > -1))
			{
				// By default it has PROT_READ | PROT_WRITE | PROT_EXEC.
				size_t memSize = ASharedMemory_getSize(SharedMemFD);
				char* SharedBuffer = (char*)mmap(NULL, memSize, PROT_READ | PROT_WRITE, MAP_SHARED, SharedMemFD, 0);
				if (ensure(SharedBuffer))
				{
					char* AppendPtr = SharedBuffer;
					auto AppendBuffer = [&AppendPtr,&SharedBuffer,&TotalSHMSize](const auto& AppendMe)
					{
						const size_t NumBytes = AppendMe.Num() * AppendMe.GetTypeSize();
						if( ensure(AppendPtr>=SharedBuffer && ((AppendPtr+NumBytes) <= (SharedBuffer+TotalSHMSize))) )
						{
							FMemory::Memcpy(AppendPtr, (const char*)AppendMe.GetData(), NumBytes);
							AppendPtr += NumBytes;
						}
					};

					AppendBuffer(VS);
					AppendBuffer(PS);
					AppendBuffer(MemoryStream);
					AppendBuffer(OptionalPSOCacheData);

					// limit access to read only
					ASharedMemory_setProt(SharedMemFD, PROT_READ);

					// dont time out if the debugger is attached.
					bool bEnableTimeOuts = !FPlatformMisc::IsDebuggerPresent();
					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_VulkanAndroid_PrecompilePSOJAVA);
						ProgramResponseObj = NewScopedJavaObject(Env, Env->CallStaticObjectMethod(VKRemoteProgramCompileJNI.PSOServiceAccessor, VKRemoteProgramCompileJNI.DispatchPSOCompileShm, *ContextJavaArray, PriorityInfo.GetPriorityInfo(), SharedMemFD, VSSize, PSSize, PSOParamsSize, PreSuppliedCacheSize, bEnableTimeOuts));
					}
					CHECK_JNI_EXCEPTIONS(Env);
					munmap(SharedBuffer, memSize);
				}
				else
				{
					UE_LOGF(LogRHI, Error, "Failed to alloc %lld bytes for external PSO compile: %d",
						TotalSHMSizeAligned,
						errno
						);
				}
				close(SharedMemFD);
			}
			else
			{
				UE_LOGF(LogRHI, Error, "Failed to alloc %lld bytes for external PSO compile: %d (%ls)",
					TotalSHMSizeAligned,
					errno,
					(errno == EMFILE) ? TEXT("too many open file descriptors") : TEXT("unknown")
				);
			}
		}

		if (ProgramResponseObj)
		{
			const bool bSucceeded = (bool)Env->GetBooleanField(*ProgramResponseObj, VKRemoteProgramCompileJNI.ProgramResponse_SuccessField);
			if (bSucceeded)
			{
				const int ProgramResultSharedHandle = Env->GetIntField(*ProgramResponseObj, VKRemoteProgramCompileJNI.ProgramResponse_SHMOutputHandleField);
				const float ProgramResultCompilationDuration = Env->GetFloatField(*ProgramResponseObj, VKRemoteProgramCompileJNI.ProgramResponse_CompilationDurationField);
				AccumulatePSOMetrics(ProgramResultCompilationDuration);
				if(ensure(ProgramResultSharedHandle > -1))
				{
					const uint32 ResultMemSize = (uint32)ASharedMemory_getSize(ProgramResultSharedHandle);
					char* ResultSharedBuffer = (char*)mmap(NULL, ResultMemSize, PROT_READ, MAP_SHARED, ProgramResultSharedHandle, 0);
					ON_SCOPE_EXIT{ if (ResultSharedBuffer) { munmap(ResultSharedBuffer, ResultMemSize); } close(ProgramResultSharedHandle); };
					if (ensure(ResultSharedBuffer))
					{
						// Actual size of data is in the first 4 bytes.
						const uint32 ResultSize = *(uint32*)ResultSharedBuffer;

						if (ensure(ResultMemSize > 0))
						{
							QUICK_SCOPE_CYCLE_COUNTER(STAT_VulkanAndroid_PrecompilePSOCreateCache);
							VkPipelineCacheCreateInfo PipelineCacheCreateInfo;
							memset(&PipelineCacheCreateInfo, 0, sizeof(VkPipelineCacheCreateInfo));
							PipelineCacheCreateInfo.sType = VK_STRUCTURE_TYPE_PIPELINE_CACHE_CREATE_INFO;
							PipelineCacheCreateInfo.flags = 0;
							PipelineCacheCreateInfo.pInitialData = ResultSharedBuffer + sizeof(ResultMemSize);
							PipelineCacheCreateInfo.initialDataSize = ResultSize;
							VERIFYVULKANRESULT(VulkanRHI::vkCreatePipelineCache(Device->GetHandle(), &PipelineCacheCreateInfo, VULKAN_CPU_ALLOCATOR, &ReturnPipelineCache));
							AfterSize = ResultSize - OptionalPSOCacheData.Num();
						}
					}
				}
			}
			else
			{
				if (AndroidVulkanService::bOneTimeErrorEncountered.exchange(true) == false)
				{
					FGenericCrashContext::SetEngineData(TEXT("Android.PSOService"), TEXT("ec"));
				}

				SetFailureMessage( FJavaHelper::FStringFromLocalRef(Env, (jstring)Env->GetObjectField(*ProgramResponseObj, VKRemoteProgramCompileJNI.ProgramResponse_ErrorField)));
			}
		}
		else
		{
			if (AndroidVulkanService::bOneTimeErrorEncountered.exchange(true) == false)
			{
				FGenericCrashContext::SetEngineData(TEXT("Android.PSOService"), TEXT("es"));
			}
			SetFailureMessage(TEXT("Remote PSO compiler failed."));
		}
	}
	else
	{
		if (AndroidVulkanService::bOneTimeErrorEncountered.exchange(true) == false)
		{
			FGenericCrashContext::SetEngineData(TEXT("Android.PSOService"), TEXT("ejni"));
		}
		SetFailureMessage(TEXT("Remote PSO compiler JNI error."));
	}

	if (ReturnPipelineCache == VK_NULL_HANDLE)
	{
		if ((AndroidVulkanService::TotalErrors++) == FPlatformDynamicRHI::GetPSOServiceFailureThreshold())
		{
			FVulkanAndroidPlatform::StopRemoteCompileServices();
			SetFailureMessage(TEXT("Remote PSO compiler failed, error count has passed threshold. Future compiles will be in-process."));
		}
	}
	return ReturnPipelineCache;
}

void FAndroidVulkanFramePacer::Init()
{
#if USE_ANDROID_VULKAN_SWAPPY
	if (FAndroidPlatformRHIFramePacer::CVarUseSwappyForFramePacing.GetValueOnAnyThread() != 0)
	{
		FVulkanAndroidPlatform::bSwappyEnabledAtRHIInit = true;
		extern void LoadSwappy();
		LoadSwappy();
	}
#if !UE_BUILD_SHIPPING
	FAndroidPlatformRHIFramePacer::CVarUseSwappyForFramePacing.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		UE_LOGF(LogRHI, Warning, "In Vulkan mode, changing a.UseSwappyForFramePacing after the RHI is initialized has no effect. Swappy is %ls.", FVulkanAndroidPlatform::bSwappyEnabledAtRHIInit ? TEXT("Enabledd") : TEXT("Disabled"));
	}));
#endif
#endif
}

bool FAndroidVulkanFramePacer::SupportsFramePace(int32 QueryFramePace)
{
	int32 TempRefreshRate, TempSyncInterval;
	return SupportsFramePaceInternal(QueryFramePace, TempRefreshRate, TempSyncInterval);
}

void FVulkanAndroidPlatform::PostInitGPU(const FVulkanDevice& InDevice)
{
	SetupImageMemoryRequirementWorkaround(InDevice);
	SetupRequiresDepthStencilFullWriteWorkaround(InDevice);
	SetupRequiresMergedDepthStencilLayoutWorkaround(InDevice);

	// The Vulkan specification does not provide concrete guidance on anisotropic filtering for 3D textures, 
	// and on some Android GPUs this results in undefined behavior
	GRHIGlobals.SupportsTexture3DAnisotropicFiltering = false;

	// start external compilers if precaching is specified.	
	static const auto CVarChunkedPSOCache = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Vulkan.UseChunkedPSOCache"));
	static const auto CVarNumRemoteProgramCompileServices = IConsoleManager::Get().FindConsoleVariable(TEXT("Android.Vulkan.NumRemoteProgramCompileServices"));
	static const auto CVarPSOPrecaching = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PSOPrecaching"));
	static const auto CVarVulkanPSOPrecaching = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Vulkan.AllowPSOPrecaching"));	
	if (CVarNumRemoteProgramCompileServices->GetInt() && CVarChunkedPSOCache->GetInt() && CVarPSOPrecaching->GetInt() && CVarVulkanPSOPrecaching->GetInt())
	{
		FVulkanAndroidPlatform::StartRemoteCompileServices(CVarNumRemoteProgramCompileServices->GetInt());
	}
}

//
// Test whether we should enable workarounds for textures
// Arm GPUs use an optimization "Arm FrameBuffer Compression - AFBC" that can significanly inflate (~5x) uncompressed texture memory requirements
// For now AFBC and similar optimizations can be disabled by using VK_IMAGE_CREATE_MUTABLE_FORMAT_BIT or VK_IMAGE_USAGE_STORAGE_BIT flags on a texture
// On Adreno GPUs ASTC textures with optimial tiling may require 8x more memory
//
void FVulkanAndroidPlatform::SetupImageMemoryRequirementWorkaround(const FVulkanDevice& InDevice)
{
	AFBCWorkaroundOption = 0;
	ASTCWorkaroundOption = 0;

	VkImageCreateInfo ImageCreateInfo;
	ZeroVulkanStruct(ImageCreateInfo, VK_STRUCTURE_TYPE_IMAGE_CREATE_INFO);
	ImageCreateInfo.imageType = VK_IMAGE_TYPE_2D;
	ImageCreateInfo.samples = VK_SAMPLE_COUNT_1_BIT;
	ImageCreateInfo.format = VK_FORMAT_B8G8R8A8_UNORM;
	ImageCreateInfo.tiling = VK_IMAGE_TILING_OPTIMAL;
	ImageCreateInfo.arrayLayers = 1;
	ImageCreateInfo.extent = {128, 128, 1};
	ImageCreateInfo.mipLevels = 8;
	ImageCreateInfo.flags = 0;
	ImageCreateInfo.usage = VK_IMAGE_USAGE_TRANSFER_DST_BIT | VK_IMAGE_USAGE_SAMPLED_BIT;
	ImageCreateInfo.sharingMode = VK_SHARING_MODE_EXCLUSIVE;
	ImageCreateInfo.queueFamilyIndexCount = 0;
	ImageCreateInfo.pQueueFamilyIndices = nullptr;
	ImageCreateInfo.initialLayout = VK_IMAGE_LAYOUT_UNDEFINED;

	// AFBC workarounds
	{
		const VkFormatFeatureFlags FormatFlags = InDevice.GetFormatProperties(VK_FORMAT_B8G8R8A8_UNORM).optimalTilingFeatures;

		VkImage Image0;
		VkMemoryRequirements Image0Mem;
		VERIFYVULKANRESULT(VulkanRHI::vkCreateImage(InDevice.GetHandle(), &ImageCreateInfo, VULKAN_CPU_ALLOCATOR, &Image0));
		VulkanRHI::vkGetImageMemoryRequirements(InDevice.GetHandle(), Image0, &Image0Mem);
		VulkanRHI::vkDestroyImage(InDevice.GetHandle(), Image0, VULKAN_CPU_ALLOCATOR);

		VkImage ImageMutable;
		VkMemoryRequirements ImageMutableMem;
		ImageCreateInfo.flags = VK_IMAGE_CREATE_MUTABLE_FORMAT_BIT;
		VERIFYVULKANRESULT(VulkanRHI::vkCreateImage(InDevice.GetHandle(), &ImageCreateInfo, VULKAN_CPU_ALLOCATOR, &ImageMutable));
		VulkanRHI::vkGetImageMemoryRequirements(InDevice.GetHandle(), ImageMutable, &ImageMutableMem);
		VulkanRHI::vkDestroyImage(InDevice.GetHandle(), ImageMutable, VULKAN_CPU_ALLOCATOR);

		VkImage ImageStorage;
		VkMemoryRequirements ImageStorageMem;
		if ((FormatFlags & VK_FORMAT_FEATURE_STORAGE_IMAGE_BIT) != 0)
		{
			ImageCreateInfo.usage |= VK_IMAGE_USAGE_STORAGE_BIT;
			ImageCreateInfo.flags = 0;
		}
		VERIFYVULKANRESULT(VulkanRHI::vkCreateImage(InDevice.GetHandle(), &ImageCreateInfo, VULKAN_CPU_ALLOCATOR, &ImageStorage));
		VulkanRHI::vkGetImageMemoryRequirements(InDevice.GetHandle(), ImageStorage, &ImageStorageMem);
		VulkanRHI::vkDestroyImage(InDevice.GetHandle(), ImageStorage, VULKAN_CPU_ALLOCATOR);

		const float MEM_SIZE_THRESHOLD = 1.5f;
		const float IMAGE0_SIZE = (float)Image0Mem.size;

		if (ImageMutableMem.size * MEM_SIZE_THRESHOLD < IMAGE0_SIZE)
		{
			AFBCWorkaroundOption = 1;
		}
		else if (ImageStorageMem.size * MEM_SIZE_THRESHOLD < IMAGE0_SIZE)
		{
			AFBCWorkaroundOption = 2;
		}

		if (AFBCWorkaroundOption != 0)
		{
			UE_LOGF(LogRHI, Display, "Enabling workaround to reduce memory requirement for BGRA textures (%ls flag). 128x128 - 8 Mips BGRA texture: %llu KiB -> %llu KiB",
				AFBCWorkaroundOption == 1 ? TEXT("MUTABLE") : TEXT("STORAGE"),
				Image0Mem.size / 1024,
				AFBCWorkaroundOption == 1 ? ImageMutableMem.size / 1024 : ImageStorageMem.size / 1024
			);
		}
	}

	// ASTC workarounds
	VkFormatProperties formatProperties{};
	VulkanRHI::vkGetPhysicalDeviceFormatProperties(InDevice.GetPhysicalHandle(), VK_FORMAT_ASTC_8x8_UNORM_BLOCK, &formatProperties);
	if ((formatProperties.linearTilingFeatures & (VK_FORMAT_FEATURE_SAMPLED_IMAGE_FILTER_LINEAR_BIT | VK_FORMAT_FEATURE_TRANSFER_DST_BIT)) == (VK_FORMAT_FEATURE_SAMPLED_IMAGE_FILTER_LINEAR_BIT | VK_FORMAT_FEATURE_TRANSFER_DST_BIT))
	{
		ImageCreateInfo.flags = 0;
		ImageCreateInfo.format = VK_FORMAT_ASTC_8x8_UNORM_BLOCK;
		ImageCreateInfo.tiling = VK_IMAGE_TILING_OPTIMAL;
		ImageCreateInfo.usage = VK_IMAGE_USAGE_TRANSFER_DST_BIT | VK_IMAGE_USAGE_SAMPLED_BIT;

		VkImage ImageOptimal_ASTC;
		VkMemoryRequirements ImageOptimalMem_ASTC;
		VERIFYVULKANRESULT(VulkanRHI::vkCreateImage(InDevice.GetHandle(), &ImageCreateInfo, VULKAN_CPU_ALLOCATOR, &ImageOptimal_ASTC));
		VulkanRHI::vkGetImageMemoryRequirements(InDevice.GetHandle(), ImageOptimal_ASTC, &ImageOptimalMem_ASTC);
		VulkanRHI::vkDestroyImage(InDevice.GetHandle(), ImageOptimal_ASTC, VULKAN_CPU_ALLOCATOR);

		ImageCreateInfo.tiling = VK_IMAGE_TILING_LINEAR;

		VkImage ImageLinear_ASTC;
		VkMemoryRequirements ImageLinearMem_ASTC;
		VERIFYVULKANRESULT(VulkanRHI::vkCreateImage(InDevice.GetHandle(), &ImageCreateInfo, VULKAN_CPU_ALLOCATOR, &ImageLinear_ASTC));
		VulkanRHI::vkGetImageMemoryRequirements(InDevice.GetHandle(), ImageLinear_ASTC, &ImageLinearMem_ASTC);
		VulkanRHI::vkDestroyImage(InDevice.GetHandle(), ImageLinear_ASTC, VULKAN_CPU_ALLOCATOR);


		const float MEM_SIZE_THRESHOLD = 2.0f;
		const float ImageOptimal_SIZE = (float)ImageOptimalMem_ASTC.size;

		if (ImageLinearMem_ASTC.size * MEM_SIZE_THRESHOLD <= ImageOptimal_SIZE)
		{
			ASTCWorkaroundOption = 1;

			UE_LOGF(LogRHI, Display, "Enabling workaround to reduce memory requirement for ASTC textures (VK_IMAGE_TILING_LINEAR). 128x128 - 8 Mips ASTC_8x8 texture: %llu KiB -> %llu KiB",
				ImageOptimalMem_ASTC.size / 1024,
				ImageLinearMem_ASTC.size / 1024
			);
		}
	}
}

void FVulkanAndroidPlatform::SetImageMemoryRequirementWorkaround(VkImageCreateInfo& ImageCreateInfo)
{
	if (AFBCWorkaroundOption != 0 &&
		ImageCreateInfo.imageType == VK_IMAGE_TYPE_2D && 
		ImageCreateInfo.format == VK_FORMAT_B8G8R8A8_UNORM && 
		ImageCreateInfo.mipLevels >= 8) // its worth enabling for 128x128 and up
	{
		if (AFBCWorkaroundOption == 1)
		{
			ImageCreateInfo.flags |= VK_IMAGE_CREATE_MUTABLE_FORMAT_BIT;
		}
		else if (AFBCWorkaroundOption == 2)
		{
			ImageCreateInfo.usage |= VK_IMAGE_USAGE_STORAGE_BIT;
		}
	}

	// Use ASTC workaround for textures ASTC_6x6 and ASTC_8x8 with mips and size up to 128x128
	if (ASTCWorkaroundOption != 0 &&
		ImageCreateInfo.imageType == VK_IMAGE_TYPE_2D &&
		(ImageCreateInfo.format >= VK_FORMAT_ASTC_6x6_UNORM_BLOCK && ImageCreateInfo.format <= VK_FORMAT_ASTC_8x8_SRGB_BLOCK) &&
		(ImageCreateInfo.mipLevels > 1 && ImageCreateInfo.extent.width <= 128 && ImageCreateInfo.extent.height <= 128))
	{
		ImageCreateInfo.tiling = VK_IMAGE_TILING_LINEAR;
	}
}

FString FVulkanAndroidPlatform::GetVulkanProfileNameForFeatureLevel(ERHIFeatureLevel::Type FeatureLevel, bool bRaytracing)
{
	// Use the generic name and add "_Android" at the end (the RT suffix get added after the platform)
	FString ProfileName = FVulkanGenericPlatform::GetVulkanProfileNameForFeatureLevel(FeatureLevel, false) + TEXT("_Android");
	if (bRaytracing)
	{
		ProfileName += TEXT("_RT");
	}
	return ProfileName;
} 

void FVulkanAndroidPlatform::OnGPUCrash(const TCHAR* Message)
{
	// trigger a fatal message which will make it visible to crash reporter.
	GLog->Panic();
	UE_LOGF(LogVulkanRHI, Fatal, "%ls", Message);
}
