// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHI.h"
#include "Android/AndroidWindow.h"

#define VK_USE_PLATFORM_ANDROID_KHR					1

#define VULKAN_ENABLE_DUMP_LAYER					0
#define VULKAN_DYNAMICALLYLOADED					1
#define VULKAN_SHOULD_ENABLE_DRAW_MARKERS			(UE_BUILD_DEVELOPMENT || UE_BUILD_DEBUG)
#define VULKAN_USE_IMAGE_ACQUIRE_FENCES				0
#define VULKAN_USE_CREATE_ANDROID_SURFACE			1
#define VULKAN_SHOULD_USE_LLM						(UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT) // If enabled Vulkan will report detailed allocation statistics, overrides some tags with custom ones
#define VULKAN_SHOULD_USE_COMMANDWRAPPERS			VULKAN_SHOULD_USE_LLM //LLM on Vulkan needs command wrappers to account for vkallocs
#define VULKAN_ENABLE_LRU_CACHE						1
#define VULKAN_SUPPORTS_GOOGLE_DISPLAY_TIMING		1
#define VULKAN_PURGE_SHADER_MODULES					0
#define VULKAN_SUPPORTS_DEDICATED_ALLOCATION		0
#define VULKAN_SUPPORTS_ASTC_DECODE_MODE			1
#define VULKAN_SUPPORTS_SCALAR_BLOCK_LAYOUT			1
#define VULKAN_SUPPORTS_TRANSIENT_RESOURCE_ALLOCATOR 0
#define VULKAN_SUPPORTS_DRIVER_PROPERTIES			0
#define VULKAN_SUPPORTS_DESCRIPTOR_INDEXING			1
#define VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH	0

// Enabling debugging gives access to features that have little cost unless enabled (Validation Layer, Renderdoc detection, etc.)
// Because Development is slower on Android, more testing using validation layer occurs on Test builds.
#define VULKAN_HAS_DEBUGGING_ENABLED				(UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT || UE_BUILD_TEST)

#define UE_VK_API_VERSION							VK_API_VERSION_1_1

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
#	include "vk_enum_string_helper.h"
#	define VK_TYPE_TO_STRING(Type, Value) ANSI_TO_TCHAR(string_##Type(Value))
//#	define VK_FLAGS_TO_STRING(Type, Value) ANSI_TO_TCHAR(string_##Type(Value).c_str())
#endif

#define ENUM_VK_ENTRYPOINTS_PLATFORM_BASE(EnumMacro)

#define ENUM_VK_ENTRYPOINTS_PLATFORM_INSTANCE(EnumMacro) \
	EnumMacro(PFN_vkCreateAndroidSurfaceKHR, vkCreateAndroidSurfaceKHR) \
	EnumMacro(PFN_vkGetAndroidHardwareBufferPropertiesANDROID, vkGetAndroidHardwareBufferPropertiesANDROID)

#define ENUM_VK_ENTRYPOINTS_OPTIONAL_PLATFORM_INSTANCE(EnumMacro) \
	EnumMacro(PFN_vkGetRefreshCycleDurationGOOGLE, vkGetRefreshCycleDurationGOOGLE) \
	EnumMacro(PFN_vkGetPastPresentationTimingGOOGLE, vkGetPastPresentationTimingGOOGLE)

// and now, include the GenericPlatform class
#include "VulkanGenericPlatform.h"

extern bool IsInAndroidEventThread();

class FVulkanAndroidPlatformWindowContext
{
public:
	FVulkanAndroidPlatformWindowContext(void* InWindowHandle)
	{
		AndroidWindow = (FAndroidWindow*)InWindowHandle;
		// in this case FVulkanAndroidPlatformWindowContext owns the FNativeAccessor.
		if (InWindowHandle)
		{
			if(FAndroidMisc::UseNewWindowBehavior())
			{
				// the context should be locked at the beginning of the update process, this overload is used for GT initiated events.
				check(IsInGameThread());
				WindowContainer = AndroidWindow->GetANativeAccessor(false);
				NativeWindow = WindowContainer.IsSet() ? WindowContainer->GetANativeWindow() : nullptr;
			}
		}
	}
	
	FVulkanAndroidPlatformWindowContext(TOptional<FAndroidWindow::FNativeAccessor> WindowContainerIn)
	{
		// in this case the FNativeAccessor has come from the event thread
		if(WindowContainerIn.IsSet())
		{
			check(FAndroidMisc::UseNewWindowBehavior());
			// the context should be locked at the beginning of the update process, this overload is used for ET initiated events.
			check(IsInAndroidEventThread());
			WindowContainer = MoveTemp(WindowContainerIn);
			NativeWindow = WindowContainer->GetANativeWindow();
			AndroidWindow = WindowContainer.GetValue().Get();
		}
	}

	// we dont have a locked window to create swapchains during present, android will use the 
	// SetRHIOnReleaseWindowCallback / SetRHIOnReInitWindowCallback to create as required.
	static bool CanCreateSwapchainOnDemand() { return false; }
	bool IsValid() const
	{
		return FAndroidMisc::UseNewWindowBehavior() ? GetANativeWindow() != nullptr : true;
	}

	ANativeWindow* GetANativeWindow() const
	{
		return NativeWindow;
	}

	void* GetWindowHandle() const
	{
		return AndroidWindow;
	}

	// Android tracks window lifetime via the native accessor, not an SDL-style window id.
	uint32 GetWindowId() const { return 0; }

private:
	ANativeWindow* NativeWindow = nullptr;
	FAndroidWindow* AndroidWindow = nullptr;
	TOptional<FAndroidWindow::FNativeAccessor> WindowContainer;
};

using FVulkanPlatformWindowContext = FVulkanAndroidPlatformWindowContext;

class FVulkanAndroidPlatform : public FVulkanGenericPlatform
{
public:
	static bool LoadVulkanLibrary();
	static bool LoadVulkanInstanceFunctions(VkInstance inInstance);
	static void FreeVulkanLibrary();

	static void InitDevice(FVulkanDevice* InDevice);
	static void PostInitGPU(const FVulkanDevice& InDevice);

	static void GetInstanceExtensions(FVulkanInstanceExtensionArray& OutExtensions);
	static void GetInstanceLayers(TArray<const ANSICHAR*>& OutLayers);
	static void GetDeviceExtensions(FVulkanDevice* Device, FVulkanDeviceExtensionArray& OutExtensions);
	static void GetDeviceLayers(TArray<const ANSICHAR*>& OutLayers);
	static void NotifyFoundDeviceLayersAndExtensions(VkPhysicalDevice PhysicalDevice, const TArray<const ANSICHAR*>& Layers, const TArray<const ANSICHAR*>& Extensions);

	static void CreateSurface(FVulkanPlatformWindowContext& WindowContext, VkInstance Instance, VkSurfaceKHR* OutSurface);

	static void* GetHardwareWindowHandle();

	static bool SupportsBCTextureFormats();
	static bool SupportsASTCTextureFormats() { return true; }
	static bool SupportsETC2TextureFormats() { return true; }
	// GLES does not support R16Unorm, so all Android has to fallback to R16F instead
	static bool SupportsR16UnormTextureFormat() { return false; }
	
	static bool SupportsQuerySurfaceProperties() { return false; }

	static void SetupFeatureLevels(TArrayView<EShaderPlatform> ShaderPlatformForFeatureLevel)
	{
		if (RequiresMobileRenderer())
		{
			ShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES2_REMOVED] = SP_NumPlatforms;
			ShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES3_1] = SP_VULKAN_ES3_1_ANDROID;
			ShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM4_REMOVED] = SP_NumPlatforms;
			ShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM5] = SP_NumPlatforms;
			ShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM6] = SP_NumPlatforms;
		}
		else
		{
			ShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES2_REMOVED] = SP_NumPlatforms;
			ShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES3_1] = SP_VULKAN_ES3_1_ANDROID;
			ShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM4_REMOVED] = SP_VULKAN_SM5_ANDROID;
			ShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM5] = SP_VULKAN_SM5_ANDROID;
			ShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM6] = SP_NumPlatforms;
		}
	}

	static bool SupportsTimestampRenderQueries();

	static bool SupportsDynamicResolution();

	static bool RequiresMobileRenderer()
	{
		#if USE_STATIC_FEATURE_LEVEL_ENUMS
		return (UE_ANDROID_STATIC_FEATURE_LEVEL == ERHIFeatureLevel::ES3_1);
		#else	
		return !FAndroidMisc::ShouldUseDesktopVulkan();
		#endif
	}

	static ERHIFeatureLevel::Type GetFeatureLevel(ERHIFeatureLevel::Type RequestedFeatureLevel)
	{
		#if USE_STATIC_FEATURE_LEVEL_ENUMS
		return UE_ANDROID_STATIC_FEATURE_LEVEL;
		#else
		return FVulkanGenericPlatform::GetFeatureLevel(RequestedFeatureLevel);
		#endif
	}

	static bool SupportsVolumeTextureRendering() { return false; }

	static void OverridePlatformHandlers(bool bInit);
#if USE_ANDROID_STANDALONE
	static bool IsWaitingForWindowCreated();
#endif

	//#todo-rco: Detect Mali?
	static bool RequiresPresentLayoutFix() { return true; }

#if PLATFORM_ANDROID_X64 	
	static bool HasUnifiedMemory();
#else
	static bool HasUnifiedMemory() { return true; }
#endif

	// Assume most devices can't use the extra cores for running parallel tasks
	static bool SupportParallelRenderingTasks() { return false; }

	//#todo-rco: Detect Mali? Doing a clear on ColorAtt layout on empty cmd buffer causes issues
	static bool RequiresSwapchainGeneralInitialLayout() { return true; }

	static bool RequiresWaitingForFrameCompletionEvent() { return bRequiresWaitingForFrameCompletionEvent; }
	
	// Does the platform allow a nullptr Pixelshader on the pipeline
	static bool SupportsNullPixelShader() { return false; }

	// Does the platform require depth to be written on stencil clear
	static bool RequiresDepthStencilFullWrite() { return bRequiresDepthStencilFullWrite; }
	static void SetupRequiresDepthStencilFullWriteWorkaround(const FVulkanDevice& Device);

	// Does the platform require to avoid VkAttachmentDescriptionStencilLayout in favor of merged depth-stencil layouts
	static bool RequiresMergedDepthStencilLayout() { return bRequiresMergedRenderPassDepthStencilLayout; }
	static void SetupRequiresMergedDepthStencilLayoutWorkaround(const FVulkanDevice& Device);

	static bool FramePace(FVulkanDevice& Device, void* WindowHandle, VkSwapchainKHR Swapchain, uint32 PresentID, VkPresentInfoKHR& Info);

	static VkResult Present(VkQueue Queue, VkPresentInfoKHR& PresentInfo);

	static VkResult CreateSwapchainKHR(FVulkanPlatformWindowContext& WindowContext, VkPhysicalDevice PhysicalDevice, VkDevice Device, const VkSwapchainCreateInfoKHR* CreateInfo, const VkAllocationCallbacks* Allocator, VkSwapchainKHR* Swapchain);

	static void DestroySwapchainKHR(VkDevice Device, VkSwapchainKHR Swapchain, const VkAllocationCallbacks* Allocator);

	// handle precompile of PSOs, send to an android specific precompile external process.
	static VkPipelineCache PrecompilePSO(FVulkanDevice* Device, const TArrayView<const uint8> ContextData, const TArrayView<const uint8> OptionalPSOCacheData, FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType PSOCompileType, const VkGraphicsPipelineCreateInfo* PipelineInfo, const FGfxPipelineDesc* GfxEntry, const FVulkanRenderTargetLayout* RTLayout, TArrayView<const uint32_t> VS, TArrayView<const uint32_t> PS, size_t& AfterSize, FString* FailureMessageOUT = nullptr);

	static FString GetCompiledPSOCacheTopFolderPath();

	static bool AreRemoteCompileServicesActive();
	static bool StartRemoteCompileServices(int NumServices);
	static void StopRemoteCompileServices();

	// Do not attempt to immediately recreate swapchain
	static bool RecreateSwapchainOnFail() { return false; }

	// original window behavior swapchain functions.
	// ignored with the new window method.
	static void RecreateSwapChain(void* NewNativeWindow);
	static void DestroySwapChain();

	static VkFormat GetPlatform5551FormatWithFallback(VkFormat& OutFallbackFormat0, VkFormat& OutFallbackFormat1) 
	{ 
		OutFallbackFormat0 = VK_FORMAT_A1R5G5B5_UNORM_PACK16; 
		OutFallbackFormat1 = VK_FORMAT_B8G8R8A8_UNORM;
		return VK_FORMAT_R5G5B5A1_UNORM_PACK16; 
	};

	// Setup platform to use a workaround to reduce textures memory requirements
	static void SetupImageMemoryRequirementWorkaround(const FVulkanDevice& InDevice);
	static void SetImageMemoryRequirementWorkaround(VkImageCreateInfo& ImageCreateInfo);

	// Returns the profile name to look up for a given feature level on a platform
	static FString GetVulkanProfileNameForFeatureLevel(ERHIFeatureLevel::Type FeatureLevel, bool bRaytracing);

	static VkShaderStageFlags RequiredWaveOpsShaderStageFlags(VkShaderStageFlags VulkanDeviceShaderStageFlags)
	{
		// Many Android Vulkan implementations do not support wave ops in vertex and geometry shaders and we don't need them there.
		return VK_SHADER_STAGE_FRAGMENT_BIT | VK_SHADER_STAGE_COMPUTE_BIT;
	}

	static void OnGPUCrash(const TCHAR* Message);

	static VkTimeDomainKHR GetTimeDomain() { return VK_TIME_DOMAIN_CLOCK_MONOTONIC_KHR; }

#if WITH_ADDITIONAL_CRASH_CONTEXTS
	static void OnCreatePipelineCrash(struct FCrashContextExtendedWriter& Writer, TConstArrayView<FVulkanShader*> Shaders, TStringView<TCHAR> ExtraInfo, bool bIsPrecache);
#endif

#if USE_ANDROID_VULKAN_SWAPPY
	static bool bSwappyEnabledAtRHIInit;
#endif

protected:
	static void* VulkanLib;
	static bool bAttemptedLoad;

#if VULKAN_SUPPORTS_GOOGLE_DISPLAY_TIMING
	static bool bHasGoogleDisplayTiming;
	static TUniquePtr<class FGDTimingFramePacer> GDTimingFramePacer;
#endif

	static TUniquePtr<struct FAndroidVulkanFramePacer> FramePacer;
	static int32 CachedFramePace;
	static int32 CachedRefreshRate;
	static int32 CachedSyncInterval;
	static int32 SuccessfulRefreshRateFrames;
	static int32 UnsuccessfulRefreshRateFrames;
	static TArray<TArray<ANSICHAR>> DebugVulkanDeviceLayers;
	static TArray<TArray<ANSICHAR>> DebugVulkanInstanceLayers;
	static TArray<TArray<ANSICHAR>> SwappyRequiredExtensions;

	static int32 AFBCWorkaroundOption;
	static int32 ASTCWorkaroundOption;

	static bool bRequiresDepthStencilFullWrite;
	static bool bRequiresMergedRenderPassDepthStencilLayout;
	static bool bRequiresWaitingForFrameCompletionEvent;

	static FAutoConsoleVariableRef CVarVulkanRequiresWaitingForFrameCompletionEvent;
};

#if VULKAN_SUPPORTS_GOOGLE_DISPLAY_TIMING
class FGDTimingFramePacer : FNoncopyable
{
public:
	FGDTimingFramePacer(VkDevice InDevice, VkSwapchainKHR InSwapChain);

	const VkPresentTimesInfoGOOGLE* GetPresentTimesInfo() const
	{
		return ((SyncDuration > 0) ? &PresentTimesInfo : nullptr);
	}

	void ScheduleNextFrame(uint32 InPresentID, int32 FramePace, int32 RefreshRate); // Call right before present

private:
	void UpdateSyncDuration(int32 FramePace, int32 RefreshRate);

	uint64 PredictLastScheduledFramePresentTime(uint32 CurrentPresentID) const;
	uint64 CalculateMinPresentTime(uint64 CpuPresentTime) const;
	uint64 CalculateMaxPresentTime(uint64 CpuPresentTime) const;
	uint64 CalculateNearestVsTime(uint64 ActualPresentTime, uint64 TargetTime) const;
	void PollPastFrameInfo();

private:
	struct FKnownFrameInfo
	{
		bool bValid = false;
		uint32 PresentID = 0;
		uint64 ActualPresentTime = 0;
	};

private:
	VkDevice Device;
	VkSwapchainKHR SwapChain;

	VkPresentTimesInfoGOOGLE PresentTimesInfo;
	VkPresentTimeGOOGLE PresentTime;
	uint64 RefreshDuration = 0;
	uint64 HalfRefreshDuration = 0;

	FKnownFrameInfo LastKnownFrameInfo;
	uint64 LastScheduledPresentTime = 0;
	uint64 SyncDuration = 0;
	int32 FramePace = 0;
};
#endif //VULKAN_SUPPORTS_GOOGLE_DISPLAY_TIMING

typedef FVulkanAndroidPlatform FVulkanPlatform;
