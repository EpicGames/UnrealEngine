// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanLinuxPlatform.h"
#include "VulkanDevice.h"
#include "VulkanRHIPrivate.h"
#include "VulkanRayTracing.h"
#include "VulkanExtensions.h"
#include <dlfcn.h>
#include <SDL3/SDL.h>
#include <SDL3/SDL_vulkan.h>
#include "Linux/LinuxPlatformApplicationMisc.h"
#include "Linux/LinuxPlatformProperties.h"

// Vulkan function pointers
#define DEFINE_VK_ENTRYPOINTS(Type,Func) Type VulkanDynamicAPI::Func = NULL;
ENUM_VK_ENTRYPOINTS_ALL(DEFINE_VK_ENTRYPOINTS)

static bool GRenderOffScreen = false;

void* FVulkanLinuxPlatform::VulkanLib = nullptr;
bool FVulkanLinuxPlatform::bAttemptedLoad = false;
bool FVulkanLinuxPlatform::bLoadedSDLVulkanLibrary = false;

bool FVulkanLinuxPlatform::IsSupported()
{
	if (!FParse::Param(FCommandLine::Get(), TEXT("RenderOffScreen")))
	{
		// If we're not running offscreen mode, make sure we have a display envvar set
		bool bHasX11Display = (getenv("DISPLAY") != nullptr);

		if (!bHasX11Display)
		{
			// check Wayland
			bool bHasWaylandSession = (getenv("WAYLAND_DISPLAY") != nullptr);

			if (!bHasWaylandSession)
			{
				UE_LOGF(LogRHI, Warning, "Could not detect DISPLAY or WAYLAND_DISPLAY environment variables");
				return false;
			}
		}
	}

	// Attempt to load the library
	return LoadVulkanLibrary();
}

// vkEnumerateInstanceVersion triggers an ASAN errors at some versions, use the filesystem to determine the version
// Only reject the loader if we were able to confirm its version number, for this reason we return MAX_uint32 on error
static uint32_t GetVulkanInstanceVersion(void* VulkanLoader, const ANSICHAR* LoaderFilename)
{
	FAnsiString FullPath;
	{
		ANSICHAR LoaderPath[PATH_MAX+1];
		FMemory::Memzero(LoaderPath);
		const int32 RetVal = dlinfo(VulkanLoader, RTLD_DI_ORIGIN, LoaderPath);
		if (RetVal < 0)
		{
			return MAX_uint32;
		}
		UE_LOGF(LogVulkanRHI, Display, "Installed Vulkan Loader Path: %ls", ANSI_TO_TCHAR(LoaderPath));
		FullPath = LoaderPath;
	}
	FullPath += "/";
	FullPath += LoaderFilename;
	
	auto IsSymLink = [](const ANSICHAR* Path)
	{
		struct stat PathStat;
		if (lstat(Path, &PathStat) == -1)
		{
			return false;
		}
		
		return S_ISLNK(PathStat.st_mode);
	};

	auto FollowLinkTarget = [](FAnsiString& LinkPath)
	{
		ANSICHAR TargetPath[PATH_MAX+1];
		FMemory::Memzero(TargetPath);
		if (readlink(*LinkPath, TargetPath, PATH_MAX) > 0)
		{
			LinkPath = TargetPath;
			return true;
		}
		return false;
	};
	
		
	while (IsSymLink(*FullPath))
	{
		FollowLinkTarget(FullPath);
	}
	
	TArray<FAnsiString> SplitStrings;
	FullPath.ParseIntoArray(SplitStrings, ".", true);
	if (SplitStrings.Num() >= 4 && 
		SplitStrings[SplitStrings.Num()-3].IsNumeric() &&
		SplitStrings[SplitStrings.Num()-2].IsNumeric() &&
		SplitStrings[SplitStrings.Num()-1].IsNumeric())
	{
		return VK_MAKE_API_VERSION(0, 
			FCStringAnsi::Atoi(*SplitStrings[SplitStrings.Num()-3]), 
			FCStringAnsi::Atoi(*SplitStrings[SplitStrings.Num()-2]), 
			FCStringAnsi::Atoi(*SplitStrings[SplitStrings.Num()-1]));
	}
	
	return MAX_uint32;
}

bool FVulkanLinuxPlatform::LoadVulkanLibrary()
{
	if (bAttemptedLoad)
	{
		return (VulkanLib != nullptr);
	}
	bAttemptedLoad = true;

	const FString UEVulkanBinariesPath = FPaths::EngineDir() + TEXT("Binaries/ThirdParty/Vulkan/Linux");

#if VULKAN_HAS_DEBUGGING_ENABLED
	const FString VulkanSDK = FPlatformMisc::GetEnvironmentVariable(TEXT("VULKAN_SDK"));
	UE_LOGF(LogVulkanRHI, Display, "Found VULKAN_SDK=%ls", *VulkanSDK);
	const bool bHasVulkanSDK = !VulkanSDK.IsEmpty();

	UE_LOGF(LogVulkanRHI, Display, "Registering provided Vulkan validation layers");

	// if vulkan SDK is installed, we'll append our built-in validation layers to VK_ADD_LAYER_PATH,
	// otherwise we append to VK_LAYER_PATH (which is probably empty)

	// Change behavior of loading Vulkan layers by setting environment variable "VarToUse" to UE specific directory
	FString VarToUse = (bHasVulkanSDK)?TEXT("VK_ADD_LAYER_PATH"):TEXT("VK_LAYER_PATH");
	FString PreviousEnvVar = FPlatformMisc::GetEnvironmentVariable(*VarToUse);
	
	if(!PreviousEnvVar.IsEmpty())
	{
		PreviousEnvVar.Append(TEXT(":"));
	}

	PreviousEnvVar.Append(*UEVulkanBinariesPath);
	FPlatformMisc::SetEnvironmentVar(*VarToUse, *PreviousEnvVar);
	UE_LOGF(LogVulkanRHI, Display, "Updated %ls=%ls", *VarToUse, *PreviousEnvVar);

	FString PreviousLibPath = FPlatformMisc::GetEnvironmentVariable(TEXT("LD_LIBRARY_PATH"));
	if (!PreviousLibPath.IsEmpty())
	{
		PreviousLibPath.Append(TEXT(":"));
	}

	PreviousLibPath.Append(*UEVulkanBinariesPath);
	FPlatformMisc::SetEnvironmentVar(TEXT("LD_LIBRARY_PATH"), *PreviousLibPath);
	UE_LOGF(LogVulkanRHI, Display, "Updated LD_LIBRARY_PATH=%ls", *PreviousLibPath);
#endif // VULKAN_HAS_DEBUGGING_ENABLED

	FAnsiString FinalVulkanLoaderPath;
		
	if (!FParse::Param(FCommandLine::Get(), TEXT("UseLocalVulkanLoader")))
	{
		const ANSICHAR* LoaderFilename = "libvulkan.so.1";
		
		// Try to load the default libvulkan.so
		void* GlobalVulkanLib = dlopen(LoaderFilename, RTLD_NOW | RTLD_LOCAL);
		FinalVulkanLoaderPath = LoaderFilename;
		
		if (FParse::Param(FCommandLine::Get(), TEXT("UseGlobalVulkanLoader")))
		{
			VulkanLib = GlobalVulkanLib;
			GlobalVulkanLib = nullptr;
		}
		else if (GlobalVulkanLib)
		{
			// Verify version and discard it if the version has issues
			const uint32_t ApiVersion = GetVulkanInstanceVersion(GlobalVulkanLib, LoaderFilename);
			UE_LOGF(LogVulkanRHI, Display, "Installed Vulkan Loader instance version %u.%u.%u.", 
				VK_API_VERSION_MAJOR(ApiVersion), VK_API_VERSION_MINOR(ApiVersion), VK_API_VERSION_PATCH(ApiVersion));
						
			if (ApiVersion > VK_MAKE_API_VERSION(0, 1, 3, 204))
			{
				VulkanLib = GlobalVulkanLib;
				GlobalVulkanLib = nullptr;
			}
		}

		if (GlobalVulkanLib)
		{
			dlclose(GlobalVulkanLib);
		}
	}

	// Try to load libvulkan.so from the included SDK
	if ((VulkanLib == nullptr) && (!FPlatformProperties::IsArm64())) // :todo: Add ARM64 versions of the layers and loader
	{
		UE_LOGF(LogVulkanRHI, Display, "Using included Vulkan loader.");
		FAnsiString VulkanLoaderPath(UEVulkanBinariesPath);
		VulkanLoaderPath.Append("/libvulkan.so");
		VulkanLib = dlopen(*VulkanLoaderPath, RTLD_NOW | RTLD_LOCAL);
		FinalVulkanLoaderPath = *VulkanLoaderPath;
	}


	if (VulkanLib == nullptr)
	{
		// be more verbose on Linux
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, 
			TEXT("Unable to load Vulkan library and/or acquire the necessary function pointers. Make sure an up-to-date libvulkan.so.1 is installed."),
			TEXT("Unable to initialize Vulkan."));

		return false;
	}
	
	bLoadedSDLVulkanLibrary = SDL_Vulkan_LoadLibrary(*FinalVulkanLoaderPath);	// SDL needs to use the same Vulkan Engine is so load it with the path we used
	
	bool bFoundAllEntryPoints = true;
#define CHECK_VK_ENTRYPOINTS(Type,Func) if (VulkanDynamicAPI::Func == NULL) { bFoundAllEntryPoints = false; UE_LOGF(LogRHI, Warning, "Failed to find entry point for %ls", TEXT(#Func)); }

	// Initialize all of the entry points we have to query manually
#define GET_VK_ENTRYPOINTS(Type,Func) VulkanDynamicAPI::Func = (Type)dlsym(VulkanLib, #Func);
	ENUM_VK_ENTRYPOINTS_BASE(GET_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_BASE(CHECK_VK_ENTRYPOINTS);
	if (!bFoundAllEntryPoints)
	{
		dlclose(VulkanLib);
		VulkanLib = nullptr;
		if (bLoadedSDLVulkanLibrary)
		{
			SDL_Vulkan_UnloadLibrary();
			bLoadedSDLVulkanLibrary = false;
		}
		return false;
	}

	ENUM_VK_ENTRYPOINTS_OPTIONAL_BASE(GET_VK_ENTRYPOINTS);
#if UE_BUILD_DEBUG
	ENUM_VK_ENTRYPOINTS_OPTIONAL_BASE(CHECK_VK_ENTRYPOINTS);
#endif

	ENUM_VK_ENTRYPOINTS_PLATFORM_BASE(GET_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_PLATFORM_BASE(CHECK_VK_ENTRYPOINTS);

#undef GET_VK_ENTRYPOINTS

	return true;
}

bool FVulkanLinuxPlatform::LoadVulkanInstanceFunctions(VkInstance inInstance)
{
	bool bFoundAllEntryPoints = true;
#define CHECK_VK_ENTRYPOINTS(Type,Func) if (VulkanDynamicAPI::Func == NULL) { bFoundAllEntryPoints = false; UE_LOGF(LogRHI, Warning, "Failed to find entry point for %ls", TEXT(#Func)); }

#define GETINSTANCE_VK_ENTRYPOINTS(Type, Func) VulkanDynamicAPI::Func = (Type)VulkanDynamicAPI::vkGetInstanceProcAddr(inInstance, #Func);
	ENUM_VK_ENTRYPOINTS_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_INSTANCE(CHECK_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_SURFACE_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_SURFACE_INSTANCE(CHECK_VK_ENTRYPOINTS);

	if (!bFoundAllEntryPoints && !FParse::Param(FCommandLine::Get(), TEXT("RenderOffScreen")))
	{
		return false;
	}

	const bool bFoundRayTracingEntries = FVulkanRayTracingPlatform::CheckVulkanInstanceFunctions(inInstance);
	if (!bFoundRayTracingEntries)
	{
		UE_LOGF(LogVulkanRHI, Warning, "Vulkan RHI ray tracing is enabled, but failed to load instance functions.");
	}

	ENUM_VK_ENTRYPOINTS_OPTIONAL_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_OPTIONAL_PLATFORM_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
#if UE_BUILD_DEBUG
	ENUM_VK_ENTRYPOINTS_OPTIONAL_INSTANCE(CHECK_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_OPTIONAL_PLATFORM_INSTANCE(CHECK_VK_ENTRYPOINTS);
#endif

	ENUM_VK_ENTRYPOINTS_PLATFORM_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_PLATFORM_INSTANCE(CHECK_VK_ENTRYPOINTS);
#undef GET_VK_ENTRYPOINTS

	return true;
}

void FVulkanLinuxPlatform::FreeVulkanLibrary()
{
	if (VulkanLib != nullptr)
	{
#define CLEAR_VK_ENTRYPOINTS(Type,Func) VulkanDynamicAPI::Func = nullptr;
		ENUM_VK_ENTRYPOINTS_ALL(CLEAR_VK_ENTRYPOINTS);

		dlclose(VulkanLib);
		VulkanLib = nullptr;

		if (bLoadedSDLVulkanLibrary)
		{
			SDL_Vulkan_UnloadLibrary();
			bLoadedSDLVulkanLibrary = false;
		}
	}
	bAttemptedLoad = false;
}

namespace
{
	void EnsureSDLIsInited()
	{
		if (!FLinuxPlatformApplicationMisc::InitSDL()) //   will not initialize more than once
		{
			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Vulkan InitSDL() failed, cannot initialize SDL."), TEXT("InitSDL Failed"));
			UE_LOGF(LogInit, Error, "Vulkan InitSDL() failed, cannot initialize SDL.");
		}
	}
}

void FVulkanLinuxPlatform::GetInstanceExtensions(FVulkanInstanceExtensionArray& OutExtensions)
{
	EnsureSDLIsInited();

	// We only support Xlib and Wayland, so check the video driver and hardcode each.
	// See FVulkanLinuxPlatform::IsSupported for the one other spot where support is hardcoded!
	//
	// Long-term, it'd be nice to replace dlopen with SDL_Vulkan_LoadLibrary so we can use
	// SDL_Vulkan_GetInstanceExtensions, but this requires moving vkGetDeviceProcAddr out of
	// the base entry points and allocating vkInstance to get all the non-global functions.
	//
	// Previously there was an Epic extension called SDL_Vulkan_GetRequiredInstanceExtensions,
	// but this effectively did what we're doing here (including depending on Xlib without a
	// fallback for xcb-only situations). Hardcoding is actually _better_ because the extension
	// broke the SDL_dynapi function table, making third-party SDL updates much harder to do.

	const char *SDLDriver = SDL_GetCurrentVideoDriver();
	if (SDLDriver == NULL)
	{
		// This should never happen if EnsureSDLIsInited passed!
		return;
	}

	if (strcmp(SDLDriver, "x11") == 0)
	{
		OutExtensions.Add(MakeUnique<FVulkanInstanceExtension>("VK_KHR_xlib_surface", VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));
	}
	else if (strcmp(SDLDriver, "wayland") == 0)
	{
		OutExtensions.Add(MakeUnique<FVulkanInstanceExtension>("VK_KHR_wayland_surface", VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));
	}
	// dummy is when we render offscreen, so ignore warning here
	else if (strcmp(SDLDriver, "dummy") != 0)
	{
		UE_LOGF(LogRHI, Warning, "Could not detect SDL video driver!");
	}
}

void FVulkanLinuxPlatform::GetDeviceExtensions(FVulkanDevice* Device, FVulkanDeviceExtensionArray& OutExtensions)
{
	OutExtensions.Add(MakeUnique<FVulkanDeviceExtension>(Device, VK_KHR_EXTERNAL_SEMAPHORE_FD_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));
	OutExtensions.Add(MakeUnique<FVulkanDeviceExtension>(Device, VK_KHR_EXTERNAL_MEMORY_FD_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));
}

FVulkanLinuxPlatformWindowContext::FVulkanLinuxPlatformWindowContext(void* InWindowHandle)
	: FVulkanGenericPlatformWindowContext(InWindowHandle)
{
	// Capture the SDL window id now, while the window is alive. The id is immutable for the window's
	// life and never recycled, so it is a safe key for releasing per-window RHI resources later.
	if (InWindowHandle)
	{
		WindowId = SDL_GetWindowID(static_cast<SDL_Window*>(InWindowHandle));
	}
}

bool FVulkanLinuxPlatformWindowContext::IsValid() const
{
	return FVulkanLinuxPlatform::IsWindowValid(GetWindowHandle());
}

void FVulkanLinuxPlatform::CreateSurface(FVulkanPlatformWindowContext& WindowContext, VkInstance Instance, VkSurfaceKHR* OutSurface)
{
	EnsureSDLIsInited();

	UE_LOGF(LogInit, Verbose, "FVulkanLinuxPlatform::CreateSurface: Window=%p, IsValid=%d",
		WindowContext.GetWindowHandle(), IsWindowValid(WindowContext.GetWindowHandle()) ? 1 : 0);
	if (SDL_Vulkan_CreateSurface((SDL_Window*)WindowContext.GetWindowHandle(), Instance, VulkanRHI::GetMemoryAllocator(nullptr), OutSurface) == false)
	{
		UE_LOGF(LogInit, Error, "Error initializing SDL Vulkan Surface: %s", SDL_GetError());
		check(0);
	}

	// The window now has a live Vulkan surface backing it; hold a ref so SDL_DestroyWindow cannot run
	// until this surface is destroyed (see DestroySurface). The id this ref is keyed under rides with
	// the window context (and onto the swapchain), so release keys on that immutable id -- never on
	// the SDL_Window*, which may be stale or reused by the time the surface is torn down.
	FLinuxPlatformApplicationMisc::AcquireWindowForRHI(WindowContext.GetWindowHandle());
}

void FVulkanLinuxPlatform::DestroySurface(uint32 WindowId, VkInstance Instance, VkSurfaceKHR Surface)
{
	// Destroy the VK surface first (it references the native window), then release the window ref by
	// the id captured at acquire time so PumpMessages can run SDL_DestroyWindow once all surfaces
	// backing the window are gone.
	VulkanRHI::vkDestroySurfaceKHR(Instance, Surface, VULKAN_CPU_ALLOCATOR);
	FLinuxPlatformApplicationMisc::ReleaseWindowForRHI(WindowId);
}

bool FVulkanLinuxPlatform::IsWindowValid(void* WindowHandle)
{
	if (!WindowHandle)
	{
		return false;
	}

	SDL_Window* Window = static_cast<SDL_Window*>(WindowHandle);

	// SDL_GetWindowID returns 0 for invalid/destroyed windows.
	if (SDL_GetWindowID(Window) == 0)
	{
		return false;
	}

	// Check if presenting is disabled for this window. This is set to false during Hide()
	// and Destroy() to prevent the RHI thread from presenting to a surface that is being
	// torn down or has had its Wayland role removed.
	if (!SDL_GetBooleanProperty(SDL_GetWindowProperties(Window), "UE.window.present_enabled", true))
	{
		return false;
	}

	// Also check SDL's own hidden flag -- a hidden window has no valid surface role on Wayland.
	if (SDL_GetWindowFlags(Window) & SDL_WINDOW_HIDDEN)
	{
		return false;
	}

	// On Wayland an xdg_popup's role is assigned lazily during Show(): SDL defers it until the popup's
	// parent surface is ready, which can land after the window has left the hidden state above. Presenting
	// in that gap commits a buffer to the still-roleless wl_surface, after which get_xdg_surface is rejected
	// with xdg_wm_base error 4 and the compositor drops the display connection. Hold off until SDL has
	// created the xdg_surface. Toplevels assign their role at creation, so they are never gated here.
	if (FLinuxPlatformApplicationMisc::IsUsingWayland() &&
		(SDL_GetWindowFlags(Window) & (SDL_WINDOW_POPUP_MENU | SDL_WINDOW_TOOLTIP)) != 0 &&
		SDL_GetPointerProperty(SDL_GetWindowProperties(Window), SDL_PROP_WINDOW_WAYLAND_XDG_SURFACE_POINTER, nullptr) == nullptr)
	{
		return false;
	}

	// On Wayland, the compositor can kill the display connection (e.g. protocol error).
	// When this happens SDL sends SDL_EVENT_QUIT. Check for it here to break out of
	// present/acquire loops that would otherwise spin forever on the dead connection.
	if (SDL_HasEvent(SDL_EVENT_QUIT))
	{
		return false;
	}

	return true;
}

bool FVulkanLinuxPlatform::TryConsumeSwapchainRecreateRequest(void* WindowHandle)
{
	if (!WindowHandle)
	{
		return false;
	}

	SDL_Window* Window = static_cast<SDL_Window*>(WindowHandle);

	// SDL_GetWindowID returns 0 on destroyed windows; don't touch properties on those.
	if (SDL_GetWindowID(Window) == 0)
	{
		return false;
	}

	SDL_PropertiesID Props = SDL_GetWindowProperties(Window);
	if (!SDL_GetBooleanProperty(Props, "UE.window.swapchain_needs_recreate", false))
	{
		return false;
	}

	SDL_SetBooleanProperty(Props, "UE.window.swapchain_needs_recreate", false);
	return true;
}

