// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"

#define VULKAN_DYNAMICALLYLOADED					1
#define VULKAN_ENABLE_DUMP_LAYER					0
#define VULKAN_SHOULD_DEBUG_IN_DEVELOPMENT			1
#define VULKAN_SHOULD_ENABLE_DRAW_MARKERS			(UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)
#define VULKAN_SIGNAL_UNIMPLEMENTED()				checkf(false, TEXT("Unimplemented vulkan functionality: %hs"), __PRETTY_FUNCTION__)
#define VULKAN_SUPPORTS_SCALAR_BLOCK_LAYOUT			1
#define VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH	1

#define UE_VK_API_VERSION							VK_API_VERSION_1_1

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
#	include "vk_enum_string_helper.h"
#	define VK_TYPE_TO_STRING(Type, Value) ANSI_TO_TCHAR(string_##Type(Value))
#	define VK_FLAGS_TO_STRING(Type, Value) ANSI_TO_TCHAR(string_##Type(Value).c_str())
#endif

#define ENUM_VK_ENTRYPOINTS_PLATFORM_BASE(EnumMacro)

#define ENUM_VK_ENTRYPOINTS_PLATFORM_INSTANCE(EnumMacro)

#define ENUM_VK_ENTRYPOINTS_OPTIONAL_PLATFORM_INSTANCE(EnumMacro)

// and now, include the GenericPlatform class
#include "../VulkanGenericPlatform.h"

// Carries the SDL_Window* plus the SDL window id captured at construction time (while the window is
// provably alive). The id rides with the context through swapchain creation so the surface can be
// released by that immutable, never-recycled id -- never by re-deriving it from a pointer that may
// be stale or reused by destroy time. Ctor/IsValid are defined out-of-line so SDL stays out of this
// header and IsValid can reach FVulkanLinuxPlatform::IsWindowValid.
class FVulkanLinuxPlatformWindowContext : public FVulkanGenericPlatformWindowContext
{
public:
	explicit FVulkanLinuxPlatformWindowContext(void* InWindowHandle);
	uint32 GetWindowId() const { return WindowId; }
	bool IsValid() const;
private:
	uint32 WindowId = 0;
};

typedef FVulkanLinuxPlatformWindowContext FVulkanPlatformWindowContext;

class FVulkanLinuxPlatform : public FVulkanGenericPlatform
{
public:
	static bool IsSupported();

	static bool LoadVulkanLibrary();
	static bool LoadVulkanInstanceFunctions(VkInstance inInstance);
	static void FreeVulkanLibrary();

	static void GetInstanceExtensions(FVulkanInstanceExtensionArray& OutExtensions);
	static void GetInstanceLayers(TArray<const ANSICHAR*>& OutLayers) {}
	static void GetDeviceExtensions(FVulkanDevice* Device, FVulkanDeviceExtensionArray& OutExtensions);
	static void GetDeviceLayers(TArray<const ANSICHAR*>& OutLayers) {}

	static void CreateSurface(FVulkanPlatformWindowContext& WindowContext, VkInstance Instance, VkSurfaceKHR* OutSurface);
	static void DestroySurface(uint32 WindowId, VkInstance Instance, VkSurfaceKHR Surface);

	/** Returns false if the native window handle has been destroyed (e.g. Wayland surface invalidated). */
	static bool IsWindowValid(void* WindowHandle);

	/** Test-and-clear the swapchain-recreate request set by FLinuxWindow::Show() after a Hide/Show cycle. */
	static bool TryConsumeSwapchainRecreateRequest(void* WindowHandle);

	static VkTimeDomainKHR GetTimeDomain() { return VK_TIME_DOMAIN_CLOCK_MONOTONIC_KHR; }

	static bool SupportsASTCTextureFormats() { return PLATFORM_LINUXARM64; }
	static bool SupportsETC2TextureFormats() { return PLATFORM_LINUXARM64; }

protected:
	static void* VulkanLib;
	static bool bLoadedSDLVulkanLibrary;
	static bool bAttemptedLoad;
};

typedef FVulkanLinuxPlatform FVulkanPlatform;
