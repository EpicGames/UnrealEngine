// Copyright Epic Games, Inc. All Rights Reserved..

#include "VulkanRHIPrivate.h"
#include "VulkanBarriers.h"
#include "VulkanContext.h"
#include "VulkanPendingState.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RHICoreTransitions.h"

// All the graphic shader stages supported by the VkDevice (VK_PIPELINE_STAGE_VERTEX_SHADER_BIT, VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT, etc)
extern VkPipelineStageFlags GVulkanDeviceGraphicsPipelineStageBits;

extern int32 GVulkanAllowConcurrentImage;

int32 GVulkanUseMemoryBarrierOpt = 1;
static FAutoConsoleVariableRef CVarVulkanUseMemoryBarrierOpt(
	TEXT("r.Vulkan.UseMemoryBarrierOpt"),
	GVulkanUseMemoryBarrierOpt,
	TEXT("Simplify buffer barriers and image barriers without layout transitions to a memory barrier.\n")
	TEXT(" 0: Do not collapse to a single memory barrier, useful for tracking single resource transitions in external tools\n")
	TEXT(" 1: Collapse to a memory barrier when appropriate (default)"),
	ECVF_Default
);

int32 GVulkanMaxBarriersPerBatch = -1;
static FAutoConsoleVariableRef CVarVulkanMaxBarriersPerBatch(
	TEXT("r.Vulkan.MaxBarriersPerBatch"),
	GVulkanMaxBarriersPerBatch,
	TEXT("Will limit the number of barriers sent per batch\n")
	TEXT(" <=0: Do not limit (default)\n")
	TEXT(" >0: Limit to the specified number\n"),
	ECVF_Default
);

int32 GVulkanAllowSplitBarriers = 1;
static FAutoConsoleVariableRef CVarVulkanAllowSplitBarriers(
	TEXT("r.Vulkan.AllowSplitBarriers"),
	GVulkanAllowSplitBarriers,
	TEXT("Will limit the number of barriers sent per batch\n")
	TEXT(" 0: Disable split barriers\n")
	TEXT(" 1: Allow split barriers using Synchronization2 events (default)\n"),
	ECVF_Default
);

//
// The following two functions are used when the RHI needs to do image layout transitions internally.
// They are not used for the transitions requested through the public API (RHICreate/Begin/EndTransition)
// unless the initial state in ERHIAccess::Unknown, in which case the tracking code kicks in.
//
static VkAccessFlags GetVkAccessMaskForLayout(const VkImageLayout Layout)
{
	VkAccessFlags Flags = 0;

	switch (Layout)
	{
		case VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL:
			Flags = VK_ACCESS_TRANSFER_READ_BIT;
			break;
		case VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL:
			Flags = VK_ACCESS_TRANSFER_WRITE_BIT;
			break;
		case VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL:
			Flags = VK_ACCESS_COLOR_ATTACHMENT_WRITE_BIT;
			break;

		case VK_IMAGE_LAYOUT_DEPTH_STENCIL_ATTACHMENT_OPTIMAL:
		case VK_IMAGE_LAYOUT_DEPTH_ATTACHMENT_OPTIMAL:
		case VK_IMAGE_LAYOUT_STENCIL_ATTACHMENT_OPTIMAL:
			Flags = VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_READ_BIT | VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_WRITE_BIT;
			break;

		case VK_IMAGE_LAYOUT_DEPTH_READ_ONLY_STENCIL_ATTACHMENT_OPTIMAL:
		case VK_IMAGE_LAYOUT_DEPTH_ATTACHMENT_STENCIL_READ_ONLY_OPTIMAL:
			Flags = VK_ACCESS_SHADER_READ_BIT | VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_READ_BIT | VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_WRITE_BIT;
			break;

		case VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL:
			Flags = VK_ACCESS_SHADER_READ_BIT;
			break;

		case VK_IMAGE_LAYOUT_DEPTH_STENCIL_READ_ONLY_OPTIMAL:
		case VK_IMAGE_LAYOUT_DEPTH_READ_ONLY_OPTIMAL:
		case VK_IMAGE_LAYOUT_STENCIL_READ_ONLY_OPTIMAL:
			Flags = VK_ACCESS_SHADER_READ_BIT | VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_READ_BIT;
			break;

		case VK_IMAGE_LAYOUT_PRESENT_SRC_KHR:
			Flags = 0;
			break;

		case VK_IMAGE_LAYOUT_FRAGMENT_DENSITY_MAP_OPTIMAL_EXT:
			Flags = VK_ACCESS_FRAGMENT_DENSITY_MAP_READ_BIT_EXT;
			break;

		case VK_IMAGE_LAYOUT_FRAGMENT_SHADING_RATE_ATTACHMENT_OPTIMAL_KHR:
			Flags = VK_ACCESS_FRAGMENT_SHADING_RATE_ATTACHMENT_READ_BIT_KHR;
			break;

		case VK_IMAGE_LAYOUT_GENERAL:
			// todo-jn: could be used for R64 in read layout
		case VK_IMAGE_LAYOUT_UNDEFINED:
			Flags = 0;
			break;

		case VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL:
			// todo-jn: sync2 currently only used by depth/stencil targets
			Flags = VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_READ_BIT | VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_WRITE_BIT;
			break;

		case VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL:
			// todo-jn: sync2 currently only used by depth/stencil targets
			Flags = VK_ACCESS_SHADER_READ_BIT;
			break;

		default:
			checkNoEntry();
			break;
	}

	return Flags;
}

static VkPipelineStageFlags GetVkStageFlagsForLayout(VkImageLayout Layout)
{
	VkPipelineStageFlags Flags = 0;

	switch (Layout)
	{
		case VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL:
			Flags = VK_PIPELINE_STAGE_TRANSFER_BIT;
			break;

		case VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL:
			Flags = VK_PIPELINE_STAGE_TRANSFER_BIT;
			break;

		case VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL:
			Flags = VK_PIPELINE_STAGE_COLOR_ATTACHMENT_OUTPUT_BIT;
			break;

		case VK_IMAGE_LAYOUT_DEPTH_STENCIL_ATTACHMENT_OPTIMAL:
		case VK_IMAGE_LAYOUT_DEPTH_ATTACHMENT_OPTIMAL:
		case VK_IMAGE_LAYOUT_STENCIL_ATTACHMENT_OPTIMAL:
			Flags = VK_PIPELINE_STAGE_EARLY_FRAGMENT_TESTS_BIT | VK_PIPELINE_STAGE_LATE_FRAGMENT_TESTS_BIT;
			break;

		case VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL:
			Flags = VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT;
			break;

		case VK_IMAGE_LAYOUT_DEPTH_STENCIL_READ_ONLY_OPTIMAL:
		case VK_IMAGE_LAYOUT_DEPTH_READ_ONLY_STENCIL_ATTACHMENT_OPTIMAL:
		case VK_IMAGE_LAYOUT_DEPTH_ATTACHMENT_STENCIL_READ_ONLY_OPTIMAL:
		case VK_IMAGE_LAYOUT_DEPTH_READ_ONLY_OPTIMAL:
		case VK_IMAGE_LAYOUT_STENCIL_READ_ONLY_OPTIMAL:
			Flags = VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT | VK_PIPELINE_STAGE_EARLY_FRAGMENT_TESTS_BIT | VK_PIPELINE_STAGE_LATE_FRAGMENT_TESTS_BIT;
			break;

		case VK_IMAGE_LAYOUT_PRESENT_SRC_KHR:
			Flags = VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT;
			break;

		case VK_IMAGE_LAYOUT_FRAGMENT_DENSITY_MAP_OPTIMAL_EXT:
			Flags = VK_PIPELINE_STAGE_FRAGMENT_DENSITY_PROCESS_BIT_EXT;
			break;

		case VK_IMAGE_LAYOUT_FRAGMENT_SHADING_RATE_ATTACHMENT_OPTIMAL_KHR:
			Flags = VK_PIPELINE_STAGE_FRAGMENT_SHADING_RATE_ATTACHMENT_BIT_KHR;
			break;
			
		case VK_IMAGE_LAYOUT_GENERAL:
		case VK_IMAGE_LAYOUT_UNDEFINED:
			Flags = VK_PIPELINE_STAGE_TOP_OF_PIPE_BIT;
			break;

		case VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL:
			// todo-jn: sync2 currently only used by depth/stencil targets
			Flags = VK_PIPELINE_STAGE_EARLY_FRAGMENT_TESTS_BIT | VK_PIPELINE_STAGE_LATE_FRAGMENT_TESTS_BIT;
			break;

		case VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL:
			// todo-jn: sync2 currently only used by depth/stencil targets
			Flags = VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT | VK_PIPELINE_STAGE_EARLY_FRAGMENT_TESTS_BIT | VK_PIPELINE_STAGE_LATE_FRAGMENT_TESTS_BIT;
			break;

		default:
			checkNoEntry();
			break;
	}

	return Flags;
}

//
// Get the Vulkan stage flags, access flags and image layout (if relevant) corresponding to an ERHIAccess value from the public API.
//
static void GetVkStageAndAccessFlags(ERHIAccess RHIAccess, FRHITransitionInfo::EType ResourceType, uint32 UsageFlags, bool bIsDepthStencil, bool bSupportsReadOnlyOptimal, VkPipelineStageFlags& StageFlags, VkAccessFlags& AccessFlags, VkImageLayout& Layout, bool bIsSourceState)
{
	// From Vulkan's point of view, when performing a multisample resolve via a render pass attachment, resolve targets are the same as render targets .
	// The caller signals this situation by setting both the RTV and ResolveDst flags, and we simply remove ResolveDst in that case,
	// to treat the resource as a render target.
	const ERHIAccess ResolveAttachmentAccess = (ERHIAccess)(ERHIAccess::RTV | ERHIAccess::ResolveDst);
	if (RHIAccess == ResolveAttachmentAccess)
	{
		RHIAccess = ERHIAccess::RTV;
	}

	// BVHRead state may be combined with SRV, but we always treat this as just BVHRead by clearing the SRV mask
	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::BVHRead))
	{
		RHIAccess &= ~ERHIAccess::SRVMask;
	}

	Layout = VK_IMAGE_LAYOUT_UNDEFINED;

	// The layout to use if SRV access is requested. In case of depth/stencil buffers, we don't need to worry about different states for the separate aspects, since that's handled explicitly elsewhere,
	// and this function is never called for depth-only or stencil-only transitions.
	const VkImageLayout SRVLayout = 
		bIsDepthStencil ? VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL : 
		bSupportsReadOnlyOptimal ? VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL : VK_IMAGE_LAYOUT_GENERAL;

	// States which cannot be combined.
	switch (RHIAccess)
	{
		case ERHIAccess::Discard:
			// FIXME: Align with Unknown for now, this could perhaps be less brutal
			StageFlags = VK_PIPELINE_STAGE_ALL_COMMANDS_BIT;
			AccessFlags = VK_ACCESS_MEMORY_READ_BIT | VK_ACCESS_MEMORY_WRITE_BIT;
			Layout = bIsSourceState ? VK_IMAGE_LAYOUT_UNDEFINED : SRVLayout;
			return;

		case ERHIAccess::Unknown:
			// We don't know where this is coming from, so we'll stall everything.
			StageFlags = VK_PIPELINE_STAGE_ALL_COMMANDS_BIT;
			AccessFlags = VK_ACCESS_MEMORY_READ_BIT | VK_ACCESS_MEMORY_WRITE_BIT;
			return;

		case ERHIAccess::CPURead:
			StageFlags = VK_PIPELINE_STAGE_HOST_BIT;
			AccessFlags = VK_ACCESS_HOST_READ_BIT;
			Layout = VK_IMAGE_LAYOUT_GENERAL;
			return;

		case ERHIAccess::Present:
			StageFlags = bIsSourceState ? VK_PIPELINE_STAGE_COLOR_ATTACHMENT_OUTPUT_BIT : VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT;
			// When transitionning out of present, the sema handles access.
			// When transitioning into present, vkQueuePresentKHR guarantees visibility.
			AccessFlags = 0;
			Layout = VK_IMAGE_LAYOUT_PRESENT_SRC_KHR;
			return;

		case ERHIAccess::RTV:
			StageFlags = VK_PIPELINE_STAGE_COLOR_ATTACHMENT_OUTPUT_BIT;
			AccessFlags = VK_ACCESS_COLOR_ATTACHMENT_READ_BIT | VK_ACCESS_COLOR_ATTACHMENT_WRITE_BIT;
			Layout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;
			return;

		case ERHIAccess::CopyDest:
			StageFlags = VK_PIPELINE_STAGE_TRANSFER_BIT;
			AccessFlags = VK_ACCESS_TRANSFER_WRITE_BIT;
			Layout = VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL;
			return;

		case ERHIAccess::ResolveDst:
			// Used when doing a resolve via RHICopyToResolveTarget. For us, it's the same as CopyDst.
			StageFlags = VK_PIPELINE_STAGE_TRANSFER_BIT;
			AccessFlags = VK_ACCESS_TRANSFER_WRITE_BIT;
			Layout = VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL;
			return;

		case ERHIAccess::BVHRead:
			// vkrt todo: Finer grain stage flags would be ideal here.
			StageFlags = VK_PIPELINE_STAGE_ACCELERATION_STRUCTURE_BUILD_BIT_KHR | VK_PIPELINE_STAGE_COMPUTE_SHADER_BIT;
			if (GRHISupportsRayTracingShaders) {
				StageFlags |= VK_PIPELINE_STAGE_RAY_TRACING_SHADER_BIT_KHR;
			}
			AccessFlags = VK_ACCESS_ACCELERATION_STRUCTURE_READ_BIT_KHR;
			return;

		case ERHIAccess::BVHWrite:
			// vkrt todo: Finer grain stage flags would be ideal here.
			StageFlags = VK_PIPELINE_STAGE_ACCELERATION_STRUCTURE_BUILD_BIT_KHR | VK_PIPELINE_STAGE_COMPUTE_SHADER_BIT;
			if (GRHISupportsRayTracingShaders) {
				StageFlags |= VK_PIPELINE_STAGE_RAY_TRACING_SHADER_BIT_KHR;
			}
			AccessFlags = VK_ACCESS_ACCELERATION_STRUCTURE_WRITE_BIT_KHR;
			return;
	}

	// If DSVWrite is set, we ignore everything else because it decides the layout.
	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::DSVWrite))
	{
		check(bIsDepthStencil);
		StageFlags = VK_PIPELINE_STAGE_EARLY_FRAGMENT_TESTS_BIT | VK_PIPELINE_STAGE_LATE_FRAGMENT_TESTS_BIT;
		AccessFlags = VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_READ_BIT | VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_WRITE_BIT;
		Layout = VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL;
		return;
	}

	// The remaining flags can be combined.
	StageFlags = 0;
	AccessFlags = 0;
	uint32 ProcessedRHIFlags = 0;

	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::IndirectArgs))
	{
		check(ResourceType != FRHITransitionInfo::EType::Texture);
		StageFlags |= VK_PIPELINE_STAGE_DRAW_INDIRECT_BIT;
		AccessFlags |= VK_ACCESS_INDIRECT_COMMAND_READ_BIT;

		ProcessedRHIFlags |= (uint32)ERHIAccess::IndirectArgs;
	}

	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::VertexOrIndexBuffer))
	{
		check(ResourceType != FRHITransitionInfo::EType::Texture);
		StageFlags |= VK_PIPELINE_STAGE_VERTEX_INPUT_BIT;
		switch (ResourceType)
		{
			case FRHITransitionInfo::EType::Buffer:
				if ((UsageFlags & VK_BUFFER_USAGE_INDEX_BUFFER_BIT) != 0)
				{
					AccessFlags |= VK_ACCESS_INDEX_READ_BIT;
				}
				if ((UsageFlags & VK_BUFFER_USAGE_VERTEX_BUFFER_BIT) != 0)
				{
					AccessFlags |= VK_ACCESS_VERTEX_ATTRIBUTE_READ_BIT;
				}
				break;
			default:
				checkNoEntry();
				break;
		}

		ProcessedRHIFlags |= (uint32)ERHIAccess::VertexOrIndexBuffer;
	}

	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::DSVRead))
	{
		check(bIsDepthStencil);
		StageFlags |= VK_PIPELINE_STAGE_EARLY_FRAGMENT_TESTS_BIT | VK_PIPELINE_STAGE_LATE_FRAGMENT_TESTS_BIT;
		AccessFlags |= VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_READ_BIT;

		// If any of the SRV flags is set, the code below will set Layout to SRVLayout again, but it's fine since
		// SRVLayout takes into account bIsDepthStencil and ends up being the same as what we set here.
		Layout = VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL;

		ProcessedRHIFlags |= (uint32)ERHIAccess::DSVRead;
	}

	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::SRVGraphicsNonPixel))
	{
		StageFlags |= (GVulkanDeviceGraphicsPipelineStageBits & ~VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT);
		AccessFlags |= VK_ACCESS_SHADER_READ_BIT;
		if ((UsageFlags & VK_BUFFER_USAGE_UNIFORM_BUFFER_BIT) != 0)
		{
			AccessFlags |= VK_ACCESS_UNIFORM_READ_BIT;
		}
		if ((UsageFlags & VK_BUFFER_USAGE_INDEX_BUFFER_BIT) != 0)
		{
			StageFlags |= VK_PIPELINE_STAGE_VERTEX_INPUT_BIT;
			AccessFlags |= VK_ACCESS_INDEX_READ_BIT;
		}
		if ((UsageFlags & VK_BUFFER_USAGE_VERTEX_BUFFER_BIT) != 0)
		{
			StageFlags |= VK_PIPELINE_STAGE_VERTEX_INPUT_BIT;
			AccessFlags |= VK_ACCESS_VERTEX_ATTRIBUTE_READ_BIT;
		}
		Layout = SRVLayout;

		ProcessedRHIFlags |= (uint32)ERHIAccess::SRVGraphicsNonPixel;
	}

	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::SRVGraphicsPixel))
	{
		StageFlags |= VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT;
		AccessFlags |= VK_ACCESS_SHADER_READ_BIT;
		if ((UsageFlags & VK_BUFFER_USAGE_UNIFORM_BUFFER_BIT) != 0)
		{
			AccessFlags |= VK_ACCESS_UNIFORM_READ_BIT;
		}
		Layout = SRVLayout;

		ProcessedRHIFlags |= (uint32)ERHIAccess::SRVGraphicsPixel;
	}

	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::SRVCompute))
	{
		StageFlags |= VK_PIPELINE_STAGE_COMPUTE_SHADER_BIT;
		AccessFlags |= VK_ACCESS_SHADER_READ_BIT;
		// There are cases where we ping-pong images between UAVCompute and SRVCompute. In that case it may be more efficient to leave the image in VK_IMAGE_LAYOUT_GENERAL
		// (at the very least, it will mean fewer image barriers). There's no good way to detect this though, so it might be better if the high level code just did UAV
		// to UAV transitions in that case, instead of SRV <-> UAV.
		Layout = SRVLayout;

		ProcessedRHIFlags |= (uint32)ERHIAccess::SRVCompute;
	}

	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::UAVGraphics))
	{
		StageFlags |= GVulkanDeviceGraphicsPipelineStageBits;
		AccessFlags |= VK_ACCESS_SHADER_READ_BIT | VK_ACCESS_SHADER_WRITE_BIT;
		Layout = VK_IMAGE_LAYOUT_GENERAL;

		ProcessedRHIFlags |= (uint32)ERHIAccess::UAVGraphics;
	}
			
	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::UAVCompute))
	{
		StageFlags |= VK_PIPELINE_STAGE_COMPUTE_SHADER_BIT;
		AccessFlags |= VK_ACCESS_SHADER_READ_BIT | VK_ACCESS_SHADER_WRITE_BIT;
		Layout = VK_IMAGE_LAYOUT_GENERAL;

		ProcessedRHIFlags |= (uint32)ERHIAccess::UAVCompute;
	}

	// ResolveSrc is used when doing a resolve via RHICopyToResolveTarget. For us, it's the same as CopySrc.
	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::CopySrc | ERHIAccess::ResolveSrc))
	{
		// If this is requested for a texture, behavior will depend on if we're combined with other flags
		if (ResourceType == FRHITransitionInfo::EType::Texture)
		{
			// If no other RHIAccess is mixed in with our CopySrc, then use proper TRANSFER_SRC layout
			if (Layout == VK_IMAGE_LAYOUT_UNDEFINED)
			{
				Layout = VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL;
				StageFlags = VK_PIPELINE_STAGE_TRANSFER_BIT;
				AccessFlags = VK_ACCESS_TRANSFER_READ_BIT;
			}
			else
			{
				// If anything else is mixed in with the CopySrc, then go to the "catch all" GENERAL layout
				Layout = VK_IMAGE_LAYOUT_GENERAL;
				StageFlags |= VK_PIPELINE_STAGE_TRANSFER_BIT;
				AccessFlags |= VK_ACCESS_TRANSFER_READ_BIT;
			}
		}
		else
		{
			StageFlags = VK_PIPELINE_STAGE_TRANSFER_BIT;
			AccessFlags = VK_ACCESS_TRANSFER_READ_BIT;
		}

		ProcessedRHIFlags |= (uint32)(ERHIAccess::CopySrc | ERHIAccess::ResolveSrc);
	}

	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::ShadingRateSource) && ValidateShadingRateDataType())
	{
		checkf(ResourceType == FRHITransitionInfo::EType::Texture, TEXT("A non-texture resource was tagged as a shading rate source; only textures (Texture2D and Texture2DArray) can be used for this purpose."));
		
		if (GRHIVariableRateShadingImageDataType == VRSImage_Palette)
		{
			StageFlags = VK_PIPELINE_STAGE_FRAGMENT_SHADING_RATE_ATTACHMENT_BIT_KHR;
			AccessFlags = VK_ACCESS_FRAGMENT_SHADING_RATE_ATTACHMENT_READ_BIT_KHR;
			Layout = VK_IMAGE_LAYOUT_FRAGMENT_SHADING_RATE_ATTACHMENT_OPTIMAL_KHR;
		}

		if (GRHIVariableRateShadingImageDataType == VRSImage_Fractional)
		{
			StageFlags = VK_PIPELINE_STAGE_FRAGMENT_DENSITY_PROCESS_BIT_EXT;
			AccessFlags = VK_ACCESS_FRAGMENT_DENSITY_MAP_READ_BIT_EXT;
			Layout = VK_IMAGE_LAYOUT_FRAGMENT_DENSITY_MAP_OPTIMAL_EXT;
		}

		ProcessedRHIFlags |= (uint32)ERHIAccess::ShadingRateSource;
	}

	uint32 RemainingFlags = (uint32)RHIAccess & (~ProcessedRHIFlags);
	ensureMsgf(RemainingFlags == 0, TEXT("Some access flags were not processed. RHIAccess=%x, ProcessedRHIFlags=%x, RemainingFlags=%x"), EnumToUnderlyingType(RHIAccess), ProcessedRHIFlags, RemainingFlags);
}


static VkImageAspectFlags GetDepthStencilAspectMask(uint32 PlaneSlice)
{
	VkImageAspectFlags AspectFlags = 0;

	if (PlaneSlice == FRHISubresourceRange::kAllSubresources)
	{
		AspectFlags = VK_IMAGE_ASPECT_DEPTH_BIT | VK_IMAGE_ASPECT_STENCIL_BIT;
	}

	if (PlaneSlice == FRHISubresourceRange::kDepthPlaneSlice)
	{
		AspectFlags = VK_IMAGE_ASPECT_DEPTH_BIT;
	}

	if (PlaneSlice == FRHISubresourceRange::kStencilPlaneSlice)
	{
		AspectFlags = VK_IMAGE_ASPECT_STENCIL_BIT;
	}

	return AspectFlags;
}


// Returns the VK_KHR_synchronization2 layout corresponding to an access type
VkImageLayout FVulkanPipelineBarrier::GetDepthOrStencilLayout(ERHIAccess Access)
{
	VkImageLayout Layout;
	if (Access == ERHIAccess::Unknown)
	{
		Layout = VK_IMAGE_LAYOUT_UNDEFINED;
	}
	else if (Access == ERHIAccess::Discard)
	{
		Layout = VK_IMAGE_LAYOUT_UNDEFINED;
	}
	else if (EnumHasAnyFlags(Access, ERHIAccess::CopySrc))
	{
		Layout = VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL;
	}
	else if (EnumHasAnyFlags(Access, ERHIAccess::CopyDest))
	{
		Layout = VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL;
	}
	else if (EnumHasAnyFlags(Access, ERHIAccess::DSVWrite))
	{
		Layout = VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL;
	}
	else
	{
		Layout = VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL;
	}
	return Layout;
}

static void GetDepthOrStencilStageAndAccessFlags(ERHIAccess Access, VkPipelineStageFlags& StageFlags, VkAccessFlags& AccessFlags)
{
	if (Access == ERHIAccess::Unknown || Access == ERHIAccess::Discard)
	{
		StageFlags = VK_PIPELINE_STAGE_ALL_COMMANDS_BIT;
		AccessFlags = VK_ACCESS_MEMORY_READ_BIT | VK_ACCESS_MEMORY_WRITE_BIT;
		return;
	}

	StageFlags = 0;
	AccessFlags = 0;
	uint32 ProcessedRHIFlags = 0;

	if (EnumHasAllFlags(Access, ERHIAccess::ResolveDst))
	{
		// Despite being a depth/stencil target, resolve operations are part of the color attachment output stage
		StageFlags |= VK_PIPELINE_STAGE_COLOR_ATTACHMENT_OUTPUT_BIT;
		AccessFlags |= VK_ACCESS_COLOR_ATTACHMENT_WRITE_BIT;
		ProcessedRHIFlags |= (uint32)ERHIAccess::ResolveDst;
	}

	if (EnumHasAnyFlags(Access, ERHIAccess::DSVWrite))
	{
		StageFlags |= VK_PIPELINE_STAGE_EARLY_FRAGMENT_TESTS_BIT | VK_PIPELINE_STAGE_LATE_FRAGMENT_TESTS_BIT;
		AccessFlags |= VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_READ_BIT | VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_WRITE_BIT;
		ProcessedRHIFlags |= (uint32)ERHIAccess::DSVWrite;
	}
	
	if (EnumHasAnyFlags(Access, ERHIAccess::DSVRead))
	{
		StageFlags |= VK_PIPELINE_STAGE_EARLY_FRAGMENT_TESTS_BIT | VK_PIPELINE_STAGE_LATE_FRAGMENT_TESTS_BIT;
		AccessFlags |= VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_READ_BIT;
		ProcessedRHIFlags |= (uint32)ERHIAccess::DSVRead;
	}

	if (EnumHasAnyFlags(Access, ERHIAccess::SRVGraphics))
	{
		StageFlags |= VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT;
		AccessFlags |= VK_ACCESS_SHADER_READ_BIT;
		ProcessedRHIFlags |= (uint32)ERHIAccess::SRVGraphics;
	}

	if (EnumHasAnyFlags(Access, ERHIAccess::UAVGraphics))
	{
		StageFlags |= VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT;
		AccessFlags |= VK_ACCESS_SHADER_READ_BIT | VK_ACCESS_SHADER_WRITE_BIT;
		ProcessedRHIFlags |= (uint32)ERHIAccess::UAVGraphics;
	}

	if (EnumHasAnyFlags(Access, ERHIAccess::SRVCompute))
	{
		StageFlags |= VK_PIPELINE_STAGE_COMPUTE_SHADER_BIT;
		AccessFlags |= VK_ACCESS_SHADER_READ_BIT;
		ProcessedRHIFlags |= (uint32)ERHIAccess::SRVCompute;
	}

	if (EnumHasAnyFlags(Access, ERHIAccess::UAVCompute))
	{
		StageFlags |= VK_PIPELINE_STAGE_COMPUTE_SHADER_BIT;
		AccessFlags |= VK_ACCESS_SHADER_READ_BIT | VK_ACCESS_SHADER_WRITE_BIT;
		ProcessedRHIFlags |= (uint32)ERHIAccess::UAVCompute;
	}

	if (EnumHasAnyFlags(Access, ERHIAccess::CopySrc))
	{
		StageFlags |= VK_PIPELINE_STAGE_TRANSFER_BIT;
		AccessFlags |= VK_ACCESS_TRANSFER_READ_BIT;
		ProcessedRHIFlags |= (uint32)ERHIAccess::CopySrc;
	}

	if (EnumHasAnyFlags(Access, ERHIAccess::CopyDest))
	{
		StageFlags |= VK_PIPELINE_STAGE_TRANSFER_BIT;
		AccessFlags |= VK_ACCESS_TRANSFER_WRITE_BIT;
		ProcessedRHIFlags |= (uint32)ERHIAccess::CopyDest;
	}

	const uint32 RemainingFlags = (uint32)Access & (~ProcessedRHIFlags);
	ensureMsgf(RemainingFlags == 0, TEXT("Some access flags were not processed. Access=%x, ProcessedRHIFlags=%x, RemainingFlags=%x"), EnumToUnderlyingType(Access), ProcessedRHIFlags, RemainingFlags);
}



//
// Helpers for filling in the fields of a VkImageMemoryBarrier structure.
//
static void SetupImageBarrier(VkImageMemoryBarrier2& ImgBarrier, VkImage Image, VkPipelineStageFlags SrcStageFlags, VkPipelineStageFlags DstStageFlags, 
	VkAccessFlags SrcAccessFlags, VkAccessFlags DstAccessFlags, VkImageLayout SrcLayout, VkImageLayout DstLayout, const VkImageSubresourceRange& SubresRange)
{
	ImgBarrier.sType = VK_STRUCTURE_TYPE_IMAGE_MEMORY_BARRIER_2;
	ImgBarrier.pNext = nullptr;
	ImgBarrier.srcStageMask = SrcStageFlags;
	ImgBarrier.dstStageMask = DstStageFlags;
	ImgBarrier.srcAccessMask = SrcAccessFlags;
	ImgBarrier.dstAccessMask = DstAccessFlags;
	ImgBarrier.oldLayout = SrcLayout;
	ImgBarrier.newLayout = DstLayout;
	ImgBarrier.srcQueueFamilyIndex = VK_QUEUE_FAMILY_IGNORED;
	ImgBarrier.dstQueueFamilyIndex = VK_QUEUE_FAMILY_IGNORED;
	ImgBarrier.image = Image;
	ImgBarrier.subresourceRange = SubresRange;
}

static void SetupImageBarrierEntireRes(VkImageMemoryBarrier2& ImgBarrier, VkImage Image, VkPipelineStageFlags SrcStageFlags, VkPipelineStageFlags DstStageFlags, 
	VkAccessFlags SrcAccessFlags, VkAccessFlags DstAccessFlags, VkImageLayout SrcLayout, VkImageLayout DstLayout, VkImageAspectFlags AspectMask)
{
	VkImageSubresourceRange SubresRange;
	SubresRange.aspectMask = AspectMask;
	SubresRange.baseMipLevel = 0;
	SubresRange.levelCount = VK_REMAINING_MIP_LEVELS;
	SubresRange.baseArrayLayer = 0;
	SubresRange.layerCount = VK_REMAINING_ARRAY_LAYERS;
	SetupImageBarrier(ImgBarrier, Image, SrcStageFlags, DstStageFlags, SrcAccessFlags, DstAccessFlags, SrcLayout, DstLayout, SubresRange);
}


void FVulkanDynamicRHI::RHICreateTransition(FRHITransition* Transition, const FRHITransitionCreateInfo& CreateInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RHICreateTransition);

	const ERHIPipeline SrcPipelines = CreateInfo.SrcPipelines;
	const ERHIPipeline DstPipelines = CreateInfo.DstPipelines;

	FVulkanTransitionData* Data = new (Transition->GetPrivateData<FVulkanTransitionData>()) FVulkanTransitionData;
	Data->TransitionInfos = CreateInfo.TransitionInfos;
	Data->SrcPipelines = SrcPipelines;
	Data->DstPipelines = DstPipelines;

	// If queue ownership transfers are necessary, always use fences on cross queue to ensure they are submitted in order
	if ((SrcPipelines != DstPipelines) && (Device->NeedsQueueFamilyOwnershipTransfers() || !EnumHasAnyFlags(CreateInfo.Flags, ERHITransitionCreateFlags::NoFence)))
	{
		Data->Semaphore = new FVulkanSemaphore(*Device);
	}

	// If we're staying on the same queue, use split barriers if they are permitted and supported
	if (GVulkanAllowSplitBarriers && (GVulkanMaxBarriersPerBatch <= 0) && Device->SupportsParallelRendering() &&
		(SrcPipelines == DstPipelines) && !EnumHasAnyFlags(CreateInfo.Flags, ERHITransitionCreateFlags::NoSplit))
	{
		// Track if host stage is used, it will prevent using split barrier
		bool bIncludesHostSync = false;
		for (const FRHITransitionInfo& Info : Data->TransitionInfos)
		{
			if (EnumHasAnyFlags(Info.AccessBefore, ERHIAccess::CPURead) || EnumHasAnyFlags(Info.AccessAfter, ERHIAccess::CPURead))
			{
				bIncludesHostSync = true;
				break;
			}
		}

		// Create an event for the split barriers
		Data->EventHandle = !bIncludesHostSync ? Device->GetBarrierEvent() : VK_NULL_HANDLE;
	}
}

void FVulkanDynamicRHI::RHIReleaseTransition(FRHITransition* Transition)
{
	Transition->GetPrivateData<FVulkanTransitionData>()->~FVulkanTransitionData();
}

// We need to keep the texture pointers around, because we need to call OnTransitionResource on them, and we need mip and layer counts for the tracking code.
struct FImageBarrierExtraData
{
	FVulkanTexture* BaseTexture = nullptr;
	bool IsAliasingBarrier = false;
	uint8 PlaneSlice = 0;
	ERHIAccess PlaneAccess = ERHIAccess::Unknown;
};


struct FLegacyBarrierArrays
{
	TArray<VkMemoryBarrier, TInlineAllocator<1>> MemoryBarriers;
	TArray<VkBufferMemoryBarrier> BufferBarriers;
	TArray<VkImageMemoryBarrier> ImageBarriers;
	TArray<FImageBarrierExtraData> ImageExtraData;

	VkPipelineStageFlags SrcStageMask = 0;
	VkPipelineStageFlags DstStageMask = 0;
};

struct FSync2BarrierArrays
{
	TArray<VkMemoryBarrier2, TInlineAllocator<1>> MemoryBarriers;
	TArray<VkBufferMemoryBarrier2> BufferBarriers;
	TArray<VkImageMemoryBarrier2> ImageBarriers;
};


template<typename BarrierArrayType>
void ConvertTransitionToBarriers(FVulkanCommandListContext& Context, const FVulkanTransitionData& Data, BarrierArrayType& OutBarriers)
{
	using MemoryBarrierType = typename decltype(OutBarriers.MemoryBarriers)::ElementType;
	using BufferBarrierType = typename decltype(OutBarriers.BufferBarriers)::ElementType;
	using ImageBarrierType = typename decltype(OutBarriers.ImageBarriers)::ElementType;

	// Legacy barriers use a single stage for all barriers
	constexpr bool bIsLegacyBarrier = std::is_same_v<FLegacyBarrierArrays, BarrierArrayType>;

	// Count the images and buffers to be able to pre-allocate the arrays.
	int32 NumTextures = 0, NumBuffers = 0;
	for (const FRHITransitionInfo& Info : Data.TransitionInfos)
	{
		if (!Info.Resource)
		{
			continue;
		}

		if (EnumHasAnyFlags(Info.AccessAfter, ERHIAccess::Discard))
		{
			// Discard as a destination is a no-op
			continue;
		}

		if (Info.Type == FRHITransitionInfo::EType::Texture)
		{
			// CPU accessible "textures" are implemented as buffers. Check if this is a real texture or a buffer.
			FVulkanTexture* Texture = ResourceCast(Info.Texture);
			if (Texture->GetCpuReadbackBuffer() == nullptr)
			{
				++NumTextures;
			}
			continue;
		}

		if (Info.Type == FRHITransitionInfo::EType::UAV)
		{
			FVulkanUnorderedAccessView* UAV = ResourceCast(Info.UAV);
			if (UAV->IsTexture())
			{
				++NumTextures;
				continue;
			}
		}

		if (Data.SrcPipelines != Data.DstPipelines)
		{
			++NumBuffers;
		}
	}


	// Presize all the arrays
	if (!GVulkanUseMemoryBarrierOpt)
	{
		OutBarriers.BufferBarriers.Reserve(OutBarriers.BufferBarriers.Num() + NumBuffers);
	}
	OutBarriers.ImageBarriers.Reserve(OutBarriers.ImageBarriers.Num() + NumTextures);


	const ERHIAccess DepthStencilFlags = ERHIAccess::DSVRead | ERHIAccess::DSVWrite;

	for (const FRHITransitionInfo& Info : Data.TransitionInfos)
	{
		if (!Info.Resource)
		{
			continue;
		}

		if (Info.AccessAfter == ERHIAccess::Discard)
		{
			// Discard as a destination is a no-op
			continue;
		}

		const UE::RHICore::FResourceState ResourceState(Context, Data.SrcPipelines, Data.DstPipelines, Info);

		FVulkanBuffer* Buffer = nullptr;
		FVulkanTexture* Texture = nullptr;
		FRHITransitionInfo::EType UnderlyingType = Info.Type;
		uint32 UsageFlags = 0;

		switch (Info.Type)
		{
		case FRHITransitionInfo::EType::Texture:
		{
			Texture = ResourceCast(Info.Texture);
			if (Texture->GetCpuReadbackBuffer())
			{
				Texture = nullptr;
			}
			break;
		}

		case FRHITransitionInfo::EType::Buffer:
		{
			Buffer = ResourceCast(Info.Buffer);
			UsageFlags = Buffer->GetBufferUsageFlags();
			break;
		}

		case FRHITransitionInfo::EType::UAV:
		{
			FVulkanUnorderedAccessView* UAV = ResourceCast(Info.UAV);
			if (UAV->IsTexture())
			{
				Texture = ResourceCast(UAV->GetTexture());
				UnderlyingType = FRHITransitionInfo::EType::Texture;
			}
			else
			{
				Buffer = ResourceCast(UAV->GetBuffer());
				UnderlyingType = FRHITransitionInfo::EType::Buffer;
				UsageFlags = Buffer->GetBufferUsageFlags();
			}
			break;
		}

		case FRHITransitionInfo::EType::BVH:
		{
			// Requires memory barrier
			break;
		}

		default:
			checkNoEntry();
			continue;
		}

		VkPipelineStageFlags SrcStageMask, DstStageMask;
		VkAccessFlags SrcAccessFlags, DstAccessFlags;
		VkImageLayout SrcLayout, DstLayout;

		const bool bIsDepthStencil = Texture && Texture->IsDepthOrStencilAspect();

		if (bIsDepthStencil)
		{
			// if we use separate transitions, then just feed them in as they are
			SrcLayout = FVulkanPipelineBarrier::GetDepthOrStencilLayout(ResourceState.AccessBefore);
			DstLayout = FVulkanPipelineBarrier::GetDepthOrStencilLayout(ResourceState.AccessAfter);
			GetDepthOrStencilStageAndAccessFlags(ResourceState.AccessBefore, SrcStageMask, SrcAccessFlags);
			GetDepthOrStencilStageAndAccessFlags(ResourceState.AccessAfter, DstStageMask, DstAccessFlags);
		}
		else
		{
			
			const bool bSupportsReadOnlyOptimal = (Texture == nullptr) || Texture->SupportsSampling();

			GetVkStageAndAccessFlags(ResourceState.AccessBefore, UnderlyingType, UsageFlags, bIsDepthStencil, bSupportsReadOnlyOptimal, SrcStageMask, SrcAccessFlags, SrcLayout, true);
			GetVkStageAndAccessFlags(ResourceState.AccessAfter, UnderlyingType, UsageFlags, bIsDepthStencil, bSupportsReadOnlyOptimal, DstStageMask, DstAccessFlags, DstLayout, false);

			// Keep this hack around until we know we can always be running with UseDedicatedNonPixelAccess
			if (!GRHIGlobals.SupportsDedicatedSRVGraphicsNonPixelAccess)
			{
				// If not compute, remove vertex pipeline bits as only compute updates vertex buffers
				// EXCEPT the special case for SRVGraphicsNonPixel where Vertex needs to be preserved
				const bool bKeepVertex = Texture && EnumHasAnyFlags(ResourceState.AccessAfter, ERHIAccess::SRVGraphicsNonPixel) && !EnumHasAnyFlags(ResourceState.AccessAfter, ERHIAccess::SRVGraphicsPixel);
				if (!bKeepVertex && !VKHasAllFlags(SrcStageMask, VK_PIPELINE_STAGE_COMPUTE_SHADER_BIT))
				{
					DstStageMask &= ~(VK_PIPELINE_STAGE_VERTEX_SHADER_BIT | VK_PIPELINE_STAGE_GEOMETRY_SHADER_BIT);
				}
			}
		}

		// Mash them all together for legacy barriers, set them for each barrier in sync2
		if constexpr (bIsLegacyBarrier)
		{
			OutBarriers.SrcStageMask |= SrcStageMask;
			OutBarriers.DstStageMask |= DstStageMask;
		}

		// If we're not transitioning across pipes and we don't need to perform layout transitions, we can express memory dependencies through a global memory barrier.
		if ((Data.SrcPipelines == Data.DstPipelines) && (Texture == nullptr || (SrcLayout == DstLayout)) && GVulkanUseMemoryBarrierOpt)
		{
			if (OutBarriers.MemoryBarriers.Num() == 0)
			{
				MemoryBarrierType& NewBarrier = OutBarriers.MemoryBarriers.AddZeroed_GetRef();
				NewBarrier.sType = bIsLegacyBarrier ? VK_STRUCTURE_TYPE_MEMORY_BARRIER : VK_STRUCTURE_TYPE_MEMORY_BARRIER_2;
			}

			const VkAccessFlags ReadMask = VK_ACCESS_INDIRECT_COMMAND_READ_BIT | VK_ACCESS_INDEX_READ_BIT | VK_ACCESS_VERTEX_ATTRIBUTE_READ_BIT | VK_ACCESS_UNIFORM_READ_BIT |
				VK_ACCESS_INPUT_ATTACHMENT_READ_BIT | VK_ACCESS_SHADER_READ_BIT | VK_ACCESS_COLOR_ATTACHMENT_READ_BIT | VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_READ_BIT |
				VK_ACCESS_TRANSFER_READ_BIT | VK_ACCESS_MEMORY_READ_BIT;

			// Mash everything into a single barrier
			MemoryBarrierType& MemoryBarrier = OutBarriers.MemoryBarriers[0];

			// We only need a memory barrier if the previous commands wrote to the buffer. In case of a transition from read, an execution barrier is enough.
			const bool SrcAccessIsRead = ((SrcAccessFlags & (~ReadMask)) == 0);
			if (!SrcAccessIsRead)
			{
				MemoryBarrier.srcAccessMask |= SrcAccessFlags;
				MemoryBarrier.dstAccessMask |= DstAccessFlags;
			}

			if constexpr (!bIsLegacyBarrier)
			{
				MemoryBarrier.srcStageMask |= SrcStageMask;
				MemoryBarrier.dstStageMask |= DstStageMask;
			}
		}
		else if (Buffer != nullptr)
		{
			BufferBarrierType& BufferBarrier = OutBarriers.BufferBarriers.AddZeroed_GetRef();
			BufferBarrier.sType = bIsLegacyBarrier ? VK_STRUCTURE_TYPE_BUFFER_MEMORY_BARRIER : VK_STRUCTURE_TYPE_BUFFER_MEMORY_BARRIER_2;
			BufferBarrier.srcAccessMask = SrcAccessFlags;
			BufferBarrier.dstAccessMask = DstAccessFlags;
			BufferBarrier.srcQueueFamilyIndex = VK_QUEUE_FAMILY_IGNORED;
			BufferBarrier.dstQueueFamilyIndex = VK_QUEUE_FAMILY_IGNORED;

			const VulkanRHI::FVulkanAllocation& BufferAlloc = Buffer->GetCurrentAllocation();
			BufferBarrier.buffer = BufferAlloc.GetBufferHandle();
			BufferBarrier.offset = BufferAlloc.Offset;
			BufferBarrier.size = BufferAlloc.Size;

			if constexpr (!bIsLegacyBarrier)
			{
				BufferBarrier.srcStageMask = SrcStageMask;
				BufferBarrier.dstStageMask = DstStageMask;
			}
		}
		else if (Texture != nullptr)
		{
			const VkImageAspectFlags AspectFlags = bIsDepthStencil ? GetDepthStencilAspectMask(Info.PlaneSlice) : Texture->GetFullAspectMask();

			ImageBarrierType& ImageBarrier = OutBarriers.ImageBarriers.AddZeroed_GetRef();
			ImageBarrier.sType = bIsLegacyBarrier ? VK_STRUCTURE_TYPE_IMAGE_MEMORY_BARRIER : VK_STRUCTURE_TYPE_IMAGE_MEMORY_BARRIER_2;
			ImageBarrier.srcAccessMask = SrcAccessFlags;
			ImageBarrier.dstAccessMask = DstAccessFlags;
			ImageBarrier.oldLayout = SrcLayout;
			ImageBarrier.newLayout = DstLayout;
			ImageBarrier.srcQueueFamilyIndex = VK_QUEUE_FAMILY_IGNORED;
			ImageBarrier.dstQueueFamilyIndex = VK_QUEUE_FAMILY_IGNORED;
			ImageBarrier.image = Texture->Image;

			// For 3D textures created with VK_IMAGE_CREATE_2D_ARRAY_COMPATIBLE_BIT when Maintenance9 is enabled,
			// a layerCount==1 refers to a single slice (used to refer to the rest of the texture).  Maintain the same behavior
			// with/without Maintenance9 (without altering values for drivers without).
			const FRHITextureDesc& Desc = Texture->GetDesc();
			const bool bUseAllSlices =
				(Desc.Dimension == ETextureDimension::Texture3D) &&
				Context.Device.GetOptionalExtensions().HasKHRMaintenance9 &&
				!EnumHasAnyFlags(Desc.Flags, TexCreate_ReservedResource);

			// Fill the ImageBarrier.subresourceRange from the FRHITransitionInfo
			ImageBarrier.subresourceRange = {
				.aspectMask = AspectFlags,
				.baseMipLevel = Info.IsAllMips() ? 0u : Info.MipIndex,
				.levelCount = Info.IsAllMips() ? VK_REMAINING_MIP_LEVELS : 1u,
				.baseArrayLayer = Info.IsAllArraySlices() ? 0u : Info.ArraySlice,
				.layerCount = (Info.IsAllArraySlices() || bUseAllSlices) ? VK_REMAINING_ARRAY_LAYERS : 1u
			};

			if constexpr (bIsLegacyBarrier)
			{
				FImageBarrierExtraData& ExtraData = OutBarriers.ImageExtraData.AddDefaulted_GetRef();;
				ExtraData.BaseTexture = Texture;
				ExtraData.IsAliasingBarrier = (ResourceState.AccessBefore == ERHIAccess::Discard);

				if (Texture->IsDepthOrStencilAspect())
				{
					ExtraData.PlaneAccess = ResourceState.AccessAfter;
					ExtraData.PlaneSlice = Info.PlaneSlice;
				}
			}
			else
			{
				ImageBarrier.srcStageMask = SrcStageMask;
				ImageBarrier.dstStageMask = DstStageMask;
			}
		}
		else
		{
			checkf(false, TEXT("Transition with no resource!"));
		}
	}
}

template <typename MemoryBarrierType>
static void MergeBarrierAccessMask(MemoryBarrierType& MemoryBarrier, VkAccessFlags InSrcAccessFlags, VkAccessFlags InDstAccessFlags)
{
	const VkAccessFlags ReadMask = VK_ACCESS_INDIRECT_COMMAND_READ_BIT | VK_ACCESS_INDEX_READ_BIT | VK_ACCESS_VERTEX_ATTRIBUTE_READ_BIT | VK_ACCESS_UNIFORM_READ_BIT |
		VK_ACCESS_INPUT_ATTACHMENT_READ_BIT | VK_ACCESS_SHADER_READ_BIT | VK_ACCESS_COLOR_ATTACHMENT_READ_BIT | VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_READ_BIT |
		VK_ACCESS_TRANSFER_READ_BIT | VK_ACCESS_MEMORY_READ_BIT;

	// We only need a memory barrier if the previous commands wrote to the buffer. In case of a transition from read, an execution barrier is enough.
	const bool SrcAccessIsRead = ((InSrcAccessFlags & (~ReadMask)) == 0);
	if (!SrcAccessIsRead)
	{
		MemoryBarrier.srcAccessMask |= InSrcAccessFlags;
		MemoryBarrier.dstAccessMask |= InDstAccessFlags;
	}
}

static void DowngradeBarrier(VkMemoryBarrier& OutBarrier, const VkMemoryBarrier2& InBarrier)
{
	OutBarrier.sType = VK_STRUCTURE_TYPE_MEMORY_BARRIER;
	OutBarrier.pNext = InBarrier.pNext;
	OutBarrier.srcAccessMask = InBarrier.srcAccessMask;
	OutBarrier.dstAccessMask = InBarrier.dstAccessMask;
}

static void DowngradeBarrier(VkBufferMemoryBarrier& OutBarrier, const VkBufferMemoryBarrier2& InBarrier)
{
	OutBarrier.sType = VK_STRUCTURE_TYPE_BUFFER_MEMORY_BARRIER;
	OutBarrier.pNext = InBarrier.pNext;
	OutBarrier.srcAccessMask = InBarrier.srcAccessMask;
	OutBarrier.dstAccessMask = InBarrier.dstAccessMask;
	OutBarrier.srcQueueFamilyIndex = InBarrier.srcQueueFamilyIndex;
	OutBarrier.dstQueueFamilyIndex = InBarrier.dstQueueFamilyIndex;
	OutBarrier.buffer = InBarrier.buffer;
	OutBarrier.offset = InBarrier.offset;
	OutBarrier.size = InBarrier.size;
}

static void DowngradeBarrier(VkImageMemoryBarrier& OutBarrier, const VkImageMemoryBarrier2& InBarrier)
{
	OutBarrier.sType = VK_STRUCTURE_TYPE_IMAGE_MEMORY_BARRIER;
	OutBarrier.pNext = InBarrier.pNext;
	OutBarrier.srcAccessMask = InBarrier.srcAccessMask;
	OutBarrier.dstAccessMask = InBarrier.dstAccessMask;
	OutBarrier.oldLayout = InBarrier.oldLayout;
	OutBarrier.newLayout = InBarrier.newLayout;
	OutBarrier.srcQueueFamilyIndex = InBarrier.srcQueueFamilyIndex;
	OutBarrier.dstQueueFamilyIndex = InBarrier.dstQueueFamilyIndex;
	OutBarrier.image = InBarrier.image;
	OutBarrier.subresourceRange = InBarrier.subresourceRange;
}

template <typename DstArrayType, typename BarrierType>
static void DowngradeBarrier(DstArrayType& TargetArray, const BarrierType& SrcBarrier, VkPipelineStageFlags& MergedSrcStageMask, VkPipelineStageFlags& MergedDstStageMask)
{
	auto& DstBarrier = TargetArray.AddDefaulted_GetRef();
	DowngradeBarrier(DstBarrier, SrcBarrier);
	MergedSrcStageMask |= SrcBarrier.srcStageMask;
	MergedDstStageMask |= SrcBarrier.dstStageMask;
}

template <typename DstArrayType, typename SrcArrayType>
static void DowngradeBarrierArray(DstArrayType& TargetArray, const SrcArrayType& SrcArray, VkPipelineStageFlags& MergedSrcStageMask, VkPipelineStageFlags& MergedDstStageMask)
{
	TargetArray.Reserve(TargetArray.Num() + SrcArray.Num());
	for (const auto& SrcBarrier : SrcArray)
	{
		auto& DstBarrier = TargetArray.AddDefaulted_GetRef();
		DowngradeBarrier(DstBarrier, SrcBarrier);
		MergedSrcStageMask |= SrcBarrier.srcStageMask;
		MergedDstStageMask |= SrcBarrier.dstStageMask;
	}
}

// Legacy manual barriers inside the RHI with FVulkanPipelineBarrier don't have access to tracking, assume same layout for both aspects
template <typename BarrierArrayType>
static void MergeDepthStencilLayouts(BarrierArrayType& TargetArray)
{
	for (auto& Barrier : TargetArray)
	{
		if (VKHasAnyFlags(Barrier.subresourceRange.aspectMask, (VK_IMAGE_ASPECT_DEPTH_BIT | VK_IMAGE_ASPECT_STENCIL_BIT)))
		{
			if (Barrier.newLayout == VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL)
			{
				Barrier.newLayout = VK_IMAGE_LAYOUT_DEPTH_STENCIL_READ_ONLY_OPTIMAL;
			}
			else if (Barrier.newLayout == VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL)
			{
				Barrier.newLayout = VK_IMAGE_LAYOUT_DEPTH_STENCIL_ATTACHMENT_OPTIMAL;
			}

			if (Barrier.oldLayout == VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL)
			{
				Barrier.oldLayout = VK_IMAGE_LAYOUT_DEPTH_STENCIL_READ_ONLY_OPTIMAL;
			}
			else if (Barrier.oldLayout == VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL)
			{
				Barrier.oldLayout = VK_IMAGE_LAYOUT_DEPTH_STENCIL_ATTACHMENT_OPTIMAL;
			}
		}
	}
}










// Legacy barriers must always submit depth and stencil together
static void MergePlanes(VkImageMemoryBarrier* ImageBarriers, const FImageBarrierExtraData* RemainingExtras, int32 RemainingCount)
{
	VkImageMemoryBarrier& ImageBarrier = ImageBarriers[0];
	FVulkanTexture* Texture = RemainingExtras->BaseTexture;
	check(Texture->Image == ImageBarrier.image);
	check(ImageBarrier.newLayout != VK_IMAGE_LAYOUT_UNDEFINED);
	check((ImageBarrier.oldLayout != VK_IMAGE_LAYOUT_UNDEFINED) || RemainingExtras->IsAliasingBarrier);

	// For Depth/Stencil formats where only one of the aspects is transitioned, look ahead for other barriers on the same resource
	if (Texture->IsDepthOrStencilAspect())
	{
		if (Texture->GetFullAspectMask() != ImageBarrier.subresourceRange.aspectMask)
		{
			check(VKHasAnyFlags(ImageBarrier.subresourceRange.aspectMask, VK_IMAGE_ASPECT_DEPTH_BIT | VK_IMAGE_ASPECT_STENCIL_BIT));

			const VkImageAspectFlagBits OtherAspectMask = (VkImageAspectFlagBits)(Texture->GetFullAspectMask() ^ ImageBarrier.subresourceRange.aspectMask);
			VkImageMemoryBarrier* OtherAspectImageBarrier = nullptr;
			for (int32 OtherBarrierIndex = 0; OtherBarrierIndex < RemainingCount; ++OtherBarrierIndex)
			{
				VkImageMemoryBarrier& OtherImageBarrier = ImageBarriers[OtherBarrierIndex];
				FVulkanTexture* OtherTexture = RemainingExtras[OtherBarrierIndex].BaseTexture;
				if ((OtherTexture->Image == ImageBarrier.image) && (OtherImageBarrier.subresourceRange.aspectMask == OtherAspectMask))
				{
					check(ImageBarrier.subresourceRange.baseArrayLayer == OtherImageBarrier.subresourceRange.baseArrayLayer);
					check(ImageBarrier.subresourceRange.baseMipLevel == OtherImageBarrier.subresourceRange.baseMipLevel);

					OtherAspectImageBarrier = &OtherImageBarrier;

					break;
				}
			}

			VkImageLayout OtherAspectOldLayout, OtherAspectNewLayout;
			if (OtherAspectImageBarrier)
			{
				OtherAspectOldLayout = OtherAspectImageBarrier->oldLayout;
				OtherAspectNewLayout = OtherAspectImageBarrier->newLayout;

				// Make it invalid so that it gets removed when we reach it
				OtherAspectImageBarrier->subresourceRange.aspectMask = 0;
			}
			else
			{
				ERHIAccess OtherPlaneAccess = Texture->AllPlanesTrackedAccess[ImageBarrier.subresourceRange.aspectMask == VK_IMAGE_ASPECT_DEPTH_BIT ? 0 : 1];

				OtherAspectOldLayout = OtherAspectNewLayout = FVulkanPipelineBarrier::GetDepthOrStencilLayout(OtherPlaneAccess);
			}

			// Merge the layout with its other half and set it in the barrier
			if (OtherAspectMask == VK_IMAGE_ASPECT_STENCIL_BIT)
			{
				ImageBarrier.oldLayout = VulkanRHI::GetMergedDepthStencilLayout(ImageBarrier.oldLayout, OtherAspectOldLayout);
				ImageBarrier.newLayout = VulkanRHI::GetMergedDepthStencilLayout(ImageBarrier.newLayout, OtherAspectNewLayout);
			}
			else
			{
				ImageBarrier.oldLayout = VulkanRHI::GetMergedDepthStencilLayout(OtherAspectOldLayout, ImageBarrier.oldLayout);
				ImageBarrier.newLayout = VulkanRHI::GetMergedDepthStencilLayout(OtherAspectNewLayout, ImageBarrier.newLayout);
			}

			ImageBarrier.subresourceRange.aspectMask |= OtherAspectMask;
		}
		else
		{
			// Transitions every aspect of the depth(-stencil) texture
			ImageBarrier.oldLayout = VulkanRHI::GetMergedDepthStencilLayout(ImageBarrier.oldLayout, ImageBarrier.oldLayout);
			ImageBarrier.newLayout = VulkanRHI::GetMergedDepthStencilLayout(ImageBarrier.newLayout, ImageBarrier.newLayout);
		}
	}

	// Once we're done with the barrier, make sure there are no sync2 states left
	check((ImageBarrier.oldLayout != VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL) && (ImageBarrier.oldLayout != VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL));
	check((ImageBarrier.newLayout != VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL) && (ImageBarrier.newLayout != VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL));
}

// Create Vulkan barriers from RHI transitions when VK_KHR_Synchronization2 is NOT supported (legacy code path)
void ProcessTransitionLegacy(FVulkanCommandListContext& Context, TArrayView<const FRHITransition*>& Transitions)
{
	for (const FRHITransition* Transition : Transitions)
	{
		const FVulkanTransitionData* Data = Transition->GetPrivateData<FVulkanTransitionData>();

		const bool bIsSingleQueue = IsSingleRHIPipeline(Data->SrcPipelines) && (Data->SrcPipelines == Data->DstPipelines);
		checkf(bIsSingleQueue, TEXT("Devices without support for Sync2 should not be using async compute."));

#if DO_GUARD_SLOW
		checkf(EnumHasAnyFlags(Data->SrcPipelines, Context.GetPipeline()) && EnumHasAnyFlags(Data->DstPipelines, Context.GetPipeline()),
			TEXT("The pipelines for this transition are [%s -> %s], but it's submitted on the [%s] queue."),
			*GetRHIPipelineName(Data->SrcPipelines),
			*GetRHIPipelineName(Data->DstPipelines),
			*GetRHIPipelineName(Context.GetPipeline())
		);
#endif

		FLegacyBarrierArrays BarrierArrays;
		ConvertTransitionToBarriers(Context, *Data, BarrierArrays);

		// Merge any depth/stencil barriers
		for (int32 Index = 0; Index < BarrierArrays.ImageExtraData.Num(); ++Index)
		{
			const FImageBarrierExtraData* RemainingExtras = &BarrierArrays.ImageExtraData[Index];
			VkImageMemoryBarrier* RemainingBarriers = &BarrierArrays.ImageBarriers[Index];

			if (RemainingExtras->BaseTexture->IsDepthOrStencilAspect())
			{
				RemainingExtras->BaseTexture->AllPlanesTrackedAccess[RemainingExtras->PlaneSlice] = RemainingExtras->PlaneAccess;
			}

			if ((RemainingBarriers->image != VK_NULL_HANDLE) && (RemainingBarriers->subresourceRange.aspectMask != 0))
			{
				const int32 RemainingCount = BarrierArrays.ImageExtraData.Num() - Index;
				MergePlanes(RemainingBarriers, RemainingExtras, RemainingCount);
			}
		}

		// Merging Depth and Stencil transitions will also result in null aspectMask for the extra transition which needs to be removed.
		for (int32 DstIndex = 0; DstIndex < BarrierArrays.ImageBarriers.Num();)
		{
			if ((BarrierArrays.ImageBarriers[DstIndex].image == VK_NULL_HANDLE) || ((BarrierArrays.ImageBarriers[DstIndex].subresourceRange.aspectMask == 0)))
			{
				BarrierArrays.ImageBarriers.RemoveAtSwap(DstIndex);
			}
			else
			{
				++DstIndex;
			}
		}

		// Submit
		if (BarrierArrays.MemoryBarriers.Num() || BarrierArrays.BufferBarriers.Num() || BarrierArrays.ImageBarriers.Num())
		{
			// Submit merged stage masks with arrays of barriers
			VulkanRHI::vkCmdPipelineBarrier(Context.GetCommandBuffer().GetHandle(), 
				BarrierArrays.SrcStageMask, BarrierArrays.DstStageMask, 0 /*VkDependencyFlags*/,
				BarrierArrays.MemoryBarriers.Num(), BarrierArrays.MemoryBarriers.GetData(),
				BarrierArrays.BufferBarriers.Num(), BarrierArrays.BufferBarriers.GetData(),
				BarrierArrays.ImageBarriers.Num(),  BarrierArrays.ImageBarriers.GetData());
		}
	}
}










// Removes stages/access that aren't supported by the compute queue
template<typename BarrierType>
static void MaskSupportedAsyncFlags(FVulkanDevice& Device, TArray<BarrierType>& InOutBarriers, bool bMaskSrc, bool bMaskDst, bool bRemoveEmpty)
{
	const VkPipelineStageFlags SupportedComputeStageMask = Device.GetComputeQueue()->GetSupportedStageBits();
	const VkAccessFlags SupportedComputeAccessMasks = Device.GetComputeQueue()->GetSupportedAccessFlags();

	for (int32 Index = InOutBarriers.Num() - 1; Index >= 0; --Index)
	{
		BarrierType& Barrier = InOutBarriers[Index];
		if (bMaskSrc)
		{
			Barrier.srcStageMask &= SupportedComputeStageMask;
			Barrier.srcAccessMask &= SupportedComputeAccessMasks;
		}
		if (bMaskDst)
		{
			Barrier.dstStageMask &= SupportedComputeStageMask;
			Barrier.dstAccessMask &= SupportedComputeAccessMasks;
		}
		if (bRemoveEmpty && ((Barrier.srcStageMask == 0) || (Barrier.dstStageMask == 0)))
		{
			InOutBarriers.RemoveAtSwap(Index);
		}
	}
}


// Patches barriers for release/acquire of resources during queue ownership transfers
template<typename BarrierType>
static void PatchCrossPipeTransitions(TArray<BarrierType>& Barriers, FVulkanCommandListContext& Context, ERHIPipeline SrcPipelines, ERHIPipeline DstPipelines, bool bIsBeginTransition)
{
	const ERHIPipeline ExecutingPipeline = Context.GetPipeline();
	const uint32 GraphicsFamilyIndex = Context.Device.GetGraphicsQueue()->GetFamilyIndex();
	const uint32 ComputeFamilyIndex = Context.Device.GetComputeQueue()->GetFamilyIndex();

	// In the case where src and dst are both single pipelines, keep the layout changes to try to do all the work in a single barrier
	if (IsSingleRHIPipeline(SrcPipelines) && IsSingleRHIPipeline(DstPipelines))
	{
		for (int32 Index = 0; Index < Barriers.Num(); ++Index)
		{
			BarrierType& Barrier = Barriers[Index];

			Barrier.srcQueueFamilyIndex = (SrcPipelines == ERHIPipeline::Graphics) ? GraphicsFamilyIndex : ComputeFamilyIndex;
			Barrier.dstQueueFamilyIndex = (DstPipelines == ERHIPipeline::Graphics) ? GraphicsFamilyIndex : ComputeFamilyIndex;

			if (bIsBeginTransition)
			{
				// Release
				check(SrcPipelines == ExecutingPipeline);
				Barrier.dstAccessMask = 0;
			}
			else
			{
				// Acquire
				check(DstPipelines == ExecutingPipeline);
				Barrier.srcAccessMask = 0;
			}
		}
	}
	else // Src/Dst is ERHIPipeline::All, add a single queue transfer for now :todo-jn:
	{
		// The cross-pipe transition will have a Graphics source when it's Graphics->All or All->AsyncCompute
		const bool bSrcIsGraphics = ((SrcPipelines == ERHIPipeline::Graphics) || (DstPipelines == ERHIPipeline::AsyncCompute));

		// When a transition uses ERHIPipeline::All, it might include stages/access that isn't supported on all queues
		MaskSupportedAsyncFlags(Context.Device, Barriers, !bSrcIsGraphics, bSrcIsGraphics, false);

		for (int32 Index = 0; Index < Barriers.Num(); ++Index)
		{
			BarrierType& Barrier = Barriers[Index];

			// Set the queue families and filter the stages for ERHIPipeline::All running on async compute
			if (bSrcIsGraphics)
			{
				Barrier.srcQueueFamilyIndex = GraphicsFamilyIndex;
				Barrier.dstQueueFamilyIndex = ComputeFamilyIndex;
			}
			else
			{
				Barrier.srcQueueFamilyIndex = ComputeFamilyIndex;
				Barrier.dstQueueFamilyIndex = GraphicsFamilyIndex;
			}

			// Remove the layout change, it gets submitted separately (on the single pipeline for 1..N and N..1 transitions)
			if constexpr (std::is_same_v<BarrierType, VkImageMemoryBarrier2>)
			{
				if (IsSingleRHIPipeline(SrcPipelines))
				{
					Barrier.oldLayout = Barrier.newLayout;
				}
				else if (IsSingleRHIPipeline(DstPipelines))
				{
					Barrier.newLayout = Barrier.oldLayout;
				}
			}

			if (bIsBeginTransition)
			{
				// Release resource from current queue.
				check(EnumHasAllFlags(SrcPipelines, ExecutingPipeline));
				Barrier.dstAccessMask = 0;
			}
			else
			{
				// Acquire resource on current queue.
				check(EnumHasAllFlags(DstPipelines, ExecutingPipeline));
				Barrier.srcAccessMask = 0;
			}
		}
	}
}



// Used to split up barrier batches in single calls to vkCmdPipelineBarrier2 on drivers with issues on larger batches
template<typename BarrierType>
static void SendBatchedBarriers(VkCommandBuffer CommandBuffer, VkDependencyInfo& BatchDependencyInfo, BarrierType*& BarrierPtr, uint32_t& BarrierCountRef, int32 TotalBarrierCount)
{
	for (int32 BatchStartIndex = 0; BatchStartIndex < TotalBarrierCount; BatchStartIndex += GVulkanMaxBarriersPerBatch)
	{
		BarrierCountRef = FMath::Min((TotalBarrierCount - BatchStartIndex), GVulkanMaxBarriersPerBatch);
		VulkanRHI::vkCmdPipelineBarrier2KHR(CommandBuffer, &BatchDependencyInfo);
		BarrierPtr += BarrierCountRef;
	}
	BarrierPtr = nullptr;
	BarrierCountRef = 0;
}



// Create Vulkan barriers from RHI transitions when VK_KHR_Synchronization2 is supported
void ProcessTransitionSync2(FVulkanCommandListContext& Context, TArrayView<const FRHITransition*>& Transitions, bool bIsBeginTransition)
{
	auto SubmitBarriers = [bIsBeginTransition, &Context](TArrayView<const VkMemoryBarrier2> MemoryBarriers, TArrayView<const VkBufferMemoryBarrier2> BufferBarriers, TArrayView<const VkImageMemoryBarrier2> ImageBarriers, VkEvent BarrierEvent = VK_NULL_HANDLE)
		{
			if (MemoryBarriers.Num() || BufferBarriers.Num() || ImageBarriers.Num())
			{
				VkDependencyInfo DependencyInfo;
				DependencyInfo.sType = VK_STRUCTURE_TYPE_DEPENDENCY_INFO;
				DependencyInfo.pNext = nullptr;
				DependencyInfo.dependencyFlags = 0;
				DependencyInfo.memoryBarrierCount = MemoryBarriers.Num();
				DependencyInfo.pMemoryBarriers = MemoryBarriers.GetData();
				DependencyInfo.bufferMemoryBarrierCount = BufferBarriers.Num();
				DependencyInfo.pBufferMemoryBarriers = BufferBarriers.GetData();
				DependencyInfo.imageMemoryBarrierCount = ImageBarriers.Num();
				DependencyInfo.pImageMemoryBarriers = ImageBarriers.GetData();

				if (BarrierEvent)
				{
					if (bIsBeginTransition)
					{
						Context.GetCommandBuffer().BeginSplitBarrier(BarrierEvent, DependencyInfo);
					}
					else
					{
						Context.GetCommandBuffer().EndSplitBarrier(BarrierEvent, DependencyInfo);
					}
				}
				else if ((GVulkanMaxBarriersPerBatch <= 0) || ((MemoryBarriers.Num() + BufferBarriers.Num() + ImageBarriers.Num()) < GVulkanMaxBarriersPerBatch))
				{
					VulkanRHI::vkCmdPipelineBarrier2KHR(Context.GetCommandBuffer().GetHandle(), &DependencyInfo);
				}
				else
				{
					VkDependencyInfo BatchDependencyInfo = DependencyInfo;
					BatchDependencyInfo.memoryBarrierCount = 0;
					BatchDependencyInfo.bufferMemoryBarrierCount = 0;
					BatchDependencyInfo.imageMemoryBarrierCount = 0;

					VkCommandBuffer CommandBuffer = Context.GetCommandBuffer().GetHandle();
					SendBatchedBarriers(CommandBuffer, BatchDependencyInfo, BatchDependencyInfo.pMemoryBarriers, BatchDependencyInfo.memoryBarrierCount, DependencyInfo.memoryBarrierCount);
					SendBatchedBarriers(CommandBuffer, BatchDependencyInfo, BatchDependencyInfo.pBufferMemoryBarriers, BatchDependencyInfo.bufferMemoryBarrierCount, DependencyInfo.bufferMemoryBarrierCount);
					SendBatchedBarriers(CommandBuffer, BatchDependencyInfo, BatchDependencyInfo.pImageMemoryBarriers, BatchDependencyInfo.imageMemoryBarrierCount, DependencyInfo.imageMemoryBarrierCount);
				}
			}
		};


	TArray<VkImageMemoryBarrier2> TempImageBarriers;

	const bool bUseOwnershipTransfers = Context.Device.HasAsyncComputeQueue() && !GVulkanAllowConcurrentImage && Context.Device.NeedsQueueFamilyOwnershipTransfers();
	for (const FRHITransition* Transition : Transitions)
	{
		const FVulkanTransitionData* Data = Transition->GetPrivateData<FVulkanTransitionData>();

		const bool bIsSingleQueue = IsSingleRHIPipeline(Data->SrcPipelines) && (Data->SrcPipelines == Data->DstPipelines);
		const ERHIPipeline TargetPipeline = bIsBeginTransition ? Data->SrcPipelines : Data->DstPipelines;
		const ERHIPipeline OtherPipeline = bIsBeginTransition ? Data->DstPipelines : Data->SrcPipelines;
		const ERHIPipeline ExecutingPipeline = Context.GetPipeline();

		checkf(EnumHasAnyFlags(TargetPipeline, Context.GetPipeline()),
			TEXT("The %s pipelines for this transition are [%s], but it's submitted on the [%s] queue."),
			bIsBeginTransition ? TEXT("SRC") : TEXT("DST"),
			*GetRHIPipelineName(TargetPipeline),
			*GetRHIPipelineName(Context.GetPipeline())
		);

		// Single queue barrier that aren't split only submit in EndTransition
		if (bIsSingleQueue && bIsBeginTransition && !Data->EventHandle)
		{
			continue;
		}

		FSync2BarrierArrays BarrierArrays;
		ConvertTransitionToBarriers(Context, *Data, BarrierArrays);

		// Submit split-barriers right away (they are always single queue)
		if (Data->EventHandle)
		{
			checkf(bIsSingleQueue, TEXT("Split barriers must remain on same queue!"));

			if (BarrierArrays.MemoryBarriers.Num() || BarrierArrays.BufferBarriers.Num() || BarrierArrays.ImageBarriers.Num())
			{
				SubmitBarriers(MakeArrayView(BarrierArrays.MemoryBarriers), MakeArrayView(BarrierArrays.BufferBarriers), MakeArrayView(BarrierArrays.ImageBarriers), Data->EventHandle);
			}
			else if (!bIsBeginTransition)
			{
				// In some cases, transitions result in no barriers (like a destination of Discard for example), make sure we discard the event properly
				Context.Device.ReleaseBarrierEvent(Data->EventHandle);
			}

			continue;
		}

		// Same queue, or single-queue to single-queue transfers, can be submitted directly without copies
		if (IsSingleRHIPipeline(Data->SrcPipelines) && IsSingleRHIPipeline(Data->DstPipelines))
		{
			// For cross-queue 1..1 transitions we will keep the layout change in the same barrier as the queue transfer
			if (Data->SrcPipelines != Data->DstPipelines)
			{
				if (bUseOwnershipTransfers)
				{
					// For buffer transitions, just remove unsupported flags for the compute queue
					if (ExecutingPipeline == ERHIPipeline::AsyncCompute)
					{
						MaskSupportedAsyncFlags(Context.Device, BarrierArrays.BufferBarriers, true, true, true);
					}

					PatchCrossPipeTransitions(BarrierArrays.ImageBarriers, Context, Data->SrcPipelines, Data->DstPipelines, bIsBeginTransition);
				}
				else
				{
					// When we don't have ownership transfers, only submit the barrier once on the graphics queue (ensures all stages are supported)
					check((TargetPipeline == ERHIPipeline::Graphics) || (OtherPipeline == ERHIPipeline::Graphics));
					if (ExecutingPipeline != ERHIPipeline::Graphics)
					{
						continue;
					}
				}
			}

			SubmitBarriers(MakeArrayView(BarrierArrays.MemoryBarriers), MakeArrayView(BarrierArrays.BufferBarriers), MakeArrayView(BarrierArrays.ImageBarriers), Data->EventHandle);
			continue;
		}

		// For 1..N or N..1 transitions we submit the barriers for layout changes on the single pipeline (rest is covered by the sema)
		const bool bNeedsPreLayoutChange = (bIsBeginTransition && ((IsSingleRHIPipeline(Data->SrcPipelines) && !IsSingleRHIPipeline(Data->DstPipelines) && (Data->SrcPipelines == ExecutingPipeline))));
		const bool bNeedsRelease = (bIsBeginTransition && (bNeedsPreLayoutChange ||
			(IsSingleRHIPipeline(Data->DstPipelines) && !IsSingleRHIPipeline(Data->SrcPipelines) && (Data->DstPipelines != ExecutingPipeline))));
		const bool bNeedsPostLayoutChange = (!bIsBeginTransition && (IsSingleRHIPipeline(Data->DstPipelines) && !IsSingleRHIPipeline(Data->SrcPipelines) && (Data->DstPipelines == ExecutingPipeline)));
		const bool bNeedsAcquire = (!bIsBeginTransition && (bNeedsPostLayoutChange ||
			(IsSingleRHIPipeline(Data->SrcPipelines) && !IsSingleRHIPipeline(Data->DstPipelines) && (Data->SrcPipelines != ExecutingPipeline))));


		// For images without concurrent sharing mode, make copies of the array so we can:
		// - wipe the layout transitions from the barriers
		// - patch in the actual queue family ownership transfer
		// NOTE: Maintenance9 being a requirement, ownership transfers aren't required for buffers
		if (bUseOwnershipTransfers && (bNeedsRelease || bNeedsAcquire) && !GVulkanAllowConcurrentImage)
		{
			TempImageBarriers = BarrierArrays.ImageBarriers;
			PatchCrossPipeTransitions(TempImageBarriers, Context, Data->SrcPipelines, Data->DstPipelines, bIsBeginTransition);
		}

		if (bNeedsPreLayoutChange)
		{
			// Remove unsupported flags if we're submitting on the compute queue
			if (ExecutingPipeline == ERHIPipeline::AsyncCompute)
			{
				MaskSupportedAsyncFlags(Context.Device, BarrierArrays.BufferBarriers, false, true, true);
				MaskSupportedAsyncFlags(Context.Device, BarrierArrays.ImageBarriers, false, true, false);
			}

			SubmitBarriers(MakeArrayView(BarrierArrays.MemoryBarriers), MakeArrayView(BarrierArrays.BufferBarriers), MakeArrayView(BarrierArrays.ImageBarriers));
		}

		if (TempImageBarriers.Num())
		{
			SubmitBarriers(MakeArrayView(BarrierArrays.MemoryBarriers), {/* no ownership transfers for buffers */}, MakeArrayView(TempImageBarriers));
			TempImageBarriers.Reset();
		}

		if (bNeedsPostLayoutChange)
		{
			// Remove unsupported flags if we're submitting on the compute queue
			if (ExecutingPipeline == ERHIPipeline::AsyncCompute)
			{
				MaskSupportedAsyncFlags(Context.Device, BarrierArrays.BufferBarriers, true, false, true);
				MaskSupportedAsyncFlags(Context.Device, BarrierArrays.ImageBarriers, true, false, false);
			}

			SubmitBarriers(MakeArrayView(BarrierArrays.MemoryBarriers), MakeArrayView(BarrierArrays.BufferBarriers), MakeArrayView(BarrierArrays.ImageBarriers));
		}
	}
}


































static void ProcessCrossQueueSemaphores(FVulkanCommandListContext& Context, TArrayView<const FRHITransition*>& Transitions, bool bIsBeginTransition)
{
	for (const FRHITransition* Transition : Transitions)
	{
		const FVulkanTransitionData* Data = Transition->GetPrivateData<FVulkanTransitionData>();
		if (Data->Semaphore)
		{
			check(Data->SrcPipelines != Data->DstPipelines);
			if (bIsBeginTransition)
			{
				if ((IsSingleRHIPipeline(Data->SrcPipelines) && (Data->SrcPipelines == Context.GetPipeline())) ||
					(IsSingleRHIPipeline(Data->DstPipelines) && (Data->DstPipelines != Context.GetPipeline())))
				{
					Context.AddSignalSemaphore(Data->Semaphore);
				}
			}
			else
			{
				if ((IsSingleRHIPipeline(Data->DstPipelines) && (Data->DstPipelines == Context.GetPipeline())) ||
					(IsSingleRHIPipeline(Data->SrcPipelines) && (Data->SrcPipelines != Context.GetPipeline())))
				{
					Context.AddWaitSemaphore(VK_PIPELINE_STAGE_ALL_COMMANDS_BIT, Data->Semaphore);
				}
			}
		}
	}
}








void FVulkanCommandListContext::RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions)
{
	if (Device.SupportsParallelRendering())
	{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
		SCOPE_CYCLE_COUNTER(STAT_VulkanBarrierTime);
#endif

		constexpr bool bIsBeginTransition = true;

		ProcessTransitionSync2(*this, Transitions, bIsBeginTransition);

		// Signal semaphores after the release barriers
		ProcessCrossQueueSemaphores(*this, Transitions, bIsBeginTransition);
	}
	else
	{
		// Nothing to do for legacy barriers on begin (no split support, no async compute support)
	}
}

void FVulkanCommandListContext::RHIEndTransitions(TArrayView<const FRHITransition*> Transitions)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanBarrierTime);
#endif

	if (Device.SupportsParallelRendering())
	{
		constexpr bool bIsBeginTransition = false;

		// Wait on semaphores before the acquire barriers
		ProcessCrossQueueSemaphores(*this, Transitions, bIsBeginTransition);

		// Update reserved resource memory mapping
		HandleReservedResourceCommits(Transitions);

		ProcessTransitionSync2(*this, Transitions, bIsBeginTransition);
	}
	else
	{
		ProcessTransitionLegacy(*this, Transitions);
	}
}













void FVulkanPipelineBarrier::AddMemoryBarrier(VkAccessFlags InSrcAccessFlags, VkAccessFlags InDstAccessFlags, VkPipelineStageFlags InSrcStageMask, VkPipelineStageFlags InDstStageMask)
{
	if (MemoryBarriers.Num() == 0)
	{
		VkMemoryBarrier2& NewBarrier = MemoryBarriers.AddDefaulted_GetRef();
		ZeroVulkanStruct(NewBarrier, VK_STRUCTURE_TYPE_MEMORY_BARRIER_2);
	}

	// Mash everything into a single barrier
	VkMemoryBarrier2& MemoryBarrier = MemoryBarriers[0];

	MergeBarrierAccessMask(MemoryBarrier, InSrcAccessFlags, InDstAccessFlags);
	MemoryBarrier.srcStageMask |= InSrcStageMask;
	MemoryBarrier.dstStageMask |= InDstStageMask;
}

//
// Methods used when the RHI itself needs to perform a layout transition. The public API functions do not call these,
// they fill in the fields of FVulkanPipelineBarrier using their own logic, based on the ERHIAccess flags.
//

void FVulkanPipelineBarrier::AddFullImageLayoutTransition(const FVulkanTexture& Texture, VkImageLayout SrcLayout, VkImageLayout DstLayout)
{
	const VkPipelineStageFlags SrcStageMask = GetVkStageFlagsForLayout(SrcLayout);
	const VkPipelineStageFlags DstStageMask = GetVkStageFlagsForLayout(DstLayout);

	const VkAccessFlags SrcAccessFlags = GetVkAccessMaskForLayout(SrcLayout);
	const VkAccessFlags DstAccessFlags = GetVkAccessMaskForLayout(DstLayout);

	const VkImageSubresourceRange SubresourceRange = MakeSubresourceRange(Texture.GetFullAspectMask());
	if (Texture.IsDepthOrStencilAspect())
	{
		SrcLayout = VulkanRHI::GetMergedDepthStencilLayout(SrcLayout, SrcLayout);
		DstLayout = VulkanRHI::GetMergedDepthStencilLayout(DstLayout, DstLayout);
	}

	VkImageMemoryBarrier2& ImgBarrier = ImageBarriers.AddDefaulted_GetRef();
	SetupImageBarrier(ImgBarrier, Texture.Image, SrcStageMask, DstStageMask, SrcAccessFlags, DstAccessFlags, SrcLayout, DstLayout, SubresourceRange);
}

void FVulkanPipelineBarrier::AddImageLayoutTransition(VkImage Image, VkImageLayout SrcLayout, VkImageLayout DstLayout, const VkImageSubresourceRange& SubresourceRange)
{
	const VkPipelineStageFlags SrcStageMask = GetVkStageFlagsForLayout(SrcLayout);
	const VkPipelineStageFlags DstStageMask = GetVkStageFlagsForLayout(DstLayout);

	const VkAccessFlags SrcAccessFlags = GetVkAccessMaskForLayout(SrcLayout);
	const VkAccessFlags DstAccessFlags = GetVkAccessMaskForLayout(DstLayout);

	VkImageMemoryBarrier2& ImgBarrier = ImageBarriers.AddDefaulted_GetRef();
	SetupImageBarrier(ImgBarrier, Image, SrcStageMask, DstStageMask, SrcAccessFlags, DstAccessFlags, SrcLayout, DstLayout, SubresourceRange);
}

void FVulkanPipelineBarrier::AddImageAccessTransition(const FVulkanTexture& Surface, ERHIAccess SrcAccess, ERHIAccess DstAccess, const VkImageSubresourceRange& SubresourceRange, VkImageLayout& InOutLayout)
{
	// This function should only be used for known states.
	check(DstAccess != ERHIAccess::Unknown);
	const bool bIsDepthStencil = Surface.IsDepthOrStencilAspect();
	const bool bSupportsReadOnlyOptimal = Surface.SupportsSampling();

	VkPipelineStageFlags ImgSrcStage, ImgDstStage;
	VkAccessFlags SrcAccessFlags, DstAccessFlags;
	VkImageLayout SrcLayout, DstLayout;
	GetVkStageAndAccessFlags(SrcAccess, FRHITransitionInfo::EType::Texture, 0, bIsDepthStencil, bSupportsReadOnlyOptimal, ImgSrcStage, SrcAccessFlags, SrcLayout, true);
	GetVkStageAndAccessFlags(DstAccess, FRHITransitionInfo::EType::Texture, 0, bIsDepthStencil, bSupportsReadOnlyOptimal, ImgDstStage, DstAccessFlags, DstLayout, false);

	// If not compute, remove vertex pipeline bits as only compute updates vertex buffers
	if (!(ImgSrcStage & VK_PIPELINE_STAGE_COMPUTE_SHADER_BIT))
	{
		ImgDstStage &= ~(VK_PIPELINE_STAGE_VERTEX_SHADER_BIT | VK_PIPELINE_STAGE_TESSELLATION_CONTROL_SHADER_BIT | VK_PIPELINE_STAGE_TESSELLATION_EVALUATION_SHADER_BIT | VK_PIPELINE_STAGE_GEOMETRY_SHADER_BIT);
	}

	if (SrcLayout == VK_IMAGE_LAYOUT_UNDEFINED)
	{
		SrcLayout = InOutLayout;
		SrcAccessFlags = GetVkAccessMaskForLayout(SrcLayout);
	}
	else
	{
		ensure(SrcLayout == InOutLayout);
	}

	if (DstLayout == VK_IMAGE_LAYOUT_UNDEFINED)
	{
		DstLayout = VK_IMAGE_LAYOUT_GENERAL;
	}

	VkImageMemoryBarrier2& ImgBarrier = ImageBarriers.AddDefaulted_GetRef();
	SetupImageBarrier(ImgBarrier, Surface.Image, ImgSrcStage, ImgDstStage, SrcAccessFlags, DstAccessFlags, SrcLayout, DstLayout, SubresourceRange);

	InOutLayout = DstLayout;
}

void FVulkanPipelineBarrier::Execute(VkCommandBuffer CmdBuffer)
{
	if (MemoryBarriers.Num() != 0 || BufferBarriers.Num() != 0 || ImageBarriers.Num() != 0)
	{
		VkPipelineStageFlags SrcStageMask = 0;
		VkPipelineStageFlags DstStageMask = 0;

		TArray<VkMemoryBarrier, TInlineAllocator<1>> TempMemoryBarriers;
		DowngradeBarrierArray(TempMemoryBarriers, MemoryBarriers, SrcStageMask, DstStageMask);

		TArray<VkBufferMemoryBarrier> TempBufferBarriers;
		DowngradeBarrierArray(TempBufferBarriers, BufferBarriers, SrcStageMask, DstStageMask);

		TArray<VkImageMemoryBarrier, TInlineAllocator<2>> TempImageBarriers;
		DowngradeBarrierArray(TempImageBarriers, ImageBarriers, SrcStageMask, DstStageMask);
		MergeDepthStencilLayouts(TempImageBarriers);

		VulkanRHI::vkCmdPipelineBarrier(CmdBuffer, SrcStageMask, DstStageMask, 0, TempMemoryBarriers.Num(), TempMemoryBarriers.GetData(), 
			TempBufferBarriers.Num(), TempBufferBarriers.GetData(), TempImageBarriers.Num(), TempImageBarriers.GetData());
	}
}

void FVulkanPipelineBarrier::Execute(FVulkanCommandBuffer* CmdBuffer)
{
	if (MemoryBarriers.Num() != 0 || BufferBarriers.Num() != 0 || ImageBarriers.Num() != 0)
	{
		if (CmdBuffer->Device.SupportsParallelRendering())
		{
			VkDependencyInfo DependencyInfo;
			DependencyInfo.sType = VK_STRUCTURE_TYPE_DEPENDENCY_INFO;
			DependencyInfo.pNext = nullptr;
			DependencyInfo.dependencyFlags = 0;
			DependencyInfo.memoryBarrierCount = MemoryBarriers.Num();
			DependencyInfo.pMemoryBarriers = MemoryBarriers.GetData();
			DependencyInfo.bufferMemoryBarrierCount = BufferBarriers.Num();
			DependencyInfo.pBufferMemoryBarriers = BufferBarriers.GetData();
			DependencyInfo.imageMemoryBarrierCount = ImageBarriers.Num();
			DependencyInfo.pImageMemoryBarriers = ImageBarriers.GetData();
			VulkanRHI::vkCmdPipelineBarrier2KHR(CmdBuffer->GetHandle(), &DependencyInfo);
		}
		else
		{
			// Call the original execute with older types
			Execute(CmdBuffer->GetHandle());
		}
	}
}

VkImageSubresourceRange FVulkanPipelineBarrier::MakeSubresourceRange(VkImageAspectFlags AspectMask, uint32 FirstMip, uint32 NumMips, uint32 FirstLayer, uint32 NumLayers)
{
	VkImageSubresourceRange Range;
	Range.aspectMask = AspectMask;
	Range.baseMipLevel = FirstMip;
	Range.levelCount = NumMips;
	Range.baseArrayLayer = FirstLayer;
	Range.layerCount = NumLayers;
	return Range;
}

//
// Used when we need to change the layout of a single image. Some plug-ins call this function from outside the RHI (Steam VR, at the time of writing this).
//
void VulkanSetImageLayout(FVulkanCommandBuffer* CmdBuffer, VkImage Image, VkImageLayout OldLayout, VkImageLayout NewLayout, const VkImageSubresourceRange& SubresourceRange)
{
	FVulkanPipelineBarrier Barrier;
	Barrier.AddImageLayoutTransition(Image, OldLayout, NewLayout, SubresourceRange);
	Barrier.Execute(CmdBuffer);
}

VkImageLayout FVulkanPipelineBarrier::GetDefaultLayout(const FVulkanTexture& VulkanTexture, ERHIAccess DesiredAccess)
{
	switch (DesiredAccess)
	{
	case ERHIAccess::SRVCompute:
	case ERHIAccess::SRVGraphics:
	case ERHIAccess::SRVGraphicsNonPixel:
	case ERHIAccess::SRVGraphicsPixel:
	case ERHIAccess::SRVMask:
	{
		if (VulkanTexture.IsDepthOrStencilAspect())
		{
			if (VulkanTexture.Device->SupportsParallelRendering())
			{
				return VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL;
			}
			else
			{
				return VK_IMAGE_LAYOUT_DEPTH_STENCIL_READ_ONLY_OPTIMAL;
			}
		}
		else
		{
			return VulkanTexture.SupportsSampling() ? VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL : VK_IMAGE_LAYOUT_GENERAL;
		}
	}

	case ERHIAccess::UAVCompute:
	case ERHIAccess::UAVGraphics:
	case ERHIAccess::UAVMask: return VK_IMAGE_LAYOUT_GENERAL;

	case ERHIAccess::CopySrc: return VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL;
	case ERHIAccess::CopyDest: return VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL;

	case ERHIAccess::DSVRead: return VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL;
	case ERHIAccess::DSVWrite: return VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL;

	case ERHIAccess::ShadingRateSource:
	{
		if (GRHIVariableRateShadingImageDataType == VRSImage_Palette)
		{
			return VK_IMAGE_LAYOUT_FRAGMENT_SHADING_RATE_ATTACHMENT_OPTIMAL_KHR;
		}
		else if (GRHIVariableRateShadingImageDataType == VRSImage_Fractional)
		{
			return VK_IMAGE_LAYOUT_FRAGMENT_DENSITY_MAP_OPTIMAL_EXT;
		}
	}

	default:
		checkNoEntry();
		return VK_IMAGE_LAYOUT_UNDEFINED;
	}
}
