// Copyright Epic Games, Inc. All Rights Reserved..

#pragma once

#include "Containers/Array.h"
#include "RHIPipeline.h"
#include "VulkanResources.h"
#include "VulkanSynchronization.h"
#include "VulkanThirdParty.h"

class FVulkanCommandBuffer;

struct FVulkanPipelineBarrier
{
	FVulkanPipelineBarrier() = default;

	using MemoryBarrierArrayType = TArray<VkMemoryBarrier2, TInlineAllocator<1>>;
	using ImageBarrierArrayType = TArray<VkImageMemoryBarrier2, TInlineAllocator<2>>;
	using BufferBarrierArrayType = TArray<VkBufferMemoryBarrier2>;

	MemoryBarrierArrayType MemoryBarriers;
	ImageBarrierArrayType ImageBarriers;
	BufferBarrierArrayType BufferBarriers;

	void AddMemoryBarrier(VkAccessFlags SrcAccessFlags, VkAccessFlags DstAccessFlags, VkPipelineStageFlags SrcStageMask, VkPipelineStageFlags DstStageMask);
	void AddFullImageLayoutTransition(const FVulkanTexture& Texture, VkImageLayout SrcLayout, VkImageLayout DstLayout);
	void AddImageLayoutTransition(VkImage Image, VkImageLayout SrcLayout, VkImageLayout DstLayout, const VkImageSubresourceRange& SubresourceRange);
	void AddImageAccessTransition(const FVulkanTexture& Surface, ERHIAccess SrcAccess, ERHIAccess DstAccess, const VkImageSubresourceRange& SubresourceRange, VkImageLayout& InOutLayout);
	void Execute(VkCommandBuffer CmdBuffer);
	void Execute(FVulkanCommandBuffer* CmdBuffer);

	static VkImageSubresourceRange MakeSubresourceRange(VkImageAspectFlags AspectMask, uint32 FirstMip = 0, uint32 NumMips = VK_REMAINING_MIP_LEVELS, uint32 FirstLayer = 0, uint32 NumLayers = VK_REMAINING_ARRAY_LAYERS);

	// Predetermined layouts for given RHIAccess
	static VkImageLayout GetDefaultLayout(const FVulkanTexture& VulkanTexture, ERHIAccess DesiredAccess);
	static VkImageLayout GetDepthOrStencilLayout(ERHIAccess Access);
};

struct FVulkanTransitionData
{
	ERHIPipeline SrcPipelines, DstPipelines;
	TRefCountPtr<FVulkanSemaphore> Semaphore;  // used for cross queue
	VkEvent EventHandle = VK_NULL_HANDLE;  // used for split barriers
	TArray<FRHITransitionInfo> TransitionInfos;
};
