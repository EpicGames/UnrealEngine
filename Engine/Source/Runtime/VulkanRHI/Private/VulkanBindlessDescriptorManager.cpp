// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanBindlessDescriptorManager.h"
#include "VulkanRHIPrivate.h"
#include "VulkanContext.h"
#include "VulkanDevice.h"
#include "RHICore.h"

int32 GVulkanBindlessMaxSamplerDescriptorCount = 2048;
static FAutoConsoleVariableRef CVarVulkanBindlessMaxSamplerDescriptorCount(
	TEXT("r.Vulkan.Bindless.MaxSamplerDescriptorCount"),
	GVulkanBindlessMaxSamplerDescriptorCount,
	TEXT("Maximum bindless sampler descriptor count"),
	ECVF_ReadOnly
);

int32 GVulkanBindlessMaxResourceDescriptorCount = 768 * 1024;
static FAutoConsoleVariableRef CVarVulkanBindlessMaxResourceDescriptorCount(
	TEXT("r.Vulkan.Bindless.MaxResourceDescriptorCount"),
	GVulkanBindlessMaxResourceDescriptorCount,
	TEXT("Maximum bindless resource descriptor count"),
	ECVF_ReadOnly
);

int32 GVulkanBindlessPreferredExtension = 0;
static FAutoConsoleVariableRef CVarVulkanBindlessPreferredExtension(
	TEXT("r.Vulkan.Bindless.PreferredExtension"),
	GVulkanBindlessPreferredExtension,
	TEXT("Preferred extension to use for bindless:\n")
	TEXT("0: VK_EXT_descriptor_buffer (default)\n")
	TEXT("1: VK_EXT_descriptor_heap\n"),
	ECVF_ReadOnly
);


DECLARE_STATS_GROUP(TEXT("Vulkan Bindless"), STATGROUP_VulkanBindless, STATCAT_Advanced);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Sampler Descriptors Allocated"),               STAT_VulkanBindlessSamplerDescriptorsAllocated,               STATGROUP_VulkanBindless);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("SampledImage Descriptors Allocated"),          STAT_VulkanBindlessSampledImageDescriptorsAllocated,          STATGROUP_VulkanBindless);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("StorageImage Descriptors Allocated"),          STAT_VulkanBindlessStorageImageDescriptorsAllocated,          STATGROUP_VulkanBindless);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("UniformTexelBuffer Descriptors Allocated"),    STAT_VulkanBindlessUniformTexelBufferDescriptorsAllocated,    STATGROUP_VulkanBindless);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("StorageTexelBuffer Descriptors Allocated"),    STAT_VulkanBindlessStorageTexelBufferDescriptorsAllocated,    STATGROUP_VulkanBindless);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("UniformBuffer Descriptors Allocated"),         STAT_VulkanBindlessUniformBufferDescriptorsAllocated,         STATGROUP_VulkanBindless);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("StorageBuffer Descriptors Allocated"),         STAT_VulkanBindlessStorageBufferDescriptorsAllocated,         STATGROUP_VulkanBindless);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("AccelerationStructure Descriptors Allocated"), STAT_VulkanBindlessAccelerationStructureDescriptorsAllocated, STATGROUP_VulkanBindless);

DECLARE_DWORD_COUNTER_STAT(TEXT("Write Per Frame"), STAT_VulkanBindlessWritePerFrame, STATGROUP_VulkanBindless);

static TStatId GetStatForDescriptorType(ERHIDescriptorType DescriptorType)
{
	switch (DescriptorType)
	{
	case ERHIDescriptorType::Sampler:               return GET_STATID(STAT_VulkanBindlessSamplerDescriptorsAllocated);
	case ERHIDescriptorType::TextureSRV:            return GET_STATID(STAT_VulkanBindlessSampledImageDescriptorsAllocated);
	case ERHIDescriptorType::TextureUAV:            return GET_STATID(STAT_VulkanBindlessStorageImageDescriptorsAllocated);
	case ERHIDescriptorType::TypedBufferSRV:        return GET_STATID(STAT_VulkanBindlessUniformTexelBufferDescriptorsAllocated);
	case ERHIDescriptorType::TypedBufferUAV:        return GET_STATID(STAT_VulkanBindlessStorageTexelBufferDescriptorsAllocated);
	case ERHIDescriptorType::BufferUAV:             return GET_STATID(STAT_VulkanBindlessStorageBufferDescriptorsAllocated);
	case ERHIDescriptorType::CBV:                   return GET_STATID(STAT_VulkanBindlessUniformBufferDescriptorsAllocated);
	case ERHIDescriptorType::BufferSRV:             return GET_STATID(STAT_VulkanBindlessUniformBufferDescriptorsAllocated);
	case ERHIDescriptorType::AccelerationStructure: return GET_STATID(STAT_VulkanBindlessAccelerationStructureDescriptorsAllocated);
	default: checkNoEntry();
	}

	return {};
}

static constexpr ERHIDescriptorTypeMask GetUEDescriptorTypeMaskFromVkDescriptorType(VkDescriptorType DescriptorType)
{
	switch (DescriptorType)
	{
	case VK_DESCRIPTOR_TYPE_SAMPLER:                    return ERHIDescriptorTypeMask::Sampler;
	case VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE:              return ERHIDescriptorTypeMask::TextureSRV;
	case VK_DESCRIPTOR_TYPE_STORAGE_IMAGE:              return ERHIDescriptorTypeMask::TextureUAV;
	case VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER:       return ERHIDescriptorTypeMask::TypedBufferSRV;
	case VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER:       return ERHIDescriptorTypeMask::TypedBufferUAV;
	case VK_DESCRIPTOR_TYPE_STORAGE_BUFFER:             return ERHIDescriptorTypeMask::BufferUAV;
	case VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER:             return ERHIDescriptorTypeMask::BufferSRV | ERHIDescriptorTypeMask::CBV;
	case VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR: return ERHIDescriptorTypeMask::AccelerationStructure;
	default: checkNoEntry();
	}

	return ERHIDescriptorTypeMask::None;
}

static constexpr VkDescriptorType GetVkDescriptorTypeFromUEDescriptorType(ERHIDescriptorType DescriptorType)
{
	switch (DescriptorType)
	{
	case ERHIDescriptorType::Sampler:               return VK_DESCRIPTOR_TYPE_SAMPLER;
	case ERHIDescriptorType::TextureSRV:            return VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE;
	case ERHIDescriptorType::TextureUAV:            return VK_DESCRIPTOR_TYPE_STORAGE_IMAGE;
	case ERHIDescriptorType::TypedBufferSRV:        return VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER;
	case ERHIDescriptorType::TypedBufferUAV:        return VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER;
	case ERHIDescriptorType::BufferUAV:             return VK_DESCRIPTOR_TYPE_STORAGE_BUFFER;
	case ERHIDescriptorType::BufferSRV:             return VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER;
	case ERHIDescriptorType::CBV:                   return VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER;
	case ERHIDescriptorType::AccelerationStructure: return VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR;
	default: checkNoEntry();
	}

	return VK_DESCRIPTOR_TYPE_MAX_ENUM;
}

extern TAutoConsoleVariable<int32> GCVarRobustBufferAccess;
static inline uint32 GetDescriptorTypeSize(FVulkanDevice& Device, VkDescriptorType DescriptorType)
{
	const bool bRobustBufferAccess = (GCVarRobustBufferAccess.GetValueOnAnyThread() > 0);
	const VkPhysicalDeviceDescriptorBufferPropertiesEXT& DescriptorBufferProperties = Device.GetOptionalExtensionProperties().DescriptorBufferProps;

	switch (DescriptorType)
	{
	case VK_DESCRIPTOR_TYPE_SAMPLER:
		return DescriptorBufferProperties.samplerDescriptorSize;
	case VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE:
		return DescriptorBufferProperties.sampledImageDescriptorSize;
	case VK_DESCRIPTOR_TYPE_STORAGE_IMAGE:
		return DescriptorBufferProperties.storageImageDescriptorSize;
	case VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER:
		return bRobustBufferAccess ? DescriptorBufferProperties.robustUniformTexelBufferDescriptorSize
			: DescriptorBufferProperties.uniformTexelBufferDescriptorSize;
	case VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER:
		return bRobustBufferAccess ? DescriptorBufferProperties.robustStorageTexelBufferDescriptorSize
			: DescriptorBufferProperties.storageTexelBufferDescriptorSize;
	case VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER:
		return bRobustBufferAccess ? DescriptorBufferProperties.robustUniformBufferDescriptorSize
			: DescriptorBufferProperties.uniformBufferDescriptorSize;
	case VK_DESCRIPTOR_TYPE_STORAGE_BUFFER:
		return bRobustBufferAccess ? DescriptorBufferProperties.robustStorageBufferDescriptorSize
			: DescriptorBufferProperties.storageBufferDescriptorSize;
	case VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR:
		return DescriptorBufferProperties.accelerationStructureDescriptorSize;
	case VK_DESCRIPTOR_TYPE_MUTABLE_EXT:
	{
		uint32 MutableDescriptorSize = FMath::Max3<uint32>(DescriptorBufferProperties.sampledImageDescriptorSize, DescriptorBufferProperties.storageImageDescriptorSize, DescriptorBufferProperties.accelerationStructureDescriptorSize);
		if (bRobustBufferAccess)
		{
			MutableDescriptorSize = FMath::Max3<uint32>(MutableDescriptorSize, DescriptorBufferProperties.robustUniformTexelBufferDescriptorSize, DescriptorBufferProperties.robustStorageTexelBufferDescriptorSize);
			MutableDescriptorSize = FMath::Max3<uint32>(MutableDescriptorSize, DescriptorBufferProperties.robustUniformBufferDescriptorSize, DescriptorBufferProperties.robustStorageBufferDescriptorSize);
		}
		else
		{
			MutableDescriptorSize = FMath::Max3<uint32>(MutableDescriptorSize, DescriptorBufferProperties.uniformTexelBufferDescriptorSize, DescriptorBufferProperties.storageTexelBufferDescriptorSize);
			MutableDescriptorSize = FMath::Max3<uint32>(MutableDescriptorSize, DescriptorBufferProperties.uniformBufferDescriptorSize, DescriptorBufferProperties.storageBufferDescriptorSize);
	}
		return MutableDescriptorSize;
}
	default: checkNoEntry();
	}
	return 0;
}


const VkPushConstantRange FVulkanBindlessDescriptorManager::PushConstantRanges[(uint8)FVulkanBindlessDescriptorManager::EPushConstantRangeType::Count] =
{
	// StaticUniformBuffers
	{
		.stageFlags = VK_SHADER_STAGE_ALL,
		.offset = 0,
		.size = (FRHIShaderBindingLayout::MaxUniformBufferEntries * sizeof(uint32))
	}

	// NOTE: Vulkan Core has a 128byte minimum (256 for Vulkan 1.4)
};


static inline VkMemoryPropertyFlags GetDescriptorBufferMemoryType(FVulkanDevice& Device)
{
	if (Device.SupportsDeviceLocalHostVisible())
	{
		return VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT | VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT | VK_MEMORY_PROPERTY_HOST_COHERENT_BIT;
	}

	return VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT | VK_MEMORY_PROPERTY_HOST_COHERENT_BIT;
}

inline FVulkanBindlessDescriptorManager::BindlessSetState& FVulkanBindlessDescriptorManager::GetBindlessState(ERHIDescriptorType RHIDescriptorType)
{
	check(RHIDescriptorType != ERHIDescriptorType::Invalid);
	return (RHIDescriptorType == ERHIDescriptorType::Sampler) ? SamplerState : ResourceState;
}

// Check all the requirements to be running in Bindless using Descriptor Heaps
static FVulkanBindlessDescriptorManager::EBindlessType VerifyDescriptorHeapSupport(FVulkanDevice& Device)
{
	const VkPhysicalDeviceProperties& GpuProps = Device.GetDeviceProperties();
	const FOptionalVulkanDeviceExtensions& OptionalDeviceExtensions = Device.GetOptionalExtensions();
	const VkPhysicalDeviceDescriptorHeapPropertiesEXT& DescriptorHeapProperties = Device.GetOptionalExtensionProperties().DescriptorHeapProps;

	const bool bMeetsExtensionsRequirements =
		OptionalDeviceExtensions.HasEXTDescriptorIndexing &&
		OptionalDeviceExtensions.HasBufferDeviceAddress &&
		OptionalDeviceExtensions.HasEXTDescriptorHeap &&
		OptionalDeviceExtensions.HasKHRMaintenance5;

	if (bMeetsExtensionsRequirements)
	{
		const uint32 ResourceDescriptorSize = FMath::Max(DescriptorHeapProperties.bufferDescriptorSize, DescriptorHeapProperties.imageDescriptorSize);
		const bool bMeetsPropertiesRequirements =
			// Our DXC does not yet support SamplerHeapEXT/ResourceHeapEXT builtins, 
			// so we bind the sampler and descriptor heaps as regular bindings in their own descriptor sets.
			// We still use an additional descriptor set for "single use" UBs, 
			// push constants with indices into the resource heap will be used to maintain compat with the push descriptors used by descriptor_buffers
			// (this might change if/when support for descriptor_buffers is dropped)
			(GpuProps.limits.maxBoundDescriptorSets >= 3) &&
			(DescriptorHeapProperties.maxResourceHeapSize >= GVulkanBindlessMaxResourceDescriptorCount * ResourceDescriptorSize + DescriptorHeapProperties.minResourceHeapReservedRange) &&
			(DescriptorHeapProperties.maxSamplerHeapSize >= GVulkanBindlessMaxSamplerDescriptorCount * DescriptorHeapProperties.samplerDescriptorSize + DescriptorHeapProperties.minSamplerHeapReservedRange) &&
			Device.GetDeviceMemoryManager().SupportsMemoryType(GetDescriptorBufferMemoryType(Device));

		if (bMeetsPropertiesRequirements)
		{
			UE_LOGF(LogRHI, Display, "Bindless will use VK_EXT_descriptor_heap extension.");
			return FVulkanBindlessDescriptorManager::EBindlessType::DescriptorHeap;
		}
		else
		{
			UE_LOGF(LogRHI, Warning, "Bindless descriptor were requested but NOT enabled because of insufficient property support.");
		}
	}
	else
	{
		UE_LOGF(LogRHI, Warning, "Bindless descriptor were requested but NOT enabled because of missing extension support.");
	}

	return FVulkanBindlessDescriptorManager::EBindlessType::Disabled;
}

// Check all the requirements to be running in Bindless using Descriptor Buffers
static FVulkanBindlessDescriptorManager::EBindlessType VerifyDescriptorBufferSupport(FVulkanDevice& Device)
{
	const VkPhysicalDeviceProperties& GpuProps = Device.GetDeviceProperties();
	const FOptionalVulkanDeviceExtensions& OptionalDeviceExtensions = Device.GetOptionalExtensions();
	const VkPhysicalDeviceDescriptorBufferPropertiesEXT& DescriptorBufferProperties = Device.GetOptionalExtensionProperties().DescriptorBufferProps;

	const bool bMeetsExtensionsRequirements =
		OptionalDeviceExtensions.HasEXTDescriptorIndexing &&
		OptionalDeviceExtensions.HasBufferDeviceAddress &&
		OptionalDeviceExtensions.HasEXTDescriptorBuffer &&
		OptionalDeviceExtensions.HasEXTMutableDescriptorType &&
		OptionalDeviceExtensions.HasKHRMaintenance5;

	if (bMeetsExtensionsRequirements)
	{
		// Requirements (lines up with descriptor sets and bindings from shader compilation):
		// - one descriptor set for sampler
		// - one descriptor set for resources
		// - one descriptor set for "single use" descriptor buffers
		// We bind one descriptor buffer for each descriptor set, so we need to support a minimum of 3 for each
		const bool bMeetsPropertiesRequirements =
			(GpuProps.limits.maxBoundDescriptorSets >= 3) &&
			(DescriptorBufferProperties.maxDescriptorBufferBindings >= 3) &&
			(DescriptorBufferProperties.maxResourceDescriptorBufferBindings >= 1) &&
			(DescriptorBufferProperties.maxSamplerDescriptorBufferBindings >= 1) &&
			Device.GetDeviceMemoryManager().SupportsMemoryType(GetDescriptorBufferMemoryType(Device));

		if (bMeetsPropertiesRequirements)
		{
			UE_LOGF(LogRHI, Display, "Bindless will use VK_EXT_descriptor_buffer extension.");
			return FVulkanBindlessDescriptorManager::EBindlessType::DescriptorBuffer;
		}
		else
		{
			UE_LOGF(LogRHI, Warning, "Bindless descriptor were requested but NOT enabled because of insufficient property support.");
		}
	}
	else
	{
		UE_LOGF(LogRHI, Warning, "Bindless descriptor were requested but NOT enabled because of missing extension support.");
	}

	return FVulkanBindlessDescriptorManager::EBindlessType::Disabled;
}

// Check all the requirements and select the descriptor extension to use (buffer vs heap)
static FVulkanBindlessDescriptorManager::EBindlessType VerifySupport(FVulkanDevice& Device, ERHIBindlessConfiguration BindlessConfig)
{
	FVulkanBindlessDescriptorManager::EBindlessType BindlessType = FVulkanBindlessDescriptorManager::EBindlessType::Disabled;

	if (BindlessConfig != ERHIBindlessConfiguration::Disabled)
	{
		if (BindlessConfig > ERHIBindlessConfiguration::RayTracing)
		{
			BindlessType = (GVulkanBindlessPreferredExtension == 0) ? VerifyDescriptorBufferSupport(Device) : VerifyDescriptorHeapSupport(Device);

			if (BindlessType == FVulkanBindlessDescriptorManager::EBindlessType::Disabled)
			{
				BindlessType = (GVulkanBindlessPreferredExtension == 0) ? VerifyDescriptorHeapSupport(Device) : VerifyDescriptorBufferSupport(Device);
			}
		}
		else
		{
			UE_LOGF(LogRHI, Warning, "Bindless in Vulkan must currently be fully enabled (all samplers and resources) or fully disabled.");
		}
	}

	if (BindlessType != FVulkanBindlessDescriptorManager::EBindlessType::Disabled)
	{
		extern TAutoConsoleVariable<int32> GDynamicGlobalUBs;
		if (GDynamicGlobalUBs->GetInt() != 0)
		{
			UE_LOGF(LogRHI, Warning, "Dynamic Uniform Buffers are enabled, but they will not be used with Vulkan bindless.");
		}

		extern int32 GVulkanEnableDefrag;
		if (GVulkanEnableDefrag != 0)  // :todo-jn: to be turned back on with new defragger
		{
			UE_LOGF(LogRHI, Warning, "Memory defrag is enabled, but it will not be used with Vulkan bindless.");
			GVulkanEnableDefrag = 0;
		}
	}

	return BindlessType;
}


FVulkanBindlessDescriptorManager::FVulkanBindlessDescriptorManager(FVulkanDevice& InDevice)
	: Device(InDevice)
{
	FMemory::Memzero(BufferBindingInfo);
	for (uint32 Index = 0; Index < VulkanBindless::MaxNumSets; Index++)
	{
		BufferIndices[Index] = Index;
	}
}

FVulkanBindlessDescriptorManager::~FVulkanBindlessDescriptorManager()
{
	checkf(BindlessPipelineLayout == VK_NULL_HANDLE, TEXT("DeInit() was not called on FVulkanBindlessDescriptorManager!"));
}

bool FVulkanBindlessDescriptorManager::SupportsBindlessAccelerationStructures() const
{
	if (!IsSupported() || !Device.GetOptionalExtensions().HasRaytracingExtensions())
	{
		return false;
	}

	// Descriptor heaps are always supported
	if (UseDescriptorHeaps())
	{
		return true;
	}

	// Need to check against mutable descriptors for descriptor buffers
	const VkDescriptorType DescriptorTypes[] = {
		VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE,
		VK_DESCRIPTOR_TYPE_STORAGE_IMAGE,
		VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER,
		VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER,
		VK_DESCRIPTOR_TYPE_STORAGE_BUFFER,
		VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER,
		VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR
	};

	const VkMutableDescriptorTypeListEXT MutableDescriptorTypeList = {
		.descriptorTypeCount = UE_ARRAY_COUNT(DescriptorTypes),
		.pDescriptorTypes = DescriptorTypes
	};

	const VkMutableDescriptorTypeCreateInfoEXT MutableDescriptorTypeCreateInfo = {
		.sType = VK_STRUCTURE_TYPE_MUTABLE_DESCRIPTOR_TYPE_CREATE_INFO_EXT,
		.pNext = nullptr,
		.mutableDescriptorTypeListCount = 1,
		.pMutableDescriptorTypeLists = &MutableDescriptorTypeList
	};

	const VkDescriptorSetLayoutBinding DescriptorSetLayoutBinding = {
		.binding = 0,
		.descriptorType = VK_DESCRIPTOR_TYPE_MUTABLE_EXT,
		.descriptorCount = 1,
		.stageFlags = VK_SHADER_STAGE_ALL,
		.pImmutableSamplers = nullptr
	};

	const VkDescriptorSetLayoutCreateInfo DescriptorSetLayoutCreateInfo = {
		.sType = VK_STRUCTURE_TYPE_DESCRIPTOR_SET_LAYOUT_CREATE_INFO,
		.pNext = &MutableDescriptorTypeCreateInfo,
		.flags = 0,
		.bindingCount = 1,
		.pBindings = &DescriptorSetLayoutBinding
	};

	VkDescriptorSetLayoutSupport LayoutSupport = {
		.sType = VK_STRUCTURE_TYPE_DESCRIPTOR_SET_LAYOUT_SUPPORT,
		.pNext = nullptr,
		.supported = false
	};
	VulkanRHI::vkGetDescriptorSetLayoutSupport(Device.GetHandle(), &DescriptorSetLayoutCreateInfo, &LayoutSupport);

	return (LayoutSupport.supported == VK_TRUE);
}

void FVulkanBindlessDescriptorManager::Init()
{
	Configuration = UE::RHICore::GetBindlessConfigurationOnStartup(GMaxRHIShaderPlatform);

	BindlessType = VerifySupport(Device, Configuration);

	switch (BindlessType)
	{
	case FVulkanBindlessDescriptorManager::EBindlessType::DescriptorHeap:
		InitDescriptorHeaps();
		break;
	case FVulkanBindlessDescriptorManager::EBindlessType::DescriptorBuffer:
		InitDescriptorBuffers();
		break;
	case FVulkanBindlessDescriptorManager::EBindlessType::Disabled:
		Configuration = ERHIBindlessConfiguration::Disabled;
		break;
	}
}

void FVulkanBindlessDescriptorManager::InitDescriptorHeaps()
{
	const VkDevice DeviceHandle = Device.GetHandle();
	const VkPhysicalDeviceDescriptorHeapPropertiesEXT& DescriptorHeapProperties = Device.GetOptionalExtensionProperties().DescriptorHeapProps;
	const bool bHasRaytracingExtensions = Device.GetOptionalExtensions().HasRaytracingExtensions();

	auto CreateDescriptorHeap = [&Device = Device](uint32 HeapSize, uint32 ReservedSize, BindlessSetState& InOutState) -> VkBindHeapInfoEXT {

		const VkBufferUsageFlags BufferUsageFlags = VK_BUFFER_USAGE_DESCRIPTOR_HEAP_BIT_EXT | VK_BUFFER_USAGE_SHADER_DEVICE_ADDRESS_BIT | VK_BUFFER_USAGE_TRANSFER_DST_BIT;

		// Create buffer for heap
		InOutState.BufferHandle = Device.CreateBuffer(HeapSize, BufferUsageFlags);

		// Allocate memory for the buffer and bind it
		{
			const VkDevice DeviceHandle = Device.GetHandle();

			VkMemoryRequirements BufferMemoryReqs;
			FMemory::Memzero(BufferMemoryReqs);
			VulkanRHI::vkGetBufferMemoryRequirements(DeviceHandle, InOutState.BufferHandle, &BufferMemoryReqs);
			check(BufferMemoryReqs.size >= HeapSize);

			uint32 MemoryTypeIndex = 0;
			VERIFYVULKANRESULT(Device.GetDeviceMemoryManager().GetMemoryTypeFromProperties(BufferMemoryReqs.memoryTypeBits, GetDescriptorBufferMemoryType(Device), &MemoryTypeIndex));

			const VkMemoryAllocateFlagsInfo FlagsInfo = {
				.sType = VK_STRUCTURE_TYPE_MEMORY_ALLOCATE_FLAGS_INFO,
				.pNext = nullptr,
				.flags = VK_MEMORY_ALLOCATE_DEVICE_ADDRESS_BIT,
				.deviceMask = 0
			};

			const VkMemoryAllocateInfo AllocateInfo = {
				.sType = VK_STRUCTURE_TYPE_MEMORY_ALLOCATE_INFO,
				.pNext = &FlagsInfo,
				.allocationSize = BufferMemoryReqs.size,
				.memoryTypeIndex = MemoryTypeIndex
			};

			VERIFYVULKANRESULT(VulkanRHI::vkAllocateMemory(DeviceHandle, &AllocateInfo, VULKAN_CPU_ALLOCATOR, &InOutState.MemoryHandle));
			VERIFYVULKANRESULT(VulkanRHI::vkBindBufferMemory(DeviceHandle, InOutState.BufferHandle, InOutState.MemoryHandle, 0));
			VERIFYVULKANRESULT(VulkanRHI::vkMapMemory(DeviceHandle, InOutState.MemoryHandle, 0, VK_WHOLE_SIZE, 0, (void**)&InOutState.MappedPointer));
			FMemory::Memzero(InOutState.MappedPointer, AllocateInfo.allocationSize);
		}

		// Return the heap info with the buffer's address
		const VkBufferDeviceAddressInfo AddressInfo = {
			.sType = VK_STRUCTURE_TYPE_BUFFER_DEVICE_ADDRESS_INFO,
			.pNext = nullptr,
			.buffer = InOutState.BufferHandle
		};
		const VkDeviceAddress BufferAddress = VulkanRHI::vkGetBufferDeviceAddressKHR(Device.GetHandle(), &AddressInfo);

		return 
		{
			.sType = VK_STRUCTURE_TYPE_BIND_HEAP_INFO_EXT,
			.pNext = nullptr,
			.heapRange = {
				.address = BufferAddress,
				.size = HeapSize,
			},
			.reservedRangeOffset = HeapSize - ReservedSize,
			.reservedRangeSize = ReservedSize,
		};
	};

	// Create the sampler heap
	{
		SamplerState.DescriptorSize = DescriptorHeapProperties.samplerDescriptorSize;

		const uint32 SamplerHeapSize = GVulkanBindlessMaxSamplerDescriptorCount * SamplerState.DescriptorSize + DescriptorHeapProperties.minSamplerHeapReservedRange;
		SamplerBindHeapInfo = CreateDescriptorHeap(SamplerHeapSize, DescriptorHeapProperties.minSamplerHeapReservedRange, SamplerState);

		TArray<TStatId, TInlineAllocator<1>> AllocatorStats;
		AllocatorStats.Emplace(GET_STATID(STAT_BindlessSamplerDescriptorsAllocated));
		SamplerState.Allocator = new FRHIHeapDescriptorAllocator(ERHIDescriptorTypeMask::Sampler, GVulkanBindlessMaxSamplerDescriptorCount, TConstArrayView<TStatId>(AllocatorStats));
	}
	
	// Create the resource heap
	{
		ResourceState.DescriptorSize = FMath::Max(DescriptorHeapProperties.bufferDescriptorSize, DescriptorHeapProperties.imageDescriptorSize);

		const uint32 ResourceHeapSize = GVulkanBindlessMaxResourceDescriptorCount * ResourceState.DescriptorSize + DescriptorHeapProperties.minResourceHeapReservedRange;
		ResourceBindHeapInfo = CreateDescriptorHeap(ResourceHeapSize, DescriptorHeapProperties.minResourceHeapReservedRange, ResourceState);

		TArray<TStatId, TInlineAllocator<1>> AllocatorStats;
		AllocatorStats.Emplace(GET_STATID(STAT_BindlessResourceDescriptorsAllocated));
		ERHIDescriptorTypeMask TypeMask{};
		EnumAddFlags(TypeMask, ERHIDescriptorTypeMask::TextureSRV);
		EnumAddFlags(TypeMask, ERHIDescriptorTypeMask::TextureUAV);
		EnumAddFlags(TypeMask, ERHIDescriptorTypeMask::TypedBufferSRV);
		EnumAddFlags(TypeMask, ERHIDescriptorTypeMask::TypedBufferUAV);
		EnumAddFlags(TypeMask, ERHIDescriptorTypeMask::BufferUAV); // VK_DESCRIPTOR_TYPE_STORAGE_BUFFER
		EnumAddFlags(TypeMask, ERHIDescriptorTypeMask::BufferSRV | ERHIDescriptorTypeMask::CBV); // VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER
		if (bHasRaytracingExtensions)
		{
			EnumAddFlags(TypeMask, ERHIDescriptorTypeMask::AccelerationStructure); // VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR
		}

		ResourceState.Allocator = new FRHIHeapDescriptorAllocator(TypeMask, GVulkanBindlessMaxResourceDescriptorCount, TConstArrayView<TStatId>(AllocatorStats));
	}

	// Create the binding mappings
	{
		HeapOnlyBindingMappings.SetNum(2);
		VkDescriptorSetAndBindingMappingEXT& SamplerHeapMapping = HeapOnlyBindingMappings[VulkanBindless::BindlessSamplerSet];
		VkDescriptorSetAndBindingMappingEXT& ResourceHeapMapping = HeapOnlyBindingMappings[VulkanBindless::BindlessResourceSet];

		// Setup a common mapping for our sampler and resources heaps (will be inserted in all arrays)
		SamplerHeapMapping = {
			.sType = VK_STRUCTURE_TYPE_DESCRIPTOR_SET_AND_BINDING_MAPPING_EXT,
			.pNext = nullptr,
			.descriptorSet = VulkanBindless::BindlessSamplerSet,
			.firstBinding = 0,
			.bindingCount = 1,
			.resourceMask = VK_SPIRV_RESOURCE_TYPE_SAMPLER_BIT_EXT,
			.source = VK_DESCRIPTOR_MAPPING_SOURCE_HEAP_WITH_CONSTANT_OFFSET_EXT,
			.sourceData = {
				.constantOffset = {
					.heapOffset = 0,
					.heapArrayStride = ResourceState.DescriptorSize,
					.pEmbeddedSampler = nullptr,
					.samplerHeapOffset = 0,
					.samplerHeapArrayStride = SamplerState.DescriptorSize
				}
			}
		};

		const VkSpirvResourceTypeFlagsEXT SpirvResourceTypeFlags = VK_SPIRV_RESOURCE_TYPE_SAMPLED_IMAGE_BIT_EXT | VK_SPIRV_RESOURCE_TYPE_READ_ONLY_IMAGE_BIT_EXT | VK_SPIRV_RESOURCE_TYPE_READ_WRITE_IMAGE_BIT_EXT |
			VK_SPIRV_RESOURCE_TYPE_UNIFORM_BUFFER_BIT_EXT | VK_SPIRV_RESOURCE_TYPE_READ_ONLY_STORAGE_BUFFER_BIT_EXT | VK_SPIRV_RESOURCE_TYPE_READ_WRITE_STORAGE_BUFFER_BIT_EXT |
			(bHasRaytracingExtensions ? VK_SPIRV_RESOURCE_TYPE_ACCELERATION_STRUCTURE_BIT_EXT : 0);

		ResourceHeapMapping = {
			.sType = VK_STRUCTURE_TYPE_DESCRIPTOR_SET_AND_BINDING_MAPPING_EXT,
			.pNext = nullptr,
			.descriptorSet = VulkanBindless::BindlessResourceSet,
			.firstBinding = 0,
			.bindingCount = 1,
			.resourceMask = SpirvResourceTypeFlags,
			.source = VK_DESCRIPTOR_MAPPING_SOURCE_HEAP_WITH_CONSTANT_OFFSET_EXT,
			.sourceData = {
				.constantOffset = {
					.heapOffset = 0,
					.heapArrayStride = ResourceState.DescriptorSize,
					.pEmbeddedSampler = nullptr,
					.samplerHeapOffset = 0,
					.samplerHeapArrayStride = SamplerState.DescriptorSize
				}
			}
		};

		// We need to offset the push constants to leave place for the static UBs
		const uint32 PushConstantOffset = (FRHIShaderBindingLayout::MaxUniformBufferEntries * sizeof(uint32));

		for (int32 StageIndex = 0; StageIndex < SF_NumGraphicsFrequencies; ++StageIndex)
		{
			const EShaderFrequency Frequency = (EShaderFrequency)StageIndex;
			check(IsValidGraphicsFrequency(Frequency));

			// Work around the limits imposed by shared support with descriptor_buffer and push_descriptors
			const int32 PushDescriptorOffset = VulkanBindless::GetOffsetForFrequency(Frequency);
			TArray<VkDescriptorSetAndBindingMappingEXT>& StageBindingMappings = GraphicsBindingMappings[StageIndex];

			// Until our DXC supports SamplerHeapEXT / ResourceHeapEXT builtins, add bindings for sampler and resource heaps
			StageBindingMappings.SetNum(VulkanBindless::MaxUniformBuffersPerStage + 2);
			StageBindingMappings[VulkanBindless::MaxUniformBuffersPerStage + 0] = SamplerHeapMapping;
			StageBindingMappings[VulkanBindless::MaxUniformBuffersPerStage + 1] = ResourceHeapMapping;

			for (uint32 UBIndex = 0; UBIndex < (uint32)VulkanBindless::MaxUniformBuffersPerStage; ++UBIndex)
			{
				const uint32 ShaderBindingIndex = (PushDescriptorOffset < 0) ? (VulkanBindless::MaxUniformBuffersTotal - UBIndex - 1) : (PushDescriptorOffset + UBIndex);

				StageBindingMappings[UBIndex] = {
					.sType = VK_STRUCTURE_TYPE_DESCRIPTOR_SET_AND_BINDING_MAPPING_EXT,
					.pNext = nullptr,
					.descriptorSet = VulkanBindless::BindlessSingleUseUniformBufferSet,
					.firstBinding = ShaderBindingIndex,
					.bindingCount = 1,
					.resourceMask = VK_SPIRV_RESOURCE_TYPE_UNIFORM_BUFFER_BIT_EXT,
					.source = VK_DESCRIPTOR_MAPPING_SOURCE_HEAP_WITH_PUSH_INDEX_EXT,
					.sourceData = {
						.pushIndex = {
							.heapOffset = 0,
							.pushOffset = GetBindingPushDataOffset(Frequency, UBIndex),
							.heapIndexStride = ResourceState.DescriptorSize,
							.heapArrayStride = ResourceState.DescriptorSize,
							.pEmbeddedSampler = nullptr,
							.useCombinedImageSamplerIndex = false,
							.samplerHeapOffset = 0,
							.samplerPushOffset = 0,
							.samplerHeapIndexStride = 0,
							.samplerHeapArrayStride = 0
						}
					}
				};
			}

			// Create another version of the mappings with a push address in slot 0
			GraphicsBindingMappingsWithPushAddr[StageIndex] = GraphicsBindingMappings[StageIndex];
			GraphicsBindingMappingsWithPushAddr[StageIndex][0].source = VK_DESCRIPTOR_MAPPING_SOURCE_PUSH_ADDRESS_EXT;
			GraphicsBindingMappingsWithPushAddr[StageIndex][0].sourceData.pushAddressOffset = GetBindingPushDataOffset(Frequency, 0);
		}

		// Until our DXC supports SamplerHeapEXT / ResourceHeapEXT builtins, add bindings for sampler and resource heaps
		ComputeBindingMappings.SetNum(VulkanBindless::MaxUniformBuffersPerStage + 2);
		ComputeBindingMappings[VulkanBindless::MaxUniformBuffersPerStage + 0] = SamplerHeapMapping;
		ComputeBindingMappings[VulkanBindless::MaxUniformBuffersPerStage + 1] = ResourceHeapMapping;
		for (uint32 UBIndex = 0; UBIndex < (uint32)VulkanBindless::MaxUniformBuffersPerStage; ++UBIndex)
		{
			ComputeBindingMappings[UBIndex] = {
				.sType = VK_STRUCTURE_TYPE_DESCRIPTOR_SET_AND_BINDING_MAPPING_EXT,
				.pNext = nullptr,
				.descriptorSet = VulkanBindless::BindlessSingleUseUniformBufferSet,
				.firstBinding = UBIndex,
				.bindingCount = 1,
				.resourceMask = VK_SPIRV_RESOURCE_TYPE_UNIFORM_BUFFER_BIT_EXT,
				.source = VK_DESCRIPTOR_MAPPING_SOURCE_HEAP_WITH_PUSH_INDEX_EXT,
				.sourceData = {
					.pushIndex = {
						.heapOffset = 0,
						.pushOffset = GetBindingPushDataOffset(SF_Compute, UBIndex),
						.heapIndexStride = ResourceState.DescriptorSize,
						.heapArrayStride = ResourceState.DescriptorSize,
						.pEmbeddedSampler = nullptr,
						.useCombinedImageSamplerIndex = false,
						.samplerHeapOffset = 0,
						.samplerPushOffset = 0,
						.samplerHeapIndexStride = 0,
						.samplerHeapArrayStride = 0
					}
				}
			};
		}

		// Create another version of the mappings with a push address in slot 0
		ComputeBindingMappingsWithPushAddr = ComputeBindingMappings;
		ComputeBindingMappingsWithPushAddr[0].source = VK_DESCRIPTOR_MAPPING_SOURCE_PUSH_ADDRESS_EXT;
		ComputeBindingMappingsWithPushAddr[0].sourceData.pushAddressOffset = GetBindingPushDataOffset(SF_Compute, 0);
	}
}

uint32 FVulkanBindlessDescriptorManager::GetBindingPushDataOffset(EShaderFrequency Frequency, uint32 BindingIndex)
{
	// Use by descriptor heaps (not bound by the limits of push descriptors)

	// Bindings are placed in push data after the static uniform buffers
	constexpr uint32 PushConstantOffset = (FRHIShaderBindingLayout::MaxUniformBufferEntries * sizeof(uint32));

	// A full stage has MaxUniformBuffersPerStage indices.  
	// The first one is 64bit in case an address is used directly (in the case of temporary buffers for globals)
	// The other bindings all use 32bit indices into the resource heap
	constexpr uint32 FullStageSize = sizeof(uint64) + ((VulkanBindless::MaxUniformBuffersPerStage - 1) * sizeof(uint32));
	const uint32 BindingOffset = (BindingIndex == 0) ? 0u : 4u + (BindingIndex * sizeof(uint32));

	switch (Frequency)
	{
	case SF_Vertex:			return PushConstantOffset + BindingOffset;
	case SF_Mesh:			return PushConstantOffset + BindingOffset;

	case SF_Pixel:			return PushConstantOffset + FullStageSize + BindingOffset;

	case SF_Geometry:		return PushConstantOffset + (2 * FullStageSize) + BindingOffset;
	case SF_Amplification:	return PushConstantOffset + (2 * FullStageSize) + BindingOffset;

	case SF_Compute:		return PushConstantOffset + BindingOffset;

	case SF_RayGen:
	case SF_RayMiss:
	case SF_RayHitGroup:
	case SF_RayCallable:	return PushConstantOffset + BindingOffset;
	default:
		checkf(false, TEXT("Invalid shader frequency %d"), (int32)Frequency);
		break;
	}

	return 0;
}

void FVulkanBindlessDescriptorManager::InitDescriptorBuffers()
{
	const VkDevice DeviceHandle = Device.GetHandle();
	const VkPhysicalDeviceDescriptorBufferPropertiesEXT& DescriptorBufferProperties = Device.GetOptionalExtensionProperties().DescriptorBufferProps;

	const VkBufferUsageFlags BufferUsageFlags = VK_BUFFER_USAGE_SHADER_DEVICE_ADDRESS_BIT | VK_BUFFER_USAGE_RESOURCE_DESCRIPTOR_BUFFER_BIT_EXT;

	{
		// Split up the descriptor types handled by each state
		// 0 - Samplers
		// 1 - All resources types (using mutable)
		{
			SamplerState.DescriptorTypes.Add(VK_DESCRIPTOR_TYPE_SAMPLER);

			ResourceState.DescriptorTypes.Add(VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE);
			ResourceState.DescriptorTypes.Add(VK_DESCRIPTOR_TYPE_STORAGE_IMAGE);
			ResourceState.DescriptorTypes.Add(VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER);
			ResourceState.DescriptorTypes.Add(VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER);
			ResourceState.DescriptorTypes.Add(VK_DESCRIPTOR_TYPE_STORAGE_BUFFER);
			ResourceState.DescriptorTypes.Add(VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER);
			if (Device.GetOptionalExtensions().HasRaytracingExtensions() && SupportsBindlessAccelerationStructures())
			{
				ResourceState.DescriptorTypes.Add(VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR);
			}

			// Add at the end so it can be excluded using Num-1 when only actual types are desired
			ResourceState.DescriptorTypes.Add(VK_DESCRIPTOR_TYPE_MUTABLE_EXT);
		}

		// Setup the sampler states
		{
			SamplerState.DescriptorSize = GetDescriptorTypeSize(Device, VK_DESCRIPTOR_TYPE_SAMPLER);
			checkf((SamplerState.DescriptorSize > 0), TEXT("Descriptor Type [VK_DESCRIPTOR_TYPE_SAMPLER] returned an invalid descriptor size!"));
			SamplerState.DescriptorCapacity = GVulkanBindlessMaxSamplerDescriptorCount;
			checkf((SamplerState.DescriptorCapacity > 0), TEXT("Descriptor Type [VK_DESCRIPTOR_TYPE_SAMPLER] returned an invalid descriptor count!"));

			const ERHIDescriptorTypeMask TypeMask = RHIDescriptorTypeMaskFromType(ERHIDescriptorType::Sampler);
			TArray<TStatId, TInlineAllocator<1>> AllocatorStats;
			AllocatorStats.Emplace(GET_STATID(STAT_BindlessSamplerDescriptorsAllocated));
			SamplerState.Allocator = new FRHIHeapDescriptorAllocator(TypeMask, SamplerState.DescriptorCapacity, TConstArrayView<TStatId>(AllocatorStats));
		}

		// Setup the mutable resource descriptor states
		{
			ResourceState.DescriptorSize = GetDescriptorTypeSize(Device, VK_DESCRIPTOR_TYPE_MUTABLE_EXT);
			checkf((ResourceState.DescriptorSize > 0), TEXT("Descriptor Type [VK_DESCRIPTOR_TYPE_MUTABLE_EXT] returned an invalid descriptor size!"));
			ResourceState.DescriptorCapacity = GVulkanBindlessMaxResourceDescriptorCount;
			checkf((ResourceState.DescriptorCapacity > 0), TEXT("Descriptor Type [VK_DESCRIPTOR_TYPE_MUTABLE_EXT] returned an invalid descriptor count!"));

			ERHIDescriptorTypeMask TypeMask{};
			TArray<TStatId, TInlineAllocator<1>> AllocatorStats;
			AllocatorStats.Emplace(GET_STATID(STAT_BindlessResourceDescriptorsAllocated));
			for (int32 TypeIndex = 0; TypeIndex < ResourceState.DescriptorTypes.Num() - 1; ++TypeIndex)
			{
				const VkDescriptorType DescriptorType = ResourceState.DescriptorTypes[TypeIndex];
				EnumAddFlags(TypeMask, GetUEDescriptorTypeMaskFromVkDescriptorType(DescriptorType));
			}
			ResourceState.Allocator = new FRHIHeapDescriptorAllocator(TypeMask, ResourceState.DescriptorCapacity, TConstArrayView<TStatId>(AllocatorStats));
		}


		// Fill the DescriptorSetLayout for a BindlessSetState
		auto CreateDescriptorSetLayout = [&, &DeviceRef=Device](const BindlessSetState& State)
		{
			const bool IsSamplerSet = (State.DescriptorTypes[0] == VK_DESCRIPTOR_TYPE_SAMPLER);

			const VkDescriptorSetLayoutBinding DescriptorSetLayoutBindings =
			{
				.binding = 0,
				.descriptorType = IsSamplerSet ? VK_DESCRIPTOR_TYPE_SAMPLER : VK_DESCRIPTOR_TYPE_MUTABLE_EXT,
				.descriptorCount = (uint32)(IsSamplerSet ? GVulkanBindlessMaxSamplerDescriptorCount : GVulkanBindlessMaxResourceDescriptorCount),
				.stageFlags = VK_SHADER_STAGE_ALL,
				.pImmutableSamplers = nullptr
			};

			const VkDescriptorBindingFlags DescriptorBindingFlags = 0; //NOTE: Eventually change to VK_DESCRIPTOR_BINDING_VARIABLE_DESCRIPTOR_COUNT_BIT
			const VkDescriptorSetLayoutBindingFlagsCreateInfo DescriptorSetLayoutBindingFlagsCreateInfo =
			{
				.sType = VK_STRUCTURE_TYPE_DESCRIPTOR_SET_LAYOUT_BINDING_FLAGS_CREATE_INFO,
				.pNext = nullptr,
				.bindingCount = 1,
				.pBindingFlags = &DescriptorBindingFlags
			};

			const uint32 MutableTypeCount = ResourceState.DescriptorTypes.Num() - 1;
			const VkMutableDescriptorTypeListEXT MutableDescriptorTypeList =
			{
				.descriptorTypeCount = MutableTypeCount,
				.pDescriptorTypes = ResourceState.DescriptorTypes.GetData()
			};

			const VkMutableDescriptorTypeCreateInfoEXT MutableDescriptorTypeCreateInfo =
			{
				.sType = VK_STRUCTURE_TYPE_MUTABLE_DESCRIPTOR_TYPE_CREATE_INFO_EXT,
				.pNext = &DescriptorSetLayoutBindingFlagsCreateInfo,
				.mutableDescriptorTypeListCount = 1,
				.pMutableDescriptorTypeLists = &MutableDescriptorTypeList
			};

			// NOTE: Eventually add VkDescriptorSetLayoutBindingFlagsCreateInfo to pNext for VK_DESCRIPTOR_BINDING_VARIABLE_DESCRIPTOR_COUNT_BIT
			const VkDescriptorSetLayoutCreateInfo DescriptorSetLayoutCreateInfo =
			{
				.sType = VK_STRUCTURE_TYPE_DESCRIPTOR_SET_LAYOUT_CREATE_INFO,
				.pNext = IsSamplerSet ? (void*)&DescriptorSetLayoutBindingFlagsCreateInfo : (void*)&MutableDescriptorTypeCreateInfo,
				.flags = VK_DESCRIPTOR_SET_LAYOUT_CREATE_DESCRIPTOR_BUFFER_BIT_EXT,
				.bindingCount = 1,
				.pBindings = &DescriptorSetLayoutBindings
			};

			VkDescriptorSetLayout DescriptorSetLayout = VK_NULL_HANDLE;
			VERIFYVULKANRESULT(VulkanRHI::vkCreateDescriptorSetLayout(DeviceHandle, &DescriptorSetLayoutCreateInfo, VULKAN_CPU_ALLOCATOR, &DescriptorSetLayout));
			VULKAN_SET_DEBUG_NAME(Device, VK_OBJECT_TYPE_DESCRIPTOR_SET_LAYOUT, DescriptorSetLayout, TEXT("BindlessDescriptorSet(%s)"), IsSamplerSet ? TEXT("Sampler") : TEXT("Resources"));

			// Safe check
			VkDeviceSize BindingOffset = 0;
			VulkanRHI::vkGetDescriptorSetLayoutBindingOffsetEXT(DeviceHandle, DescriptorSetLayout, 0u, &BindingOffset);
			check(BindingOffset == 0);

			return DescriptorSetLayout;
		};


		// Create the descriptor buffer for a BindlessSetState
		auto CreateDescriptorBuffer = [&](BindlessSetState& InOutState, VkDescriptorBufferBindingInfoEXT& OutBufferBindingInfo)
		{
			const bool IsSamplerSet = (InOutState.DescriptorTypes[0] == VK_DESCRIPTOR_TYPE_SAMPLER);
			const VkBufferUsageFlags BufferUsageFlags =
				VK_BUFFER_USAGE_SHADER_DEVICE_ADDRESS_BIT |
				VK_BUFFER_USAGE_TRANSFER_SRC_BIT |
				VK_BUFFER_USAGE_TRANSFER_DST_BIT |
				(IsSamplerSet ? VK_BUFFER_USAGE_SAMPLER_DESCRIPTOR_BUFFER_BIT_EXT : VK_BUFFER_USAGE_RESOURCE_DESCRIPTOR_BUFFER_BIT_EXT);

			const uint32 DescriptorBufferSize = InOutState.DescriptorSize * InOutState.DescriptorCapacity;
			InOutState.CPUDescriptorMemory = reinterpret_cast<uint8*>(FMemory::MallocZeroed(DescriptorBufferSize));

			VkDeviceSize LayoutSizeInBytes = 0;
			VulkanRHI::vkGetDescriptorSetLayoutSizeEXT(DeviceHandle, InOutState.DescriptorSetLayout, &LayoutSizeInBytes);
			// Double check that the layout follows the rules for a single binding with an array of descriptors that are tightly packed
			if (LayoutSizeInBytes == 0)
			{
				// Reported this way by older drivers, log it as a warning.
				UE_LOGF(LogRHI, Warning, "vkGetDescriptorSetLayoutSizeEXT returned 0! Expected size: %u.", DescriptorBufferSize);
			}
			else
			{
				checkf((LayoutSizeInBytes == DescriptorBufferSize),
					TEXT("vkGetDescriptorSetLayoutSizeEXT returned %u instead of expected size of %u."),
					(uint32)LayoutSizeInBytes, DescriptorBufferSize);
			}

			if (IsSamplerSet)
			{
				checkf(DescriptorBufferSize < DescriptorBufferProperties.samplerDescriptorBufferAddressSpaceSize,
					TEXT("Sampler descriptor buffer size [%u] exceeded maximum [%llu]."),
					DescriptorBufferSize, DescriptorBufferProperties.samplerDescriptorBufferAddressSpaceSize);
			}
			else
			{
				checkf(DescriptorBufferSize < DescriptorBufferProperties.resourceDescriptorBufferAddressSpaceSize,
					TEXT("Combined resource descriptor buffer size of [%u] exceeded maximum [%llu]."),
					DescriptorBufferSize, DescriptorBufferProperties.resourceDescriptorBufferAddressSpaceSize);
			}

			// Create descriptor buffer
			{
				InOutState.BufferHandle = Device.CreateBuffer(DescriptorBufferSize, BufferUsageFlags);
			}

			// Allocate buffer memory, bind and map
			{
				VkMemoryRequirements BufferMemoryReqs;
				FMemory::Memzero(BufferMemoryReqs);
				VulkanRHI::vkGetBufferMemoryRequirements(DeviceHandle, InOutState.BufferHandle, &BufferMemoryReqs);
				check(BufferMemoryReqs.size >= DescriptorBufferSize);

				uint32 MemoryTypeIndex = 0;
				VERIFYVULKANRESULT(Device.GetDeviceMemoryManager().GetMemoryTypeFromProperties(BufferMemoryReqs.memoryTypeBits, GetDescriptorBufferMemoryType(Device), &MemoryTypeIndex));

				VkMemoryAllocateFlagsInfo FlagsInfo;
				ZeroVulkanStruct(FlagsInfo, VK_STRUCTURE_TYPE_MEMORY_ALLOCATE_FLAGS_INFO);
				FlagsInfo.flags = VK_MEMORY_ALLOCATE_DEVICE_ADDRESS_BIT;

				VkMemoryAllocateInfo AllocateInfo;
				ZeroVulkanStruct(AllocateInfo, VK_STRUCTURE_TYPE_MEMORY_ALLOCATE_INFO);
				AllocateInfo.allocationSize = BufferMemoryReqs.size;
				AllocateInfo.memoryTypeIndex = MemoryTypeIndex;
				AllocateInfo.pNext = &FlagsInfo;

				VERIFYVULKANRESULT(VulkanRHI::vkAllocateMemory(DeviceHandle, &AllocateInfo, VULKAN_CPU_ALLOCATOR, &InOutState.MemoryHandle));
				VERIFYVULKANRESULT(VulkanRHI::vkBindBufferMemory(DeviceHandle, InOutState.BufferHandle, InOutState.MemoryHandle, 0));
				VERIFYVULKANRESULT(VulkanRHI::vkMapMemory(DeviceHandle, InOutState.MemoryHandle, 0, VK_WHOLE_SIZE, 0, (void**)&InOutState.MappedPointer));
				FMemory::Memzero(InOutState.MappedPointer, AllocateInfo.allocationSize);
			}

			// Setup the binding info
			{
				VkBufferDeviceAddressInfo AddressInfo;
				ZeroVulkanStruct(AddressInfo, VK_STRUCTURE_TYPE_BUFFER_DEVICE_ADDRESS_INFO);
				AddressInfo.buffer = InOutState.BufferHandle;

				ZeroVulkanStruct(OutBufferBindingInfo, VK_STRUCTURE_TYPE_DESCRIPTOR_BUFFER_BINDING_INFO_EXT);
				OutBufferBindingInfo.address = VulkanRHI::vkGetBufferDeviceAddressKHR(DeviceHandle, &AddressInfo);
				OutBufferBindingInfo.usage = BufferUsageFlags;
			}

			// Add the types to the global groups
			GRHIGlobals.DescriptorTypeGroups.Emplace(InOutState.Allocator->GetTypeMask());
		};

		// Fill in one state for each descriptor type
		SamplerState.DescriptorSetLayout = CreateDescriptorSetLayout(SamplerState);
		CreateDescriptorBuffer(SamplerState, BufferBindingInfo[VulkanBindless::BindlessSamplerSet]);
		ResourceState.DescriptorSetLayout = CreateDescriptorSetLayout(ResourceState);
		CreateDescriptorBuffer(ResourceState, BufferBindingInfo[VulkanBindless::BindlessResourceSet]);

		// Fill in the state for single-use UB
		// Uniform buffer descriptor set layout differ from the other resources, we reserve a fixed number of descriptors per stage for each draw/dispatch
		{
			// Guaranteed to be min 32 by Vulkan 1.4
			constexpr uint32 NumTotalBindings = VulkanBindless::MaxUniformBuffersTotal;

			TArray<VkDescriptorSetLayoutBinding> DescriptorSetLayoutBindings;
			DescriptorSetLayoutBindings.SetNumZeroed(NumTotalBindings);
			for (uint32 BindingIndex = 0; BindingIndex < NumTotalBindings; ++BindingIndex)
			{
				DescriptorSetLayoutBindings[BindingIndex].binding = BindingIndex;
				DescriptorSetLayoutBindings[BindingIndex].descriptorType = VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER;
				DescriptorSetLayoutBindings[BindingIndex].descriptorCount = 1;
				DescriptorSetLayoutBindings[BindingIndex].stageFlags = VK_SHADER_STAGE_ALL;
			}

			VkDescriptorSetLayoutCreateInfo DescriptorSetLayoutCreateInfo;
			ZeroVulkanStruct(DescriptorSetLayoutCreateInfo, VK_STRUCTURE_TYPE_DESCRIPTOR_SET_LAYOUT_CREATE_INFO);
			DescriptorSetLayoutCreateInfo.pBindings = DescriptorSetLayoutBindings.GetData();
			DescriptorSetLayoutCreateInfo.bindingCount = NumTotalBindings;
			DescriptorSetLayoutCreateInfo.flags = VK_DESCRIPTOR_SET_LAYOUT_CREATE_DESCRIPTOR_BUFFER_BIT_EXT | VK_DESCRIPTOR_SET_LAYOUT_CREATE_PUSH_DESCRIPTOR_BIT_KHR;
			DescriptorSetLayoutCreateInfo.pNext = nullptr;

			checkSlow(SingleUseUBDescriptorSetLayout == VK_NULL_HANDLE);
			VERIFYVULKANRESULT(VulkanRHI::vkCreateDescriptorSetLayout(DeviceHandle, &DescriptorSetLayoutCreateInfo, VULKAN_CPU_ALLOCATOR, &SingleUseUBDescriptorSetLayout));
			VULKAN_SET_DEBUG_NAME(Device, VK_OBJECT_TYPE_DESCRIPTOR_SET_LAYOUT, SingleUseUBDescriptorSetLayout, TEXT("UBPushDescriptors(%d)"), NumTotalBindings);
		}
	}

	// Now create the single pipeline layout used by everything
	{
		BindlessDescriptorSetLayouts[VulkanBindless::BindlessSamplerSet] = SamplerState.DescriptorSetLayout;
		BindlessDescriptorSetLayouts[VulkanBindless::BindlessResourceSet] = ResourceState.DescriptorSetLayout;
		BindlessDescriptorSetLayouts[VulkanBindless::BindlessSingleUseUniformBufferSet] = SingleUseUBDescriptorSetLayout;
		check(VulkanBindless::BindlessSingleUseUniformBufferSet < BindlessDescriptorSetLayouts.Num());

		const VkPipelineLayoutCreateInfo PipelineLayoutCreateInfo =
		{
			.sType = VK_STRUCTURE_TYPE_PIPELINE_LAYOUT_CREATE_INFO,
			.pNext = nullptr,
			.flags = 0,
			.setLayoutCount = (uint32)BindlessDescriptorSetLayouts.Num(),
			.pSetLayouts = BindlessDescriptorSetLayouts.GetData(),
			.pushConstantRangeCount = UE_ARRAY_COUNT(PushConstantRanges),
			.pPushConstantRanges = PushConstantRanges
		};

		checkf((uint32)EPushConstantRangeType::Count == UE_ARRAY_COUNT(PushConstantRanges),
			TEXT("Mismatch between expected push constant range count (%d) and provided array size (%zu)"),
			(uint32)EPushConstantRangeType::Count, UE_ARRAY_COUNT(PushConstantRanges));

		VERIFYVULKANRESULT(VulkanRHI::vkCreatePipelineLayout(DeviceHandle, &PipelineLayoutCreateInfo, VULKAN_CPU_ALLOCATOR, &BindlessPipelineLayout));
		VULKAN_SET_DEBUG_NAME(Device, VK_OBJECT_TYPE_PIPELINE_LAYOUT, BindlessPipelineLayout, TEXT("BindlessPipelineLayout(SetCount=%d)"), VulkanBindless::MaxNumSets);
	}
}

void FVulkanBindlessDescriptorManager::Deinit()
{
	const VkDevice DeviceHandle = Device.GetHandle();

	if (IsSupported())
	{
		VulkanRHI::vkDestroyPipelineLayout(DeviceHandle, BindlessPipelineLayout, VULKAN_CPU_ALLOCATOR);
		BindlessPipelineLayout = VK_NULL_HANDLE;

		auto DestroyBindlessState = [DeviceHandle](BindlessSetState& State)
		{
			if (State.DescriptorSetLayout)
			{
				VulkanRHI::vkDestroyDescriptorSetLayout(DeviceHandle, State.DescriptorSetLayout, VULKAN_CPU_ALLOCATOR);
				State.DescriptorSetLayout = VK_NULL_HANDLE;
			}

			VulkanRHI::vkDestroyBuffer(DeviceHandle, State.BufferHandle, VULKAN_CPU_ALLOCATOR);
			State.BufferHandle = VK_NULL_HANDLE;

			VulkanRHI::vkUnmapMemory(DeviceHandle, State.MemoryHandle);
			VulkanRHI::vkFreeMemory(DeviceHandle, State.MemoryHandle, VULKAN_CPU_ALLOCATOR);
			State.MemoryHandle = VK_NULL_HANDLE;

			if (State.Allocator)
			{
				delete State.Allocator;
				State.Allocator = nullptr;
			}
		};

		DestroyBindlessState(SamplerState);
		DestroyBindlessState(ResourceState);

		VulkanRHI::vkDestroyDescriptorSetLayout(DeviceHandle, SingleUseUBDescriptorSetLayout, VULKAN_CPU_ALLOCATOR);
		SingleUseUBDescriptorSetLayout = VK_NULL_HANDLE;
	}
}

void FVulkanBindlessDescriptorManager::BindDescriptorBuffers(VkCommandBuffer CommandBuffer, VkPipelineStageFlags SupportedStages)
{
	checkf(IsSupported(), TEXT("Trying to BindDescriptorBuffers but bindless is not supported!"));
	check(UseDescriptorBuffers());

	VulkanRHI::vkCmdBindDescriptorBuffersEXT(CommandBuffer, VulkanBindless::NumBindlessSets, BufferBindingInfo);

	VkDeviceSize BufferOffsets[VulkanBindless::NumBindlessSets];
	FMemory::Memzero(BufferOffsets);
	if (SupportedStages & VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT)
	{
		VulkanRHI::vkCmdSetDescriptorBufferOffsetsEXT(CommandBuffer, VK_PIPELINE_BIND_POINT_GRAPHICS, BindlessPipelineLayout, 0, VulkanBindless::NumBindlessSets, BufferIndices, BufferOffsets);
	}
	if (SupportedStages & VK_PIPELINE_STAGE_COMPUTE_SHADER_BIT)
	{
		VulkanRHI::vkCmdSetDescriptorBufferOffsetsEXT(CommandBuffer, VK_PIPELINE_BIND_POINT_COMPUTE, BindlessPipelineLayout, 0, VulkanBindless::NumBindlessSets, BufferIndices, BufferOffsets);
	}
	if (SupportedStages & VK_PIPELINE_STAGE_RAY_TRACING_SHADER_BIT_KHR)
	{
		VulkanRHI::vkCmdSetDescriptorBufferOffsetsEXT(CommandBuffer, VK_PIPELINE_BIND_POINT_RAY_TRACING_KHR, BindlessPipelineLayout, 0, VulkanBindless::NumBindlessSets, BufferIndices, BufferOffsets);
	}
}

void FVulkanBindlessDescriptorManager::RegisterUniformBuffers(FVulkanCommandListContext& Context, VkShaderStageFlags StageFlags, TConstArrayView<VkWriteDescriptorSet> UBDescriptors)
{
	checkf(IsSupported(), TEXT("Trying to RegisterUniformBuffers but bindless is not supported!"));
	check(UBDescriptors.Num() < VulkanBindless::MaxUniformBuffersTotal);
	check(UseDescriptorBuffers());

	const VkPushDescriptorSetInfo PushDescriptorSetInfo =
	{
		.sType = VK_STRUCTURE_TYPE_PUSH_DESCRIPTOR_SET_INFO,
		.pNext = nullptr,
		.stageFlags = StageFlags,
		.layout = BindlessPipelineLayout,
		.set = VulkanBindless::BindlessSingleUseUniformBufferSet,
		.descriptorWriteCount = (uint32)UBDescriptors.Num(),
		.pDescriptorWrites = UBDescriptors.GetData()
	};

	VulkanRHI::vkCmdPushDescriptorSet2KHR(Context.GetCommandBuffer().GetHandle(), &PushDescriptorSetInfo);
}

FRHIDescriptorHandle FVulkanBindlessDescriptorManager::AllocateDescriptor(ERHIDescriptorType DescriptorType)
{
	if (IsSupported())
	{
		INC_DWORD_STAT_BY_FName(GetStatForDescriptorType(DescriptorType).GetName(), 1);
		return GetBindlessState(DescriptorType).Allocator->Allocate(DescriptorType);
	}

	return FRHIDescriptorHandle();
}

TOptional<FRHIDescriptorAllocation> FVulkanBindlessDescriptorManager::AllocateDescriptors(ERHIDescriptorType DescriptorType, uint32 DescriptorCount)
{
	if (IsSupported())
	{
		INC_DWORD_STAT_BY_FName(GetStatForDescriptorType(DescriptorType).GetName(), DescriptorCount);
		return GetBindlessState(DescriptorType).Allocator->Allocate(DescriptorCount);
	}

	return TOptional<FRHIDescriptorAllocation>();
}

void FVulkanBindlessDescriptorManager::FreeDescriptor(FRHIDescriptorHandle DescriptorHandle)
{
	if (DescriptorHandle.IsValid())
	{
		DEC_DWORD_STAT_BY_FName(GetStatForDescriptorType(DescriptorHandle.GetType()).GetName(), 1);

		BindlessSetState& State = GetBindlessState(DescriptorHandle.GetType());
		State.Allocator->Free(DescriptorHandle);

		if (UseDescriptorBuffers())
		{
			const uint32 ByteOffset = DescriptorHandle.GetIndex() * State.DescriptorSize;
			FMemory::Memzero(&State.CPUDescriptorMemory[ByteOffset], State.DescriptorSize); // easier for debugging for now
		}
	}
}

void FVulkanBindlessDescriptorManager::FreeDescriptors(FRHIDescriptorAllocation Allocation, ERHIDescriptorType DescriptorType)
{
	if (Allocation.Count > 0)
	{
		DEC_DWORD_STAT_BY_FName(GetStatForDescriptorType(DescriptorType).GetName(), Allocation.Count);

		BindlessSetState& State = GetBindlessState(DescriptorType);
		State.Allocator->Free(Allocation);

		if (UseDescriptorBuffers())
		{
			const uint32 ByteOffset = Allocation.StartIndex * State.DescriptorSize;
			FMemory::Memzero(&State.CPUDescriptorMemory[ByteOffset], State.DescriptorSize * Allocation.Count); // easier for debugging for now
		}
	}
}

void FVulkanBindlessDescriptorManager::UpdateDescriptorInternal(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, VkDescriptorDataEXT DescriptorData, bool bImmediateUpdate)
{
	check(UseDescriptorBuffers());
	checkf(DescriptorHandle.IsValid(), TEXT("Attemping to update invalid descriptor handle!"));

	const VkDescriptorType DescriptorType = GetVkDescriptorTypeFromUEDescriptorType(DescriptorHandle.GetType());

	BindlessSetState& State = GetBindlessState(DescriptorHandle.GetType());
	const uint32 ByteOffset = DescriptorHandle.GetIndex() * State.DescriptorSize;
	checkSlow(State.DescriptorTypes.Contains(DescriptorType));

	uint8* const CPUDescriptor = &State.CPUDescriptorMemory[ByteOffset];

	VkDescriptorGetInfoEXT Info;
	ZeroVulkanStruct(Info, VK_STRUCTURE_TYPE_DESCRIPTOR_GET_INFO_EXT);
	Info.type = DescriptorType;
	Info.data = DescriptorData;
	VulkanRHI::vkGetDescriptorEXT(Device.GetHandle(), &Info, GetDescriptorTypeSize(Device, DescriptorType), CPUDescriptor);

	if (bImmediateUpdate)
	{
		FMemory::Memcpy(&State.MappedPointer[ByteOffset], CPUDescriptor, State.DescriptorSize);
	}
	else
	{
		if (FVulkanCommandListContext * Context = Contexts[ERHIPipeline::Graphics])
		{
			FVulkanCommandBuffer* CmdBuffer = Context->GetActiveCmdBuffer();

			// :todo-jn:  Hack to avoid barriers/copies in renderpasses
			if (CmdBuffer->IsInsideRenderPass())
			{
				FMemory::Memcpy(&State.MappedPointer[ByteOffset], CPUDescriptor, State.DescriptorSize);
			}
			else
			{
				VulkanRHI::FStagingBuffer* StagingBuffer = Device.GetStagingManager().AcquireBuffer(State.DescriptorSize);
				FMemory::Memcpy(StagingBuffer->GetMappedPointer(), CPUDescriptor, State.DescriptorSize);

				VkMemoryBarrier2 MemoryBarrier;
				ZeroVulkanStruct(MemoryBarrier, VK_STRUCTURE_TYPE_MEMORY_BARRIER_2);
				MemoryBarrier.srcStageMask = VK_PIPELINE_STAGE_2_ALL_COMMANDS_BIT;
				MemoryBarrier.srcAccessMask = VK_ACCESS_2_DESCRIPTOR_BUFFER_READ_BIT_EXT | VK_ACCESS_2_MEMORY_READ_BIT | VK_ACCESS_2_MEMORY_WRITE_BIT;
				MemoryBarrier.dstStageMask = VK_PIPELINE_STAGE_2_TRANSFER_BIT;
				MemoryBarrier.dstAccessMask = VK_ACCESS_2_TRANSFER_READ_BIT | VK_ACCESS_2_TRANSFER_WRITE_BIT;

				VkDependencyInfo DependencyInfo;
				ZeroVulkanStruct(DependencyInfo, VK_STRUCTURE_TYPE_DEPENDENCY_INFO);
				DependencyInfo.memoryBarrierCount = 1;
				DependencyInfo.pMemoryBarriers = &MemoryBarrier;
				VulkanRHI::vkCmdPipelineBarrier2KHR(CmdBuffer->GetHandle(), &DependencyInfo);

				VkBufferCopy Region = {};
				Region.srcOffset = 0;
				Region.dstOffset = ByteOffset;
				Region.size = State.DescriptorSize;
				VulkanRHI::vkCmdCopyBuffer(CmdBuffer->GetHandle(), StagingBuffer->GetHandle(), State.BufferHandle, 1, &Region);

				MemoryBarrier.srcStageMask = VK_PIPELINE_STAGE_2_TRANSFER_BIT;
				MemoryBarrier.srcAccessMask = VK_ACCESS_2_TRANSFER_READ_BIT | VK_ACCESS_2_TRANSFER_WRITE_BIT;
				MemoryBarrier.dstStageMask = VK_PIPELINE_STAGE_2_ALL_COMMANDS_BIT;
				MemoryBarrier.dstAccessMask = VK_ACCESS_2_DESCRIPTOR_BUFFER_READ_BIT_EXT;
				VulkanRHI::vkCmdPipelineBarrier2KHR(CmdBuffer->GetHandle(), &DependencyInfo);

				Device.GetStagingManager().ReleaseBuffer(Context, StagingBuffer);
			}
		}
	}
}

void FVulkanBindlessDescriptorManager::UpdateDescriptor(FRHIDescriptorHandle DescriptorHandle, VkSampler VulkanSampler)
{
	if (IsSupported())
	{
		check(UseDescriptorBuffers());
		VkDescriptorDataEXT DescriptorData;
		DescriptorData.pSampler = &VulkanSampler;
		UpdateDescriptorInternal({}, DescriptorHandle, DescriptorData, true);
	}
}

void FVulkanBindlessDescriptorManager::UpdateDescriptor(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, VkImageView ImageView, bool bIsDepthStencil, bool bImmediateUpdate)
{
	if (IsSupported())
	{
		check(UseDescriptorBuffers());
		check(DescriptorHandle.GetType() == ERHIDescriptorType::TextureSRV || DescriptorHandle.GetType() == ERHIDescriptorType::TextureUAV);

		VkDescriptorImageInfo DescriptorImageInfo;
		DescriptorImageInfo.sampler = VK_NULL_HANDLE;
		DescriptorImageInfo.imageView = ImageView;
		DescriptorImageInfo.imageLayout =
			DescriptorHandle.GetType() == ERHIDescriptorType::TextureUAV
			? VK_IMAGE_LAYOUT_GENERAL
			: (bIsDepthStencil ? VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL : VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL);

		VkDescriptorDataEXT DescriptorData;
		DescriptorData.pSampledImage = &DescriptorImageInfo;  // same pointer for storage, it's a union
		UpdateDescriptorInternal(Contexts, DescriptorHandle, DescriptorData, bImmediateUpdate);
	}
}

void FVulkanBindlessDescriptorManager::UpdateDescriptor(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, VkBuffer VulkanBuffer, VkDeviceSize BufferOffset, VkDeviceSize BufferSize, bool bImmediateUpdate)
{
	if (IsSupported())
	{
		const VkBufferDeviceAddressInfo BufferInfo =
		{
			.sType = VK_STRUCTURE_TYPE_BUFFER_DEVICE_ADDRESS_INFO,
			.pNext = nullptr,
			.buffer = VulkanBuffer
		};
		const VkDeviceAddress BufferAddress = VulkanRHI::vkGetBufferDeviceAddressKHR(Device.GetHandle(), &BufferInfo);

		if (UseDescriptorBuffers())
		{
			check(DescriptorHandle.GetType() == ERHIDescriptorType::CBV || DescriptorHandle.GetType() == ERHIDescriptorType::BufferSRV || DescriptorHandle.GetType() == ERHIDescriptorType::BufferUAV);
			UpdateDescriptor(Contexts, DescriptorHandle, BufferAddress + BufferOffset, BufferSize, bImmediateUpdate);
		}
		else
		{
			check(DescriptorHandle.GetType() == ERHIDescriptorType::CBV || DescriptorHandle.GetType() == ERHIDescriptorType::BufferSRV || DescriptorHandle.GetType() == ERHIDescriptorType::BufferUAV || DescriptorHandle.GetType() == ERHIDescriptorType::AccelerationStructure);
			check(UseDescriptorHeaps());

			const VkDeviceAddressRangeEXT DeviceAddressRange = {
				.address = BufferAddress + BufferOffset,
				.size = BufferSize
			};

			UpdateDescriptor(Contexts, DescriptorHandle, DeviceAddressRange, bImmediateUpdate);
		}
	}
}

void FVulkanBindlessDescriptorManager::UpdateDescriptor(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, VkDeviceAddress BufferAddress, VkDeviceSize BufferSize, bool bImmediateUpdate)
{
	if (IsSupported())
	{
		check(DescriptorHandle.GetType() == ERHIDescriptorType::BufferSRV || DescriptorHandle.GetType() == ERHIDescriptorType::CBV || DescriptorHandle.GetType() == ERHIDescriptorType::BufferUAV);

		if (UseDescriptorBuffers())
		{
			VkDescriptorAddressInfoEXT AddressInfo;
			ZeroVulkanStruct(AddressInfo, VK_STRUCTURE_TYPE_DESCRIPTOR_ADDRESS_INFO_EXT);
			AddressInfo.address = BufferAddress;
			AddressInfo.range = BufferSize;

			VkDescriptorDataEXT DescriptorData;
			DescriptorData.pStorageBuffer = &AddressInfo;  // same pointer for uniform, it's a union
			UpdateDescriptorInternal(Contexts, DescriptorHandle, DescriptorData, bImmediateUpdate);
		}
		else
		{
			check(UseDescriptorHeaps());

			const VkDeviceAddressRangeEXT DeviceAddressRange = {
				.address = BufferAddress,
				.size = BufferSize
			};

			UpdateDescriptor(Contexts, DescriptorHandle, DeviceAddressRange, bImmediateUpdate);
		}
	}
}

void FVulkanBindlessDescriptorManager::UpdateDescriptor(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, const VkBufferViewCreateInfo& ViewInfo, bool bImmediateUpdate)
{
	if (IsSupported())
	{
		check(DescriptorHandle.GetType() == ERHIDescriptorType::TypedBufferSRV || DescriptorHandle.GetType() == ERHIDescriptorType::TypedBufferUAV);

		// :todo-jn: start caching buffer addresses in resources to avoid the extra call
		const VkBufferDeviceAddressInfo BufferInfo = {
			.sType = VK_STRUCTURE_TYPE_BUFFER_DEVICE_ADDRESS_INFO,
			.pNext = nullptr,
			.buffer = ViewInfo.buffer
		};
		const VkDeviceAddress BufferAddress = VulkanRHI::vkGetBufferDeviceAddressKHR(Device.GetHandle(), &BufferInfo);

		if (UseDescriptorBuffers())
		{
			VkDescriptorAddressInfoEXT AddressInfo;
			ZeroVulkanStruct(AddressInfo, VK_STRUCTURE_TYPE_DESCRIPTOR_ADDRESS_INFO_EXT);
			AddressInfo.address = BufferAddress + ViewInfo.offset;
			AddressInfo.range = ViewInfo.range;
			AddressInfo.format = ViewInfo.format;

			VkDescriptorDataEXT DescriptorData;
			DescriptorData.pUniformTexelBuffer = &AddressInfo;  // same pointer for storage, it's a union
			UpdateDescriptorInternal(Contexts, DescriptorHandle, DescriptorData, bImmediateUpdate);
		}
		else
		{
			check(UseDescriptorHeaps());

			const VkTexelBufferDescriptorInfoEXT TexelBufferDescriptorInfo = {
				.sType = VK_STRUCTURE_TYPE_TEXEL_BUFFER_DESCRIPTOR_INFO_EXT,
				.pNext = nullptr,
				.format = ViewInfo.format,
				.addressRange = {
					.address = BufferAddress + ViewInfo.offset,
					.size = ViewInfo.range
				}
			};

			UpdateDescriptor(Contexts, DescriptorHandle, TexelBufferDescriptorInfo, bImmediateUpdate);
		}
	}
}

void FVulkanBindlessDescriptorManager::UpdateDescriptor(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, VkAccelerationStructureKHR AccelerationStructure, bool bImmediateUpdate)
{
	if (IsSupported())
	{
		check(UseDescriptorBuffers());

		// :todo-jn: start caching AccelerationStructure in resources to avoid the extra call
		VkAccelerationStructureDeviceAddressInfoKHR AccelerationStructureDeviceAddressInfo;
		ZeroVulkanStruct(AccelerationStructureDeviceAddressInfo, VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_DEVICE_ADDRESS_INFO_KHR);
		AccelerationStructureDeviceAddressInfo.accelerationStructure = AccelerationStructure;
		const VkDeviceAddress BufferAddress = VulkanRHI::vkGetAccelerationStructureDeviceAddressKHR(Device.GetHandle(), &AccelerationStructureDeviceAddressInfo);

		VkDescriptorDataEXT DescriptorData;
		DescriptorData.accelerationStructure = BufferAddress;
		UpdateDescriptorInternal(Contexts, DescriptorHandle, DescriptorData, bImmediateUpdate);
	}
}


void FVulkanBindlessDescriptorManager::UpdateDescriptor(FRHIDescriptorHandle DescriptorHandle, const VkSamplerCreateInfo& SamplerCreateInfo)
{
	if (IsSupported())
	{
		check(UseDescriptorHeaps());

		const uint32 ByteOffset = DescriptorHandle.GetIndex() * SamplerState.DescriptorSize;

		const VkHostAddressRangeEXT HostAddressRangeEXT = {
			.address = &SamplerState.MappedPointer[ByteOffset],
			.size = SamplerState.DescriptorSize
		};

		VERIFYVULKANRESULT(VulkanRHI::vkWriteSamplerDescriptorsEXT(Device.GetHandle(), 1 /*samplerCount*/, &SamplerCreateInfo, &HostAddressRangeEXT));
	}
}

void FVulkanBindlessDescriptorManager::UpdateDescriptorInternal(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, const VkResourceDescriptorDataEXT& DescriptorData, bool bImmediateUpdate)
{
	if (IsSupported())
	{
		check(UseDescriptorHeaps());

		const VkResourceDescriptorInfoEXT ResourceDescriptorInfo = {
			.sType = VK_STRUCTURE_TYPE_RESOURCE_DESCRIPTOR_INFO_EXT,
			.pNext = nullptr,
			.type = GetVkDescriptorTypeFromUEDescriptorType(DescriptorHandle.GetType()),
			.data = DescriptorData
		};

		const uint32 ByteOffset = DescriptorHandle.GetIndex() * ResourceState.DescriptorSize;

		// :todo-jn:  Hack to avoid barriers/copies in renderpasses
		FVulkanCommandListContext* Context = Contexts[ERHIPipeline::Graphics];
		const bool bIsInsideRenderPass = Context && Context->GetCommandBuffer().IsInsideRenderPass();

		if (bImmediateUpdate || bIsInsideRenderPass)
		{
			const VkHostAddressRangeEXT HostAddressRangeEXT = {
				.address = &ResourceState.MappedPointer[ByteOffset],
				.size = ResourceState.DescriptorSize
			};

			VERIFYVULKANRESULT(VulkanRHI::vkWriteResourceDescriptorsEXT(Device.GetHandle(), 1 /*resourceCount*/, &ResourceDescriptorInfo, &HostAddressRangeEXT));
		}
		else
		{
			checkf(Context, TEXT("Context was NULL, could not update descriptor"));

			VulkanRHI::FStagingBuffer* StagingBuffer = Device.GetStagingManager().AcquireBuffer(ResourceState.DescriptorSize);
			const VkHostAddressRangeEXT HostAddressRangeEXT = {
				.address = StagingBuffer->GetMappedPointer(),
				.size = ResourceState.DescriptorSize
			};
			VERIFYVULKANRESULT(VulkanRHI::vkWriteResourceDescriptorsEXT(Device.GetHandle(), 1 /*resourceCount*/, &ResourceDescriptorInfo, &HostAddressRangeEXT));

			VkMemoryBarrier2 MemoryBarrier = {
				.sType = VK_STRUCTURE_TYPE_MEMORY_BARRIER_2,
				.pNext = nullptr,
				.srcStageMask = VK_PIPELINE_STAGE_2_ALL_COMMANDS_BIT,
				.srcAccessMask = VK_ACCESS_2_RESOURCE_HEAP_READ_BIT_EXT | VK_ACCESS_2_MEMORY_READ_BIT | VK_ACCESS_2_MEMORY_WRITE_BIT,
				.dstStageMask = VK_PIPELINE_STAGE_2_TRANSFER_BIT,
				.dstAccessMask = VK_ACCESS_2_TRANSFER_READ_BIT | VK_ACCESS_2_TRANSFER_WRITE_BIT
			};

			FVulkanCommandBuffer& CommandBuffer = Context->GetCommandBuffer();

			VkDependencyInfo DependencyInfo;
			ZeroVulkanStruct(DependencyInfo, VK_STRUCTURE_TYPE_DEPENDENCY_INFO);
			DependencyInfo.memoryBarrierCount = 1;
			DependencyInfo.pMemoryBarriers = &MemoryBarrier;
			VulkanRHI::vkCmdPipelineBarrier2KHR(CommandBuffer.GetHandle(), &DependencyInfo);

			const VkBufferCopy Region = {
				.srcOffset = 0,
				.dstOffset = ByteOffset,
				.size = ResourceState.DescriptorSize
			};
			VulkanRHI::vkCmdCopyBuffer(CommandBuffer.GetHandle(), StagingBuffer->GetHandle(), ResourceState.BufferHandle, 1, &Region);

			MemoryBarrier.srcStageMask = VK_PIPELINE_STAGE_2_TRANSFER_BIT;
			MemoryBarrier.srcAccessMask = VK_ACCESS_2_TRANSFER_READ_BIT | VK_ACCESS_2_TRANSFER_WRITE_BIT;
			MemoryBarrier.dstStageMask = VK_PIPELINE_STAGE_2_ALL_COMMANDS_BIT;
			MemoryBarrier.dstAccessMask = VK_ACCESS_2_RESOURCE_HEAP_READ_BIT_EXT;
			VulkanRHI::vkCmdPipelineBarrier2KHR(CommandBuffer.GetHandle(), &DependencyInfo);

			Device.GetStagingManager().ReleaseBuffer(Context, StagingBuffer);
		}
	}
}

TConstArrayView<VkDescriptorSetAndBindingMappingEXT> FVulkanBindlessDescriptorManager::GetBindingMappings(EShaderFrequency Frequency, bool bHasGlobals) const
{
	if (UseDescriptorBuffers())
	{
		return {};
	}

	if (IsValidGraphicsFrequency(Frequency))
	{
		return bHasGlobals ? GraphicsBindingMappingsWithPushAddr[Frequency] : GraphicsBindingMappings[Frequency];
	}
	else if ((Frequency == SF_Compute) || (Frequency == SF_RayGen))
	{
		return bHasGlobals ? ComputeBindingMappingsWithPushAddr : ComputeBindingMappings;
	}
	else
	{
		return HeapOnlyBindingMappings;
	}
}
