// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#include "RHIDefinitions.h"
#include "VulkanCommon.h"
#include "VulkanMemory.h"
#include "VulkanThirdParty.h"
#include "RHIDescriptorAllocator.h"

class FVulkanDevice;
class FVulkanBuffer;

// Manager for resource descriptors used in bindless rendering.
class FVulkanBindlessDescriptorManager
{
public:
	enum class EBindlessType : uint8
	{
		DescriptorHeap,
		DescriptorBuffer,
		Disabled
	};

	enum class EPushConstantRangeType : uint8
	{
		StaticUniformBuffers,

		Count
	};

	static constexpr uint32 RequiredPushDataSize = 256u;
	static uint32 GetBindingPushDataOffset(EShaderFrequency Frequency, uint32 BindingIndex);

	FVulkanBindlessDescriptorManager(FVulkanDevice& InDevice);
	~FVulkanBindlessDescriptorManager();

	void Init();
	void Deinit();

	FRHIDescriptorHandle AllocateDescriptor(ERHIDescriptorType DescriptorType);
	TOptional<FRHIDescriptorAllocation> AllocateDescriptors(ERHIDescriptorType DescriptorType, uint32 DescriptorCount);
	void FreeDescriptor(FRHIDescriptorHandle DescriptorHandle);
	void FreeDescriptors(FRHIDescriptorAllocation Allocation, ERHIDescriptorType DescriptorType);

	// Descriptor Heap update functions
	void UpdateDescriptor(FRHIDescriptorHandle DescriptorHandle, const VkSamplerCreateInfo& SamplerCreateInfo);
	void UpdateDescriptor(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, const VkImageDescriptorInfoEXT& ImageDescriptorInfo, bool bImmediateUpdate = true)
	{
		check(DescriptorHandle.GetType() == ERHIDescriptorType::TextureSRV || DescriptorHandle.GetType() == ERHIDescriptorType::TextureUAV);
		UpdateDescriptorInternal(Contexts, DescriptorHandle, VkResourceDescriptorDataEXT{ .pImage = &ImageDescriptorInfo }, bImmediateUpdate);
	}
	void UpdateDescriptor(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, const VkTexelBufferDescriptorInfoEXT& TexelBufferDescriptorInfo, bool bImmediateUpdate = true)
	{
		check(DescriptorHandle.GetType() == ERHIDescriptorType::TypedBufferSRV || DescriptorHandle.GetType() == ERHIDescriptorType::TypedBufferUAV);
		UpdateDescriptorInternal(Contexts, DescriptorHandle, VkResourceDescriptorDataEXT{ .pTexelBuffer = &TexelBufferDescriptorInfo }, bImmediateUpdate);
	}
	void UpdateDescriptor(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, const VkDeviceAddressRangeEXT& DeviceAddressRange, bool bImmediateUpdate = true)
	{
		check(DescriptorHandle.GetType() == ERHIDescriptorType::CBV || DescriptorHandle.GetType() == ERHIDescriptorType::BufferSRV ||DescriptorHandle.GetType() == ERHIDescriptorType::BufferUAV || DescriptorHandle.GetType() == ERHIDescriptorType::AccelerationStructure);
		UpdateDescriptorInternal(Contexts, DescriptorHandle, VkResourceDescriptorDataEXT{ .pAddressRange = &DeviceAddressRange }, bImmediateUpdate);
	}

	// Descriptor Buffer functions
	void BindDescriptorBuffers(VkCommandBuffer CommandBuffer, VkPipelineStageFlags SupportedStages);
	void UpdateDescriptor(FRHIDescriptorHandle DescriptorHandle, VkSampler VulkanSampler);
	void UpdateDescriptor(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, VkImageView VulkanImage, bool bIsDepthStencil, bool bImmediateUpdate = true);
	void UpdateDescriptor(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, VkBuffer VulkanBuffer, VkDeviceSize BufferOffset, VkDeviceSize BufferSize, bool bImmediateUpdate = true);
	void UpdateDescriptor(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, VkDeviceAddress BufferAddress, VkDeviceSize BufferSize, bool bImmediateUpdate = true);
	void UpdateDescriptor(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, const VkBufferViewCreateInfo& ViewInfo, bool bImmediateUpdate = true);
	void UpdateDescriptor(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, VkAccelerationStructureKHR AccelerationStructure, bool bImmediateUpdate = true);
	void RegisterUniformBuffers(FVulkanCommandListContext& Context, VkShaderStageFlags StageFlags, TConstArrayView<VkWriteDescriptorSet> UBDescriptors);

	TConstArrayView<VkDescriptorSetAndBindingMappingEXT> GetBindingMappings(EShaderFrequency Frequency, bool bHasGlobals) const;

	ERHIBindlessConfiguration GetConfiguration() const
	{
		return Configuration;
	}

	bool IsSupported() const
	{
		return !IsBindlessDisabled(Configuration);
	}

	VkPipelineCreateFlagBits2 GetPipelineCreateFlag() const
	{
		return UseDescriptorBuffers() ? VK_PIPELINE_CREATE_2_DESCRIPTOR_BUFFER_BIT_EXT :
			   UseDescriptorHeaps() ? VK_PIPELINE_CREATE_2_DESCRIPTOR_HEAP_BIT_EXT : 0ull;
	}

	VkPipelineLayout GetPipelineLayout() const
	{
		return UseDescriptorBuffers() ? BindlessPipelineLayout : nullptr;
	}

	// Returns descriptor set layouts for a single stage
	TConstArrayView<VkDescriptorSetLayout> GetDescriptorSetLayouts() const
	{
		return UseDescriptorBuffers() ? BindlessDescriptorSetLayouts : TConstArrayView<VkDescriptorSetLayout>();
	}

	TConstArrayView<VkPushConstantRange> GetPushConstantRanges() const
	{
		return UseDescriptorBuffers() ? PushConstantRanges : TConstArrayView<VkPushConstantRange>();
	}

	bool UseDescriptorHeaps() const
	{
		return BindlessType == EBindlessType::DescriptorHeap;
	}

	bool UseDescriptorBuffers() const
	{
		return BindlessType == EBindlessType::DescriptorBuffer;
	}

	const VkBindHeapInfoEXT& GetSamplerBindHeapInfo() const
	{
		return SamplerBindHeapInfo;
	}

	const VkBindHeapInfoEXT& GetResourceBindHeapInfo() const
	{
		return ResourceBindHeapInfo;
	}

	bool SupportsBindlessAccelerationStructures() const;

private:
	FVulkanDevice& Device;

	ERHIBindlessConfiguration Configuration{};
	ERHIPipeline              ConfiguredPipelines = ERHIPipeline::None;

	EBindlessType BindlessType = EBindlessType::Disabled;

	struct BindlessSetState
	{
		~BindlessSetState()
		{
			if (CPUDescriptorMemory)
			{
				FMemory::Free(CPUDescriptorMemory);
				CPUDescriptorMemory = nullptr;
			}

			if (Allocator)
			{
				delete Allocator;
				Allocator = nullptr;
			}
		}

		FRHIHeapDescriptorAllocator* Allocator = nullptr;

		uint32 DescriptorSize = 0;

		VkBuffer BufferHandle = VK_NULL_HANDLE;
		VkDeviceMemory MemoryHandle = VK_NULL_HANDLE;
		uint8* MappedPointer = nullptr;

		// Descriptor Buffer specific
		TArray<VkDescriptorType, TInlineAllocator<2>> DescriptorTypes;
		VkDescriptorSetLayout DescriptorSetLayout = VK_NULL_HANDLE;
		uint32 DescriptorCapacity = 0;
		uint8* CPUDescriptorMemory = nullptr;
	};
	BindlessSetState SamplerState;
	BindlessSetState ResourceState;

	BindlessSetState& GetBindlessState(ERHIDescriptorType RHIDescriptorType);

	// Descriptor Heap specific values
	VkBindHeapInfoEXT SamplerBindHeapInfo;
	VkBindHeapInfoEXT ResourceBindHeapInfo;

	// Only provides binding mappings for the Sampler and Resource Heaps
	// NOTE: Won't be necessary when our DXC supports SamplerHeapEXT / ResourceHeapEXT builtins
	TArray<VkDescriptorSetAndBindingMappingEXT> HeapOnlyBindingMappings;
	// Binding mappins where only bindless handles are used (no temporary allocation for globals)
	TArray<VkDescriptorSetAndBindingMappingEXT> GraphicsBindingMappings[SF_NumGraphicsFrequencies];
	TArray<VkDescriptorSetAndBindingMappingEXT> ComputeBindingMappings;
	// Bindings with a push address in Binding0 for the temporary allocation used for globals
	TArray<VkDescriptorSetAndBindingMappingEXT> GraphicsBindingMappingsWithPushAddr[SF_NumGraphicsFrequencies];
	TArray<VkDescriptorSetAndBindingMappingEXT> ComputeBindingMappingsWithPushAddr;

	// Descriptor Heap functions
	void InitDescriptorHeaps();
	void UpdateDescriptorInternal(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, const VkResourceDescriptorDataEXT& DescriptorData, bool bImmediateUpdate);

	// Descriptor Buffer specific values
	VkDescriptorSetLayout SingleUseUBDescriptorSetLayout = VK_NULL_HANDLE;
	VkDescriptorBufferBindingInfoEXT BufferBindingInfo[VulkanBindless::NumBindlessSets];
	uint32_t BufferIndices[VulkanBindless::MaxNumSets];
	VkPipelineLayout BindlessPipelineLayout = VK_NULL_HANDLE;
	TStaticArray<VkDescriptorSetLayout, VulkanBindless::MaxNumSets> BindlessDescriptorSetLayouts;
	static const VkPushConstantRange PushConstantRanges[(uint8)EPushConstantRangeType::Count];

	// Descriptor Buffer functions
	void InitDescriptorBuffers();
	void UpdateDescriptorInternal(const FVulkanContextArray& Contexts, FRHIDescriptorHandle DescriptorHandle, VkDescriptorDataEXT DescriptorData, bool bImmediateUpdate);
};
