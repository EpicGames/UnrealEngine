// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanIndexBuffer.cpp: Vulkan Index buffer RHI implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanDevice.h"
#include "VulkanContext.h"
#include "Containers/ResourceArray.h"
#include "VulkanLLM.h"
#include "VulkanRayTracing.h"
#include "VulkanTransientResourceAllocator.h"
#include "RHICoreBufferInitializer.h"
#include "RHICoreStats.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"

struct FVulkanPendingBufferLock
{
	VulkanRHI::FStagingBuffer* StagingBuffer = nullptr;
	uint32 Offset = 0;
	uint32 Size = 0;
	EResourceLockMode LockMode = RLM_Num;
	bool FirstLock = false;
};

static TMap<FVulkanBuffer*, FVulkanPendingBufferLock> GPendingLocks;
static FCriticalSection GPendingLockMutex;

int32 GVulkanForceStagingBufferOnLock = 0;
static FAutoConsoleVariableRef CVarVulkanForceStagingBufferOnLock(
	TEXT("r.Vulkan.ForceStagingBufferOnLock"),
	GVulkanForceStagingBufferOnLock,
	TEXT("When nonzero, non-volatile buffer locks will always use staging buffers. Useful for debugging.\n")
	TEXT("default: 0"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVulkanSparseBufferAllocSizeMB(
	TEXT("r.Vulkan.SparseBufferAllocSizeMB"),
	16,
	TEXT("Size of the backing memory blocks for sparse buffers in megabytes (default 16MB)."),
	ECVF_ReadOnly
);


static FORCEINLINE FVulkanPendingBufferLock GetPendingBufferLock(FVulkanBuffer* Buffer)
{
	FVulkanPendingBufferLock PendingLock;

	// Found only if it was created for Write
	FScopeLock ScopeLock(&GPendingLockMutex);
	const bool bFound = GPendingLocks.RemoveAndCopyValue(Buffer, PendingLock);

	checkf(bFound, TEXT("Mismatched Buffer Lock/Unlock!"));
	return PendingLock;
}

static FORCEINLINE void AddPendingBufferLock(FVulkanBuffer* Buffer, FVulkanPendingBufferLock& PendingLock)
{
	FScopeLock ScopeLock(&GPendingLockMutex);
	check(!GPendingLocks.Contains(Buffer));
	GPendingLocks.Add(Buffer, PendingLock);
}

static void UpdateVulkanBufferStats(const FRHIBufferDesc& BufferDesc, int64 BufferSize, bool bAllocating)
{
	UE::RHICore::UpdateGlobalBufferStats(BufferDesc, BufferSize, bAllocating);
}

static VkDeviceAddress GetBufferDeviceAddress(FVulkanDevice& Device, VkBuffer Buffer)
{
	if (Device.GetOptionalExtensions().HasBufferDeviceAddress)
	{
		VkBufferDeviceAddressInfoKHR DeviceAddressInfo;
		ZeroVulkanStruct(DeviceAddressInfo, VK_STRUCTURE_TYPE_BUFFER_DEVICE_ADDRESS_INFO);
		DeviceAddressInfo.buffer = Buffer;
		return VulkanRHI::vkGetBufferDeviceAddressKHR(Device.GetHandle(), &DeviceAddressInfo);
	}
	return 0;
}

VkBufferUsageFlags FVulkanBuffer::UEToVKBufferUsageFlags(FVulkanDevice& InDevice, EBufferUsageFlags InUEUsage, bool bZeroSize)
{
	// Always include TRANSFER_SRC since hardware vendors confirmed it wouldn't have any performance cost and we need it for some debug functionalities.
	VkBufferUsageFlags OutVkUsage = VK_BUFFER_USAGE_TRANSFER_SRC_BIT;

	auto TranslateFlag = [&OutVkUsage, &InUEUsage](EBufferUsageFlags SearchUEFlag, VkBufferUsageFlags AddedIfFound, VkBufferUsageFlags AddedIfNotFound = 0)
	{
		const bool HasFlag = EnumHasAnyFlags(InUEUsage, SearchUEFlag);
		OutVkUsage |= HasFlag ? AddedIfFound : AddedIfNotFound;
	};

	TranslateFlag(BUF_VertexBuffer, VK_BUFFER_USAGE_VERTEX_BUFFER_BIT);
	TranslateFlag(BUF_IndexBuffer, VK_BUFFER_USAGE_INDEX_BUFFER_BIT);
	TranslateFlag(BUF_ByteAddressBuffer|BUF_StructuredBuffer, VK_BUFFER_USAGE_STORAGE_BUFFER_BIT);
	TranslateFlag(BUF_UniformBuffer, VK_BUFFER_USAGE_UNIFORM_BUFFER_BIT);
	TranslateFlag(BUF_AccelerationStructure, VK_BUFFER_USAGE_ACCELERATION_STRUCTURE_STORAGE_BIT_KHR);

	if (!bZeroSize)
	{
		TranslateFlag(BUF_UnorderedAccess, VK_BUFFER_USAGE_STORAGE_TEXEL_BUFFER_BIT);
		TranslateFlag(BUF_DrawIndirect, VK_BUFFER_USAGE_INDIRECT_BUFFER_BIT);
		TranslateFlag(BUF_KeepCPUAccessible, (VK_BUFFER_USAGE_TRANSFER_SRC_BIT | VK_BUFFER_USAGE_TRANSFER_DST_BIT));
		TranslateFlag(BUF_ShaderResource, VK_BUFFER_USAGE_UNIFORM_TEXEL_BUFFER_BIT);

		TranslateFlag(BUF_Volatile, 0, VK_BUFFER_USAGE_TRANSFER_DST_BIT);

		if (InDevice.GetOptionalExtensions().HasRaytracingExtensions())
		{
			OutVkUsage |= VK_BUFFER_USAGE_SHADER_DEVICE_ADDRESS_BIT;

			TranslateFlag(BUF_AccelerationStructure, 0, VK_BUFFER_USAGE_ACCELERATION_STRUCTURE_BUILD_INPUT_READ_ONLY_BIT_KHR);

			// Add storage bit for when we bind hit groups in FVulkanRayTracingGeometry::SetupHitGroupSystemParameters
			if (EnumHasAnyFlags(InUEUsage, BUF_VertexBuffer | BUF_IndexBuffer) && !EnumHasAnyFlags(InUEUsage, BUF_AccelerationStructure))
			{
				OutVkUsage |= VK_BUFFER_USAGE_STORAGE_BUFFER_BIT;
			}
		}

		// For descriptors buffers
		if (InDevice.GetOptionalExtensions().HasBufferDeviceAddress)
		{
			OutVkUsage |= VK_BUFFER_USAGE_SHADER_DEVICE_ADDRESS_BIT;
		}
	}

	return OutVkUsage;
}

FVulkanBuffer::FVulkanBuffer(FVulkanDevice& InDevice, const FRHIBufferCreateDesc& CreateDesc, const FRHITransientHeapAllocation* InTransientHeapAllocation)
	: FRHIBuffer(CreateDesc)
	, Device(InDevice)
{
	VULKAN_TRACK_OBJECT_CREATE(FVulkanBuffer, this);

	const bool bZeroSize = (CreateDesc.Size == 0);
	BufferUsageFlags = UEToVKBufferUsageFlags(InDevice, CreateDesc.Usage, bZeroSize);

	if (!bZeroSize)
	{
		const bool bUnifiedMem = InDevice.HasUnifiedMemory();
		const uint32 BufferAlignment = VulkanRHI::FMemoryManager::CalculateBufferAlignment(InDevice, CreateDesc.Usage, bZeroSize);
		const bool bIsSparseResource = EnumHasAnyFlags(CreateDesc.Usage, BUF_ReservedResource);

		if (InTransientHeapAllocation != nullptr)
		{
			checkf(!bIsSparseResource, TEXT("Resources with the ReservedResource flag cannot be backed by a transient allocation."));
			CurrentBufferAlloc.Alloc = FVulkanTransientHeap::GetVulkanAllocation(*InTransientHeapAllocation);
			CurrentBufferAlloc.HostPtr = bUnifiedMem ? CurrentBufferAlloc.Alloc.GetMappedPointer(&Device) : nullptr;
			CurrentBufferAlloc.DeviceAddress = GetBufferDeviceAddress(InDevice, CurrentBufferAlloc.Alloc.GetBufferHandle()) + CurrentBufferAlloc.Alloc.Offset;
			check(CurrentBufferAlloc.Alloc.Offset % BufferAlignment == 0);
			check(CurrentBufferAlloc.Alloc.Size >= CreateDesc.Size);
		}
		else if (bIsSparseResource)
		{
			checkf(!EnumHasAnyFlags(GetUsage(), BUF_Dynamic) && !EnumHasAnyFlags(GetUsage(), BUF_Volatile), TEXT("The ReservedResource flag is incompatible with dynamic and volatile buffers."));

			const uint64 SparseBlockSize = (uint64)CVarVulkanSparseBufferAllocSizeMB.GetValueOnAnyThread() * 1024 * 1024;
			const uint32 BufferSize = Align(GetSize(), GRHIGlobals.ReservedResources.TileSizeInBytes);
			const VkBufferCreateFlags BufferCreateFlags = VK_BUFFER_CREATE_SPARSE_BINDING_BIT | VK_BUFFER_CREATE_SPARSE_RESIDENCY_BIT;

			CurrentBufferAlloc.Alloc.Size = BufferSize;
			const VkBuffer BufferHandle = Device.CreateBuffer(BufferSize, BufferUsageFlags, BufferCreateFlags);
			CurrentBufferAlloc.Alloc.VulkanHandle = (uint64)BufferHandle;
			CurrentBufferAlloc.DeviceAddress = GetBufferDeviceAddress(Device, BufferHandle);

			if (UseVulkanDescriptorCache())
			{
				CurrentBufferAlloc.Alloc.HandleId = ++GVulkanBufferViewHandleIdCounter;
			}

			ReservedResourceData = MakeUnique<FVulkanReservedResourceData>();
			ReservedResourceData->LastBlockSize = Align(GetSize() % SparseBlockSize, GRHIGlobals.ReservedResources.TileSizeInBytes);
			ReservedResourceData->BlockCount = GetSize() / SparseBlockSize;
			if (ReservedResourceData->LastBlockSize == 0)
			{
				ReservedResourceData->LastBlockSize = SparseBlockSize;
			}
			else
			{
				ReservedResourceData->BlockCount++;
			}
			VulkanRHI::vkGetBufferMemoryRequirements(Device.GetHandle(), BufferHandle, &ReservedResourceData->MemoryRequirements);
			checkf(SparseBlockSize % ReservedResourceData->MemoryRequirements.alignment == 0, 
				TEXT("The value of r.Vulkan.SparseBufferAllocSizeMB (%dMB) is not compatible with this resource's tile size requirements (%lld)."),
				CVarVulkanSparseBufferAllocSizeMB.GetValueOnAnyThread(), ReservedResourceData->MemoryRequirements.alignment);
			checkf(ReservedResourceData->LastBlockSize % ReservedResourceData->MemoryRequirements.alignment == 0,
				TEXT("The value of GRHIGlobals.ReservedResources.TileSizeInBytes (%d) is not compatible with this resource's tile size requirements (%lld)."),
				GRHIGlobals.ReservedResources.TileSizeInBytes, ReservedResourceData->MemoryRequirements.alignment);
			UpdateVulkanBufferStats(GetDesc(), BufferSize, true);
		}
		else
		{
			AllocateMemory(CurrentBufferAlloc);
		}
	}
}

FVulkanBuffer::~FVulkanBuffer()
{
	if (ReservedResourceData)
	{
		Device.GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::Buffer, GetHandle());
		CommitReservedResource(0);
	}

	VULKAN_TRACK_OBJECT_DELETE(FVulkanBuffer, this);
	ReleaseOwnership();
}


void FVulkanBuffer::AllocateMemory(FBufferAlloc& OutAlloc)
{
	VkMemoryPropertyFlags BufferMemFlags = 0;
	const bool bUnifiedMem = Device.HasUnifiedMemory();
	const bool bDynamic = EnumHasAnyFlags(GetUsage(), BUF_Dynamic) || EnumHasAnyFlags(GetUsage(), BUF_Volatile);
	if (bUnifiedMem)
	{
		BufferMemFlags = VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT | VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT | VK_MEMORY_PROPERTY_HOST_COHERENT_BIT;
	}
	else if (bDynamic)
	{
		BufferMemFlags = VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT | VK_MEMORY_PROPERTY_HOST_COHERENT_BIT;
	}
	else
	{
		BufferMemFlags = VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT;
	}
	const uint32 BufferSize = Align(GetSize(), 4);  // keep allocated size a multiple of 4 (for use with vkCmdFillBuffer)
	const uint32 BufferAlignment = VulkanRHI::FMemoryManager::CalculateBufferAlignment(Device, GetUsage(), (BufferSize == 0));

	FBufferAlloc& NewBufferAlloc = OutAlloc;
	if (!Device.GetMemoryManager().AllocateBufferPooled(NewBufferAlloc.Alloc, nullptr, BufferSize, BufferAlignment, BufferUsageFlags, BufferMemFlags, VulkanRHI::EVulkanAllocationMetaMultiBuffer, __FILE__, __LINE__))
	{
		Device.GetMemoryManager().HandleOOM();
	}
	NewBufferAlloc.HostPtr = (bUnifiedMem || bDynamic) ? NewBufferAlloc.Alloc.GetMappedPointer(&Device) : nullptr;
	NewBufferAlloc.DeviceAddress = GetBufferDeviceAddress(Device, NewBufferAlloc.Alloc.GetBufferHandle()) + NewBufferAlloc.Alloc.Offset;

	UpdateVulkanBufferStats(GetDesc(), NewBufferAlloc.Alloc.Size, true);
}

TArray<VkSparseMemoryBind> FVulkanBuffer::CommitReservedResource(uint64 RequiredCommitSizeInBytes)
{
	checkf(RequiredCommitSizeInBytes <= GetSize(), TEXT("Commit size too large for reserve resource's maximum size."));
	checkf(ReservedResourceData, TEXT("CommitReservedResource called on a resource without any ReservedResourceData.  Resource needs to be created with the ReservedResource flag."));

	// Figure out the allocation needs
	const uint64 SparseBlockSize = (uint64)CVarVulkanSparseBufferAllocSizeMB.GetValueOnAnyThread() * 1024 * 1024;
	const uint64 RequiredBlockCount = Align(RequiredCommitSizeInBytes, SparseBlockSize) / SparseBlockSize;
	const uint64 CurrentBlockCount = ReservedResourceData->Allocations.Num();

	// Figure out the tile needs
	const uint64 TileSize = GRHIGlobals.ReservedResources.TileSizeInBytes;
	const uint64 RequiredTileCount = Align(RequiredCommitSizeInBytes, TileSize) / TileSize;
	const uint64 CurrentTileCount = ReservedResourceData->NumCommittedTiles;

	TArray<VkSparseMemoryBind> Binds;
	if (RequiredTileCount > CurrentTileCount)
	{
		// Make sure we will have all the memory we need
		ReservedResourceData->Allocations.Reserve(RequiredBlockCount);
		for (uint64 BlockCount = CurrentBlockCount; BlockCount < RequiredBlockCount; BlockCount++)
		{
			const uint32 BlockSize = (BlockCount == ReservedResourceData->BlockCount - 1) ? ReservedResourceData->LastBlockSize : SparseBlockSize;
			FVulkanReservedResourceData::FSparseAllocation& NewBlockAlloc = ReservedResourceData->Allocations.AddDefaulted_GetRef();
			if (!Device.GetMemoryManager().AllocateBufferPooled(
				NewBlockAlloc.Allocation, nullptr, BlockSize,
				ReservedResourceData->MemoryRequirements.alignment, BufferUsageFlags,
				VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT, VulkanRHI::EVulkanAllocationMetaMultiBuffer,
				__FILE__, __LINE__))
			{
				Device.GetMemoryManager().HandleOOM();
			}
			NewBlockAlloc.ResourceOffset = BlockCount * SparseBlockSize;
		}

		// Bind all the individual tiles
		Binds.Reserve(RequiredTileCount - CurrentTileCount);
		for (uint64 TileIndex = CurrentTileCount; TileIndex < RequiredTileCount; TileIndex++)
		{
			const uint32 TileOffset = (TileIndex * TileSize);

			const uint32 AllocIndex = TileOffset / SparseBlockSize;
			const FVulkanReservedResourceData::FSparseAllocation& SparseAlloc = ReservedResourceData->Allocations[AllocIndex];

			VkSparseMemoryBind& NewBind = Binds.AddZeroed_GetRef();
			NewBind.size = TileSize;
			NewBind.memory = SparseAlloc.Allocation.GetDeviceMemoryHandle(&Device);
			NewBind.memoryOffset = SparseAlloc.Allocation.Offset + TileOffset - SparseAlloc.ResourceOffset;
			NewBind.resourceOffset = TileOffset;
		}

		const int64 CommitDeltaInBytes = TileSize * Binds.Num();
		UE::RHICore::UpdateReservedResourceStatsOnCommit(CommitDeltaInBytes, true /* bBuffer */, true /* bCommitting */);
	}
	else if (RequiredTileCount < CurrentTileCount)
	{
		Binds.Reserve(CurrentTileCount - RequiredTileCount);
		for (int64 TileIndex = CurrentTileCount - 1; TileIndex >= (int64)RequiredTileCount; TileIndex--)
		{
			VkSparseMemoryBind& NewBind = Binds.AddZeroed_GetRef();
			NewBind.size = TileSize;
			NewBind.memory = VK_NULL_HANDLE;
			NewBind.memoryOffset = 0;
			NewBind.resourceOffset = (TileIndex * TileSize);
		}

		// Free up unused blocks (deletion is deferred, don't have to wait for binds to be submitted)
		while (ReservedResourceData->Allocations.Num() > RequiredBlockCount)
		{
			ReservedResourceData->Allocations.Last().Allocation.Free(Device);
			ReservedResourceData->Allocations.Pop();
		}

		const int64 CommitDeltaInBytes = TileSize * Binds.Num();
		UE::RHICore::UpdateReservedResourceStatsOnCommit(CommitDeltaInBytes, true /* bBuffer */, false /* bCommitting */);
	}

	ReservedResourceData->NumCommittedTiles = RequiredTileCount;
	return Binds;
}

void* FVulkanBuffer::Lock(FRHICommandListBase& RHICmdList, EResourceLockMode LockMode, uint32 LockSize, uint32 Offset)
{
	void* Data = nullptr;
	uint32 DataOffset = 0;

	check(LockStatus == ELockStatus::Unlocked);

	LockStatus = ELockStatus::Locked;
	const bool bIsFirstLock = (0 == LockCounter++);

	// Dynamic:    Allocate a new Host_Visible buffer, swap this new buffer in on RHI thread and update views.  
	//             GPU reads directly from host memory, but no copy is required so it can be used in render passes.
	// Static:     A single Device_Local buffer is allocated at creation.  For Lock/Unlock, use a staging buffer for the upload:
	//             host writes to staging buffer on lock, a copy on GPU is issued on unlock to update the device_local memory.

	const bool bUnifiedMem = Device.HasUnifiedMemory();
	const bool bDynamic = EnumHasAnyFlags(GetUsage(), BUF_Dynamic) || EnumHasAnyFlags(GetUsage(), BUF_Volatile);
	const bool bStatic = EnumHasAnyFlags(GetUsage(), BUF_Static) || !bDynamic;
	const bool bUAV = EnumHasAnyFlags(GetUsage(), BUF_UnorderedAccess);
	const bool bSR = EnumHasAnyFlags(GetUsage(), BUF_ShaderResource);
	const bool bIsSparseResource = EnumHasAnyFlags(GetUsage(), BUF_ReservedResource);

	check(bStatic || bDynamic || bUAV || bSR);

	if (LockMode == RLM_ReadOnly)
	{
		check(IsInRenderingThread());

		if (bUnifiedMem)
		{
			Data = CurrentBufferAlloc.HostPtr;
			DataOffset = Offset;
			LockStatus = ELockStatus::PersistentMapping;
		}
		else 
		{
			// Create a staging buffer we can use to copy data from device to cpu.
			VulkanRHI::FStagingBuffer* StagingBuffer = Device.GetStagingManager().AcquireBuffer(LockSize, VK_BUFFER_USAGE_TRANSFER_DST_BIT, VK_MEMORY_PROPERTY_HOST_CACHED_BIT);

			VkBufferCopy Regions;
			Regions.size = LockSize;
			Regions.srcOffset = Offset + CurrentBufferAlloc.Alloc.Offset;
			Regions.dstOffset = 0;

			VkBuffer BufferHandle = CurrentBufferAlloc.Alloc.GetBufferHandle();

			FRHICommandListImmediate& ImmCmdList = RHICmdList.GetAsImmediate();
			ImmCmdList.EnqueueLambda([StagingBuffer, Regions, BufferHandle](FRHICommandListBase& ExecutingCmdList)
				{
					FVulkanCommandListContext& Context = FVulkanCommandListContext::Get(ExecutingCmdList);
					FVulkanCommandBuffer& CommandBuffer = Context.GetCommandBuffer();

					// Make sure any previous tasks have finished on the source buffer.
					VkMemoryBarrier BarrierBefore = { VK_STRUCTURE_TYPE_MEMORY_BARRIER, nullptr, VK_ACCESS_MEMORY_WRITE_BIT, VK_ACCESS_MEMORY_READ_BIT };
					VulkanRHI::vkCmdPipelineBarrier(CommandBuffer.GetHandle(), VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT, VK_PIPELINE_STAGE_TRANSFER_BIT, 0, 1, &BarrierBefore, 0, nullptr, 0, nullptr);

					// Fill the staging buffer with the data on the device.
					VulkanRHI::vkCmdCopyBuffer(CommandBuffer.GetHandle(), BufferHandle, StagingBuffer->GetHandle(), 1, &Regions);

					// Setup barrier.
					VkMemoryBarrier BarrierAfter = { VK_STRUCTURE_TYPE_MEMORY_BARRIER, nullptr, VK_ACCESS_MEMORY_WRITE_BIT, VK_ACCESS_HOST_READ_BIT };
					VulkanRHI::vkCmdPipelineBarrier(CommandBuffer.GetHandle(), VK_PIPELINE_STAGE_TRANSFER_BIT, VK_PIPELINE_STAGE_HOST_BIT, 0, 1, &BarrierAfter, 0, nullptr, 0, nullptr);

				});

			// We need to execute the command list so we can read the data from the map below
			ImmCmdList.SubmitAndBlockUntilGPUIdle();

			// Flush.
			StagingBuffer->FlushMappedMemory();

			// Get mapped pointer. 
			Data = StagingBuffer->GetMappedPointer();

			// Release temp staging buffer during unlock.
			FVulkanPendingBufferLock PendingLock;
			PendingLock.Offset = 0;
			PendingLock.Size = LockSize;
			PendingLock.LockMode = LockMode;
			PendingLock.StagingBuffer = StagingBuffer;
			AddPendingBufferLock(this, PendingLock);
		}
	}
	else
	{
		check((LockMode == RLM_WriteOnly) || (LockMode == RLM_WriteOnly_NoOverwrite));

		// When the entire buffer is locked on unified memory devices, 
		// we can use a new allocation instead of staging+copy (at no extra cost since all memory is host visible)
		const bool bEntireBuffer = bUnifiedMem && (LockSize == GetSize()) && (Offset == 0);

		// If this is the first lock on host visible memory, then the memory is still untouched so use it directly
		const bool bIsHostVisible = (bUnifiedMem || bDynamic);
		if (bIsHostVisible && (bIsFirstLock || (LockMode == RLM_WriteOnly_NoOverwrite)))
		{
			check(CurrentBufferAlloc.HostPtr);
			Data = CurrentBufferAlloc.HostPtr;
			DataOffset = Offset;
			LockStatus = ELockStatus::PersistentMapping;
		}
		else if ((bStatic && !bEntireBuffer) || GVulkanForceStagingBufferOnLock || bIsSparseResource)
		{
			FVulkanPendingBufferLock PendingLock;
			PendingLock.Offset = Offset;
			PendingLock.Size = LockSize;
			PendingLock.LockMode = LockMode; 
			PendingLock.FirstLock = bIsFirstLock;

			VulkanRHI::FStagingBuffer* StagingBuffer = Device.GetStagingManager().AcquireBuffer(LockSize, VK_BUFFER_USAGE_TRANSFER_SRC_BIT);
			PendingLock.StagingBuffer = StagingBuffer;
			Data = StagingBuffer->GetMappedPointer();

			AddPendingBufferLock(this, PendingLock);
		}
		else
		{
			FBufferAlloc NewAlloc;
			AllocateMemory(NewAlloc);
			NewAlloc.Alloc.Disown();

			RHICmdList.EnqueueLambdaMultiPipe(GetEnabledRHIPipelines(), FRHICommandListBase::EThreadFence::Enabled, TEXT("FVulkanBuffer::Lock"),
				[Buffer = this, NewAlloc](const FVulkanContextArray& Contexts)
				{
					UpdateVulkanBufferStats(Buffer->GetDesc(), Buffer->CurrentBufferAlloc.Alloc.Size, false);
					Buffer->CurrentBufferAlloc.Alloc.Free(Buffer->Device);
					Buffer->CurrentBufferAlloc = NewAlloc;
					Buffer->CurrentBufferAlloc.Alloc.Own();
					Buffer->UpdateLinkedViews(Contexts);
				});

			Data = NewAlloc.HostPtr;
			DataOffset = Offset;
			LockStatus = ELockStatus::PersistentMapping;
		}
	}

	check(Data);
	return (uint8*)Data + DataOffset;
}


void FVulkanBuffer::Unlock(FRHICommandListBase& RHICmdList)
{
	const bool bUnifiedMem = Device.HasUnifiedMemory();
	const bool bDynamic = EnumHasAnyFlags(GetUsage(), BUF_Dynamic) || EnumHasAnyFlags(GetUsage(), BUF_Volatile);
	const bool bStatic = EnumHasAnyFlags(GetUsage(), BUF_Static) || !bDynamic;
	const bool bSR = EnumHasAnyFlags(GetUsage(), BUF_ShaderResource);

	check(LockStatus != ELockStatus::Unlocked);

	if (LockStatus == ELockStatus::PersistentMapping)
	{
		// Do nothing
	}
	else
	{
		check(bStatic || bDynamic || bSR);

		FVulkanPendingBufferLock PendingLock = GetPendingBufferLock(this);

		const bool bNeedsTransitions = RHICmdList.NeedsExtraTransitions();
		RHICmdList.EnqueueLambda(TEXT("FVulkanBuffer::Unlock"), [Buffer=this, PendingLock, bNeedsTransitions](FRHICommandListBase& ExecutingCmdList)
		{
			VulkanRHI::FStagingBuffer* StagingBuffer = PendingLock.StagingBuffer;
			check(StagingBuffer);
			StagingBuffer->FlushMappedMemory();

			if (PendingLock.LockMode == RLM_ReadOnly)
			{
				// Just remove the staging buffer here.
				Buffer->Device.GetStagingManager().ReleaseBuffer(nullptr, StagingBuffer);
			}
			else if (PendingLock.LockMode == RLM_WriteOnly)
			{
				// We need to do this on the active command buffer instead of using an upload command buffer. The high level code sometimes reuses the same
				// buffer in sequences of upload / dispatch, upload / dispatch, so we need to order the copy commands correctly with respect to the dispatches.
				// Unless this is the first time any data is pushed into this buffer, then ordering doesn't matter and UploadContext can be used
				FVulkanContextCommon* CommonContext = nullptr;
				if (PendingLock.FirstLock)
				{
					CommonContext = &FVulkanUploadContext::Get(ExecutingCmdList);
				}
				else
				{
					CommonContext = &FVulkanCommandListContext::Get(ExecutingCmdList);
				}
				FVulkanCommandBuffer& CommandBuffer = CommonContext->GetCommandBuffer();
				check(CommandBuffer.IsOutsideRenderPass());
				VkCommandBuffer CommandBufferHandle = CommandBuffer.GetHandle();

				VulkanRHI::DebugHeavyWeightBarrier(CommandBufferHandle, 16);

				VkBufferCopy Region;
				FMemory::Memzero(Region);
				Region.size = PendingLock.Size;
				//Region.srcOffset = 0;
				Region.dstOffset = PendingLock.Offset + Buffer->CurrentBufferAlloc.Alloc.Offset;
				VulkanRHI::vkCmdCopyBuffer(CommandBufferHandle, StagingBuffer->GetHandle(), Buffer->CurrentBufferAlloc.Alloc.GetBufferHandle(), 1, &Region);

				// High level code expects the data in Buffer to be ready to read, unless the transitions are batched in a pass with multiple copies
				if (bNeedsTransitions)
				{
					VkMemoryBarrier BarrierAfter = { VK_STRUCTURE_TYPE_MEMORY_BARRIER, nullptr, VK_ACCESS_TRANSFER_WRITE_BIT, VK_ACCESS_MEMORY_READ_BIT | VK_ACCESS_MEMORY_WRITE_BIT };
					VulkanRHI::vkCmdPipelineBarrier(CommandBufferHandle, VK_PIPELINE_STAGE_TRANSFER_BIT, VK_PIPELINE_STAGE_ALL_COMMANDS_BIT, 0, 1, &BarrierAfter, 0, nullptr, 0, nullptr);
				}

				Buffer->Device.GetStagingManager().ReleaseBuffer(CommonContext, StagingBuffer);
			}
		});
	}

	LockStatus = ELockStatus::Unlocked;
}

void FVulkanBuffer::TakeOwnership(FVulkanBuffer& Other)
{
	check(Other.LockStatus == ELockStatus::Unlocked);

	// Clean up any resource this buffer already owns
	ReleaseOwnership();

	// Transfer ownership of Other's resources to this instance
	FRHIBuffer::TakeOwnership(Other);

	BufferUsageFlags   = Other.BufferUsageFlags;
	CurrentBufferAlloc = Other.CurrentBufferAlloc;

	Other.BufferUsageFlags   = {};
	Other.CurrentBufferAlloc = {};
}

void FVulkanBuffer::ReleaseOwnership()
{
	check(LockStatus == ELockStatus::Unlocked);

	if (CurrentBufferAlloc.Alloc.HasAllocation())
	{
		UpdateVulkanBufferStats(GetDesc(), CurrentBufferAlloc.Alloc.Size, false);
		Device.GetMemoryManager().FreeVulkanAllocation(CurrentBufferAlloc.Alloc);
	}

	FRHIBuffer::ReleaseOwnership();
}

FRHIBufferInitializer FVulkanDynamicRHI::RHICreateBufferInitializer(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& CreateDesc)
{
#if VULKAN_USE_LLM
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanBuffers);
#else
	LLM_SCOPE(EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::VertexBuffer | EBufferUsageFlags::IndexBuffer) ? ELLMTag::Meshes : ELLMTag::RHIMisc);
#endif
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.OwnerName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CreateDesc.DebugName, CreateDesc.GetTraceClassName(), CreateDesc.OwnerName);

	FVulkanBuffer* Buffer = new FVulkanBuffer(*Device, CreateDesc);

	if (CreateDesc.IsNull() || CreateDesc.InitAction == ERHIBufferInitAction::Default)
	{
		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
	}

	if (CreateDesc.InitAction == ERHIBufferInitAction::Zeroed)
	{
		if (void* HostPointer = Buffer->GetCurrentHostPointer())
		{
			FMemory::Memzero(HostPointer, CreateDesc.Size);

			return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
		}

		return UE::RHICore::FCustomBufferInitializer(RHICmdList, Buffer, nullptr, CreateDesc.Size,
			[Buffer = TRefCountPtr<FVulkanBuffer>(Buffer)](FRHICommandListBase& RHICmdList) mutable
			{
				FVulkanUploadContext& UploadContext = FVulkanUploadContext::Get(RHICmdList);

				FVulkanCommandBuffer& CommandBuffer = UploadContext.GetCommandBuffer();
				const VulkanRHI::FVulkanAllocation& Allocation = Buffer->GetCurrentAllocation();

				const auto Offset = Buffer->GetCurrentAllocation().Offset;
				
				VulkanRHI::vkCmdFillBuffer(
					CommandBuffer.GetHandle(),
					Allocation.GetBufferHandle(),
					Allocation.Offset,
					Allocation.Size,
					0
				);

				// High level code expects the data in Buffer to be ready to read
				const VkMemoryBarrier BarrierAfter =
				{
					VK_STRUCTURE_TYPE_MEMORY_BARRIER, nullptr, VK_ACCESS_TRANSFER_WRITE_BIT, VK_ACCESS_MEMORY_READ_BIT | VK_ACCESS_MEMORY_WRITE_BIT
				};
				VulkanRHI::vkCmdPipelineBarrier(
					CommandBuffer.GetHandle(),
					VK_PIPELINE_STAGE_TRANSFER_BIT,
					VK_PIPELINE_STAGE_ALL_COMMANDS_BIT,
					0,
					1,
					&BarrierAfter,
					0,
					nullptr,
					0,
					nullptr
				);

				return TRefCountPtr<FRHIBuffer>(MoveTemp(Buffer));
			});
	}

	void* WritableData = nullptr;
	bool bUsingLock = false;

	if (void* HostPointer = Buffer->GetCurrentHostPointer())
	{
		WritableData = HostPointer;
		Buffer->IncrementLockCounter();
	}
	else
	{
		WritableData = Buffer->Lock(RHICmdList, RLM_WriteOnly, CreateDesc.Size, 0);
		bUsingLock = true;
	}

	if (CreateDesc.InitAction == ERHIBufferInitAction::ResourceArray)
	{
		check(CreateDesc.InitialData);

		FMemory::Memcpy(WritableData, CreateDesc.InitialData->GetResourceData(), CreateDesc.InitialData->GetResourceDataSize());

		// Discard the resource array's contents.
		CreateDesc.InitialData->Discard();

		if (bUsingLock)
		{
			Buffer->Unlock(RHICmdList);
		}

		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
	}

	if (CreateDesc.InitAction == ERHIBufferInitAction::Initializer)
	{
		return UE::RHICore::FCustomBufferInitializer(RHICmdList, Buffer, WritableData, CreateDesc.Size,
			[Buffer = TRefCountPtr<FVulkanBuffer>(Buffer), bUsingLock](FRHICommandListBase& RHICmdList) mutable
			{
				if (bUsingLock)
				{
					Buffer->Unlock(RHICmdList);
				}
				return TRefCountPtr<FRHIBuffer>(MoveTemp(Buffer));
			});
	}

	return UE::RHICore::HandleUnknownBufferInitializerInitAction(RHICmdList, CreateDesc);
}

void* FVulkanDynamicRHI::LockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI, uint32 Offset, uint32 Size, EResourceLockMode LockMode)
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanBuffers);
	FVulkanBuffer* Buffer = ResourceCast(BufferRHI);
	return Buffer->Lock(RHICmdList, LockMode, Size, Offset);
}

void FVulkanDynamicRHI::UnlockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI)
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanBuffers);
	FVulkanBuffer* Buffer = ResourceCast(BufferRHI);
	Buffer->Unlock(RHICmdList);
}

void* FVulkanDynamicRHI::RHILockBuffer(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI, uint32 Offset, uint32 Size, EResourceLockMode LockMode)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_LockBuffer_RenderThread);
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanBuffers);
	FVulkanBuffer* Buffer = ResourceCast(BufferRHI);
	return Buffer->Lock(RHICmdList, LockMode, Size, Offset);
}

void FVulkanDynamicRHI::RHIUnlockBuffer(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UnlockBuffer_RenderThread);
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanBuffers);
	FVulkanBuffer* Buffer = ResourceCast(BufferRHI);
	Buffer->Unlock(RHICmdList);
}

#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
void FVulkanDynamicRHI::RHIUpdateAllocationTags(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer)
{
	check(RHICmdList.IsBottomOfPipe());
	// LLM tracking happens through LLM_TRACK_VULKAN_HIGH_LEVEL_ALLOC but the pointer used is the heap itself, where buffers are sub-allocated, so it's not trivial to 
	// move tags as long as we don't track the GPU VA directly
}
#endif

