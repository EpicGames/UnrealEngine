// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanChunkedPipelineCache.cpp: 
=============================================================================*/

#include "VulkanChunkedPipelineCache.h"
#include "VulkanRHIPrivate.h"
#include "VulkanPipeline.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "VulkanPendingState.h"
#include "VulkanContext.h"
#include "GlobalShader.h"
#include "VulkanLLM.h"
#include "Misc/ScopeRWLock.h"
#include "HAL/PlatformFramePacer.h"
#include "Templates/Greater.h"
#include "Async/MappedFileHandle.h"

/*
* Chunked PSO cache is a collection of multiple VK PSO cache objects (referred to as a cache chunks)
* Cache chunks accumulate only, we do not currently delete them except at startup when the entire cache is removed (when it is deemed over sized).
* 
* FVulkanChunkedPipelineCacheManagerImpl implements the external API, it contains:
* a map of PSO precache hash -> cache chunk key, a map of chunkkey->FVulkanPipelineCacheChunk,
* and an LRU of recently used cache chunks, (used when cache eviction is active)
* 
* when a PSO is created its hash is checked against the PSOPrecacheHash->cache chunk map, if found FVulkanPipelineCacheChunk is responsible for creating and managing the VK PSO object.
* If not found the most recent FVulkanPipelineCacheChunk is queried for capacity, if space is available the new PSO is added to the chunk.
* If the most recent chunk is full, it is closed and its data is appended to the permanent storage file. a new chunk is created and the PSO is added to it.
* 
* At startup the entire cache file is scanned to build the PSOPrecacheHash->chunk map, the entire cache is mmapped but the PSO cache data is not initially loaded.
* FVulkanPipelineCacheChunk will load its data from the mmapped cache file on demand. the cache is then considered 'resident'
* 
* If the sum of the resident cache chunks goes beyond a cvar specified memory threshold then the least recently used cache chunk is evicted. (eviction means the VK PSO cache object is destroyed and its memory freed up.)
* Only chunks that are mmapped can be evicted.
* 
*/

#define LOGCACHEINFO 0

#if LOGCACHEINFO
#include "ProfilingDebugging/ScopedTimers.h"
#define FScopedTimeToLog FScopedDurationTimeLogger
#else
class FScopedTimeToLog
{
public:
	FScopedTimeToLog(FString, class FOutputDevice* output = nullptr) {};
};
#endif
#include "Containers/LockFreeList.h"

#if PLATFORM_ANDROID && USE_ANDROID_FILE
// TODO:
extern FString GExternalFilePath;
#endif
namespace UE
{
	namespace Vulkan
	{
		static int32 GUseChunkedPSOCache = PLATFORM_ANDROID;
		static FAutoConsoleVariableRef GVulkanGUseNewCacheCodeCVar(
			TEXT("r.Vulkan.UseChunkedPSOCache"),
			GUseChunkedPSOCache,
			TEXT("\n")
			TEXT("")
			, ECVF_RenderThreadSafe | ECVF_ReadOnly
		);

		static FVulkanDevice* GetVulkanDevice()
		{
			return GVulkanRHI->GetDevice();
		}

		static VkDevice GetVulkanDeviceHandle()
		{
			return GetVulkanDevice()->GetHandle();
		}

		int32 GMaxPSOsPerChunk				= 20;
		int32 GTargetResidentCacheSizeMb	= 40;
		int32 GMaxTotalCacheSizeMb			= 300;
		int32 GUntouchedChunkEvictTimeSeconds = 60; 

		static FAutoConsoleVariableRef GMaxSingleCachePSOCountCVar(
			TEXT("r.Vulkan.ChunkedPSOCache.MaxSingleCachePSOCount"),
			GMaxPSOsPerChunk,
			TEXT("The target PSO count for an individual PSO cache.\n")
			TEXT("existing caches with different PSO counts are discarded at startup.\n")
			TEXT("(default) 20")
			, ECVF_RenderThreadSafe | ECVF_ReadOnly
		);

		static FAutoConsoleVariableRef GTargetResidentCacheSizeCVar(
			TEXT("r.Vulkan.ChunkedPSOCache.TargetResidentCacheSizeMb"),
			GTargetResidentCacheSizeMb,
			TEXT("A target resident cache size in MB, if the combined memory usage of all the currently loaded cache chunks is above this threshold\n")
			TEXT("the least recently used chunks will be considered for eviction.\n")
			TEXT("(default) 40")
			, ECVF_RenderThreadSafe | ECVF_ReadOnly
		);

		static FAutoConsoleVariableRef GMaxTotalCacheSizeMbCVar(
			TEXT("r.Vulkan.ChunkedPSOCache.MaxTotalCacheSizeMb"),
			GMaxTotalCacheSizeMb,
			TEXT("At startup, if the entire cache is above this threshold the cache will be deleted\n")
			TEXT("and rebuilt during the subsequent run.\n")			
			TEXT("(default) 300\n")
			TEXT("0 to disable cache size limit, note that the cache will grow indefinitely.")
			, ECVF_RenderThreadSafe | ECVF_ReadOnly
		);

		static bool GMemoryMapChunkedPSOCache = true;
		static FAutoConsoleVariableRef CVarMemoryMapChunkedPSOCache(
			TEXT("r.Vulkan.MemoryMapChunkedPSOCache"),
			GMemoryMapChunkedPSOCache,
			TEXT("If true enabled memory mapping of the chunked vulkan PSO cache. (default)\n")
			TEXT("\n")
			TEXT("")
			,
			ECVF_ReadOnly | ECVF_RenderThreadSafe
		);

		static FAutoConsoleVariableRef GChunkEvictTimeCVar(
			TEXT("r.Vulkan.ChunkedPSOCache.ChunkEvictTime"),
			GUntouchedChunkEvictTimeSeconds,
			TEXT("Time in seconds for a cache chunk to be unused before it can be evicted from ram.\n")
			TEXT("(default) 60")
			, ECVF_RenderThreadSafe | ECVF_ReadOnly
		);

		bool CanMemoryMapChunkedPSOCache()
		{
			return FPlatformProperties::SupportsMemoryMappedFiles() && GMemoryMapChunkedPSOCache;
		}

		static FCriticalSection CacheVersionedFolderCriticalSection;
		static FString GetPSOBinaryCacheVersionedFolder()
		{
			static FString BinaryCacheVersionKey;
			if (BinaryCacheVersionKey.IsEmpty())
			{
				FScopeLock Lock(&CacheVersionedFolderCriticalSection);
				if (BinaryCacheVersionKey.IsEmpty())
				{
					BinaryCacheVersionKey.Append(LegacyShaderPlatformToShaderFormat(GMaxRHIShaderPlatform).ToString());
					const VkPhysicalDeviceProperties& DeviceProperties = GetVulkanDevice()->GetDeviceProperties();

					BinaryCacheVersionKey.Append(FString::Printf(TEXT(".%x.%x.%x"), FCrc::MemCrc32(DeviceProperties.pipelineCacheUUID, VK_UUID_SIZE), DeviceProperties.vendorID, DeviceProperties.deviceID));
#if PLATFORM_ANDROID
					// Apparently we can't rely on version alone to assume binary compatibility.
					// Some devices have reported binary compatibility errors after minor OS updates even though the driver version number does not change.
					const FString BuildNumber = FAndroidMisc::GetDeviceBuildNumber();
					BinaryCacheVersionKey.Append(BuildNumber);

					// Optional configrule variable for triggering a rebuild of the cache.
					const FString* ConfigRulesVulkanProgramKey = FAndroidMisc::GetConfigRulesVariable(TEXT("VulkanProgramCacheKey"));
					if (ConfigRulesVulkanProgramKey && !ConfigRulesVulkanProgramKey->IsEmpty())
					{
						BinaryCacheVersionKey.Append(*ConfigRulesVulkanProgramKey);
					}
#endif
				}
			}
			check(!BinaryCacheVersionKey.IsEmpty());

			return BinaryCacheVersionKey;
		}

		static FString GetRHICacheRootFolder()
		{
#if PLATFORM_ANDROID && USE_ANDROID_FILE
			static FString RHICacheTopFolderPath = GExternalFilePath / TEXT("RHICache") / GVulkanRHI->GetName();
#else
			static FString RHICacheTopFolderPath = FPaths::ProjectSavedDir() / TEXT("RHICache") / GVulkanRHI->GetName();
#endif
			return RHICacheTopFolderPath;
		}

		using FPipelineCacheChunkKey = uint32;
		using FVulkanGraphicsPipelineStateLRU = TDoubleLinkedList<FPipelineCacheChunkKey>;
		using FVulkanPipelineCacheChunkLRUNode = FVulkanGraphicsPipelineStateLRU::TDoubleLinkedListNode;

		std::atomic<int32> TotalResidentCacheSize = 0;
	}
}

using EPSOOperation = FVulkanChunkedPipelineCacheManager::EPSOOperation;

// this class manages a file that combines all of the cache chunks
// It manages access to each chunk via a single mmap alloc. mmap support is relied on for perf, as a fallback where it's not supported synchronous file access is used.
class FVulkanCombinedChunkCacheFile
{
	TUniquePtr<IMappedFileHandle> MappedCacheFile;
	
	mutable FRWLock MappingLock;
	TUniquePtr<IMappedFileRegion> MappedRegion;

	TUniquePtr<FArchive> PSOFileWriter = nullptr;

	static const uint32 CacheFileVersion = 5;
	constexpr static TCHAR FileName[] = TEXT("VulkanPSOChunks");

	void UpdateMapping(uint32 Size)
	{
		FRWScopeLock Lock(MappingLock, SLT_Write);
		MappedRegion = TUniquePtr<IMappedFileRegion>(MappedCacheFile->MapRegion(0, Size));
	}

public:

	static FVulkanCombinedChunkCacheFile& Get()
	{
		static FVulkanCombinedChunkCacheFile Impl;
		return Impl;
	}

	const TCHAR* GetFilename() const { return FileName; }
	
	const FString& GetFullCachePath()
	{
		static FString FullCachePath;
		if(FullCachePath.IsEmpty())
		{
			const FString RootCacheFolder = UE::Vulkan::GetRHICacheRootFolder();
			const FString CacheSubDir = UE::Vulkan::GetPSOBinaryCacheVersionedFolder();
			const FString CombinedCacheSubDir = FPaths::Combine(RootCacheFolder, CacheSubDir);
			FullCachePath = FPaths::Combine(CombinedCacheSubDir, FileName);
		}
		return FullCachePath;
	}

	// hash of parameters used while building the cache
	// a clash would mean some cache chunks would not be honoring the cvar size limits.
	static uint32 GetCacheBuildingParamHash()
	{
		uint32 ParamHash = 0;
		ParamHash = FCrc::MemCrc32(&UE::Vulkan::GMaxPSOsPerChunk, sizeof(UE::Vulkan::GMaxPSOsPerChunk), ParamHash);
		return ParamHash;
	}

	static void WriteFileHeader(FArchive& Archive, uint32 LastValidOffset)
	{
		uint32 Version = CacheFileVersion;
		uint32 ParamHash = GetCacheBuildingParamHash();
		uint32 PrecacheHashVersion = FVulkanDynamicRHI::GetPrecachePSOHashVersion();
		Archive << Version;
		Archive << PrecacheHashVersion;
		Archive << ParamHash;
		Archive << LastValidOffset;
	}

	class FPSOArchiveReader
	{
		const uint8* PSOBytes;
		TUniquePtr<FArchive> Archive;
		TUniquePtr<FRWScopeLock> MappingLock;
	public:
		explicit FPSOArchiveReader(TUniquePtr<FArchive>&& InArchive, const uint8* MappedBytes, TUniquePtr<FRWScopeLock>&& LockIn)
			: PSOBytes(MappedBytes), Archive(MoveTemp(InArchive)), MappingLock(MoveTemp(LockIn))
		{ }
		explicit FPSOArchiveReader(TUniquePtr<FArchive>&& InArchive)
			: PSOBytes(nullptr), Archive(MoveTemp(InArchive))
		{ }

		bool IsValid() const { return Archive.IsValid(); }
		const uint8* GetData() const { return PSOBytes; }
		FArchive* GetArchive() { return Archive.Get(); }
	};

	bool ReadAllCacheChunks(FPSOArchiveReader& ArchiveReader, TUniqueFunction<void(FPSOArchiveReader& ArchiveReader)> OnFoundCacheChunk)
	{
		const FString FullCachePath = GetFullCachePath();

		FArchive& Archive = *ArchiveReader.GetArchive();
		uint32 Version;
		Archive << Version;
		if (Version != CacheFileVersion)
		{
			UE_LOGF(LogVulkanRHI, Log, "FVulkanChunkedPipelineCacheManager: incorrect Cache file version (%d, expected %d)", Version, CacheFileVersion);
			return false;
		}

		uint32 PrecacheHashVersion;
		Archive << PrecacheHashVersion;
		if(PrecacheHashVersion != FVulkanDynamicRHI::GetPrecachePSOHashVersion())
		{
			UE_LOGF(LogVulkanRHI, Log, "FVulkanChunkedPipelineCacheManager: mismatched hash version (%d, expected %d)", PrecacheHashVersion, FVulkanDynamicRHI::GetPrecachePSOHashVersion());
			return false;
		}

		uint32 ParamHash;
		uint32 LastValidOffset;
		Archive << ParamHash;
		Archive << LastValidOffset;
		if (LastValidOffset == 0 || ParamHash != GetCacheBuildingParamHash())
		{
			UE_LOGF(LogVulkanRHI, Log, "FVulkanChunkedPipelineCacheManager: Cache file params have changed (%d, %x=%x)", LastValidOffset, ParamHash, GetCacheBuildingParamHash());
			return false;
		}

		if (UE::Vulkan::CanMemoryMapChunkedPSOCache())
		{
			FOpenMappedResult Result = FPlatformFileManager::Get().GetPlatformFile().OpenMappedEx(*FullCachePath);
			MappedCacheFile = Result.HasError() ? nullptr : Result.StealValue();
			UpdateMapping(LastValidOffset);
		}

		for (int ChunkIdx = 0; Archive.Tell() < LastValidOffset; ChunkIdx++)
		{
			OnFoundCacheChunk(ArchiveReader);
		}

		check(Archive.Tell() == LastValidOffset);
		return true;
	}

	// open the file and callback OnFoundCacheChunk for each contained cache chunk
	bool LoadAllCacheChunks(TUniqueFunction<void(FPSOArchiveReader& ArchiveReader)> OnFoundCacheChunk)
	{
		const FString FullCachePath = GetFullCachePath();
		
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		FFileStatData Stats = PlatformFile.GetStatData(*FullCachePath);

		bool bReadSuccess = false;

		TArray<int32> OffsetsToChunkData;
		// TODO: simplistic 'GC', blow cache away and rebuild.
		// This should be acceptable as the cache rebuild needs to be as transparent as possible..
		if (Stats.FileSize < (UE::Vulkan::GMaxTotalCacheSizeMb * 1024 * 1024))
		{
			FPSOArchiveReader ArchiveReader(TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*FullCachePath, FILEREAD_AllowWrite)), nullptr, nullptr);

			if (ArchiveReader.IsValid())
			{
				UE_LOGF(LogVulkanRHI, Log, "FVulkanChunkedPipelineCacheManager: loading %ls", *FullCachePath);
				bReadSuccess = ReadAllCacheChunks(ArchiveReader, MoveTemp(OnFoundCacheChunk));
			}
			else
			{
				UE_LOGF(LogVulkanRHI, Log, "FVulkanChunkedPipelineCacheManager: Cache file could not open %ls for reading", *FullCachePath);
			}
		}
		else
		{
			UE_LOG(LogVulkanRHI, Log, TEXT("FVulkanChunkedPipelineCacheManager: deleted oversized cache (%" INT64_FMT ", %d)"), Stats.FileSize , (UE::Vulkan::GMaxTotalCacheSizeMb * 1024 * 1024));
		}

		// prepare the output handle.
		bool bWriteSuccess = OpenWriteHandle(bReadSuccess);
		check(bWriteSuccess);
		if (!bReadSuccess && bWriteSuccess)
		{
			// fill in the new header and flush to ensure there's actually something for 'OpenMapped' to use.
			WriteFileHeader(*PSOFileWriter, 0);
			PSOFileWriter->Flush();
		}

		if (UE::Vulkan::CanMemoryMapChunkedPSOCache() && !MappedCacheFile.IsValid())
		{
			FOpenMappedResult Result = FPlatformFileManager::Get().GetPlatformFile().OpenMappedEx(*FullCachePath);
			MappedCacheFile = Result.HasError() ? nullptr : Result.StealValue();
		}

		return bWriteSuccess;
	}

	FPSOArchiveReader GetReader(uint32 Offset)
	{
		if (UE::Vulkan::CanMemoryMapChunkedPSOCache())
		{
			check(MappedRegion);
			TUniquePtr<FRWScopeLock> MappingScopeLock(new FRWScopeLock(FVulkanCombinedChunkCacheFile::Get().MappingLock, SLT_ReadOnly));
			FMemoryView MemView(MappedRegion->GetMappedPtr() + Offset, MappedRegion->GetMappedSize() - Offset);
			return FPSOArchiveReader(TUniquePtr<FArchive>(new FMemoryReaderView(MemView)), (const uint8*)MemView.GetData(), MoveTemp(MappingScopeLock));
		}
		else
		{
			FPSOArchiveReader Reader(TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*GetFullCachePath(), FILEREAD_AllowWrite)));
			if (Reader.GetArchive())
			{
				Reader.GetArchive()->Seek(Offset);
			}
			return Reader;
		}
	}

	bool OpenWriteHandle(bool bAppend)
	{
		check(PSOFileWriter == nullptr);
		const FString& FullCachePath = GetFullCachePath();
		uint32 WriteFlags = EFileWrite::FILEWRITE_AllowRead  | (bAppend ? EFileWrite::FILEWRITE_Append : 0);
		PSOFileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*FullCachePath, WriteFlags ));
		UE_CLOGF(PSOFileWriter, LogRHI, Log, "Opened binary cache for write (%ls)", *FullCachePath);
		UE_CLOGF(PSOFileWriter == nullptr, LogRHI, Warning, "Failed to open Vulkan binary cache output file. (%ls)", *FullCachePath);
		UE_CLOGF(PSOFileWriter && (PSOFileWriter->IsError() || PSOFileWriter->IsCriticalError()), LogRHI, Error, "Vulkan binary cache output archive error (%ls, %d,%d)", *FullCachePath, PSOFileWriter->IsError(), PSOFileWriter->IsCriticalError());

		return PSOFileWriter != nullptr;
	}

	FArchive& GetWriter() 
	{
		check(PSOFileWriter.IsValid());
		return *PSOFileWriter.Get(); 
	}

	void FlushWriteHandle()
	{
		FArchive& Archive = *PSOFileWriter;
		Archive.Flush();

		uint32 CurrentOffsetPos = Archive.Tell();
		Archive.Seek(0);
		WriteFileHeader(Archive, CurrentOffsetPos);
		Archive.Seek(CurrentOffsetPos);
		Archive.Flush();

		// Bring the mmap up the new write position.
		UpdateMapping(CurrentOffsetPos);
	}
};


using FPSOArchiveReader = FVulkanCombinedChunkCacheFile::FPSOArchiveReader;

// A FVulkanPipelineCacheChunk represents a single instance of the VK PSO cache object.
// GMaxPSOsPerChunk is used to limit the size of a chunk during creation.
class FVulkanPipelineCacheChunk
{
	using FVulkanPipelineCacheChunkLRUNode = UE::Vulkan::FVulkanPipelineCacheChunkLRUNode;

	FVulkanPipelineCacheChunkLRUNode* LRUNode;
public:

	explicit FVulkanPipelineCacheChunk(FVulkanPipelineCacheChunkLRUNode* LRUNodeIn = nullptr) : LRUNode(LRUNodeIn) { }

	~FVulkanPipelineCacheChunk()
	{
		if (PipelineCacheObj != VK_NULL_HANDLE)
		{
			VulkanRHI::vkDestroyPipelineCache(UE::Vulkan::GetVulkanDeviceHandle(), PipelineCacheObj, VULKAN_CPU_ALLOCATOR);
		}
	}

	void SetLRUNode(FVulkanPipelineCacheChunkLRUNode* InLRUNode) { LRUNode = InLRUNode; }
	FVulkanPipelineCacheChunkLRUNode* GetLRUNode() { return LRUNode; }

	uint32 OffsetWithinFile = 0; // Location of the binary data as required by 
	void SetCacheOffset(uint32 OffsetWithinFileIn) { OffsetWithinFile = OffsetWithinFileIn; }
	uint32 GetCacheOffset() { return OffsetWithinFile; }

	void InitNewCache()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineChunk_init);

		FRWScopeLock PipelineLock(PipelineCacheObjLock, SLT_Write);
		FRWScopeLock Lock(CacheStateLock, SLT_Write);

		VkPipelineCacheCreateInfo PipelineCacheInfo;
		ZeroVulkanStruct(PipelineCacheInfo, VK_STRUCTURE_TYPE_PIPELINE_CACHE_CREATE_INFO);
		check(PipelineCacheObj == VK_NULL_HANDLE);
		VERIFYVULKANRESULT(VulkanRHI::vkCreatePipelineCache(UE::Vulkan::GetVulkanDeviceHandle(), &PipelineCacheInfo, VULKAN_CPU_ALLOCATOR, &PipelineCacheObj));
		CacheState = ECacheState::Building;
		check(LastSaveSize == 0);
		check(BinaryCacheFileInfo.Filename.IsEmpty());
		BinaryCacheFileInfo.Filename = GenerateNewFileName();
	}

	void Touch()
	{
		LastUsedFrame = GFrameNumber;
		FRWScopeLock PipelineLock(PipelineCacheObjLock, SLT_ReadOnly);
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_CREATE_Chunk_Touch);
			FRWScopeLock Lock(CacheStateLock, SLT_ReadOnly);
			if (CacheState != ECacheState::FinalizedEvicted)
			{
				return;
			}
		}
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_CREATE_Chunk_TouchReinstate);
			PipelineLock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
			FRWScopeLock Lock(CacheStateLock, SLT_Write);
			if(CacheState == ECacheState::FinalizedEvicted)
			{
				ReinstateDriverBlobInternal();
			}
		}
	}

	enum class EPSOCacheFindResult
	{
		NotFound,			// PSO was not found and should contribute to the binary cache.
		MatchedExisting,	// The PSO hash was not found but the PSO would not create a new entry in the cache, so we say there is a match in the cache.
		Found,				// PSO hash has an entry in 
	};

	template<class TPipelineState>
	bool PSORequiresCompile(TPipelineState* Initializer, FVulkanChunkedPipelineCacheManager::FPSOCreateCallbackFunc<TPipelineState>& PSOCreateFunc)
	{
		const bool bCanTestForExistence = UE::Vulkan::GetVulkanDevice()->GetOptionalExtensions().HasEXTPipelineCreationCacheControl;
	
		if(!bCanTestForExistence)
		{
			return true;
		}

		FVulkanChunkedPipelineCacheManager::FPSOCreateFuncParams<TPipelineState> Params(Initializer, PipelineCacheObj, EPSOOperation::CreateIfPresent, PipelineCacheObjLock);
		VkResult Result = PSOCreateFunc(Params);
		check(Result == VK_SUCCESS || Result == VK_PIPELINE_COMPILE_REQUIRED_EXT);
		return Result != VK_SUCCESS;
	}

	template<class TPipelineState>
	VkResult CreatePSO(TPipelineState* Initializer, EPSOCacheFindResult PSOCacheFindResult, FVulkanChunkedPipelineCacheManager::FPSOCreateCallbackFunc<TPipelineState> PSOCreateFunc)
	{
		FScopedTimeToLog Timer(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: FVulkanPipelineCacheChunk.CreatePSO  tot %d "), FPlatformTLS::GetCurrentThreadId()));
		Touch();

		QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_CREATE_CreatePSO);

		VkResult retcode;
		uint64 PSOHash;
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_CREATE_CreatePSOFUNC);
			FScopedTimeToLog Timer2(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: FVulkanPipelineCacheChunk.CreatePSO(lock  tot %d "), FPlatformTLS::GetCurrentThreadId()));
			PSOHash = GetPrecacheKey(Initializer);
			if (PSOCacheFindResult == EPSOCacheFindResult::Found)
			{
				// it's possible to still add a new PSO if we have hash collisions or imperfect PSO hash calc.
				QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_CacheManagerCreatePSO_WARM);
				FVulkanChunkedPipelineCacheManager::FPSOCreateFuncParams<TPipelineState> Params (Initializer, PipelineCacheObj, EPSOOperation::CreateAndStorePSO, PipelineCacheObjLock);
				retcode = PSOCreateFunc(Params);
			}
			else if(PSOCacheFindResult == EPSOCacheFindResult::NotFound)
			{
			
				{
					FRWScopeLock Lock(CacheStateLock, SLT_ReadOnly);
					check(CacheState == ECacheState::Building || CacheState == ECacheState::Closing);
				}
				// Even though we know we're modifying the cache, we're not taking the write lock.
				// Holding the write lock for the duration of the create is too costly, better to let the driver manage this.
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_CacheManagerCreatePSO_COLD);
					FVulkanChunkedPipelineCacheManager::FPSOCreateFuncParams<TPipelineState> Params(Initializer, PipelineCacheObj, EPSOOperation::CreateAndStorePSO, PipelineCacheObjLock);
					retcode = PSOCreateFunc(Params);
				}
			}
			else
			{
				retcode = VK_SUCCESS;
			}
		}

		if (PSOCacheFindResult != EPSOCacheFindResult::Found)
		{
			// if EPSOCacheFindResult::MatchedExisting we record the hash only. 
			// 'existing' pso's will not contribute to the cache, we dont consider them for pending compiles.
			FRWScopeLock Lock(CacheStateLock, SLT_Write);
			static_assert(sizeof(void*) == sizeof(uint64));
			PSOsToBeFlushed2.Push((void*)PSOHash);
			TotalNumPSOs++;

			if (PSOCacheFindResult == EPSOCacheFindResult::NotFound)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_CREATE_Chunk_AddTo);
				FScopedTimeToLog Timer3(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: FVulkanPipelineCacheChunk.CreatePSO AddTo tot %d "), FPlatformTLS::GetCurrentThreadId()));

				TotalNumUniquePSOs++;

				check(PendingAddToCompiles.load() != 0);
				--PendingAddToCompiles;
				if (CacheState == ECacheState::Closing && PendingAddToCompiles.load() == 0)
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_CREATE_Chunk_Flush);
#if LOGCACHEINFO
					UE_LOGF(LogVulkanRHI, Log, "FVulkanChunkedPipelineCacheManager: create pso - Chunk Capacity reached %ls, Finalizing as No pending jobs remain..", *BinaryCacheFileInfo.Filename);
#endif
					SavePSOCacheInternal();
				}
			}
		}
		return retcode;
	}
	
	// Reserve is protected by the ChunkedPipelineCacheLock mutex.
	void ReservePendingPSO()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineChunk_Reserve);
		FRWScopeLock PipelineLock(PipelineCacheObjLock, SLT_ReadOnly);
		FRWScopeLock Lock(CacheStateLock, SLT_ReadOnly); // take the cache readlock, 

		check(CacheState == ECacheState::Building);
		++PendingAddToCompiles;
		if ((TotalNumUniquePSOs + PendingAddToCompiles) >= UE::Vulkan::GMaxPSOsPerChunk)
		{
 			Lock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineChunk_ReserveWrite);
			if ((TotalNumUniquePSOs + PendingAddToCompiles) >= UE::Vulkan::GMaxPSOsPerChunk)
			{
				// become closed..
#if LOGCACHEINFO
				UE_LOGF(LogVulkanRHI, Log, "FVulkanChunkedPipelineCacheManager: Chunk Capacity reached , cachestate %d, CacheSize %d, TotalNumPSOs %d, TotalNumUniquePSOs %d, PendingAddToCompiles %d, %ls, pending finalize..", CacheState, CacheSize, TotalNumPSOs.Load(EMemoryOrder::Relaxed), TotalNumUniquePSOs.Load(EMemoryOrder::Relaxed), PendingAddToCompiles.load(), *BinaryCacheFileInfo.Filename);
#endif
				CacheState = ECacheState::Closing;
			}
		}
	} 

	void LogStats(FString&& LogInfo)
	{
		FRWScopeLock PipelineLock(PipelineCacheObjLock, SLT_ReadOnly);
		FRWScopeLock Lock(CacheStateLock, SLT_ReadOnly); // take the cache readlock, 
		UE_LOGF(LogVulkanRHI, Log, "FVulkanChunkedPipelineCacheManager: %ls Cache name %ls num PSOs %d (%d unique), cache size %d, last used %llu, resident %d, state %d", *LogInfo, *BinaryCacheFileInfo.Filename, TotalNumPSOs.Load(EMemoryOrder::Relaxed), TotalNumUniquePSOs.Load(EMemoryOrder::Relaxed), CacheSize, LastUsedFrame.load(), CacheState != ECacheState::FinalizedEvicted, (int)CacheState);
	}

	bool CanBeEvicted() const
	{
		FRWScopeLock PipelineLock(PipelineCacheObjLock, SLT_ReadOnly);
		FRWScopeLock Lock(CacheStateLock, SLT_ReadOnly); // take the cache readlock, 
		return CacheState == ECacheState::Finalized;
	}

	bool CheckCapacityReached()
	{
		FRWScopeLock PipelineLock(PipelineCacheObjLock, SLT_ReadOnly);
		FRWScopeLock Lock(CacheStateLock, SLT_ReadOnly); // take the cache readlock, 
		return CacheState != ECacheState::Building;
	}

	uint32 GetResidentSize() const
	{
		FRWScopeLock PipelineLock(PipelineCacheObjLock, SLT_ReadOnly);
		FRWScopeLock Lock(CacheStateLock, SLT_ReadOnly); // take the cache readlock, 
		uint32 Size = CacheState == ECacheState::FinalizedEvicted ? 0 : CacheSize;
		return Size;
	}

	uint64 GetLastUsedFrame() const
	{
		return LastUsedFrame.load();
	}

	void Unload()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineChunk_Unload);
		FRWScopeLock PipelineLock(PipelineCacheObjLock, SLT_Write);
		FRWScopeLock Lock(CacheStateLock, SLT_Write);
		check(CacheState == ECacheState::Finalized);
#if LOGCACHEINFO
		UE_LOGF(LogVulkanRHI, Log, "FVulkanChunkedPipelineCacheManager: Evicting cache %ls, %d bytes", *GetCombinedFilePath(), CacheSize);
#endif
		// remove vulkan object from mem
		VulkanRHI::vkDestroyPipelineCache(UE::Vulkan::GetVulkanDeviceHandle(), PipelineCacheObj, VULKAN_CPU_ALLOCATOR);
		PipelineCacheObj = VK_NULL_HANDLE;

		CacheState = ECacheState::FinalizedEvicted;
		UE::Vulkan::TotalResidentCacheSize -= CacheSize;
	}

	enum class ECacheChunkLoadType { LoadAsEvicted, LoadAllData };
	void Load(FPSOArchiveReader& ArchiveReader, TArray<uint64>* PSOsFoundOUT, ECacheChunkLoadType LoadType)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineChunk_Reserve);
		FRWScopeLock PipelineLock(PipelineCacheObjLock, SLT_Write);
		FRWScopeLock Lock(CacheStateLock, SLT_Write);
		FArchive& Archive = *ArchiveReader.GetArchive();
		check(PSOsFoundOUT->IsEmpty());
		check(CacheState == ECacheState::Initialized);
		SetCacheOffset(Archive.Tell());

		LoadInternal(ArchiveReader, PSOsFoundOUT, LoadType);
		// if we're loading as evicted then we're in a completed (RO) state.
		CacheState = LoadType == ECacheChunkLoadType::LoadAsEvicted ? ECacheState::FinalizedEvicted : ECacheState::Building;
	}

	private:
		mutable FRWLock PipelineCacheObjLock;  // This locks access to the PipelineCacheObj, used for create/destroy operations. individual PSO creates use read lock, driver is thread safe, this lock can be held for 100s of ms.

		mutable FRWLock CacheStateLock;


		VkPipelineCache PipelineCacheObj = VK_NULL_HANDLE;

// 		TArray<uint64> PSOsToBeFlushed;
		TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> PSOsToBeFlushed2;
		int32 PSOsNotFlushed = 0;
		int32 CacheSize = 0;
		int32 LastSaveSize = 0;
		TAtomic<int32> TotalNumPSOs = 0; // the number of PSO hashes that are known to be represented in the cache.
		TAtomic<int32> TotalNumUniquePSOs = 0; // the number of PSOs that have contributed to the cache.

		static inline const TCHAR TempFileSuffix[] = TEXT("write");

		struct FFileInfo
		{
			FString Filename;
			uint32 RawDataOffset = 0xffffffff;
		} BinaryCacheFileInfo;

		std::atomic<uint64> LastUsedFrame = 0;

		std::atomic<uint64> PendingAddToCompiles = 0;

		FString GetCombinedFilePath() const
		{
			const FString RootCacheFolder = UE::Vulkan::GetRHICacheRootFolder();
			const FString CacheSubDir = UE::Vulkan::GetPSOBinaryCacheVersionedFolder();
			const FString CombinedCacheSubDir = FPaths::Combine(RootCacheFolder, CacheSubDir);
			return FPaths::Combine(CombinedCacheSubDir, BinaryCacheFileInfo.Filename);
		}

		static FString GenerateNewFileName()
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			return FString("VulkanPSO_") + FGuid::NewGuid().ToString();
		}

		void ReinstateDriverBlobInternal()
		{
			FScopedTimeToLog Timer(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: ReinstateDriverBlobInternal tot %d "), FPlatformTLS::GetCurrentThreadId()));

			check(CacheState == ECacheState::FinalizedEvicted);		

			FPSOArchiveReader ArchiveReader = FVulkanCombinedChunkCacheFile::Get().GetReader(GetCacheOffset());

			if(ensure(ArchiveReader.IsValid()))
			{
				LoadInternal(ArchiveReader, nullptr, ECacheChunkLoadType::LoadAllData);
				CacheState = ECacheState::Finalized;
			}
		}

		void CreateVKCacheInternal(TConstArrayView<uint8> CacheBytes)
		{
			FScopedTimeToLog Timer(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: CreateVKCacheInternal %s "), *BinaryCacheFileInfo.Filename));
			check(CacheState == ECacheState::FinalizedEvicted || CacheState == ECacheState::Initialized);
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_CacheManagerLoadCache);
			VkPipelineCacheCreateInfo PipelineCacheInfo;
			ZeroVulkanStruct(PipelineCacheInfo, VK_STRUCTURE_TYPE_PIPELINE_CACHE_CREATE_INFO);
			PipelineCacheInfo.pInitialData = CacheBytes.GetData();
			PipelineCacheInfo.initialDataSize = CacheBytes.Num();
			check(PipelineCacheObj == VK_NULL_HANDLE);
			VERIFYVULKANRESULT(VulkanRHI::vkCreatePipelineCache(UE::Vulkan::GetVulkanDeviceHandle(), &PipelineCacheInfo, VULKAN_CPU_ALLOCATOR, &PipelineCacheObj));
		}

		uint64 GetPrecacheKey(const FVulkanGraphicsPipelineState* GraphicsPipelineState)
		{
			return GraphicsPipelineState->PrecacheKey;
		}


		void SavePSOCacheInternal()
		{
			static FCriticalSection ArchiveMutex;
			FScopeLock Lock(&ArchiveMutex);
			CacheState = ECacheState::PendingFlush;

			FArchive& Archive = FVulkanCombinedChunkCacheFile::Get().GetWriter();
			SavePSOCacheInternal(Archive);
			FVulkanCombinedChunkCacheFile::Get().FlushWriteHandle();
		}
		enum class ECacheState
		{
			Unknown,
			Initialized,
			Building,			// still accumulating PSOs
			Closing,			// cache reached the size limit waiting for remaining PSOs to complete.
			PendingFlush,		// capacity reached and all pending PSOs are stored. Cache has not been flushed to storage.
			Finalized,			// a finished and resident cache. Finalized caches are always backed by storage.
			FinalizedEvicted,	// a finished cache that is not resident in RAM. 
		};
		ECacheState CacheState = ECacheState::Initialized;

		void LoadInternal(FPSOArchiveReader& ArchiveReader, TArray<uint64>* PSOsInFileOUT, ECacheChunkLoadType LoadType)
		{
			FScopedTimeToLog Timer1(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: LoadInternal TOT  ")));
 			FArchive& Archive = *ArchiveReader.GetArchive();
			int32 TotalNumUniquePSOsRead;
			Archive << TotalNumUniquePSOsRead;
			TotalNumUniquePSOs = TotalNumUniquePSOsRead;
			Archive << CacheSize;
			LastSaveSize = CacheSize;
			BinaryCacheFileInfo.RawDataOffset = Archive.Tell();
			if (LoadType == ECacheChunkLoadType::LoadAllData)
			{
				if (ArchiveReader.GetData())
				{
					TConstArrayView<uint8> CacheBytes;
					{
						FScopedTimeToLog Timer3(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: LoadInternal MappedSerialize %d"), CacheSize));
						// this avoids a pointless memcpy when mmapping is in use.
						const uint8* RawData = ArchiveReader.GetData() + Archive.Tell();
						CacheBytes = MakeArrayView(RawData, CacheSize);
						Archive.Seek(Archive.Tell() + CacheSize);
					}

					CreateVKCacheInternal(CacheBytes);
				}
				else
				{
					TArray<uint8> CacheBytes;
					{
						FScopedTimeToLog Timer3(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: LoadInternal Serialize %d"), CacheSize));
						CacheBytes.SetNumUninitialized(CacheSize);
						Archive.Serialize(CacheBytes.GetData(), CacheSize);;
						check(!Archive.IsError());
					}
					CreateVKCacheInternal(CacheBytes);
				}
				UE::Vulkan::TotalResidentCacheSize += CacheSize;
			}
			else
			{
				Archive.Seek(Archive.Tell() + CacheSize);
			}

			if (PSOsInFileOUT)
			{
				uint32 NumHashes = 0;
				Archive << NumHashes;
				PSOsInFileOUT->SetNumUninitialized(NumHashes);
				Archive.Serialize(PSOsInFileOUT->GetData(), (int64)PSOsInFileOUT->Num() * PSOsInFileOUT->GetTypeSize());
				TotalNumPSOs = PSOsInFileOUT->Num();
			}
		}

		// Save this cache chunk to the archive. We have the state writelock but cache object read lock, its possible for the cache to be changed (added to).
		void SavePSOCacheInternal(FArchive& Archive)
		{
			FScopedTimeToLog Timer(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: SavePSOCacheInternal TOT %s "), *BinaryCacheFileInfo.Filename));

			check(CacheState == ECacheState::PendingFlush);

			QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_CacheManagerWriteCache);

			TArray<uint8> CacheBytes;
			size_t TotalSize = 0;
			{
				FScopedTimeToLog Timer2(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: SavePSOCacheInternal get vk data %s "), *BinaryCacheFileInfo.Filename));
				VulkanRHI::vkGetPipelineCacheData(UE::Vulkan::GetVulkanDeviceHandle(), PipelineCacheObj, &TotalSize, nullptr);
				CacheBytes.SetNumUninitialized(TotalSize);
				VulkanRHI::vkGetPipelineCacheData(UE::Vulkan::GetVulkanDeviceHandle(), PipelineCacheObj, &TotalSize, CacheBytes.GetData());
			}
			CacheSize = (uint32)TotalSize;

			{
				FScopedTimeToLog Timer4(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: SavePSOCacheInternal serialize %s %zu "), *BinaryCacheFileInfo.Filename, TotalSize));

				SetCacheOffset((uint32)Archive.Tell());

				int32 TotalNumUniquePSOsWrite = TotalNumUniquePSOs.Load(EMemoryOrder::Relaxed);
				Archive << TotalNumUniquePSOsWrite;
				uint32 RawDataSize = CacheBytes.Num();
				Archive << RawDataSize;
				BinaryCacheFileInfo.RawDataOffset = Archive.Tell();
				Archive.Serialize(CacheBytes.GetData(), RawDataSize);


				TArray<void*> FlushedPSOHashes;
				PSOsToBeFlushed2.PopAll(FlushedPSOHashes);
				// Just write out the void* as uint64s.
				static_assert(sizeof(void*) == sizeof(uint64));
				uint32 HashCount = FlushedPSOHashes.Num();
				Archive << HashCount;
				Archive.Serialize(FlushedPSOHashes.GetData(), (int64)FlushedPSOHashes.Num() * FlushedPSOHashes.GetTypeSize());
				check(!Archive.IsError());
				CacheState = ECacheState::Finalized;
				UE::Vulkan::TotalResidentCacheSize += CacheSize;
			}
		}
};

//////////////////////////////////////////////////////////////////////////

class FVulkanChunkedPipelineCacheManagerImpl
{
	mutable FRWLock ChunkedPipelineCacheLock; // This guards access to the manager but not individual chunks within the map.

	using FCacheChunkKey = UE::Vulkan::FPipelineCacheChunkKey;
	using FCacheChunksMap = TMap<FCacheChunkKey, TUniquePtr<FVulkanPipelineCacheChunk>>;

	uint32 NumChunks = 0;
	FCacheChunksMap CacheChunksMap;

	// track number of PSOs created/found. logging use only.
	std::atomic<int32> FoundPSOs = 0;

	TMap<uint64, FCacheChunkKey> PrecachePSOToCacheChunkMap;

	FCriticalSection LRUCS;
	using FVulkanPipelineCacheChunkLRUNode = UE::Vulkan::FVulkanPipelineCacheChunkLRUNode;
	using FVulkanGraphicsPipelineStateLRU = UE::Vulkan::FVulkanGraphicsPipelineStateLRU;

	UE::Vulkan::FVulkanGraphicsPipelineStateLRU CacheChunkLRU;



public:
	static FVulkanChunkedPipelineCacheManagerImpl& Get()
	{
		static FVulkanChunkedPipelineCacheManagerImpl VulkanPipelineCacheManager;
		return VulkanPipelineCacheManager;
	}

	FVulkanChunkedPipelineCacheManagerImpl()
	{
		LoadAllCaches();
	}

	template<class TPipelineState>
	VkResult CreatePSO(TPipelineState* GraphicsPipelineState, bool bIsPrecompileJob, FVulkanChunkedPipelineCacheManager::FPSOCreateCallbackFunc<TPipelineState> PSOCreateFunc)
	{
		FScopedTimeToLog Timer(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: CreatePSO (precomp %d) tot %d "), bIsPrecompileJob, FPlatformTLS::GetCurrentThreadId()));

		FCacheChunkKey ChunkKey;
		FVulkanPipelineCacheChunk::EPSOCacheFindResult FindResult;
		FVulkanPipelineCacheChunk* Chunk = GetOrAddCache(PSOCreateFunc, GraphicsPipelineState, FindResult, ChunkKey);

		// dont need to lock ChunkedPipelineCacheLock, we never remove an Chunk once it's added. There should be no PSO create tasks during cache shutdown.
		// Do not create cached precompile PSOs
		if (!bIsPrecompileJob || FindResult != FVulkanPipelineCacheChunk::EPSOCacheFindResult::Found)
		{
			FScopedTimeToLog Timer4(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: CreatePSO actualcreate %d "), FPlatformTLS::GetCurrentThreadId()));
			uint64 Before = Chunk->GetLastUsedFrame();

			VkResult Result = Chunk->CreatePSO(GraphicsPipelineState, FindResult, MoveTemp(PSOCreateFunc));
			uint64 After = Chunk->GetLastUsedFrame();

			if (Before != After)
			{
				FScopedTimeToLog Timer3(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: CreatePSO lru %d "), FPlatformTLS::GetCurrentThreadId()));
				FScopeLock Lock(&LRUCS);
				FVulkanPipelineCacheChunkLRUNode* LRUNode = Chunk->GetLRUNode();
				if(CacheChunkLRU.GetHead() != LRUNode)
				{
					if (LRUNode->GetNextNode() || LRUNode->GetPrevNode() )
					{
						// if evicted it may not be in the lru yet..
						CacheChunkLRU.RemoveNode(LRUNode, false);
					}
					CacheChunkLRU.AddHead(LRUNode);
				}
			}
			return Result;
		}

		return VK_SUCCESS;
	}

	void Tick()
	{
#if LOGCACHEINFO
		{ // Logging
			FRWScopeLock Lock(ChunkedPipelineCacheLock, SLT_ReadOnly);
			static int32 CachedChunkCount = 0;
			bool bLogMe = CachedChunkCount != PrecachePSOToCacheChunkMap.Num();
			if (bLogMe)
			{
				UE_LOGF(LogVulkanRHI, Log, "FVulkanChunkedPipelineCacheManager: Total precache PSOs stored %d, Num Chunks %d", PrecachePSOToCacheChunkMap.Num(), CacheChunksMap.Num());
				for (auto& ChunkPair : CacheChunksMap)
				{
					ChunkPair.Value.Get()->LogStats(FString::Printf(TEXT("Chunk id %d"), ChunkPair.Key));
				}
				UE_LOGF(LogVulkanRHI, Log, "FVulkanChunkedPipelineCacheManager: PSOs created from cache %d ", FoundPSOs.load());

				CachedChunkCount = PrecachePSOToCacheChunkMap.Num();
			}
		}
#endif
		if (UE::Vulkan::GTargetResidentCacheSizeMb)
		{
			TryUnloadCacheChunks();
		}
	}

private:

	bool LoadAllCaches()
	{
		FRWScopeLock WriteLock(ChunkedPipelineCacheLock, SLT_Write);

		check(CacheChunksMap.IsEmpty() && PrecachePSOToCacheChunkMap.IsEmpty());

		// the CacheSubDir is specific to the device+driver+shaderplatform.
		// We clean out everything else from the cache folder.
		const FString RootCacheFolder = UE::Vulkan::GetRHICacheRootFolder();
		const FString CacheSubDir = UE::Vulkan::GetPSOBinaryCacheVersionedFolder();
		const FString CombinedCacheSubDir = FPaths::Combine(RootCacheFolder, CacheSubDir);

		const bool bDeleteAll = FParse::Param(FCommandLine::Get(), TEXT("ClearVulkanBinaryProgramCache")) || FParse::Param(FCommandLine::Get(), TEXT("clearPSODriverCache"));

		// delete anything unexpected from the RHI cache root folder.
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		TArray<FString> FoundFiles;
		IFileManager::Get().FindFiles(FoundFiles, *(RootCacheFolder / TEXT("*")), true, true);
		for (FString& FoundFile : FoundFiles)
		{
			const FString FullPath = (RootCacheFolder / FoundFile);
			const bool bIsDir = PlatformFile.DirectoryExists(*FullPath);
			if (FoundFile != CacheSubDir || !bIsDir || bDeleteAll)
			{
				bool bSuccess;
				if (bIsDir)
				{
					bSuccess = PlatformFile.DeleteDirectoryRecursively(*FullPath);
				}
				else
				{
					bSuccess = PlatformFile.DeleteFile(*FullPath);
				}
				UE_LOGF(LogRHI, Verbose, "FVulkanChunkedPipelineCacheManagerImpl: Deleting %ls %ls", bIsDir ? TEXT("dir") : TEXT("file"), *FullPath);
				UE_CLOGF(!bSuccess, LogRHI, Warning, "FVulkanChunkedPipelineCacheManagerImpl: Failed to delete %ls", *FullPath);
			}
		}

		FoundFiles.Reset();
		IFileManager::Get().FindFiles(FoundFiles, *(CombinedCacheSubDir / TEXT("*")), true, false);

		for (FString& FoundFile : FoundFiles)
		{
			if (!FoundFile.Equals(FVulkanCombinedChunkCacheFile::Get().GetFilename()))
			{
				FString FullPath = FPaths::Combine(CombinedCacheSubDir, FoundFile);
				UE_LOGF(LogVulkanRHI, Log, "FVulkanChunkedPipelineCacheManager: deleting unexpected Cache %ls.", *FullPath);
				PlatformFile.DeleteFile(*FullPath);
			}
		}

		bool bReadFailed = false;
		TUniqueFunction<void(FPSOArchiveReader& ArchiveReader)> OnFoundCacheChunk = [this, &bReadFailed](FPSOArchiveReader& ArchiveReader)
		{
			TUniquePtr<FVulkanPipelineCacheChunk> PendingChunk = MakeUnique<FVulkanPipelineCacheChunk>();
			TArray<uint64> PSOsOut;
			PendingChunk->Load(ArchiveReader, &PSOsOut, FVulkanPipelineCacheChunk::ECacheChunkLoadType::LoadAsEvicted);

			FCacheChunkKey NewChunkKey = ++NumChunks;
			check(!CacheChunksMap.Contains(NewChunkKey));

			FVulkanPipelineCacheChunkLRUNode* NewLRUNode = new FVulkanPipelineCacheChunkLRUNode(NewChunkKey);
			{
				FScopeLock Lock(&LRUCS);
				CacheChunkLRU.AddHead(NewLRUNode);
				PendingChunk->SetLRUNode(NewLRUNode);
			}

			CacheChunksMap.Emplace(NewChunkKey, MoveTemp(PendingChunk));
			for (uint64 PSOHash : PSOsOut)
			{
				PrecachePSOToCacheChunkMap.Add(PSOHash, NewChunkKey);
			}
		};

		bool bSuccess = FVulkanCombinedChunkCacheFile::Get().LoadAllCacheChunks(MoveTemp(OnFoundCacheChunk));
		if (!bSuccess)
		{
			UE_LOGF(LogVulkanRHI, Log, "FVulkanChunkedPipelineCacheManager: failed to open chunked PSO cache file, PSO caching is disabled.");
			UE::Vulkan::GUseChunkedPSOCache = 0;
		}
		return bSuccess;
	}

	static uint64 GetPrecacheHash(const FVulkanGraphicsPipelineState* GFXState) { return GFXState->PrecacheKey; }

	template<class TPipelineState, class TInitializer>
	FVulkanPipelineCacheChunk* GetChunk(FVulkanChunkedPipelineCacheManager::FPSOCreateCallbackFunc<TPipelineState>& PSOCreateFunc, const TInitializer& Initializer, FVulkanPipelineCacheChunk::EPSOCacheFindResult& FindResult, FCacheChunkKey& ChunkKeyOUT, FRWScopeLock& Lock, const bool bTryAdd = false)
	{
		FVulkanPipelineCacheChunk* ReturnChunk = nullptr;
		uint64 PSOPrecacheKey = GetPrecacheHash(Initializer);

		const FCacheChunkKey* FoundChunkKey = PrecachePSOToCacheChunkMap.Find(PSOPrecacheKey);
		if (FoundChunkKey)
		{
			// we have a cache for this PSO
			ReturnChunk = CacheChunksMap.FindChecked(*FoundChunkKey).Get();
			FoundPSOs++;
			FindResult = FVulkanPipelineCacheChunk::EPSOCacheFindResult::Found;
			ChunkKeyOUT = *FoundChunkKey;
		}
		else if (!bTryAdd)
		{
			FindResult = FVulkanPipelineCacheChunk::EPSOCacheFindResult::NotFound;
			// Try again with the write lock, add if it's still missing.
			Lock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
			return GetChunk(PSOCreateFunc, Initializer, FindResult, ChunkKeyOUT, Lock, true);
		}

		if (!ReturnChunk)
		{
			check(bTryAdd);

			auto FindChunk = [this,&FindResult,&Initializer,&PSOCreateFunc]
			{
				FVulkanPipelineCacheChunk* ReturnChunk = nullptr;

				TUniquePtr<FVulkanPipelineCacheChunk>* Found = CacheChunksMap.Find(NumChunks);
				if (Found)
				{
					ReturnChunk = (*Found).Get();
				}
				if ((!ReturnChunk) || (ReturnChunk && ReturnChunk->CheckCapacityReached()))
				{
					// create and return a new cache chunk.
					FCacheChunkKey NewChunk = ++NumChunks;
					check(!CacheChunksMap.Contains(NewChunk));
					FVulkanPipelineCacheChunkLRUNode* NewLRUNode = new FVulkanPipelineCacheChunkLRUNode(NewChunk);
					{
						FScopeLock Lock(&LRUCS);
						CacheChunkLRU.AddHead(NewLRUNode);
						ReturnChunk = CacheChunksMap.Emplace(NewChunk, MakeUnique<FVulkanPipelineCacheChunk>(NewLRUNode)).Get();
					}
					ReturnChunk->InitNewCache();
				}

				if(ReturnChunk->PSORequiresCompile(Initializer, PSOCreateFunc))
				{
					ReturnChunk->ReservePendingPSO();
				}
				else
				{
#if LOGCACHEINFO
					UE_LOGF(LogVulkanRHI, Log, "FVulkanChunkedPipelineCacheManager: redundant PSO ..");
#endif
					FindResult = FVulkanPipelineCacheChunk::EPSOCacheFindResult::MatchedExisting;
				}

				return ReturnChunk;
			};

			uint32 CurrentChunk = NumChunks;
			ReturnChunk = FindChunk();
			ChunkKeyOUT = NumChunks;
			PrecachePSOToCacheChunkMap.Add(PSOPrecacheKey, NumChunks);
		}

		check(ReturnChunk);
		return ReturnChunk;
	}

	template<class TPipelineState, class TInitializer>
	FVulkanPipelineCacheChunk* GetOrAddCache(FVulkanChunkedPipelineCacheManager::FPSOCreateCallbackFunc<TPipelineState>& PSOCreateFunc, const TInitializer& Initializer, FVulkanPipelineCacheChunk::EPSOCacheFindResult& FindResult, FCacheChunkKey& ChunkKeyOUT)
	{
		FScopedTimeToLog Timer(FString::Printf(TEXT("FVulkanChunkedPipelineCacheManager: GetOrAddCache tot %d "), FPlatformTLS::GetCurrentThreadId()));
		FRWScopeLock Lock(ChunkedPipelineCacheLock, SLT_ReadOnly);
		return GetChunk(PSOCreateFunc, Initializer, FindResult, ChunkKeyOUT, Lock);
	}

	void TryUnloadCacheChunks()
	{
		const uint32 FramePace = FPlatformRHIFramePacer::GetFramePace();
		const uint32 LastFrameRequired = GFrameNumber - FMath::Min(GFrameNumber, (uint32)(FramePace * UE::Vulkan::GUntouchedChunkEvictTimeSeconds));

		FRWScopeLock CacheLock(ChunkedPipelineCacheLock, SLT_ReadOnly);
		FScopeLock LRULock(&LRUCS);

		int32 CurrentResidentSize = UE::Vulkan::TotalResidentCacheSize;

		const int32 TargetResidentCacheSizeBytes = (UE::Vulkan::GTargetResidentCacheSizeMb * 1024 * 1024);
		const int32 MaxToUnloadPerTick = 3;

		// unload oldest
		FVulkanPipelineCacheChunkLRUNode* CurrentNode = CacheChunkLRU.GetTail();
		for (int32 UnloadCount = 0; CurrentNode && UnloadCount < MaxToUnloadPerTick && CurrentResidentSize > TargetResidentCacheSizeBytes;)
		{
			FVulkanPipelineCacheChunkLRUNode* NextNode = CurrentNode->GetPrevNode();

			UE::Vulkan::FPipelineCacheChunkKey ChunkKey = CurrentNode->GetValue();
			FVulkanPipelineCacheChunk* FoundCacheChunk = CacheChunksMap.FindChecked(ChunkKey).Get();

			int CacheSize = FoundCacheChunk->GetResidentSize();
			if (FoundCacheChunk->CanBeEvicted())
			{
				if (LastFrameRequired < FoundCacheChunk->GetLastUsedFrame())
				{
					// exit, everything else will be too recent
					break;
				}

				FoundCacheChunk->Unload();
				CacheChunkLRU.RemoveNode(CurrentNode, false);
				CurrentResidentSize -= CacheSize;
				UnloadCount++;
			}
			CurrentNode = NextNode;
		}
	}
};

//////////////////////////////////////////////////////////////////////////
// public interface:
static FVulkanChunkedPipelineCacheManager VulkanPipelineCacheManager;

bool FVulkanChunkedPipelineCacheManager::IsEnabled()
{
	return UE::Vulkan::GUseChunkedPSOCache != 0;
}

void FVulkanChunkedPipelineCacheManager::Init()
{
	if (UE::Vulkan::GUseChunkedPSOCache == 0)
	{
		return;
	}

	check(!VulkanPipelineCacheManager.VulkanPipelineCacheManagerImpl.IsValid());
	VulkanPipelineCacheManager.VulkanPipelineCacheManagerImpl = MakeUnique<FVulkanChunkedPipelineCacheManagerImpl>();
}

void FVulkanChunkedPipelineCacheManager::Shutdown()
{
	if (UE::Vulkan::GUseChunkedPSOCache == 0)
	{
		return;
	}

	if (VulkanPipelineCacheManager.VulkanPipelineCacheManagerImpl.IsValid())
	{
		VulkanPipelineCacheManager.VulkanPipelineCacheManagerImpl = nullptr;
	}
}

FVulkanChunkedPipelineCacheManager& FVulkanChunkedPipelineCacheManager::Get()
{
	check(UE::Vulkan::GUseChunkedPSOCache);
	check(VulkanPipelineCacheManager.VulkanPipelineCacheManagerImpl.IsValid());
	return VulkanPipelineCacheManager;
}


template<class TPipelineState>
VkResult FVulkanChunkedPipelineCacheManager::CreatePSO(TPipelineState* GraphicsPipelineState, bool bIsPrecompileJob, FPSOCreateCallbackFunc<TPipelineState> PSOCreateFunc)
{
	check(UE::Vulkan::GUseChunkedPSOCache);
	return VulkanPipelineCacheManagerImpl->CreatePSO(GraphicsPipelineState, bIsPrecompileJob, MoveTemp(PSOCreateFunc));
}

void FVulkanChunkedPipelineCacheManager::Tick()
{
	if (UE::Vulkan::GUseChunkedPSOCache == 0)
	{
		return;
	}

	VulkanPipelineCacheManagerImpl->Tick();
}

template VkResult FVulkanChunkedPipelineCacheManager::CreatePSO(FVulkanGraphicsPipelineState* GraphicsPipelineState, bool bIsPrecompileJob, FPSOCreateCallbackFunc<FVulkanGraphicsPipelineState> PSOCreateFunc);

