// Copyright Epic Games, Inc. All Rights Reserved..

/*=============================================================================
	VulkanPipelineCache.h: Private Vulkan RHI definitions.
=============================================================================*/

#pragma once
#include "VulkanRHIPrivate.h"
#include "VulkanResources.h"
#include "VulkanShaderResources.h"
#include "VulkanDescriptorSets.h"
#include "ShaderPipelineCache.h"
#include "Templates/UniquePtr.h"

class FVulkanChunkedPipelineCacheManager
{
	TUniquePtr<class FVulkanChunkedPipelineCacheManagerImpl> VulkanPipelineCacheManagerImpl;
	friend class FVulkanPipelineCacheEntry;
public:
	static void Init();
	static void Shutdown();
	static bool IsEnabled();
	static FVulkanChunkedPipelineCacheManager& Get();

	enum class EPSOOperation
	{
		CreateIfPresent,	// checks the PSO is already present in the cache, VK api requires it creates the PSO too.
		CreateAndStorePSO,	// Create and store the PSO in the cache.
	};


	template<class TPipelineState>
	struct FPSOCreateFuncParams
	{
		FPSOCreateFuncParams(TPipelineState* PSOIn, VkPipelineCache DestPipelineCacheIn, FVulkanChunkedPipelineCacheManager::EPSOOperation PSOOperationIn, FRWLock& DestPipelineCacheLockIn)
			: PSO(PSOIn), DestPipelineCache(DestPipelineCacheIn), PSOOperation(PSOOperationIn), DestPipelineCacheLock(DestPipelineCacheLockIn)
		{}

		TPipelineState* PSO;
		VkPipelineCache DestPipelineCache;
		FVulkanChunkedPipelineCacheManager::EPSOOperation PSOOperation;
		FRWLock& DestPipelineCacheLock; // ensure lock is acquired before use of DestPipelineCache.
	};

	template<class TPipelineState>
	using FPSOCreateCallbackFunc = TUniqueFunction<VkResult(FPSOCreateFuncParams<TPipelineState>& Params)>;

	template<class TPipelineState>
	VkResult CreatePSO(TPipelineState* GraphicsPipelineState, bool bIsPrecompileJob, FPSOCreateCallbackFunc<TPipelineState> PSOCreateFunc);

	void Tick();
};
