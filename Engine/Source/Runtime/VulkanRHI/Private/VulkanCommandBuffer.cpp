// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanCommandBuffer.cpp: Vulkan device RHI implementation.
=============================================================================*/

#include "VulkanCommandBuffer.h"
#include "VulkanContext.h"
#include "VulkanFramebuffer.h"
#include "VulkanRenderpass.h"
#include "VulkanDescriptorSets.h"
#include "VulkanMemory.h"
#include "VulkanRayTracing.h"
#include "VulkanBindlessDescriptorManager.h"
#include "VulkanPipeline.h"

#define CMD_BUFFER_TIME_TO_WAIT_BEFORE_DELETING		10

#if VULKAN_USE_DEBUG_NAMES
static std::atomic_int32_t GNextCommandBufferID{1};
#endif

FVulkanCommandBuffer::FVulkanCommandBuffer(FVulkanDevice& InDevice, FVulkanCommandBufferPool& InCommandBufferPool)
	: Device(InDevice)
	, CommandBufferPool(InCommandBufferPool)
	, EventStream(InCommandBufferPool.GetQueue().GetProfilerQueue())
{
	FScopeLock ScopeLock(CommandBufferPool.GetCS());
	AllocMemory();

	if (InDevice.GetOptionalExtensions().HasAnyExtendedDynamicState() || InDevice.SupportsShaderObjects())
	{
		LastDynamicStateUpdate = new FGfxPipelineDesc();
		FMemory::Memzero(*LastDynamicStateUpdate);
	}
}

void FVulkanCommandBuffer::AllocMemory()
{
	// Assumes we are inside a lock for the pool
	check(State == EState::NotAllocated);

	VkCommandBufferAllocateInfo CreateCmdBufInfo;
	ZeroVulkanStruct(CreateCmdBufInfo, VK_STRUCTURE_TYPE_COMMAND_BUFFER_ALLOCATE_INFO);
	CreateCmdBufInfo.level = (GetCommandBufferType() == EVulkanCommandBufferType::Secondary) ? VK_COMMAND_BUFFER_LEVEL_SECONDARY : VK_COMMAND_BUFFER_LEVEL_PRIMARY;
	CreateCmdBufInfo.commandBufferCount = 1;
	CreateCmdBufInfo.commandPool = CommandBufferPool.GetHandle();

	VERIFYVULKANRESULT(VulkanRHI::vkAllocateCommandBuffers(Device.GetHandle(), &CreateCmdBufInfo, &CommandBufferHandle));

	bNeedsFullDynamicStateUpdate = 1;
	bHasPipeline = 0;
	bHasViewport = 0;
	bHasScissor = 0;
	bHasStencilRef = 0;
	bHasBoundDescriptorHeap = 0;
	State = EState::ReadyForBegin;

	ResetCurrentState();

	INC_DWORD_STAT(STAT_VulkanNumCmdBuffers);

#if VULKAN_USE_DEBUG_NAMES
	int32 ID = GNextCommandBufferID++;
	VULKAN_SET_DEBUG_NAME(Device, VK_OBJECT_TYPE_COMMAND_BUFFER, CommandBufferHandle, TEXT("FVulkanCommandBuffer %d"), ID);
#endif
}

FVulkanCommandBuffer::~FVulkanCommandBuffer()
{
	if (State != EState::NotAllocated)
	{
		FreeMemory();
	}

	if (LastDynamicStateUpdate)
	{
		delete LastDynamicStateUpdate;
		LastDynamicStateUpdate = nullptr;
	}
}

void FVulkanCommandBuffer::FreeMemory()
{
	// Assumes we are inside a lock for the pool
	check(State != EState::NotAllocated);
	check(CommandBufferHandle != VK_NULL_HANDLE);
	VulkanRHI::vkFreeCommandBuffers(Device.GetHandle(), CommandBufferPool.GetHandle(), 1, &CommandBufferHandle);
	CommandBufferHandle = VK_NULL_HANDLE;

	DEC_DWORD_STAT(STAT_VulkanNumCmdBuffers);
	State = EState::NotAllocated;
}

void FVulkanCommandBuffer::EndRenderPass()
{
	checkf(IsInsideRenderPass(), TEXT("Can't EndRP as we're NOT inside one! CmdBuffer 0x%p State=%d"), CommandBufferHandle, (int32)State);

	if (Device.GetOptionalExtensions().HasKHRRenderPass2)
	{
		VkSubpassEndInfo SubpassEndInfo;
		ZeroVulkanStruct(SubpassEndInfo, VK_STRUCTURE_TYPE_SUBPASS_END_INFO);

		VkRenderPassFragmentDensityMapOffsetEndInfoEXT FDMOffsetInfo;
		VkOffset2D Offsets[2];
		if (Device.GetOptionalExtensions().HasEXTFragmentDensityMapOffset && GRHISupportsVariableRateShadingImageOffsets)
		{
			Offsets[0].x = GRHIVariableRateShadingImageOffsets[0].X;
			Offsets[0].y = GRHIVariableRateShadingImageOffsets[0].Y;
			Offsets[1].x = GRHIVariableRateShadingImageOffsets[1].X;
			Offsets[1].y = GRHIVariableRateShadingImageOffsets[1].Y;

			ZeroVulkanStruct(FDMOffsetInfo, VK_STRUCTURE_TYPE_RENDER_PASS_FRAGMENT_DENSITY_MAP_OFFSET_END_INFO_EXT);
			FDMOffsetInfo.pFragmentDensityOffsets = Offsets;
			FDMOffsetInfo.fragmentDensityOffsetCount = 2;
			FDMOffsetInfo.pFragmentDensityOffsets = Offsets;
			FDMOffsetInfo.pNext = nullptr;
 		
			SubpassEndInfo.pNext = &FDMOffsetInfo;
		}
		
		VulkanRHI::vkCmdEndRenderPass2KHR(CommandBufferHandle, &SubpassEndInfo);
	}
	else
	{
		VulkanRHI::vkCmdEndRenderPass(CommandBufferHandle);
	}

	State = EState::IsInsideBegin;
	CurrentMultiViewCount = 0;
}

void FVulkanCommandBuffer::BeginRenderPass(const FVulkanBeginRenderPassInfo& BeginRenderPassInfo, const VkClearValue* AttachmentClearValues)
{
	checkf(IsOutsideRenderPass(), TEXT("Can't BeginRP as already inside one! CmdBuffer 0x%p State=%d"), CommandBufferHandle, (int32)State);

	const FVulkanRenderTargetLayout& Layout = BeginRenderPassInfo.RenderPass.GetLayout();

	VkRenderPassBeginInfo Info;
	ZeroVulkanStruct(Info, VK_STRUCTURE_TYPE_RENDER_PASS_BEGIN_INFO);
	Info.renderPass = BeginRenderPassInfo.RenderPass.GetHandle();
	Info.framebuffer = BeginRenderPassInfo.Framebuffer.GetHandle();
	Info.renderArea = BeginRenderPassInfo.Framebuffer.GetRenderArea();
	Info.clearValueCount = Layout.GetNumUsedClearValues();
	Info.pClearValues = AttachmentClearValues;

	const VkSubpassContents SubpassContents = BeginRenderPassInfo.bIsParallelRenderPass ?
		VK_SUBPASS_CONTENTS_SECONDARY_COMMAND_BUFFERS : VK_SUBPASS_CONTENTS_INLINE;

	if (Device.GetOptionalExtensions().HasKHRRenderPass2)
	{
		VkSubpassBeginInfo SubpassInfo;
		ZeroVulkanStruct(SubpassInfo, VK_STRUCTURE_TYPE_SUBPASS_BEGIN_INFO);
		SubpassInfo.contents = SubpassContents;
		VulkanRHI::vkCmdBeginRenderPass2KHR(CommandBufferHandle, &Info, &SubpassInfo);
	}
	else
	{
		VulkanRHI::vkCmdBeginRenderPass(CommandBufferHandle, &Info, SubpassContents);
	}

	CurrentMultiViewCount = BeginRenderPassInfo.RenderPass.GetLayout().GetMultiViewCount();
	State = EState::IsInsideRenderPass;
}

void FVulkanCommandBuffer::BeginDynamicRendering(const VkRenderingInfo& RenderingInfo, FVulkanQueryPool* OptionalQueryPool)
{
	check(GetCommandBufferType() != EVulkanCommandBufferType::Secondary);

	VulkanRHI::vkCmdBeginRenderingKHR(CommandBufferHandle, &RenderingInfo);

	if (OptionalQueryPool)
	{
		check(GetCommandBufferType() == EVulkanCommandBufferType::Parallel);
		auto& Event = EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FBeginWork>(0, UINT64_MAX);
		const uint32 IndexInPool = OptionalQueryPool->ReserveQuery(&Event.GPUTimestampTOP);
		VulkanRHI::vkCmdWriteTimestamp(GetHandle(), VK_PIPELINE_STAGE_TOP_OF_PIPE_BIT, OptionalQueryPool->GetHandle(), IndexInPool);
	}

	VkRenderingInputAttachmentIndexInfo InputAttachmentIndexInfo = GetDefaultRenderingInputAttachmentIndexInfo(RenderingInfo.colorAttachmentCount);
	VulkanRHI::vkCmdSetRenderingInputAttachmentIndicesKHR(CommandBufferHandle, &InputAttachmentIndexInfo);

	LastDynamicRenderingFlags = RenderingInfo.flags;
	State = EState::IsInsideRenderPass;
}

void FVulkanCommandBuffer::EndDynamicRendering(FVulkanQueryPool* OptionalQueryPool)
{
	check(GetCommandBufferType() != EVulkanCommandBufferType::Secondary);

	if (OptionalQueryPool)
	{
		check(GetCommandBufferType() == EVulkanCommandBufferType::Parallel);
		auto& Event = EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FEndWork>(0);
		const uint32 IndexInPool = OptionalQueryPool->ReserveQuery(&Event.GPUTimestampBOP);
		VulkanRHI::vkCmdWriteTimestamp(GetHandle(), VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT, OptionalQueryPool->GetHandle(), IndexInPool);
	}

	// vkCmdEndRendering2KHR is provided by VK_EXT_fragment_density_map_offset to attach the offset info struct
	if (Device.GetOptionalExtensions().HasEXTFragmentDensityMapOffset && GRHISupportsVariableRateShadingImageOffsets)
	{
		VkRenderingEndInfoKHR SubpassEndInfo;
		ZeroVulkanStruct(SubpassEndInfo, VK_STRUCTURE_TYPE_RENDERING_END_INFO_KHR);

		VkRenderPassFragmentDensityMapOffsetEndInfoEXT FDMOffsetInfo;
		VkOffset2D Offsets[2];

		Offsets[0].x = GRHIVariableRateShadingImageOffsets[0].X;
		Offsets[0].y = GRHIVariableRateShadingImageOffsets[0].Y;
		Offsets[1].x = GRHIVariableRateShadingImageOffsets[1].X;
		Offsets[1].y = GRHIVariableRateShadingImageOffsets[1].Y;

		ZeroVulkanStruct(FDMOffsetInfo, VK_STRUCTURE_TYPE_RENDER_PASS_FRAGMENT_DENSITY_MAP_OFFSET_END_INFO_EXT);
		FDMOffsetInfo.pFragmentDensityOffsets = Offsets;
		FDMOffsetInfo.fragmentDensityOffsetCount = 2;
		FDMOffsetInfo.pFragmentDensityOffsets = Offsets;
		FDMOffsetInfo.pNext = nullptr;
 		
		SubpassEndInfo.pNext = &FDMOffsetInfo;
		VulkanRHI::vkCmdEndRendering2KHR(CommandBufferHandle, &SubpassEndInfo);
	}
	else
	{
		VulkanRHI::vkCmdEndRenderingKHR(CommandBufferHandle);
	}

	State = EState::IsInsideBegin;
}

void FVulkanCommandBuffer::End(FVulkanQueryPool* OptionalQueryPool)
{
	checkf(IsOutsideRenderPass() || (GetCommandBufferType() == EVulkanCommandBufferType::Secondary), 
		TEXT("Can't End as we're inside a render pass! CmdBuffer 0x%p State=%d"), CommandBufferHandle, (int32)State);

	// Reset barrier events for next use
	for (VkEvent BarrierEvent : EndedBarrierEvents)
	{
		VulkanRHI::vkCmdResetEvent(GetHandle(), BarrierEvent, VK_PIPELINE_STAGE_ALL_COMMANDS_BIT);
	}

	if (OptionalQueryPool)
	{
		auto& Event = EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FEndWork>(0);
		const uint32 IndexInPool = OptionalQueryPool->ReserveQuery(&Event.GPUTimestampBOP);
		VulkanRHI::vkCmdWriteTimestamp(GetHandle(), VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT, OptionalQueryPool->GetHandle(), IndexInPool);
	}

	VERIFYVULKANRESULT(VulkanRHI::vkEndCommandBuffer(GetHandle()));
	State = EState::HasEnded;
}

void FVulkanCommandBuffer::Begin(FVulkanQueryPool* OptionalQueryPool, VkRenderPass RenderPassHandle)
{
	const bool bIsSecondaryCommandBuffer = GetCommandBufferType() == EVulkanCommandBufferType::Secondary;
	checkf(!bIsSecondaryCommandBuffer || (RenderPassHandle != VK_NULL_HANDLE),
		TEXT("Secondary command buffers require the render pass handle!"));

	{
		FScopeLock ScopeLock(CommandBufferPool.GetCS());
		if (State == EState::NeedReset)
		{
			VulkanRHI::vkResetCommandBuffer(CommandBufferHandle, VK_COMMAND_BUFFER_RESET_RELEASE_RESOURCES_BIT);
		}
		else
		{
			checkf(State == EState::ReadyForBegin, TEXT("Can't Begin as we're NOT ready! CmdBuffer 0x%p State=%d"), CommandBufferHandle, (int32)State);
		}
		State = (RenderPassHandle != VK_NULL_HANDLE) ? EState::IsInsideRenderPass : EState::IsInsideBegin;
	}

	VkCommandBufferBeginInfo BeginInfo;
	ZeroVulkanStruct(BeginInfo, VK_STRUCTURE_TYPE_COMMAND_BUFFER_BEGIN_INFO);
	BeginInfo.flags = VK_COMMAND_BUFFER_USAGE_ONE_TIME_SUBMIT_BIT;

	FVulkanBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
	VkCommandBufferInheritanceInfo InheritanceInfo;
	VkCommandBufferInheritanceDescriptorHeapInfoEXT DescriptorHeapInfo;
	if (bIsSecondaryCommandBuffer)
	{
		BeginInfo.flags |= VK_COMMAND_BUFFER_USAGE_RENDER_PASS_CONTINUE_BIT;
		BeginInfo.pInheritanceInfo = &InheritanceInfo;

		ZeroVulkanStruct(InheritanceInfo, VK_STRUCTURE_TYPE_COMMAND_BUFFER_INHERITANCE_INFO);
		InheritanceInfo.renderPass = RenderPassHandle;

		if (BindlessDescriptorManager->UseDescriptorHeaps())
		{
			DescriptorHeapInfo = {
				.sType = VK_STRUCTURE_TYPE_COMMAND_BUFFER_INHERITANCE_DESCRIPTOR_HEAP_INFO_EXT,
				.pNext = nullptr,
				.pSamplerHeapBindInfo = &BindlessDescriptorManager->GetSamplerBindHeapInfo(),
				.pResourceHeapBindInfo = &BindlessDescriptorManager->GetResourceBindHeapInfo()
			};

			AddToPNext(InheritanceInfo, DescriptorHeapInfo);
		}
	}

	VERIFYVULKANRESULT(VulkanRHI::vkBeginCommandBuffer(CommandBufferHandle, &BeginInfo));

	if (Device.SupportsBindless())
	{
		if (BindlessDescriptorManager->UseDescriptorBuffers())
		{
			const VkPipelineStageFlags SupportedStages = CommandBufferPool.GetQueue().GetSupportedStageBits();
			BindlessDescriptorManager->BindDescriptorBuffers(CommandBufferHandle, SupportedStages);
			bHasBoundDescriptorHeap = true;
		}
		else if (BindlessDescriptorManager->UseDescriptorHeaps() && !bIsSecondaryCommandBuffer)
		{
			VulkanRHI::vkCmdBindSamplerHeapEXT(CommandBufferHandle, &BindlessDescriptorManager->GetSamplerBindHeapInfo());
			VulkanRHI::vkCmdBindResourceHeapEXT(CommandBufferHandle, &BindlessDescriptorManager->GetResourceBindHeapInfo());
			bHasBoundDescriptorHeap = true;
		}
	}

	if (OptionalQueryPool)
	{
		auto& Event = EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FBeginWork>(0, UINT64_MAX);
		const uint32 IndexInPool = OptionalQueryPool->ReserveQuery(&Event.GPUTimestampTOP);
		VulkanRHI::vkCmdWriteTimestamp(GetHandle(), VK_PIPELINE_STAGE_TOP_OF_PIPE_BIT, OptionalQueryPool->GetHandle(), IndexInPool);
	}

	ResetBoundShaderObject();

	bNeedsFullDynamicStateUpdate = true;
}

void FVulkanCommandBuffer::SetUnusedGraphicsDynamicStates()
{
	// Some unused states have been made dynamic with shader objects, set a default value at the start of each new command buffer
	// :todo-jn: Look for a way to batch these settings (it was suggested to use a baked secondary command buffer)
	if (Device.SupportsShaderObjects())
	{
		VulkanRHI::vkCmdSetRasterizerDiscardEnableEXT(CommandBufferHandle, false);
		VulkanRHI::vkCmdSetDepthClampEnableEXT(CommandBufferHandle, false);
		VulkanRHI::vkCmdSetAlphaToOneEnableEXT(CommandBufferHandle, false);
		VulkanRHI::vkCmdSetLogicOpEnableEXT(CommandBufferHandle, false);
		//VulkanRHI::vkCmdSetLogicOpEXT(CommandBufferHandle, VK_LOGIC_OP_CLEAR);  // ignored when LogicOpEnable is false
		VulkanRHI::vkCmdSetPrimitiveRestartEnableEXT(CommandBufferHandle, false);
		VulkanRHI::vkCmdSetFrontFaceEXT(CommandBufferHandle, VK_FRONT_FACE_CLOCKWISE);
		VulkanRHI::vkCmdSetLineWidth(CommandBufferHandle, 1.0f);

		const float blendConstants[4] = {1.0f, 1.0f, 1.0f, 1.0f};
		VulkanRHI::vkCmdSetBlendConstants(CommandBufferHandle, blendConstants);

		const VkSampleMask SampleMask[] = { MAX_uint32, MAX_uint32 };
		VulkanRHI::vkCmdSetSampleMaskEXT(CommandBufferHandle, VK_SAMPLE_COUNT_64_BIT, SampleMask);
	}
}

void FVulkanCommandBuffer::ResetBoundShaderObject()
{
	if (Device.SupportsShaderObjects())
	{
		CurrentComputeShaderObject = VK_NULL_HANDLE;

		// Clear all graphics stages, including the unused tess stages
		if (CommandBufferPool.GetQueue().GetQueueType() == EVulkanQueueType::Graphics)
		{
			FMemory::Memzero(CurrentGraphicsShaderObjects);

			uint32 StageCount = 0;
			TStaticArray<VkShaderStageFlagBits, ShaderStage::NumGraphicsStages + 2> ShaderStages;

			ShaderStages[StageCount++] = VK_SHADER_STAGE_TESSELLATION_CONTROL_BIT; // unused
			ShaderStages[StageCount++] = VK_SHADER_STAGE_TESSELLATION_EVALUATION_BIT; // unused

			ShaderStages[StageCount++] = VK_SHADER_STAGE_VERTEX_BIT;
			ShaderStages[StageCount++] = VK_SHADER_STAGE_FRAGMENT_BIT;

#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
			if (Device.GetPhysicalDeviceFeatures().Core_1_0.geometryShader)
			{
				ShaderStages[StageCount++] = VK_SHADER_STAGE_GEOMETRY_BIT;
			}
#endif

#if PLATFORM_SUPPORTS_MESH_SHADERS
			if (Device.GetOptionalExtensions().HasEXTMeshShader)
			{
				ShaderStages[StageCount++] = VK_SHADER_STAGE_MESH_BIT_EXT;
				ShaderStages[StageCount++] = VK_SHADER_STAGE_TASK_BIT_EXT;
			}
#endif
			VulkanRHI::vkCmdBindShadersEXT(CommandBufferHandle, StageCount, ShaderStages.GetData(), nullptr);
		}
	}
}

void FVulkanCommandBuffer::ResetCurrentState()
{
	CurrentViewports.Empty();
	CurrentScissors.Empty();
	CurrentStencilRef = 0;

	CurrentPipelineHandle = VK_NULL_HANDLE;

	CurrentPushConstants.SetNumUninitialized(FVulkanBindlessDescriptorManager::RequiredPushDataSize / sizeof(uint32));
	FMemory::Memset(CurrentPushConstants.GetData(), 0xFF, CurrentPushConstants.NumBytes());
}


void FVulkanCommandBuffer::Reset()
{
	// Reset the secondary command buffers we executed from this one
	for (FVulkanCommandBuffer* SecondaryCommand : ExecutedSecondaryCommandBuffers)
	{
		SecondaryCommand->Reset();
	}
	ExecutedSecondaryCommandBuffers.Empty();

	// Hold lock while State is altered
	FScopeLock ScopeLock(CommandBufferPool.GetCS());
	if (State == EState::Submitted)
	{
		bNeedsFullDynamicStateUpdate = true;
		bHasPipeline = false;
		bHasViewport = false;
		bHasScissor = false;
		bHasStencilRef = false;
		bHasBoundDescriptorHeap = false;

		ResetCurrentState();

		for (VkEvent BarrierEvent : EndedBarrierEvents)
		{
			Device.ReleaseBarrierEvent(BarrierEvent);
		}
		EndedBarrierEvents.Reset();

		// Change state at the end to be safe
		State = EState::NeedReset;
	}
}

void FVulkanCommandBuffer::BindPipeline(VkPipelineBindPoint BindPoint, VkPipeline PipelineHandle, bool bBindless)
{
	if (PipelineHandle != CurrentPipelineHandle)
	{
		VulkanRHI::vkCmdBindPipeline(CommandBufferHandle, BindPoint, PipelineHandle);
		CurrentPipelineHandle = PipelineHandle;
	}

	if (bBindless && Device.SupportsBindless())
	{
		if (!bHasBoundDescriptorHeap)
		{
			FVulkanBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
			const bool bIsSecondaryCommandBuffer = GetCommandBufferType() == EVulkanCommandBufferType::Secondary;
			if (BindlessDescriptorManager->UseDescriptorBuffers())
			{
				const VkPipelineStageFlags SupportedStages = CommandBufferPool.GetQueue().GetSupportedStageBits();
				BindlessDescriptorManager->BindDescriptorBuffers(CommandBufferHandle, SupportedStages);
				bHasBoundDescriptorHeap = true;
			}
			else if (BindlessDescriptorManager->UseDescriptorHeaps() && !bIsSecondaryCommandBuffer)
			{
				VulkanRHI::vkCmdBindSamplerHeapEXT(CommandBufferHandle, &BindlessDescriptorManager->GetSamplerBindHeapInfo());
				VulkanRHI::vkCmdBindResourceHeapEXT(CommandBufferHandle, &BindlessDescriptorManager->GetResourceBindHeapInfo());
				bHasBoundDescriptorHeap = true;
			}
		}
	}
	else
	{
		// Using regular pipelines invalidates heap bindings
		bHasBoundDescriptorHeap = false;
	}
}

void FVulkanCommandBuffer::BindGraphicsShaderObjects(TArrayView<FVulkanShader*> VulkanShaders)
{
	check(VulkanShaders.Num() == ShaderStage::NumGraphicsStages);
	VkShaderStageFlagBits ShaderStages[ShaderStage::NumGraphicsStages];
	VkShaderEXT ShaderObjectHandles[ShaderStage::NumGraphicsStages] = { VK_NULL_HANDLE };

	uint32 ChangeCount = 0;
	auto AddStage = [this, &ChangeCount, &ShaderStages, &ShaderObjectHandles, &VulkanShaders](ShaderStage::EStage Stage, VkShaderStageFlagBits StageFlag)
	{

		FVulkanShader* VulkanShader = VulkanShaders[Stage];
		if (VulkanShader)
		{
			VulkanShader->WaitForAsyncCompile();
			ShaderObjectHandles[ChangeCount] = VulkanShader->GetShaderObject();
		}
		else
		{
			ShaderObjectHandles[ChangeCount] = VK_NULL_HANDLE;
		}

		if (CurrentGraphicsShaderObjects[Stage] != ShaderObjectHandles[ChangeCount])
		{
			ShaderStages[ChangeCount] = StageFlag;
			CurrentGraphicsShaderObjects[Stage] = ShaderObjectHandles[ChangeCount];
			ChangeCount++;
		}
	};

	AddStage(ShaderStage::Vertex, VK_SHADER_STAGE_VERTEX_BIT);
	AddStage(ShaderStage::Pixel, VK_SHADER_STAGE_FRAGMENT_BIT);

#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	if (Device.GetPhysicalDeviceFeatures().Core_1_0.geometryShader)
	{
		AddStage(ShaderStage::Geometry, VK_SHADER_STAGE_GEOMETRY_BIT);
	}
#endif

#if PLATFORM_SUPPORTS_MESH_SHADERS
	if (Device.GetOptionalExtensions().HasEXTMeshShader)
	{
		AddStage(ShaderStage::Mesh, VK_SHADER_STAGE_MESH_BIT_EXT);
		AddStage(ShaderStage::Task, VK_SHADER_STAGE_TASK_BIT_EXT);
	}
#endif

	if (ChangeCount > 0)
	{
		VulkanRHI::vkCmdBindShadersEXT(CommandBufferHandle, ChangeCount, ShaderStages, ShaderObjectHandles);
	}
}

void FVulkanCommandBuffer::BindComputeShaderObject(FVulkanShader* VulkanShader)
{
	check(VulkanShader);
	VulkanShader->WaitForAsyncCompile();

	static const VkShaderStageFlagBits ShaderStage = VK_SHADER_STAGE_COMPUTE_BIT;
	const VkShaderEXT ShaderObjectHandle = VulkanShader->GetShaderObject();
	check(ShaderObjectHandle);

	if (CurrentComputeShaderObject != ShaderObjectHandle)
	{
		VulkanRHI::vkCmdBindShadersEXT(CommandBufferHandle, 1, &ShaderStage, &ShaderObjectHandle);
		CurrentComputeShaderObject = ShaderObjectHandle;
	}
}

void FVulkanCommandBuffer::SetSubmitted()
{
	for (FVulkanCommandBuffer* SecondaryCommand : ExecutedSecondaryCommandBuffers)
	{
		SecondaryCommand->SetSubmitted();
	}

	FScopeLock Lock(CommandBufferPool.GetCS());
	State = FVulkanCommandBuffer::EState::Submitted;
	SubmittedTime = FPlatformTime::Seconds();
}

void FVulkanCommandBuffer::BeginSplitBarrier(VkEvent BarrierEvent, const VkDependencyInfo& DependencyInfo)
{
	VulkanRHI::vkCmdSetEvent2KHR(GetHandle(), BarrierEvent, &DependencyInfo);
}

void FVulkanCommandBuffer::EndSplitBarrier(VkEvent BarrierEvent, const VkDependencyInfo& DependencyInfo)
{
	VulkanRHI::vkCmdWaitEvents2KHR(GetHandle(), 1, &BarrierEvent, &DependencyInfo);
	EndedBarrierEvents.Add(BarrierEvent);
}

void FVulkanCommandBuffer::SetStaticUniformBufferPushConstants(TConstArrayView<uint32> StaticUniformBuffers)
{
	check(StaticUniformBuffers.Num() <= FRHIShaderBindingLayout::MaxUniformBufferEntries);
	if (StaticUniformBuffers.IsEmpty())
	{
		return;
	}

	// trim what we send
	int32 StartIndex = 0;
	for (; StartIndex < StaticUniformBuffers.Num(); ++StartIndex)
	{
		if (CurrentPushConstants[StartIndex] != StaticUniformBuffers[StartIndex])
		{
			break;
		}
	}
	int32 EndIndex = StaticUniformBuffers.Num() - 1;
	for (; EndIndex >= 0; --EndIndex)
	{
		if (CurrentPushConstants[EndIndex] != StaticUniformBuffers[EndIndex])
		{
			break;
		}
	}

	if (StartIndex <= EndIndex)
	{
		FMemory::Memcpy(CurrentPushConstants.GetData(), StaticUniformBuffers.GetData(), StaticUniformBuffers.NumBytes());

		const uint32 NumBytes = (EndIndex - StartIndex + 1) * sizeof(uint32);
		if (Device.GetBindlessDescriptorManager()->UseDescriptorHeaps())
		{
			const VkPushDataInfoEXT PushDataInfo = {
				.sType = VK_STRUCTURE_TYPE_PUSH_DATA_INFO_EXT,
				.pNext = nullptr,
				.offset = StartIndex * sizeof(uint32),
				.data = {
					.address = &CurrentPushConstants[StartIndex],
					.size = NumBytes
				}
			};
			VulkanRHI::vkCmdPushDataEXT(CommandBufferHandle, &PushDataInfo);
		}
		else
		{
			VulkanRHI::vkCmdPushConstants(
				CommandBufferHandle, 
				Device.GetBindlessDescriptorManager()->GetPipelineLayout(),
				VK_SHADER_STAGE_ALL,
				StartIndex * sizeof(uint32),
				NumBytes,
				&CurrentPushConstants[StartIndex]
			);
		}
	}
}

void FVulkanCommandBuffer::SetBindingPushConstants(EShaderFrequency Frequency, uint64 GlobalsAddressOrFirstBinding, TConstArrayView<uint32> Bindings)
{
	checkf(Device.GetBindlessDescriptorManager()->UseDescriptorHeaps(), TEXT("Push constant bindings are only used by descriptor buffers"));
	check(Bindings.Num() <= VulkanBindless::MaxUniformBuffersPerStage-1);

	const uint32 ByteOffset = FVulkanBindlessDescriptorManager::GetBindingPushDataOffset(Frequency, 0);
	const uint32 StartIndex = ByteOffset / sizeof(uint32);
	if (GlobalsAddressOrFirstBinding != MAX_uint64)
	{
		FMemory::Memcpy(&CurrentPushConstants[StartIndex], &GlobalsAddressOrFirstBinding, sizeof(GlobalsAddressOrFirstBinding));
	}

	uint32 NumExtraValues = Bindings.Num();
	if (Bindings.Num())
	{
		// trim what we send to the commandbuffer
		const uint32 FirstExtraIndex = StartIndex + 2u; // Go past our initial 64bit value
		for (; NumExtraValues > 0u; --NumExtraValues)
		{
			if (CurrentPushConstants[FirstExtraIndex + NumExtraValues - 1u] != Bindings[NumExtraValues - 1u])
			{
				break;
			}
		}

		FMemory::Memcpy(&CurrentPushConstants[FirstExtraIndex], Bindings.GetData(), Bindings.NumBytes());
	}

	const VkPushDataInfoEXT PushDataInfo = {
		.sType = VK_STRUCTURE_TYPE_PUSH_DATA_INFO_EXT,
		.pNext = nullptr,
		.offset = ByteOffset,
		.data = {
			.address = &CurrentPushConstants[StartIndex],
			.size = (2u + NumExtraValues) * sizeof(uint32)
		}
	};
	VulkanRHI::vkCmdPushDataEXT(CommandBufferHandle, &PushDataInfo);
}




FVulkanCommandBufferPool::FVulkanCommandBufferPool(FVulkanDevice& InDevice, FVulkanQueue& InQueue, EVulkanCommandBufferType InCommandBufferType)
	: Device(InDevice)
	, Queue(InQueue)
	, CommandBufferType(InCommandBufferType)
{
	VkCommandPoolCreateInfo CmdPoolInfo;
	ZeroVulkanStruct(CmdPoolInfo, VK_STRUCTURE_TYPE_COMMAND_POOL_CREATE_INFO);
	CmdPoolInfo.queueFamilyIndex = InQueue.GetFamilyIndex();
	CmdPoolInfo.flags = VK_COMMAND_POOL_CREATE_RESET_COMMAND_BUFFER_BIT; // :todo: Investigate use of VK_COMMAND_POOL_CREATE_TRANSIENT_BIT?
	VERIFYVULKANRESULT(VulkanRHI::vkCreateCommandPool(Device.GetHandle(), &CmdPoolInfo, VULKAN_CPU_ALLOCATOR, &Handle));
}

FVulkanCommandBufferPool::~FVulkanCommandBufferPool()
{
	for (int32 Index = 0; Index < CmdBuffers.Num(); ++Index)
	{
		FVulkanCommandBuffer* CmdBuffer = CmdBuffers[Index];
		delete CmdBuffer;
	}

	for (int32 Index = 0; Index < FreeCmdBuffers.Num(); ++Index)
	{
		FVulkanCommandBuffer* CmdBuffer = FreeCmdBuffers[Index];
		delete CmdBuffer;
	}

	VulkanRHI::vkDestroyCommandPool(Device.GetHandle(), Handle, VULKAN_CPU_ALLOCATOR);
	Handle = VK_NULL_HANDLE;
}

void FVulkanCommandBufferPool::FreeUnusedCmdBuffers(FVulkanQueue* InQueue, bool bTrimMemory)
{
#if VULKAN_DELETE_STALE_CMDBUFFERS
	FScopeLock ScopeLock(&CS);
	
	if (bTrimMemory)
	{
		VulkanRHI::vkTrimCommandPool(Device.GetHandle(), Handle, 0);
		return;
	}

	const double CurrentTime = FPlatformTime::Seconds();

	for (int32 Index = CmdBuffers.Num() - 1; Index >= 0; --Index)
	{
		FVulkanCommandBuffer* CmdBuffer = CmdBuffers[Index];
		if ((CmdBuffer->State == FVulkanCommandBuffer::EState::ReadyForBegin || CmdBuffer->State == FVulkanCommandBuffer::EState::NeedReset) &&
			((CurrentTime - CmdBuffer->SubmittedTime) > CMD_BUFFER_TIME_TO_WAIT_BEFORE_DELETING))
		{
			CmdBuffer->FreeMemory();
			CmdBuffers.RemoveAtSwap(Index, EAllowShrinking::No);
			FreeCmdBuffers.Add(CmdBuffer);
		}
	}
#endif
}

FVulkanCommandBuffer* FVulkanCommandBufferPool::Create()
{
	// Assumes we are inside a lock for the pool
	if (FreeCmdBuffers.Num())
	{
		FVulkanCommandBuffer* CmdBuffer = FreeCmdBuffers[0];
		FreeCmdBuffers.RemoveAtSwap(0);
		CmdBuffer->AllocMemory();
		CmdBuffers.Add(CmdBuffer);
		return CmdBuffer;
	}

	FVulkanCommandBuffer* CmdBuffer = new FVulkanCommandBuffer(Device, *this);
	CmdBuffers.Add(CmdBuffer);
	check(CmdBuffer);
	return CmdBuffer;
}

