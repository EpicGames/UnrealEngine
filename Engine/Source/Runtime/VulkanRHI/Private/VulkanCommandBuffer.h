// Copyright Epic Games, Inc. All Rights Reserved..

/*=============================================================================
	VulkanCommandBuffer.h: Private Vulkan RHI definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "VulkanConfiguration.h"
#include "VulkanThirdParty.h"

class FVulkanDevice;
class FVulkanCommandBufferPool;
class FVulkanContextCommon;
class FVulkanQueue;
class FVulkanQueryPool;
class FVulkanShader;
struct FVulkanBeginRenderPassInfo;
struct FGfxPipelineDesc;

namespace VulkanRHI
{
	class FFence;
	class FSemaphore;
}

enum class EVulkanCommandBufferType : uint8
{
	Primary,

	// Secondary command buffer with an inherited render pass that will get executed from a primary cmd buffer
	// Used for parallel contexts without dynamic rendering
	Secondary,

	// Primary command buffer that suspend/resume the current renderpass 
	// Used for parallel contexts with dynamic rendering
	Parallel,

	Count
};

class FVulkanCommandBuffer
{
protected:
	friend class FVulkanCommandBufferPool;
	friend class FVulkanQueue;

	FVulkanCommandBuffer(FVulkanDevice& InDevice, FVulkanCommandBufferPool& InCommandBufferPool);
	~FVulkanCommandBuffer();

public:
	bool IsInsideRenderPass() const
	{
		return State == EState::IsInsideRenderPass;
	}

	bool IsOutsideRenderPass() const
	{
		return State == EState::IsInsideBegin;
	}

	bool HasBegun() const
	{
		return State == EState::IsInsideBegin || State == EState::IsInsideRenderPass;
	}

	bool HasEnded() const
	{
		return State == EState::HasEnded;
	}

	bool IsSubmitted() const
	{
		return State == EState::Submitted;
	}

	bool IsAllocated() const
	{
		return State != EState::NotAllocated;
	}

	VkCommandBuffer GetHandle() const
	{
		return CommandBufferHandle;
	}

	EVulkanCommandBufferType GetCommandBufferType() const;

	void SetSubmitted();

	void Begin(FVulkanQueryPool* OptionalQueryPool, VkRenderPass RenderPassHandle);
	void End(FVulkanQueryPool* OptionalQueryPool);

	enum class EState : uint8
	{
		ReadyForBegin,
		IsInsideBegin,
		IsInsideRenderPass,
		HasEnded,
		Submitted,
		NotAllocated,
		NeedReset,
	};

	TArray<VkViewport, TInlineAllocator<2>> CurrentViewports;
	TArray<VkRect2D, TInlineAllocator<2>> CurrentScissors;
	uint32 CurrentStencilRef = 0;
	EState State = EState::NotAllocated;

	FGfxPipelineDesc* LastDynamicStateUpdate = nullptr;

	uint8 bNeedsFullDynamicStateUpdate	: 1 = 1;
	uint8 bHasPipeline					: 1 = 0;
	uint8 bHasViewport					: 1 = 0;
	uint8 bHasScissor					: 1 = 0;
	uint8 bHasStencilRef				: 1 = 0;
	uint8 bHasBoundDescriptorHeap		: 1 = 0;

	uint8 CurrentMultiViewCount = 0;

	// Every secondary command buffer executed from this one with tied lifetimes
	TArray<FVulkanCommandBuffer*> ExecutedSecondaryCommandBuffers;

	// You never want to call Begin/EndRenderPass directly as it will mess up the layout manager.
	void BeginRenderPass(const FVulkanBeginRenderPassInfo& BeginRenderPassInfo, const VkClearValue* AttachmentClearValues);
	void EndRenderPass();

	// Begin/End calls for when using VK_KHR_dynamic_rendering
	void BeginDynamicRendering(const VkRenderingInfo& RenderingInfo, FVulkanQueryPool* OptionalQueryPool);
	void EndDynamicRendering(FVulkanQueryPool* OptionalQueryPool);

	// Flags used with the last dynamic rendering pass
	VkRenderingFlags LastDynamicRenderingFlags = 0;

	// Split barriers
	void BeginSplitBarrier(VkEvent BarrierEvent, const VkDependencyInfo& DependencyInfo);
	void EndSplitBarrier(VkEvent BarrierEvent, const VkDependencyInfo& DependencyInfo);

	template <typename TEventType, typename... TArgs>
	TEventType& EmplaceProfilerEvent(TArgs&&... Args)
	{
		TEventType& Data = EventStream.Emplace<TEventType>(Forward<TArgs>(Args)...);

		if constexpr (std::is_same_v<UE::RHI::GPUProfiler::FEvent::FBeginWork, TEventType>)
		{
			// Store BeginEvents in a separate array as the CPUTimestamp field needs updating at submit time.
			BeginEvents.Add(&Data);
		}

		return Data;
	}

	void FlushProfilerEvents(UE::RHI::GPUProfiler::FEventStream& Destination, uint64 CPUTimestamp)
	{
		for (UE::RHI::GPUProfiler::FEvent::FBeginWork* BeginEvent : BeginEvents)
		{
			checkSlow((BeginEvent->CPUTimestamp == 0) && (BeginEvent->GPUTimestampTOP == UINT64_MAX));
			BeginEvent->CPUTimestamp = CPUTimestamp;
		}
		BeginEvents.Reset();

		Destination.Append(MoveTemp(EventStream));
	}

	// Public immutable reference to be accessed directly
	FVulkanDevice& Device;

	void SetUnusedGraphicsDynamicStates();

	void BindPipeline(VkPipelineBindPoint BindPoint, VkPipeline PipelineHandle, bool bBindless);
	void BindGraphicsShaderObjects(TArrayView<FVulkanShader*> VulkanShaders);
	void BindComputeShaderObject(FVulkanShader* VulkanShader);

	void SetStaticUniformBufferPushConstants(TConstArrayView<uint32> StaticUniformBuffers);
	void SetBindingPushConstants(EShaderFrequency Frequency, uint64 GlobalsAddressOrFirstBinding, TConstArrayView<uint32> Bindings);

private:
	VkCommandBuffer CommandBufferHandle = VK_NULL_HANDLE;
	double SubmittedTime = 0.0f;

	void Reset();

	FVulkanCommandBufferPool& CommandBufferPool;

	VkPipeline CurrentPipelineHandle = VK_NULL_HANDLE;
	VkShaderEXT CurrentComputeShaderObject = VK_NULL_HANDLE;
	VkShaderEXT CurrentGraphicsShaderObjects[ShaderStage::NumGraphicsStages] = { VK_NULL_HANDLE };
	TArray<uint32> CurrentPushConstants;

	void ResetCurrentState();
	void ResetBoundShaderObject();

	TArray<VkEvent> EndedBarrierEvents;

	void AllocMemory();
	void FreeMemory();

	UE::RHI::GPUProfiler::FEventStream EventStream;
	TArray<UE::RHI::GPUProfiler::FEvent::FBeginWork*, TInlineAllocator<8>> BeginEvents;

	friend class FVulkanDynamicRHI;
	friend FVulkanContextCommon;
};

class FVulkanCommandBufferPool
{
public:
	FVulkanCommandBufferPool(FVulkanDevice& InDevice, FVulkanQueue& InQueue, EVulkanCommandBufferType InCommandBufferType);
	~FVulkanCommandBufferPool();

	VkCommandPool GetHandle() const
	{
		return Handle;
	}

	FCriticalSection* GetCS()
	{
		return &CS;
	}

	FVulkanQueue& GetQueue()
	{
		return Queue;
	}

	EVulkanCommandBufferType GetCommandBufferType() const
	{
		return CommandBufferType;
	}

	void FreeUnusedCmdBuffers(FVulkanQueue* Queue, bool bTrimMemory);

private:
	FVulkanDevice& Device;
	FVulkanQueue& Queue;

	const EVulkanCommandBufferType CommandBufferType;

	VkCommandPool Handle = VK_NULL_HANDLE;

	TArray<FVulkanCommandBuffer*> CmdBuffers;
	TArray<FVulkanCommandBuffer*> FreeCmdBuffers;

	FCriticalSection CS;

	FVulkanCommandBuffer* Create();

	friend FVulkanContextCommon;
};


inline EVulkanCommandBufferType FVulkanCommandBuffer::GetCommandBufferType() const
{
	return CommandBufferPool.GetCommandBufferType();
}
