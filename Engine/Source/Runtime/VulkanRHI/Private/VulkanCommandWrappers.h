// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanCommandWrappers.h: Wrap all Vulkan API functions so we can add our own 'layers'
=============================================================================*/

#pragma once

#include "VulkanConfiguration.h"
#include "VulkanLoader.h"
#include "VulkanThirdParty.h"

#if !VULKAN_COMMANDWRAPPERS_ENABLE
	#if VULKAN_DYNAMICALLYLOADED
		// Bring functions from VulkanDynamicAPI to VulkanRHI
		#define VK_DYNAMICAPI_TO_VULKANRHI(Type,Func) using VulkanDynamicAPI::Func;

		namespace VulkanRHI
		{
			ENUM_VK_ENTRYPOINTS_ALL(VK_DYNAMICAPI_TO_VULKANRHI);
		}
	#else // VULKAN_DYNAMICALLYLOADED
		#error "Statically linked vulkan api must be wrapped!"
	#endif // VULKAN_DYNAMICALLYLOADED

#else // VULKAN_COMMANDWRAPPERS_ENABLE

#if VULKAN_DYNAMICALLYLOADED
	// Vulkan API is defined in VulkanDynamicAPI namespace.
	#define VULKANAPINAMESPACE VulkanDynamicAPI
#else
	// Vulkan API is in the global namespace.
	#define VULKANAPINAMESPACE
#endif

#if VULKAN_ENABLE_WRAP_LAYER
	#define  VULKAN_LAYER_BODY		;
	#define	VULKAN_EXTERN_EXPORT	VULKANRHI_API
#else
	#define VULKAN_LAYER_BODY		{}
	#define	VULKAN_EXTERN_EXPORT
#endif

struct FWrapLayer
{
	// Pass in VK_RESULT_MAX_ENUM for Prolog calls; for Epilog use a different value or the actual Result if available

	static void CreateInstance(VkResult Result, const VkInstanceCreateInfo* CreateInfo, VkInstance* Instance) VULKAN_LAYER_BODY
	static void DestroyInstance(VkResult Result, VkInstance Instance) VULKAN_LAYER_BODY
	static void EnumeratePhysicalDevices(VkResult Result, VkInstance Instance, uint32* PhysicalDeviceCount, VkPhysicalDevice* PhysicalDevices) VULKAN_LAYER_BODY
	static void EnumerateInstanceVersion(VkResult Result, uint32_t* ApiVersion) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceFeatures(VkResult Result, VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceFeatures* Features) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceFormatProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, VkFormat Format, VkFormatProperties* FormatProperties) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceFormatProperties2(VkResult Result, VkPhysicalDevice PhysicalDevice, VkFormat Format, VkFormatProperties2* FormatProperties) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceImageFormatProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, VkFormat Format, VkImageType Type, VkImageTiling Tiling, VkImageUsageFlags Usage, VkImageCreateFlags Flags, VkImageFormatProperties* pImageFormatProperties) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceImageFormatProperties2(VkResult Result, VkPhysicalDevice PhysicalDevice, const VkPhysicalDeviceImageFormatInfo2* FormatInfo, VkImageFormatProperties2* FormatProperties) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceProperties* FormatProperties) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceQueueFamilyProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, uint32* QueueFamilyPropertyCount, VkQueueFamilyProperties* QueueFamilyProperties) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceQueueFamilyProperties2(VkResult Result, VkPhysicalDevice PhysicalDevice, uint32* QueueFamilyPropertyCount, VkQueueFamilyProperties2* QueueFamilyProperties) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceMemoryProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceMemoryProperties* Properties) VULKAN_LAYER_BODY
	static void GetInstanceProcAddr(VkResult Result, VkInstance Instance, const char* Name, PFN_vkVoidFunction VoidFunction) VULKAN_LAYER_BODY
	VULKAN_EXTERN_EXPORT static void GetDeviceProcAddr(VkResult Result, VkDevice Device, const char* Name, PFN_vkVoidFunction VoidFunction) VULKAN_LAYER_BODY
	static void CreateDevice(VkResult Result, VkPhysicalDevice PhysicalDevice, const VkDeviceCreateInfo* CreateInfo, VkDevice* Device) VULKAN_LAYER_BODY
	static void DestroyDevice(VkResult Result, VkDevice Device) VULKAN_LAYER_BODY
	VULKAN_EXTERN_EXPORT static void EnumerateInstanceExtensionProperties(VkResult Result, const char* LayerName, uint32* PropertyCount, VkExtensionProperties* Properties) VULKAN_LAYER_BODY
	VULKAN_EXTERN_EXPORT static void EnumerateDeviceExtensionProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, const char* LayerName, uint32* PropertyCount, VkExtensionProperties* Properties) VULKAN_LAYER_BODY
	static void EnumerateInstanceLayerProperties(VkResult Result, uint32* PropertyCount, VkLayerProperties* Properties) VULKAN_LAYER_BODY
	static void EnumerateDeviceLayerProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, uint32* PropertyCount, VkLayerProperties* Properties) VULKAN_LAYER_BODY
	static void GetDeviceQueue(VkResult Result, VkDevice Device, uint32 QueueFamilyIndex, uint32 QueueIndex, VkQueue* Queue) VULKAN_LAYER_BODY
	static void QueueSubmit(VkResult Result, VkQueue Queue, uint32 SubmitCount, const VkSubmitInfo* Submits, VkFence Fence) VULKAN_LAYER_BODY
	static void QueueWaitIdle(VkResult Result, VkQueue Queue) VULKAN_LAYER_BODY
	static void DeviceWaitIdle(VkResult Result, VkDevice Device) VULKAN_LAYER_BODY
	static void AllocateMemory(VkResult Result, VkDevice Device, const VkMemoryAllocateInfo* AllocateInfo, VkDeviceMemory* Memory) VULKAN_LAYER_BODY
	static void FreeMemory(VkResult Result, VkDevice Device, VkDeviceMemory Memory) VULKAN_LAYER_BODY
	static void MapMemory(VkResult Result, VkDevice Device, VkDeviceMemory Memory, VkDeviceSize Offset, VkDeviceSize Size, VkMemoryMapFlags Flags, void** Data) VULKAN_LAYER_BODY
	static void UnmapMemory(VkResult Result, VkDevice Device, VkDeviceMemory Memory) VULKAN_LAYER_BODY
	static void FlushMappedMemoryRanges(VkResult Result, VkDevice Device, uint32 MemoryRangeCount, const VkMappedMemoryRange* pMemoryRanges) VULKAN_LAYER_BODY
	static void InvalidateMappedMemoryRanges(VkResult Result, VkDevice Device, uint32 MemoryRangeCount, const VkMappedMemoryRange* pMemoryRanges) VULKAN_LAYER_BODY
	static void GetDeviceMemoryCommitment(VkResult Result, VkDevice Device, VkDeviceMemory Memory, VkDeviceSize* pCommittedMemoryInBytes) VULKAN_LAYER_BODY
	static void BindBufferMemory(VkResult Result, VkDevice Device, VkBuffer Buffer, VkDeviceMemory Memory, VkDeviceSize MemoryOffset) VULKAN_LAYER_BODY
	VULKAN_EXTERN_EXPORT static void BindImageMemory(VkResult Result, VkDevice Device, VkImage Image, VkDeviceMemory Memory, VkDeviceSize MemoryOffset) VULKAN_LAYER_BODY
	static void GetBufferMemoryRequirements(VkResult Result, VkDevice Device, VkBuffer Buffer, VkMemoryRequirements* MemoryRequirements) VULKAN_LAYER_BODY
	static void GetImageMemoryRequirements(VkResult Result, VkDevice Device, VkImage Image, VkMemoryRequirements* MemoryRequirements) VULKAN_LAYER_BODY
	static void GetImageSparseMemoryRequirements(VkResult Result, VkDevice Device, VkImage Image, uint32_t* pSparseMemoryRequirementCount, VkSparseImageMemoryRequirements* pSparseMemoryRequirements) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceSparseImageFormatProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, VkFormat Format, VkImageType Type, VkSampleCountFlagBits Samples, VkImageUsageFlags Usage, VkImageTiling Tiling, uint32_t* pPropertyCount, VkSparseImageFormatProperties* pProperties) VULKAN_LAYER_BODY
	static void QueueBindSparse(VkResult Result, VkQueue Queue, uint32_t BindInfoCount, const VkBindSparseInfo* pBindInfo, VkFence Fence) VULKAN_LAYER_BODY
	static void CreateFence(VkResult Result, VkDevice Device, const VkFenceCreateInfo* CreateInfo, VkFence* Fence) VULKAN_LAYER_BODY
	static void DestroyFence(VkResult Result, VkDevice Device, VkFence Fence) VULKAN_LAYER_BODY
	static void ResetFences(VkResult Result, VkDevice Device, uint32 FenceCount, const VkFence* Fences) VULKAN_LAYER_BODY
	static void GetFenceStatus(VkResult Result, VkDevice Device, VkFence Fence) VULKAN_LAYER_BODY
	static void WaitForFences(VkResult Result, VkDevice Device, uint32 FenceCount, const VkFence* Fences, VkBool32 bWaitAll, uint64_t Timeout) VULKAN_LAYER_BODY
	static void CreateSemaphore(VkResult Result, VkDevice Device, const VkSemaphoreCreateInfo* CreateInfo, VkSemaphore* Semaphore) VULKAN_LAYER_BODY
	static void DestroySemaphore(VkResult Result, VkDevice Device, VkSemaphore Semaphore) VULKAN_LAYER_BODY
	static void CreateEvent(VkResult Result, VkDevice Device, const VkEventCreateInfo* CreateInfo, VkEvent* Event) VULKAN_LAYER_BODY
	static void DestroyEvent(VkResult Result, VkDevice Device, VkEvent Event) VULKAN_LAYER_BODY
	static void GetEventStatus(VkResult Result, VkDevice Device, VkEvent Event) VULKAN_LAYER_BODY
	static void SetEvent(VkResult Result, VkDevice Device, VkEvent Event) VULKAN_LAYER_BODY
	static void ResetEvent(VkResult Result, VkDevice Device, VkEvent Event) VULKAN_LAYER_BODY
	static void CreateQueryPool(VkResult Result, VkDevice Device, const VkQueryPoolCreateInfo* CreateInfo, VkQueryPool* QueryPool) VULKAN_LAYER_BODY
	static void DestroyQueryPool(VkResult Result, VkDevice Device, VkQueryPool QueryPool) VULKAN_LAYER_BODY
	static void GetQueryPoolResults(VkResult Result, VkDevice Device, VkQueryPool QueryPool, uint32 FirstQuery, uint32 QueryCount, size_t DataSize, void* Data, VkDeviceSize Stride, VkQueryResultFlags Flags) VULKAN_LAYER_BODY
	static void CreateBuffer(VkResult Result, VkDevice Device, const VkBufferCreateInfo* CreateInfo, VkBuffer* Buffer) VULKAN_LAYER_BODY
	static void DestroyBuffer(VkResult Result, VkDevice Device, VkBuffer Buffer) VULKAN_LAYER_BODY
	static void CreateBufferView(VkResult Result, VkDevice Device, const VkBufferViewCreateInfo* CreateInfo, VkBufferView* BufferView) VULKAN_LAYER_BODY
	static void DestroyBufferView(VkResult Result, VkDevice Device, VkBufferView BufferView) VULKAN_LAYER_BODY
	VULKAN_EXTERN_EXPORT static void CreateImage(VkResult Result, VkDevice Device, const VkImageCreateInfo* CreateInfo, VkImage* Image) VULKAN_LAYER_BODY
	VULKAN_EXTERN_EXPORT static void DestroyImage(VkResult Result, VkDevice Device, VkImage Image) VULKAN_LAYER_BODY
	static void GetImageSubresourceLayout(VkResult Result, VkDevice Device, VkImage Image, const VkImageSubresource* Subresource, VkSubresourceLayout* Layout) VULKAN_LAYER_BODY
	static void GetImageSubresourceLayout2(VkResult Result, VkDevice Device, VkImage Image, const VkImageSubresource2KHR* Subresource, VkSubresourceLayout2KHR* Layout) VULKAN_LAYER_BODY
	VULKAN_EXTERN_EXPORT static void CreateImageView(VkResult Result, VkDevice Device, const VkImageViewCreateInfo* CreateInfo, VkImageView* ImageView) VULKAN_LAYER_BODY
	VULKAN_EXTERN_EXPORT static void DestroyImageView(VkResult Result, VkDevice Device, VkImageView ImageView) VULKAN_LAYER_BODY
	static void CreateShaderModule(VkResult Result, VkDevice Device, const VkShaderModuleCreateInfo* CreateInfo, VkShaderModule* ShaderModule) VULKAN_LAYER_BODY
	static void DestroyShaderModule(VkResult Result, VkDevice Device, VkShaderModule ShaderModule) VULKAN_LAYER_BODY
	static void CreatePipelineCache(VkResult Result, VkDevice Device, const VkPipelineCacheCreateInfo* CreateInfo, VkPipelineCache* PipelineCache) VULKAN_LAYER_BODY
	static void DestroyPipelineCache(VkResult Result, VkDevice Device, VkPipelineCache PipelineCache) VULKAN_LAYER_BODY
	static void GetPipelineCacheData(VkResult Result, VkDevice Device, VkPipelineCache PipelineCache, size_t* DataSize, void* Data) VULKAN_LAYER_BODY
	static void MergePipelineCaches(VkResult Result, VkDevice Device, VkPipelineCache DestCache, uint32 SourceCacheCount, const VkPipelineCache* SrcCaches) VULKAN_LAYER_BODY
	static void CreateGraphicsPipelines(VkResult Result, VkDevice Device, VkPipelineCache PipelineCache, uint32 CreateInfoCount, const VkGraphicsPipelineCreateInfo* CreateInfos, VkPipeline* Pipelines) VULKAN_LAYER_BODY
	static void CreateComputePipelines(VkResult Result, VkDevice Device, VkPipelineCache PipelineCache, uint32 CreateInfoCount, const VkComputePipelineCreateInfo* CreateInfos, VkPipeline* Pipelines) VULKAN_LAYER_BODY
	static void DestroyPipeline(VkResult Result, VkDevice Device, VkPipeline Pipeline) VULKAN_LAYER_BODY
	static void CreatePipelineLayout(VkResult Result, VkDevice Device, const VkPipelineLayoutCreateInfo* CreateInfo, VkPipelineLayout* PipelineLayout) VULKAN_LAYER_BODY
	static void DestroyPipelineLayout(VkResult Result, VkDevice Device, VkPipelineLayout PipelineLayout) VULKAN_LAYER_BODY
	VULKAN_EXTERN_EXPORT static void CreateSampler(VkResult Result, VkDevice Device, const VkSamplerCreateInfo* CreateInfo, VkSampler* Sampler) VULKAN_LAYER_BODY  
	VULKAN_EXTERN_EXPORT static void DestroySampler(VkResult Result, VkDevice Device, VkSampler Sampler) VULKAN_LAYER_BODY
	static void CreateDescriptorSetLayout(VkResult Result, VkDevice Device, const VkDescriptorSetLayoutCreateInfo* CreateInfo, VkDescriptorSetLayout* SetLayout) VULKAN_LAYER_BODY
	static void DestroyDescriptorSetLayout(VkResult Result, VkDevice Device, VkDescriptorSetLayout DescriptorSetLayout) VULKAN_LAYER_BODY
	static void CreateDescriptorPool(VkResult Result, VkDevice Device, const VkDescriptorPoolCreateInfo* CreateInfo, VkDescriptorPool* DescriptorPool) VULKAN_LAYER_BODY
	static void DestroyDescriptorPool(VkResult Result, VkDevice Device, VkDescriptorPool DescriptorPool) VULKAN_LAYER_BODY
	static void ResetDescriptorPool(VkResult Result, VkDevice Device, VkDescriptorPool DescriptorPool, VkDescriptorPoolResetFlags Flags) VULKAN_LAYER_BODY
	static void AllocateDescriptorSets(VkResult Result, VkDevice Device, const VkDescriptorSetAllocateInfo* AllocateInfo, VkDescriptorSet* DescriptorSets) VULKAN_LAYER_BODY
	static void FreeDescriptorSets(VkResult Result, VkDevice Device, VkDescriptorPool DescriptorPool, uint32 DescriptorSetCount, const VkDescriptorSet* DescriptorSets) VULKAN_LAYER_BODY
	static void UpdateDescriptorSets(VkResult Result, VkDevice Device, uint32 DescriptorWriteCount, const VkWriteDescriptorSet* DescriptorWrites, uint32 DescriptorCopyCount, const VkCopyDescriptorSet* DescriptorCopies) VULKAN_LAYER_BODY
	static void CreateFramebuffer(VkResult Result, VkDevice Device, const VkFramebufferCreateInfo* CreateInfo, VkFramebuffer* Framebuffer) VULKAN_LAYER_BODY
	static void DestroyFramebuffer(VkResult Result, VkDevice Device, VkFramebuffer Framebuffer) VULKAN_LAYER_BODY
	static void CreateRenderPass(VkResult Result, VkDevice Device, const VkRenderPassCreateInfo* CreateInfo, VkRenderPass* RenderPass) VULKAN_LAYER_BODY
	static void CreateRenderPass2KHR(VkResult Result, VkDevice Device, const VkRenderPassCreateInfo2* CreateInfo, VkRenderPass* RenderPass) VULKAN_LAYER_BODY
	static void DestroyRenderPass(VkResult Result, VkDevice Device, VkRenderPass RenderPass) VULKAN_LAYER_BODY
	static void GetRenderAreaGranularity(VkResult Result, VkDevice Device, VkRenderPass RenderPass, VkExtent2D* pGranularity) VULKAN_LAYER_BODY
	static void CreateCommandPool(VkResult Result, VkDevice Device, const VkCommandPoolCreateInfo* CreateInfo, VkCommandPool* CommandPool) VULKAN_LAYER_BODY
	static void DestroyCommandPool(VkResult Result, VkDevice Device, VkCommandPool CommandPool) VULKAN_LAYER_BODY
	static void ResetCommandPool(VkResult Result, VkDevice Device, VkCommandPool CommandPool, VkCommandPoolResetFlags Flags) VULKAN_LAYER_BODY
	static void TrimCommandPool(VkResult Result, VkDevice Device, VkCommandPool CommandPool, VkCommandPoolTrimFlags Flags) VULKAN_LAYER_BODY
	static void AllocateCommandBuffers(VkResult Result, VkDevice Device, const VkCommandBufferAllocateInfo* AllocateInfo, VkCommandBuffer* CommandBuffers) VULKAN_LAYER_BODY
	static void FreeCommandBuffers(VkResult Result, VkDevice Device, VkCommandPool CommandPool, uint32 CommandBufferCount, const VkCommandBuffer* CommandBuffers) VULKAN_LAYER_BODY
	static void BeginCommandBuffer(VkResult Result, VkCommandBuffer CommandBuffer, const VkCommandBufferBeginInfo* BeginInfo) VULKAN_LAYER_BODY
	static void EndCommandBuffer(VkResult Result, VkCommandBuffer CommandBuffer) VULKAN_LAYER_BODY
	static void ResetCommandBuffer(VkResult Result, VkCommandBuffer CommandBuffer, VkCommandBufferResetFlags Flags) VULKAN_LAYER_BODY

	static void CmdBindPipeline(VkResult Result, VkCommandBuffer CommandBuffer, VkPipelineBindPoint PipelineBindPoint, VkPipeline Pipeline) VULKAN_LAYER_BODY
	static void CmdSetViewport(VkResult Result, VkCommandBuffer CommandBuffer, uint32 FirstViewport, uint32 ViewportCount, const VkViewport* Viewports) VULKAN_LAYER_BODY
	static void CmdSetScissor(VkResult Result, VkCommandBuffer CommandBuffer, uint32 FirstScissor, uint32 ScissorCount, const VkRect2D* Scissors) VULKAN_LAYER_BODY
	static void CmdSetLineWidth(VkResult Result, VkCommandBuffer CommandBuffer, float LineWidth) VULKAN_LAYER_BODY
	static void CmdSetDepthBias(VkResult Result, VkCommandBuffer CommandBuffer, float DepthBiasConstantFactor, float DepthBiasClamp, float DepthBiasSlopeFactor) VULKAN_LAYER_BODY
	static void CmdSetBlendConstants(VkResult Result, VkCommandBuffer CommandBuffer, const float BlendConstants[4]) VULKAN_LAYER_BODY
	static void CmdSetDepthBounds(VkResult Result, VkCommandBuffer CommandBuffer, float MinDepthBounds, float MaxDepthBounds) VULKAN_LAYER_BODY
	static void CmdSetStencilCompareMask(VkResult Result, VkCommandBuffer CommandBuffer, VkStencilFaceFlags FaceMask, uint32 CompareMask) VULKAN_LAYER_BODY
	static void CmdSetStencilWriteMask(VkResult Result, VkCommandBuffer CommandBuffer, VkStencilFaceFlags FaceMask, uint32 WriteMask) VULKAN_LAYER_BODY
	static void CmdSetStencilReference(VkResult Result, VkCommandBuffer CommandBuffer, VkStencilFaceFlags FaceMask, uint32 Reference) VULKAN_LAYER_BODY
	static void CmdBindDescriptorSets(VkResult Result, VkCommandBuffer CommandBuffer, VkPipelineBindPoint PipelineBindPoint, VkPipelineLayout Layout, uint32 FirstSet, uint32 DescriptorSetCount, const VkDescriptorSet* DescriptorSets, uint32 DynamicOffsetCount, const uint32* DynamicOffsets) VULKAN_LAYER_BODY
	static void CmdBindIndexBuffer(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer IndexBuffer, VkDeviceSize Offset, VkIndexType IndexType) VULKAN_LAYER_BODY
	static void CmdBindVertexBuffers(VkResult Result, VkCommandBuffer CommandBuffer, uint32 FirstBinding, uint32 BindingCount, const VkBuffer* Buffers, const VkDeviceSize* pOffsets) VULKAN_LAYER_BODY
	static void CmdDraw(VkResult Result, VkCommandBuffer CommandBuffer, uint32 VertexCount, uint32 InstanceCount, uint32 FirstVertex, uint32 FirstInstance) VULKAN_LAYER_BODY
	static void CmdDrawIndexed(VkResult Result, VkCommandBuffer CommandBuffer, uint32 IndexCount, uint32 InstanceCount, uint32 FirstIndex, int32_t VertexOffset, uint32 FirstInstance) VULKAN_LAYER_BODY
	static void CmdDrawIndirect(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer Buffer, VkDeviceSize Offset, uint32 DrawCount, uint32 Stride) VULKAN_LAYER_BODY
	static void CmdDrawIndexedIndirect(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer Buffer, VkDeviceSize Offset, uint32 DrawCount, uint32 Stride) VULKAN_LAYER_BODY
	static void CmdDispatch(VkResult Result, VkCommandBuffer CommandBuffer, uint32 X, uint32 Y, uint32 Z) VULKAN_LAYER_BODY
	static void CmdDispatchIndirect(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer Buffer, VkDeviceSize Offset) VULKAN_LAYER_BODY
	static void CmdCopyBuffer(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer SrcBuffer, VkBuffer DstBuffer, uint32 RegionCount, const VkBufferCopy* Regions) VULKAN_LAYER_BODY
	static void CmdCopyImage(VkResult Result, VkCommandBuffer CommandBuffer, VkImage SrcImage, VkImageLayout SrcImageLayout, VkImage DstImage, VkImageLayout DstImageLayout, uint32 RegionCount, const VkImageCopy* Regions) VULKAN_LAYER_BODY
	VULKAN_EXTERN_EXPORT static void CmdBlitImage(VkResult Result, VkCommandBuffer CommandBuffer, VkImage SrcImage, VkImageLayout SrcImageLayout, VkImage DstImage, VkImageLayout DstImageLayout, uint32 RegionCount, const VkImageBlit* Regions, VkFilter Filter) VULKAN_LAYER_BODY
	static void CmdCopyBufferToImage(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer SrcBuffer, VkImage DstImage, VkImageLayout DstImageLayout, uint32 RegionCount, const VkBufferImageCopy* Regions) VULKAN_LAYER_BODY
	static void CmdCopyImageToBuffer(VkResult Result, VkCommandBuffer CommandBuffer, VkImage SrcImage, VkImageLayout SrcImageLayout, VkBuffer DstBuffer, uint32 RegionCount, const VkBufferImageCopy* Regions) VULKAN_LAYER_BODY
	static void CmdUpdateBuffer(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer DstBuffer, VkDeviceSize DstOffset, VkDeviceSize DataSize, const void* pData) VULKAN_LAYER_BODY
	static void CmdFillBuffer(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer DstBuffer, VkDeviceSize DstOffset, VkDeviceSize Size, uint32 Data) VULKAN_LAYER_BODY
	VULKAN_EXTERN_EXPORT static void CmdClearColorImage(VkResult Result, VkCommandBuffer CommandBuffer, VkImage Image, VkImageLayout ImageLayout, const VkClearColorValue* ColorValue, uint32 RangeCount, const VkImageSubresourceRange* Ranges) VULKAN_LAYER_BODY
	VULKAN_EXTERN_EXPORT static void CmdClearDepthStencilImage(VkResult Result, VkCommandBuffer CommandBuffer, VkImage Image, VkImageLayout ImageLayout, const VkClearDepthStencilValue* DepthStencil, uint32 RangeCount, const VkImageSubresourceRange* Ranges) VULKAN_LAYER_BODY
	static void CmdClearAttachments(VkResult Result, VkCommandBuffer CommandBuffer, uint32 AttachmentCount, const VkClearAttachment* Attachments, uint32 RectCount, const VkClearRect* Rects) VULKAN_LAYER_BODY
	static void CmdResolveImage(VkResult Result, VkCommandBuffer CommandBuffer, VkImage SrcImage, VkImageLayout SrcImageLayout, VkImage DstImage, VkImageLayout DstImageLayout, uint32 RegionCount, const VkImageResolve* Regions) VULKAN_LAYER_BODY
	static void CmdSetEvent(VkResult Result, VkCommandBuffer CommandBuffer, VkEvent Event, VkPipelineStageFlags StageMask) VULKAN_LAYER_BODY
	static void CmdResetEvent(VkResult Result, VkCommandBuffer CommandBuffer, VkEvent Event, VkPipelineStageFlags StageMask) VULKAN_LAYER_BODY
	static void CmdWaitEvents(VkResult Result, VkCommandBuffer CommandBuffer, uint32 EventCount, const VkEvent* Events, VkPipelineStageFlags SrcStageMask, VkPipelineStageFlags DstStageMask, uint32 MemoryBarrierCount, const VkMemoryBarrier* pMemoryBarriers, uint32 BufferMemoryBarrierCount, const VkBufferMemoryBarrier* pBufferMemoryBarriers, uint32 ImageMemoryBarrierCount, const VkImageMemoryBarrier* pImageMemoryBarriers) VULKAN_LAYER_BODY
	static void CmdPipelineBarrier(VkResult Result, VkCommandBuffer CommandBuffer, VkPipelineStageFlags SrcStageMask, VkPipelineStageFlags DstStageMask, VkDependencyFlags DependencyFlags, uint32 MemoryBarrierCount, const VkMemoryBarrier* MemoryBarriers, uint32 BufferMemoryBarrierCount, const VkBufferMemoryBarrier* BufferMemoryBarriers, uint32 ImageMemoryBarrierCount, const VkImageMemoryBarrier* ImageMemoryBarriers) VULKAN_LAYER_BODY
	static void CmdBeginQuery(VkResult Result, VkCommandBuffer CommandBuffer, VkQueryPool QueryPool, uint32 Query, VkQueryControlFlags Flags) VULKAN_LAYER_BODY
	static void CmdEndQuery(VkResult Result, VkCommandBuffer CommandBuffer, VkQueryPool QueryPool, uint32 Query) VULKAN_LAYER_BODY
	static void CmdResetQueryPool(VkResult Result, VkCommandBuffer CommandBuffer, VkQueryPool QueryPool, uint32 FirstQuery, uint32 QueryCount) VULKAN_LAYER_BODY
	static void CmdWriteTimestamp(VkResult Result, VkCommandBuffer CommandBuffer, VkPipelineStageFlagBits PipelineStage, VkQueryPool QueryPool, uint32 Query) VULKAN_LAYER_BODY
	static void CmdCopyQueryPoolResults(VkResult Result, VkCommandBuffer CommandBuffer, VkQueryPool QueryPool, uint32 FirstQuery, uint32 QueryCount, VkBuffer DstBuffer, VkDeviceSize DstOffset, VkDeviceSize Stride, VkQueryResultFlags Flags) VULKAN_LAYER_BODY
	static void CmdPushConstants(VkResult Result, VkCommandBuffer CommandBuffer, VkPipelineLayout Layout, VkShaderStageFlags StageFlags, uint32_t Offset, uint32_t Size, const void* pValues) VULKAN_LAYER_BODY
	static void CmdPushDescriptorSetKHR(VkResult Result, VkCommandBuffer CommandBuffer, VkPipelineBindPoint PipelineBindPoint, VkPipelineLayout Layout, uint32_t Set, uint32_t DescriptorWriteCount, const VkWriteDescriptorSet* DescriptorWrites) VULKAN_LAYER_BODY
	static void CmdPushDescriptorSet2KHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkPushDescriptorSetInfo* PushDescriptorSetInfo) VULKAN_LAYER_BODY
	static void CmdBeginRenderPass(VkResult Result, VkCommandBuffer CommandBuffer, const VkRenderPassBeginInfo* RenderPassBegin, VkSubpassContents Contents) VULKAN_LAYER_BODY
	static void CmdBeginRenderPass2KHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkRenderPassBeginInfo* RenderPassBegin, const VkSubpassBeginInfo* pSubpassBeginInfo) VULKAN_LAYER_BODY
	static void CmdNextSubpass(VkResult Result, VkCommandBuffer CommandBuffer, VkSubpassContents Contents) VULKAN_LAYER_BODY
	static void CmdEndRenderPass(VkResult Result, VkCommandBuffer CommandBuffer) VULKAN_LAYER_BODY
	static void CmdEndRenderPass2KHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkSubpassEndInfo* SubpassEndInfo) VULKAN_LAYER_BODY
	static void CmdBeginRenderingKHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkRenderingInfo* RenderingInfo) VULKAN_LAYER_BODY
	static void CmdEndRenderingKHR(VkResult Result, VkCommandBuffer CommandBuffer) VULKAN_LAYER_BODY
	static void CmdEndRendering2KHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkRenderingEndInfoKHR* RenderingEndInfo) VULKAN_LAYER_BODY
	static void CmdExecuteCommands(VkResult Result, VkCommandBuffer CommandBuffer, uint32 CommandBufferCount, const VkCommandBuffer* pCommandBuffers) VULKAN_LAYER_BODY

	static void QueuePresent(VkResult Result, VkQueue Queue, const VkPresentInfoKHR* PresentInfo) VULKAN_LAYER_BODY
	static void GetSwapChainImagesKHR(VkResult Result, VkDevice Device, VkSwapchainKHR Swapchain, uint32_t* SwapchainImageCount, VkImage* SwapchainImages) VULKAN_LAYER_BODY
	static void CreateSwapchainKHR(VkResult Result, VkDevice Device, const VkSwapchainCreateInfoKHR* CreateInfo, VkSwapchainKHR* Swapchain) VULKAN_LAYER_BODY
	static void AcquireNextImageKHR(VkResult Result, VkDevice Device, VkSwapchainKHR Swapchain, uint64_t Timeout, VkSemaphore Semaphore, VkFence Fence, uint32_t* ImageIndex) VULKAN_LAYER_BODY
	static void DestroySurfaceKHR(VkResult Result, VkInstance Instance, VkSurfaceKHR SurfaceKHR) VULKAN_LAYER_BODY
	static void DestroySwapchainKHR(VkResult Result, VkDevice Device, VkSwapchainKHR Swapchain) VULKAN_LAYER_BODY
	static void GetImageMemoryRequirements2(VkResult Result, VkDevice Device, const VkImageMemoryRequirementsInfo2* Info, VkMemoryRequirements2* MemoryRequirements) VULKAN_LAYER_BODY
	static void GetBufferMemoryRequirements2(VkResult Result, VkDevice Device, const VkBufferMemoryRequirementsInfo2* Info, VkMemoryRequirements2* MemoryRequirements) VULKAN_LAYER_BODY

	VULKAN_EXTERN_EXPORT static void GetPhysicalDeviceMemoryProperties2(VkResult Result, VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceMemoryProperties2* MemoryProperties) VULKAN_LAYER_BODY 
	VULKAN_EXTERN_EXPORT static void GetPhysicalDeviceProperties2(VkResult Result, VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceProperties2* Properties) VULKAN_LAYER_BODY
	VULKAN_EXTERN_EXPORT static void GetPhysicalDeviceFeatures2(VkResult Result, VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceFeatures2* Features) VULKAN_LAYER_BODY
	VULKAN_EXTERN_EXPORT static void GetPhysicalDeviceFragmentShadingRatesKHR(VkResult, VkPhysicalDevice PhysicalDevice, uint32* FragmentShadingRateCount, VkPhysicalDeviceFragmentShadingRateKHR* FragmentShadingRates) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceSurfaceCapabilitiesKHR(VkResult Result, VkPhysicalDevice PhysicalDevice, VkSurfaceKHR Surface, VkSurfaceCapabilitiesKHR* SurfaceCapabilities) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceSurfaceFormatsKHR(VkResult Result, VkPhysicalDevice PhysicalDevice, VkSurfaceKHR Surface, uint32_t* SurfaceFormatCountPtr, VkSurfaceFormatKHR* SurfaceFormats) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceSurfaceSupportKHR(VkResult Result, VkPhysicalDevice PhysicalDevice, uint32_t QueueFamilyIndex, VkSurfaceKHR Surface, VkBool32* SupportedPtr) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceSurfacePresentModesKHR(VkResult Result, VkPhysicalDevice PhysicalDevice, VkSurfaceKHR Surface, uint32_t* PresentModeCountPtr, VkPresentModeKHR* PresentModesPtr) VULKAN_LAYER_BODY
#if VULKAN_USE_CREATE_ANDROID_SURFACE
	static void CreateAndroidSurfaceKHR(VkResult Result, VkInstance Instance, const VkAndroidSurfaceCreateInfoKHR* CreateInfo, VkSurfaceKHR* Surface) VULKAN_LAYER_BODY
#endif
#if VULKAN_USE_CREATE_WIN32_SURFACE
	static void CreateWin32SurfaceKHR(VkResult Result, VkInstance instance, const VkWin32SurfaceCreateInfoKHR* pCreateInfo, VkSurfaceKHR* pSurface);
#endif

	static void CreateAccelerationStructureKHR(VkResult Result, VkDevice Device, const VkAccelerationStructureCreateInfoKHR* CreateInfo, const VkAllocationCallbacks* Allocator, VkAccelerationStructureKHR* AccelerationStructure) VULKAN_LAYER_BODY
	static void DestroyAccelerationStructureKHR(VkResult Result, VkDevice Device, VkAccelerationStructureKHR AccelerationStructure, const VkAllocationCallbacks* Allocator) VULKAN_LAYER_BODY
	static void CmdBuildAccelerationStructuresKHR(VkResult Result, VkCommandBuffer CommandBuffer, uint32 InfoCount, const VkAccelerationStructureBuildGeometryInfoKHR* Infos, const VkAccelerationStructureBuildRangeInfoKHR* const* BuildRangeInfos) VULKAN_LAYER_BODY
	static void GetAccelerationStructureBuildSizesKHR(VkResult Result, VkDevice Device, VkAccelerationStructureBuildTypeKHR BuildType, const VkAccelerationStructureBuildGeometryInfoKHR* BuildInfo, const uint32* MaxPrimitiveCounts, VkAccelerationStructureBuildSizesInfoKHR* SizeInfo) VULKAN_LAYER_BODY
	static void GetAccelerationStructureDeviceAddressKHR(VkResult Result, VkDevice Device, const VkAccelerationStructureDeviceAddressInfoKHR* Info) VULKAN_LAYER_BODY
	static void CmdTraceRaysKHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkStridedDeviceAddressRegionKHR* RaygenShaderBindingTable, const VkStridedDeviceAddressRegionKHR* MissShaderBindingTable, const VkStridedDeviceAddressRegionKHR* HitShaderBindingTable, const VkStridedDeviceAddressRegionKHR* CallableShaderBindingTable, uint32 width, uint32 height, uint32 depth) VULKAN_LAYER_BODY
	static void CmdTraceRaysIndirectKHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkStridedDeviceAddressRegionKHR* RaygenShaderBindingTable, const VkStridedDeviceAddressRegionKHR* MissShaderBindingTable, const VkStridedDeviceAddressRegionKHR* HitShaderBindingTable, const VkStridedDeviceAddressRegionKHR* CallableShaderBindingTable, VkDeviceAddress IndirectDeviceAddress) VULKAN_LAYER_BODY
	static void CmdTraceRaysIndirect2KHR(VkResult Result, VkCommandBuffer CommandBuffer, VkDeviceAddress IndirectDeviceAddress) VULKAN_LAYER_BODY
	static void CreateRayTracingPipelinesKHR(VkResult Result, VkDevice Device, VkDeferredOperationKHR DeferredOperation, VkPipelineCache PipelineCache, uint32_t CreateInfoCount, const VkRayTracingPipelineCreateInfoKHR* CreateInfos, const VkAllocationCallbacks* Allocator, VkPipeline* Pipelines) VULKAN_LAYER_BODY
	static void GetRayTracingShaderGroupHandlesKHR(VkResult Result, VkDevice Device, VkPipeline Pipeline, uint32_t FirstGroup, uint32_t GroupCount, size_t DataSize, void* Data) VULKAN_LAYER_BODY
	static void CmdWriteAccelerationStructuresPropertiesKHR(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t AccelerationStructureCount, const VkAccelerationStructureKHR* AccelerationStructures, VkQueryType QueryType, VkQueryPool QueryPool, uint32_t FirstQuery) VULKAN_LAYER_BODY
	static void CmdCopyAccelerationStructureKHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkCopyAccelerationStructureInfoKHR* Info) VULKAN_LAYER_BODY

	static void GetBufferDeviceAddressKHR(VkResult Result, VkDevice Device, const VkBufferDeviceAddressInfo* Info) VULKAN_LAYER_BODY
	static void GetDeviceImageMemoryRequirementsKHR(VkResult Result, VkDevice Device, const VkDeviceImageMemoryRequirements* Info, VkMemoryRequirements2* MemoryRequirements) VULKAN_LAYER_BODY
	static void GetDeviceBufferMemoryRequirementsKHR(VkResult Result, VkDevice Device, const VkDeviceBufferMemoryRequirements* Info, VkMemoryRequirements2* MemoryRequirements) VULKAN_LAYER_BODY
	static void ResetQueryPoolEXT(VkResult Result, VkDevice Device, VkQueryPool QueryPool, uint32_t FirstQuery, uint32_t QueryCount) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceCalibrateableTimeDomainsEXT(VkResult Result, VkPhysicalDevice PhysicalDevice, uint32_t* TimeDomainCount, VkTimeDomainEXT* TimeDomains) VULKAN_LAYER_BODY
	static void GetCalibratedTimestampsEXT(VkResult Result, VkDevice Device, uint32_t TimestampCount, const VkCalibratedTimestampInfoEXT* TimestampInfos, uint64_t* Timestamps, uint64_t* MaxDeviation) VULKAN_LAYER_BODY
	static void BindBufferMemory2(VkResult Result, VkDevice Device, uint32_t BindInfoCount, const VkBindBufferMemoryInfo* BindInfos) VULKAN_LAYER_BODY
	static void BindImageMemory2(VkResult Result, VkDevice Device, uint32_t BindInfoCount, const VkBindImageMemoryInfo* BindInfos) VULKAN_LAYER_BODY
	static void CmdPipelineBarrier2KHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkDependencyInfo* DependencyInfo) VULKAN_LAYER_BODY
	static void CmdResetEvent2KHR(VkResult Result, VkCommandBuffer CommandBuffer, VkEvent Event, VkPipelineStageFlags2 StageMask) VULKAN_LAYER_BODY
	static void CmdSetEvent2KHR(VkResult Result, VkCommandBuffer CommandBuffer, VkEvent Event, const VkDependencyInfo* DependencyInfo) VULKAN_LAYER_BODY
	static void CmdWaitEvents2KHR(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t EventCount, const VkEvent* Events, const VkDependencyInfo* DependencyInfos) VULKAN_LAYER_BODY
	static void QueueSubmit2KHR(VkResult Result, VkQueue Queue, uint32_t SubmitCount, const VkSubmitInfo2* Submits, VkFence Fence) VULKAN_LAYER_BODY
	static void GetDescriptorSetLayoutSizeEXT(VkResult Result, VkDevice Device, VkDescriptorSetLayout Layout, VkDeviceSize* OutLayoutSizeInBytes) VULKAN_LAYER_BODY
	static void GetDescriptorSetLayoutBindingOffsetEXT(VkResult Result, VkDevice Device, VkDescriptorSetLayout Layout, uint32_t Binding, VkDeviceSize* Offset) VULKAN_LAYER_BODY
	static void CmdBindDescriptorBuffersEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t BufferCount, const VkDescriptorBufferBindingInfoEXT* BindingInfos) VULKAN_LAYER_BODY
	static void CmdSetDescriptorBufferOffsetsEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkPipelineBindPoint PipelineBindPoint, VkPipelineLayout Layout, uint32_t FirstSet, uint32_t SetCount, const uint32_t* BufferIndices, const VkDeviceSize* Offsets) VULKAN_LAYER_BODY
	static void CmdSetDescriptorBufferOffsets2EXT(VkResult Result, VkCommandBuffer CommandBuffer, const VkSetDescriptorBufferOffsetsInfoEXT* SetDescriptorBufferOffsetsInfo) VULKAN_LAYER_BODY
	static void GetDescriptorEXT(VkResult Result, VkDevice Device, const VkDescriptorGetInfoEXT* DescriptorInfo, size_t DataSize, void* Descriptor) VULKAN_LAYER_BODY
	static void CreateDeferredOperationKHR(VkResult Result, VkDevice Device, const VkAllocationCallbacks* Allocator, VkDeferredOperationKHR* DeferredOperation) VULKAN_LAYER_BODY
	static void DestroyDeferredOperationKHR(VkResult Result, VkDevice Device, VkDeferredOperationKHR DeferredOperation, const VkAllocationCallbacks* Allocator) VULKAN_LAYER_BODY
	static void DeferredOperationJoinKHR(VkResult Result, VkDevice Device, VkDeferredOperationKHR DeferredOperation) VULKAN_LAYER_BODY
	static void GetDeferredOperationMaxConcurrencyKHR(VkResult Result, VkDevice Device, VkDeferredOperationKHR DeferredOperation) VULKAN_LAYER_BODY
	static void GetDeferredOperationResultKHR(VkResult Result, VkDevice Device, VkDeferredOperationKHR DeferredOperation) VULKAN_LAYER_BODY
	static void GetDeviceFaultInfoEXT(VkResult Result, VkDevice Device, VkDeviceFaultCountsEXT* FaultCounts, VkDeviceFaultInfoEXT* FaultInfo) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceToolPropertiesEXT(VkResult Result, VkPhysicalDevice PhysicalDevice, uint32_t* ToolCount, VkPhysicalDeviceToolProperties* ToolProperties) VULKAN_LAYER_BODY
	static void CreateSamplerYcbcrConversion(VkResult Result, VkDevice Device, const VkSamplerYcbcrConversionCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkSamplerYcbcrConversion* YcbcrConversion) VULKAN_LAYER_BODY
	static void DestroySamplerYcbcrConversion(VkResult Result, VkDevice Device, VkSamplerYcbcrConversion YcbcrConversion, const VkAllocationCallbacks* Allocator) VULKAN_LAYER_BODY
#if PLATFORM_ANDROID
	static void GetAndroidHardwareBufferPropertiesANDROID(VkResult Result, VkDevice Device, const struct AHardwareBuffer* Buffer, VkAndroidHardwareBufferPropertiesANDROID* Properties) VULKAN_LAYER_BODY
#endif
	static void GetSemaphoreCounterValueKHR(VkResult Result, VkDevice Device, VkSemaphore Semaphore, uint64_t* Value) VULKAN_LAYER_BODY
	static void WaitSemaphoresKHR(VkResult Result, VkDevice Device, const VkSemaphoreWaitInfo* WaitInfo, uint64_t Timeout) VULKAN_LAYER_BODY
	static void SignalSemaphoreKHR(VkResult Result, VkDevice Device, const VkSemaphoreSignalInfo* SignalInfo) VULKAN_LAYER_BODY

	static void CreateShadersEXT(VkResult Result, VkDevice Device, uint32_t CreateInfoCount, const VkShaderCreateInfoEXT* CreateInfos, const VkAllocationCallbacks* Allocator, VkShaderEXT* Shaders) VULKAN_LAYER_BODY
	static void DestroyShaderEXT(VkResult Result, VkDevice Device, VkShaderEXT Shader, const VkAllocationCallbacks* Allocator) VULKAN_LAYER_BODY
	static void GetShaderBinaryDataEXT(VkResult Result, VkDevice Device, VkShaderEXT Shader, size_t* DataSize, void* Data) VULKAN_LAYER_BODY
	static void CmdBindShadersEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t StageCount, const VkShaderStageFlagBits* Stages, const VkShaderEXT* Shaders) VULKAN_LAYER_BODY
	static void CmdBindVertexBuffers2EXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t FirstBinding, uint32_t BindingCount, const VkBuffer* Buffers, const VkDeviceSize* Offsets, const VkDeviceSize* Sizes, const VkDeviceSize* Strides) VULKAN_LAYER_BODY
	static void CmdSetCullModeEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkCullModeFlags CullMode) VULKAN_LAYER_BODY
	static void CmdSetColorBlendEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t FirstAttachment, uint32_t AttachmentCount, const VkBool32* ColorBlendEnables) VULKAN_LAYER_BODY
	static void CmdSetColorBlendEquationEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t FirstAttachment, uint32_t AttachmentCount, const VkColorBlendEquationEXT* ColorBlendEquations) VULKAN_LAYER_BODY
	static void CmdSetColorWriteMaskEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t FirstAttachment, uint32_t AttachmentCount, const VkColorComponentFlags* ColorWriteMasks) VULKAN_LAYER_BODY
	static void CmdSetFrontFaceEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkFrontFace FrontFace) VULKAN_LAYER_BODY
	static void CmdSetPolygonModeEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkPolygonMode PolygonMode) VULKAN_LAYER_BODY
	static void CmdSetPrimitiveTopologyEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkPrimitiveTopology PrimitiveTopology) VULKAN_LAYER_BODY
	static void CmdSetRasterizationSamplesEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkSampleCountFlagBits RasterizationSamples) VULKAN_LAYER_BODY
	static void CmdSetScissorWithCountEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t ScissorCount, const VkRect2D* Scissors) VULKAN_LAYER_BODY
	static void CmdSetStencilOpEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkStencilFaceFlags FaceMask, VkStencilOp FailOp, VkStencilOp passOp, VkStencilOp depthFailOp, VkCompareOp compareOp) VULKAN_LAYER_BODY
	static void CmdSetStencilTestEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 StencilTestEnable) VULKAN_LAYER_BODY
	static void CmdSetVertexInputEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t VertexBindingDescriptionCount, const VkVertexInputBindingDescription2EXT* pVertexBindingDescriptions, uint32_t vertexAttributeDescriptionCount, const VkVertexInputAttributeDescription2EXT* pVertexAttributeDescriptions) VULKAN_LAYER_BODY
	static void CmdSetViewportWithCountEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t ViewportCount, const VkViewport* pViewports) VULKAN_LAYER_BODY
	static void CmdSetDepthBiasEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 DepthBiasEnable) VULKAN_LAYER_BODY
	static void CmdSetDepthBoundsTestEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 DepthBoundsTestEnable) VULKAN_LAYER_BODY
	static void CmdSetDepthClampEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 DepthClampEnable) VULKAN_LAYER_BODY
	static void CmdSetDepthCompareOpEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkCompareOp DepthCompareOp) VULKAN_LAYER_BODY
	static void CmdSetDepthTestEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 DepthTestEnable) VULKAN_LAYER_BODY
	static void CmdSetDepthWriteEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 DepthWriteEnable) VULKAN_LAYER_BODY
	static void CmdSetRasterizerDiscardEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 RasterizerDiscardEnable) VULKAN_LAYER_BODY
	static void CmdSetPrimitiveRestartEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 PrimitiveRestartEnable) VULKAN_LAYER_BODY
	static void CmdSetLogicOpEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 LogicOpEnable) VULKAN_LAYER_BODY
	static void CmdSetAlphaToCoverageEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 AlphaToCoverageEnable) VULKAN_LAYER_BODY
	static void CmdSetAlphaToOneEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 AlphaToOneEnable) VULKAN_LAYER_BODY
	static void CmdSetSampleMaskEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkSampleCountFlagBits Samples, const VkSampleMask* SampleMask) VULKAN_LAYER_BODY
	static void CmdSetFragmentShadingRateKHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkExtent2D* FragmentSize, const VkFragmentShadingRateCombinerOpKHR combinerOps[2]) VULKAN_LAYER_BODY

	static void CmdExecuteGeneratedCommandsEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 IsPreprocessed, const VkGeneratedCommandsInfoEXT* GeneratedCommandsInfo) VULKAN_LAYER_BODY
	static void CmdPreprocessGeneratedCommandsEXT(VkResult Result, VkCommandBuffer CommandBuffer, const VkGeneratedCommandsInfoEXT* GeneratedCommandsInfo, VkCommandBuffer StateCommandBuffer) VULKAN_LAYER_BODY
	static void CreateIndirectCommandsLayoutEXT(VkResult Result, VkDevice Device, const VkIndirectCommandsLayoutCreateInfoEXT* CreateInfo, const VkAllocationCallbacks* Allocator, VkIndirectCommandsLayoutEXT* IndirectCommandsLayout) VULKAN_LAYER_BODY
	static void CreateIndirectExecutionSetEXT(VkResult Result, VkDevice Device, const VkIndirectExecutionSetCreateInfoEXT* CreateInfo, const VkAllocationCallbacks* Allocator, VkIndirectExecutionSetEXT* IndirectExecutionSet) VULKAN_LAYER_BODY
	static void DestroyIndirectCommandsLayoutEXT(VkResult Result, VkDevice Device, VkIndirectCommandsLayoutEXT IndirectCommandsLayout, const VkAllocationCallbacks* Allocator) VULKAN_LAYER_BODY
	static void DestroyIndirectExecutionSetEXT(VkResult Result, VkDevice Device, VkIndirectExecutionSetEXT IndirectExecutionSet, const VkAllocationCallbacks* Allocator) VULKAN_LAYER_BODY
	static void GetGeneratedCommandsMemoryRequirementsEXT(VkResult Result, VkDevice Device, const VkGeneratedCommandsMemoryRequirementsInfoEXT* Info, VkMemoryRequirements2* MemoryRequirements) VULKAN_LAYER_BODY
	static void UpdateIndirectExecutionSetPipelineEXT(VkResult Result, VkDevice Device, VkIndirectExecutionSetEXT IndirectExecutionSet, uint32_t ExecutionSetWriteCount, const VkWriteIndirectExecutionSetPipelineEXT* ExecutionSetWrites) VULKAN_LAYER_BODY
	static void UpdateIndirectExecutionSetShaderEXT(VkResult Result, VkDevice Device, VkIndirectExecutionSetEXT IndirectExecutionSet, uint32_t ExecutionSetWriteCount, const VkWriteIndirectExecutionSetShaderEXT* ExecutionSetWrites) VULKAN_LAYER_BODY

	static void CopyMemoryToImageEXT(VkResult Result, VkDevice Device, const VkCopyMemoryToImageInfo* CopyMemoryToImageInfo) VULKAN_LAYER_BODY
	static void TransitionImageLayoutEXT(VkResult Result, VkDevice Device, uint32_t TransitionCount, const VkHostImageLayoutTransitionInfo* Transitions) VULKAN_LAYER_BODY

	static void GetPastPresentationTimingEXT(VkResult Result, VkDevice Device, const VkPastPresentationTimingInfoEXT* PastPresentationTimingInfo, VkPastPresentationTimingPropertiesEXT* PastPresentationTimingProperties) VULKAN_LAYER_BODY
	static void GetSwapchainTimeDomainPropertiesEXT(VkResult Result, VkDevice Device, VkSwapchainKHR Swapchain, VkSwapchainTimeDomainPropertiesEXT* SwapchainTimeDomainProperties, uint64_t* TimeDomainsCounter) VULKAN_LAYER_BODY
	static void GetSwapchainTimingPropertiesEXT(VkResult Result, VkDevice Device, VkSwapchainKHR Swapchain, VkSwapchainTimingPropertiesEXT* SwapchainTimingProperties, uint64_t* SwapchainTimingPropertiesCounter) VULKAN_LAYER_BODY
	static void SetSwapchainPresentTimingQueueSizeEXT(VkResult Result, VkDevice Device, VkSwapchainKHR Swapchain, uint32_t Size) VULKAN_LAYER_BODY

	static void CmdBindResourceHeapEXT(VkResult Result, VkCommandBuffer CommandBuffer, const VkBindHeapInfoEXT* BindInfo) VULKAN_LAYER_BODY
	static void CmdBindSamplerHeapEXT(VkResult Result, VkCommandBuffer CommandBuffer, const VkBindHeapInfoEXT* BindInfo) VULKAN_LAYER_BODY
	static void CmdPushDataEXT(VkResult Result, VkCommandBuffer CommandBuffer, const VkPushDataInfoEXT* PushDataInfo) VULKAN_LAYER_BODY
	static void GetPhysicalDeviceDescriptorSizeEXT(VkDeviceSize OutSize, VkPhysicalDevice PhysicalDevice, VkDescriptorType DescriptorType) VULKAN_LAYER_BODY
	static void WriteResourceDescriptorsEXT(VkResult Result, VkDevice Device, uint32_t ResourceCount, const VkResourceDescriptorInfoEXT* Resources, const VkHostAddressRangeEXT* Descriptors) VULKAN_LAYER_BODY
	static void WriteSamplerDescriptorsEXT(VkResult Result, VkDevice Device, uint32_t SamplerCount, const VkSamplerCreateInfo* Samplers, const VkHostAddressRangeEXT* Descriptors) VULKAN_LAYER_BODY

	static void CmdSetRenderingAttachmentLocationsKHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkRenderingAttachmentLocationInfo* LocationInfo) VULKAN_LAYER_BODY
	static void CmdSetRenderingInputAttachmentIndicesKHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkRenderingInputAttachmentIndexInfo* InputAttachmentIndexInfo) VULKAN_LAYER_BODY

	static void GetDescriptorSetLayoutSupport(VkResult Result, VkDevice Device, const VkDescriptorSetLayoutCreateInfo* CreateInfo, VkDescriptorSetLayoutSupport* OutSupport) VULKAN_LAYER_BODY
};


#undef VULKAN_LAYER_BODY

namespace VulkanRHI
{
	void FlushDebugWrapperLog();

	FORCEINLINE_DEBUGGABLE VkResult  vkCreateInstance(const VkInstanceCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkInstance* Instance)
	{
		FWrapLayer::CreateInstance(VK_RESULT_MAX_ENUM, CreateInfo, Instance);
		VkResult Result = VULKANAPINAMESPACE::vkCreateInstance(CreateInfo, Allocator, Instance);
		FWrapLayer::CreateInstance(Result, CreateInfo, Instance);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyInstance(VkInstance Instance, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyInstance(VK_RESULT_MAX_ENUM, Instance);
		VULKANAPINAMESPACE::vkDestroyInstance(Instance, Allocator);
		FWrapLayer::DestroyInstance(VK_SUCCESS, Instance);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkEnumeratePhysicalDevices(VkInstance Instance, uint32* PhysicalDeviceCount, VkPhysicalDevice* PhysicalDevices)
	{
		FWrapLayer::EnumeratePhysicalDevices(VK_RESULT_MAX_ENUM, Instance, PhysicalDeviceCount, PhysicalDevices);
		VkResult Result = VULKANAPINAMESPACE::vkEnumeratePhysicalDevices(Instance, PhysicalDeviceCount, PhysicalDevices);
		FWrapLayer::EnumeratePhysicalDevices(Result, Instance, PhysicalDeviceCount, PhysicalDevices);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkEnumerateInstanceVersion(uint32_t* ApiVersion)
	{
		FWrapLayer::EnumerateInstanceVersion(VK_RESULT_MAX_ENUM, ApiVersion);
		const VkResult Result = VULKANAPINAMESPACE::vkEnumerateInstanceVersion(ApiVersion);
		FWrapLayer::EnumerateInstanceVersion(Result, ApiVersion);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetPhysicalDeviceFeatures(VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceFeatures* Features)
	{
		FWrapLayer::GetPhysicalDeviceFeatures(VK_RESULT_MAX_ENUM, PhysicalDevice, Features);
		VULKANAPINAMESPACE::vkGetPhysicalDeviceFeatures(PhysicalDevice, Features);
		FWrapLayer::GetPhysicalDeviceFeatures(VK_SUCCESS, PhysicalDevice, Features);
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetPhysicalDeviceFormatProperties(VkPhysicalDevice PhysicalDevice, VkFormat Format, VkFormatProperties* FormatProperties)
	{
		FWrapLayer::GetPhysicalDeviceFormatProperties(VK_RESULT_MAX_ENUM, PhysicalDevice, Format, FormatProperties);
		VULKANAPINAMESPACE::vkGetPhysicalDeviceFormatProperties(PhysicalDevice, Format, FormatProperties);
		FWrapLayer::GetPhysicalDeviceFormatProperties(VK_SUCCESS, PhysicalDevice, Format, FormatProperties);
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetPhysicalDeviceFormatProperties2(VkPhysicalDevice PhysicalDevice, VkFormat Format, VkFormatProperties2* FormatProperties)
	{
		FWrapLayer::GetPhysicalDeviceFormatProperties2(VK_RESULT_MAX_ENUM, PhysicalDevice, Format, FormatProperties);
		VULKANAPINAMESPACE::vkGetPhysicalDeviceFormatProperties2(PhysicalDevice, Format, FormatProperties);
		FWrapLayer::GetPhysicalDeviceFormatProperties2(VK_SUCCESS, PhysicalDevice, Format, FormatProperties);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkGetPhysicalDeviceImageFormatProperties(VkPhysicalDevice PhysicalDevice, VkFormat Format, VkImageType Type, VkImageTiling Tiling, VkImageUsageFlags Usage, VkImageCreateFlags Flags, VkImageFormatProperties* pImageFormatProperties)
	{
		FWrapLayer::GetPhysicalDeviceImageFormatProperties(VK_RESULT_MAX_ENUM, PhysicalDevice, Format, Type, Tiling, Usage, Flags, pImageFormatProperties);
		VkResult Result = VULKANAPINAMESPACE::vkGetPhysicalDeviceImageFormatProperties(PhysicalDevice, Format, Type, Tiling, Usage, Flags, pImageFormatProperties);
		FWrapLayer::GetPhysicalDeviceImageFormatProperties(Result, PhysicalDevice, Format, Type, Tiling, Usage, Flags, pImageFormatProperties);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetPhysicalDeviceImageFormatProperties2(VkPhysicalDevice PhysicalDevice, const VkPhysicalDeviceImageFormatInfo2* FormatInfo, VkImageFormatProperties2* FormatProperties)
	{
		FWrapLayer::GetPhysicalDeviceImageFormatProperties2(VK_RESULT_MAX_ENUM, PhysicalDevice, FormatInfo, FormatProperties);
		VkResult Result = VULKANAPINAMESPACE::vkGetPhysicalDeviceImageFormatProperties2(PhysicalDevice, FormatInfo, FormatProperties);
		FWrapLayer::GetPhysicalDeviceImageFormatProperties2(Result, PhysicalDevice, FormatInfo, FormatProperties);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetPhysicalDeviceProperties(VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceProperties* Properties)
	{
		FWrapLayer::GetPhysicalDeviceProperties(VK_RESULT_MAX_ENUM, PhysicalDevice, Properties);
		VULKANAPINAMESPACE::vkGetPhysicalDeviceProperties(PhysicalDevice, Properties);
		FWrapLayer::GetPhysicalDeviceProperties(VK_SUCCESS, PhysicalDevice, Properties);
	}

	static FORCEINLINE_DEBUGGABLE void vkGetPhysicalDeviceMemoryProperties2(VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceMemoryProperties2* pMemoryProperties)
	{
		FWrapLayer::GetPhysicalDeviceMemoryProperties2(VK_RESULT_MAX_ENUM, PhysicalDevice, pMemoryProperties);
		VULKANAPINAMESPACE::vkGetPhysicalDeviceMemoryProperties2(PhysicalDevice, pMemoryProperties);
		FWrapLayer::GetPhysicalDeviceMemoryProperties2(VK_SUCCESS, PhysicalDevice, pMemoryProperties);
	}

	static FORCEINLINE_DEBUGGABLE void vkGetPhysicalDeviceProperties2(VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceProperties2* Properties)
	{
		FWrapLayer::GetPhysicalDeviceProperties2(VK_RESULT_MAX_ENUM, PhysicalDevice, Properties);
		VULKANAPINAMESPACE::vkGetPhysicalDeviceProperties2(PhysicalDevice, Properties);
		FWrapLayer::GetPhysicalDeviceProperties2(VK_SUCCESS, PhysicalDevice, Properties);
	}

	static FORCEINLINE_DEBUGGABLE void vkGetPhysicalDeviceFeatures2(VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceFeatures2* Features)
	{
		FWrapLayer::GetPhysicalDeviceFeatures2(VK_RESULT_MAX_ENUM, PhysicalDevice, Features);
		VULKANAPINAMESPACE::vkGetPhysicalDeviceFeatures2(PhysicalDevice, Features);
		FWrapLayer::GetPhysicalDeviceFeatures2(VK_SUCCESS, PhysicalDevice, Features);
	}

	static FORCEINLINE_DEBUGGABLE void vkGetPhysicalDeviceFragmentShadingRatesKHR(VkPhysicalDevice PhysicalDevice, uint32* FragmentShadingRateCount, VkPhysicalDeviceFragmentShadingRateKHR* FragmentShadingRates)
	{
		FWrapLayer::GetPhysicalDeviceFragmentShadingRatesKHR(VK_RESULT_MAX_ENUM, PhysicalDevice, FragmentShadingRateCount, FragmentShadingRates);
		VULKANAPINAMESPACE::vkGetPhysicalDeviceFragmentShadingRatesKHR(PhysicalDevice, FragmentShadingRateCount, FragmentShadingRates);
		FWrapLayer::GetPhysicalDeviceFragmentShadingRatesKHR(VK_SUCCESS, PhysicalDevice, FragmentShadingRateCount, FragmentShadingRates);
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetPhysicalDeviceQueueFamilyProperties(VkPhysicalDevice PhysicalDevice, uint32* QueueFamilyPropertyCount, VkQueueFamilyProperties* QueueFamilyProperties)
	{
		FWrapLayer::GetPhysicalDeviceQueueFamilyProperties(VK_RESULT_MAX_ENUM, PhysicalDevice, QueueFamilyPropertyCount, QueueFamilyProperties);
		VULKANAPINAMESPACE::vkGetPhysicalDeviceQueueFamilyProperties(PhysicalDevice, QueueFamilyPropertyCount, QueueFamilyProperties);
		FWrapLayer::GetPhysicalDeviceQueueFamilyProperties(VK_SUCCESS, PhysicalDevice, QueueFamilyPropertyCount, QueueFamilyProperties);
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetPhysicalDeviceQueueFamilyProperties2(VkPhysicalDevice PhysicalDevice, uint32* QueueFamilyPropertyCount, VkQueueFamilyProperties2* QueueFamilyProperties)
	{
		FWrapLayer::GetPhysicalDeviceQueueFamilyProperties2(VK_RESULT_MAX_ENUM, PhysicalDevice, QueueFamilyPropertyCount, QueueFamilyProperties);
		VULKANAPINAMESPACE::vkGetPhysicalDeviceQueueFamilyProperties2(PhysicalDevice, QueueFamilyPropertyCount, QueueFamilyProperties);
		FWrapLayer::GetPhysicalDeviceQueueFamilyProperties2(VK_SUCCESS, PhysicalDevice, QueueFamilyPropertyCount, QueueFamilyProperties);
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetPhysicalDeviceMemoryProperties(VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceMemoryProperties* MemoryProperties)
	{
		FWrapLayer::GetPhysicalDeviceMemoryProperties(VK_RESULT_MAX_ENUM, PhysicalDevice, MemoryProperties);
		VULKANAPINAMESPACE::vkGetPhysicalDeviceMemoryProperties(PhysicalDevice, MemoryProperties);
		FWrapLayer::GetPhysicalDeviceMemoryProperties(VK_SUCCESS, PhysicalDevice, MemoryProperties);
	}

	static FORCEINLINE_DEBUGGABLE PFN_vkVoidFunction  vkGetInstanceProcAddr(VkInstance Instance, const char* Name)
	{
		FWrapLayer::GetInstanceProcAddr(VK_RESULT_MAX_ENUM, Instance, Name, nullptr);
		PFN_vkVoidFunction Function = VULKANAPINAMESPACE::vkGetInstanceProcAddr(Instance, Name);
		FWrapLayer::GetInstanceProcAddr(VK_SUCCESS, Instance, Name, Function);
		return Function;
	}

	static FORCEINLINE_DEBUGGABLE PFN_vkVoidFunction  vkGetDeviceProcAddr(VkDevice Device, const char* Name)
	{
		FWrapLayer::GetDeviceProcAddr(VK_RESULT_MAX_ENUM, Device, Name, nullptr);
		PFN_vkVoidFunction Function = VULKANAPINAMESPACE::vkGetDeviceProcAddr(Device, Name);
		FWrapLayer::GetDeviceProcAddr(VK_SUCCESS, Device, Name, Function);
		return Function;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateDevice(VkPhysicalDevice PhysicalDevice, const VkDeviceCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkDevice* Device)
	{
		FWrapLayer::CreateDevice(VK_RESULT_MAX_ENUM, PhysicalDevice, CreateInfo, Device);
		VkResult Result = VULKANAPINAMESPACE::vkCreateDevice(PhysicalDevice, CreateInfo, Allocator, Device);
		FWrapLayer::CreateDevice(Result, PhysicalDevice, CreateInfo, Device);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyDevice(VkDevice Device, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyDevice(VK_RESULT_MAX_ENUM, Device);
		VULKANAPINAMESPACE::vkDestroyDevice(Device, Allocator);
		FWrapLayer::DestroyDevice(VK_SUCCESS, Device);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkEnumerateInstanceExtensionProperties(const char* LayerName, uint32* PropertyCount, VkExtensionProperties* Properties)
	{
		FWrapLayer::EnumerateInstanceExtensionProperties(VK_RESULT_MAX_ENUM, LayerName, PropertyCount, Properties);
		VkResult Result = VULKANAPINAMESPACE::vkEnumerateInstanceExtensionProperties(LayerName, PropertyCount, Properties);
		//FWrapLayer::EnumerateInstanceExtensionProperties(VK_SUCCESS, LayerName, PropertyCount, Properties);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkEnumerateDeviceExtensionProperties(VkPhysicalDevice PhysicalDevice, const char* LayerName, uint32* PropertyCount, VkExtensionProperties* Properties)
	{
		FWrapLayer::EnumerateDeviceExtensionProperties(VK_RESULT_MAX_ENUM, PhysicalDevice, LayerName, PropertyCount, Properties);
		VkResult Result = VULKANAPINAMESPACE::vkEnumerateDeviceExtensionProperties(PhysicalDevice, LayerName, PropertyCount, Properties);
		FWrapLayer::EnumerateDeviceExtensionProperties(VK_SUCCESS, PhysicalDevice, LayerName, PropertyCount, Properties);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkEnumerateInstanceLayerProperties(uint32* PropertyCount, VkLayerProperties* Properties)
	{
		FWrapLayer::EnumerateInstanceLayerProperties(VK_RESULT_MAX_ENUM, PropertyCount, Properties);
		VkResult Result = VULKANAPINAMESPACE::vkEnumerateInstanceLayerProperties(PropertyCount, Properties);
		FWrapLayer::EnumerateInstanceLayerProperties(VK_SUCCESS, PropertyCount, Properties);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkEnumerateDeviceLayerProperties(VkPhysicalDevice PhysicalDevice, uint32* PropertyCount, VkLayerProperties* Properties)
	{
		FWrapLayer::EnumerateDeviceLayerProperties(VK_RESULT_MAX_ENUM, PhysicalDevice, PropertyCount, Properties);
		VkResult Result = VULKANAPINAMESPACE::vkEnumerateDeviceLayerProperties(PhysicalDevice, PropertyCount, Properties);
		FWrapLayer::EnumerateDeviceLayerProperties(VK_SUCCESS, PhysicalDevice, PropertyCount, Properties);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetDeviceQueue(VkDevice Device, uint32 QueueFamilyIndex, uint32 QueueIndex, VkQueue* Queue)
	{
		FWrapLayer::GetDeviceQueue(VK_RESULT_MAX_ENUM, Device, QueueFamilyIndex, QueueIndex, Queue);
		VULKANAPINAMESPACE::vkGetDeviceQueue(Device, QueueFamilyIndex, QueueIndex, Queue);
		FWrapLayer::GetDeviceQueue(VK_SUCCESS, Device, QueueFamilyIndex, QueueIndex, Queue);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkQueueSubmit(VkQueue Queue, uint32 SubmitCount, const VkSubmitInfo* Submits, VkFence Fence)
	{
		FWrapLayer::QueueSubmit(VK_RESULT_MAX_ENUM, Queue, SubmitCount, Submits, Fence);
		VkResult Result = VULKANAPINAMESPACE::vkQueueSubmit(Queue, SubmitCount, Submits, Fence);
		FWrapLayer::QueueSubmit(Result, Queue, SubmitCount, Submits, Fence);
		return Result;
	}
	static FORCEINLINE_DEBUGGABLE VkResult  vkQueueWaitIdle(VkQueue Queue)
	{
		FWrapLayer::QueueWaitIdle(VK_RESULT_MAX_ENUM, Queue);
		VkResult Result = VULKANAPINAMESPACE::vkQueueWaitIdle(Queue);
		FWrapLayer::QueueWaitIdle(Result, Queue);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkDeviceWaitIdle(VkDevice Device)
	{
		FWrapLayer::DeviceWaitIdle(VK_RESULT_MAX_ENUM, Device);
		VkResult Result = VULKANAPINAMESPACE::vkDeviceWaitIdle(Device);
		FWrapLayer::DeviceWaitIdle(VK_SUCCESS, Device);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkAllocateMemory(VkDevice Device, const VkMemoryAllocateInfo* AllocateInfo, const VkAllocationCallbacks* Allocator, VkDeviceMemory* Memory)
	{
		FWrapLayer::AllocateMemory(VK_RESULT_MAX_ENUM, Device, AllocateInfo, Memory);
		VkResult Result = VULKANAPINAMESPACE::vkAllocateMemory(Device, AllocateInfo, Allocator, Memory);
		FWrapLayer::AllocateMemory(Result, Device, AllocateInfo, Memory);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkFreeMemory(VkDevice Device, VkDeviceMemory Memory, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::FreeMemory(VK_RESULT_MAX_ENUM, Device, Memory);
		VULKANAPINAMESPACE::vkFreeMemory(Device, Memory, Allocator);
		FWrapLayer::FreeMemory(VK_SUCCESS, Device, Memory);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkMapMemory(VkDevice Device, VkDeviceMemory Memory, VkDeviceSize Offset, VkDeviceSize Size, VkMemoryMapFlags Flags, void** Data)
	{
		FWrapLayer::MapMemory(VK_RESULT_MAX_ENUM, Device, Memory, Offset, Size, Flags, Data);
		VkResult Result = VULKANAPINAMESPACE::vkMapMemory(Device, Memory, Offset, Size, Flags, Data);
		FWrapLayer::MapMemory(Result, Device, Memory, Offset, Size, Flags, Data);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkUnmapMemory(VkDevice Device, VkDeviceMemory Memory)
	{
		FWrapLayer::UnmapMemory(VK_RESULT_MAX_ENUM, Device, Memory);
		VULKANAPINAMESPACE::vkUnmapMemory(Device, Memory);
		FWrapLayer::UnmapMemory(VK_SUCCESS, Device, Memory);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkFlushMappedMemoryRanges(VkDevice Device, uint32 MemoryRangeCount, const VkMappedMemoryRange* MemoryRanges)
	{
		FWrapLayer::FlushMappedMemoryRanges(VK_RESULT_MAX_ENUM, Device, MemoryRangeCount, MemoryRanges);
		VkResult Result = VULKANAPINAMESPACE::vkFlushMappedMemoryRanges(Device, MemoryRangeCount, MemoryRanges);
		FWrapLayer::FlushMappedMemoryRanges(Result, Device, MemoryRangeCount, MemoryRanges);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkInvalidateMappedMemoryRanges(VkDevice Device, uint32 MemoryRangeCount, const VkMappedMemoryRange* MemoryRanges)
	{
		FWrapLayer::InvalidateMappedMemoryRanges(VK_RESULT_MAX_ENUM, Device, MemoryRangeCount, MemoryRanges);
		VkResult Result = VULKANAPINAMESPACE::vkInvalidateMappedMemoryRanges(Device, MemoryRangeCount, MemoryRanges);
		FWrapLayer::InvalidateMappedMemoryRanges(Result, Device, MemoryRangeCount, MemoryRanges);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetDeviceMemoryCommitment(VkDevice Device, VkDeviceMemory Memory, VkDeviceSize* pCommittedMemoryInBytes)
	{
		FWrapLayer::GetDeviceMemoryCommitment(VK_RESULT_MAX_ENUM, Device, Memory, pCommittedMemoryInBytes);
		VULKANAPINAMESPACE::vkGetDeviceMemoryCommitment(Device, Memory, pCommittedMemoryInBytes);
		FWrapLayer::GetDeviceMemoryCommitment(VK_SUCCESS, Device, Memory, pCommittedMemoryInBytes);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkBindBufferMemory(VkDevice Device, VkBuffer Buffer, VkDeviceMemory Memory, VkDeviceSize MemoryOffset)
	{
		FWrapLayer::BindBufferMemory(VK_RESULT_MAX_ENUM, Device, Buffer, Memory, MemoryOffset);
		VkResult Result = VULKANAPINAMESPACE::vkBindBufferMemory(Device, Buffer, Memory, MemoryOffset);
		FWrapLayer::BindBufferMemory(Result, Device, Buffer, Memory, MemoryOffset);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkBindImageMemory(VkDevice Device, VkImage Image, VkDeviceMemory Memory, VkDeviceSize MemoryOffset)
	{
		FWrapLayer::BindImageMemory(VK_RESULT_MAX_ENUM, Device, Image, Memory, MemoryOffset);
		VkResult Result = VULKANAPINAMESPACE::vkBindImageMemory(Device, Image, Memory, MemoryOffset);
		FWrapLayer::BindImageMemory(Result, Device, Image, Memory, MemoryOffset);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetBufferMemoryRequirements(VkDevice Device, VkBuffer Buffer, VkMemoryRequirements* MemoryRequirements)
	{
		FWrapLayer::GetBufferMemoryRequirements(VK_RESULT_MAX_ENUM, Device, Buffer, MemoryRequirements);
		VULKANAPINAMESPACE::vkGetBufferMemoryRequirements(Device, Buffer, MemoryRequirements);
		FWrapLayer::GetBufferMemoryRequirements(VK_SUCCESS, Device, Buffer, MemoryRequirements);
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetImageMemoryRequirements(VkDevice Device, VkImage Image, VkMemoryRequirements* MemoryRequirements)
	{
		FWrapLayer::GetImageMemoryRequirements(VK_RESULT_MAX_ENUM, Device, Image, MemoryRequirements);
		VULKANAPINAMESPACE::vkGetImageMemoryRequirements(Device, Image, MemoryRequirements);
		FWrapLayer::GetImageMemoryRequirements(VK_SUCCESS, Device, Image, MemoryRequirements);
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetImageSparseMemoryRequirements(VkDevice Device, VkImage Image, uint32* pSparseMemoryRequirementCount, VkSparseImageMemoryRequirements* pSparseMemoryRequirements)
	{
		FWrapLayer::GetImageSparseMemoryRequirements(VK_RESULT_MAX_ENUM, Device, Image, pSparseMemoryRequirementCount, pSparseMemoryRequirements);
		VULKANAPINAMESPACE::vkGetImageSparseMemoryRequirements(Device, Image, pSparseMemoryRequirementCount, pSparseMemoryRequirements);
		FWrapLayer::GetImageSparseMemoryRequirements(VK_SUCCESS, Device, Image, pSparseMemoryRequirementCount, pSparseMemoryRequirements);
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetPhysicalDeviceSparseImageFormatProperties(VkPhysicalDevice PhysicalDevice, VkFormat Format, VkImageType Type, VkSampleCountFlagBits Samples, VkImageUsageFlags Usage, VkImageTiling Tiling, uint32* pPropertyCount, VkSparseImageFormatProperties* pProperties)
	{
		FWrapLayer::GetPhysicalDeviceSparseImageFormatProperties(VK_RESULT_MAX_ENUM, PhysicalDevice, Format, Type, Samples, Usage, Tiling, pPropertyCount, pProperties);
		VULKANAPINAMESPACE::vkGetPhysicalDeviceSparseImageFormatProperties(PhysicalDevice, Format, Type, Samples, Usage, Tiling, pPropertyCount, pProperties);
		FWrapLayer::GetPhysicalDeviceSparseImageFormatProperties(VK_SUCCESS, PhysicalDevice, Format, Type, Samples, Usage, Tiling, pPropertyCount, pProperties);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkQueueBindSparse(VkQueue Queue, uint32 BindInfoCount, const VkBindSparseInfo* pBindInfo, VkFence Fence)
	{
		FWrapLayer::QueueBindSparse(VK_RESULT_MAX_ENUM, Queue, BindInfoCount, pBindInfo, Fence);
		VkResult Result = VULKANAPINAMESPACE::vkQueueBindSparse(Queue, BindInfoCount, pBindInfo, Fence);
		FWrapLayer::QueueBindSparse(Result, Queue, BindInfoCount, pBindInfo, Fence);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateFence(VkDevice Device, const VkFenceCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkFence* Fence)
	{
		FWrapLayer::CreateFence(VK_RESULT_MAX_ENUM, Device, CreateInfo, Fence);
		VkResult Result = VULKANAPINAMESPACE::vkCreateFence(Device, CreateInfo, Allocator, Fence);
		FWrapLayer::CreateFence(Result, Device, CreateInfo, Fence);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyFence(VkDevice Device, VkFence Fence, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyFence(VK_RESULT_MAX_ENUM, Device, Fence);
		VULKANAPINAMESPACE::vkDestroyFence(Device, Fence, Allocator);
		FWrapLayer::DestroyFence(VK_SUCCESS, Device, Fence);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkResetFences(VkDevice Device, uint32 FenceCount, const VkFence* Fences)
	{
		FWrapLayer::ResetFences(VK_RESULT_MAX_ENUM, Device, FenceCount, Fences);
		VkResult Result = VULKANAPINAMESPACE::vkResetFences(Device, FenceCount, Fences);
		FWrapLayer::ResetFences(Result, Device, FenceCount, Fences);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkGetFenceStatus(VkDevice Device, VkFence Fence)
	{
		FWrapLayer::GetFenceStatus(VK_RESULT_MAX_ENUM, Device, Fence);
		VkResult Result = VULKANAPINAMESPACE::vkGetFenceStatus(Device, Fence);
		FWrapLayer::GetFenceStatus(Result, Device, Fence);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkWaitForFences(VkDevice Device, uint32 FenceCount, const VkFence* Fences, VkBool32 bWaitAll, uint64_t Timeout)
	{
		FWrapLayer::WaitForFences(VK_RESULT_MAX_ENUM, Device, FenceCount, Fences, bWaitAll, Timeout);
		VkResult Result = VULKANAPINAMESPACE::vkWaitForFences(Device, FenceCount, Fences, bWaitAll, Timeout);
		FWrapLayer::WaitForFences(Result, Device, FenceCount, Fences, bWaitAll, Timeout);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateSemaphore(VkDevice Device, const VkSemaphoreCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkSemaphore* Semaphore)
	{
		FWrapLayer::CreateSemaphore(VK_RESULT_MAX_ENUM, Device, CreateInfo, Semaphore);
		VkResult Result = VULKANAPINAMESPACE::vkCreateSemaphore(Device, CreateInfo, Allocator, Semaphore);
		FWrapLayer::CreateSemaphore(Result, Device, CreateInfo, Semaphore);
		return Result;
	}


	static FORCEINLINE_DEBUGGABLE void  vkDestroySemaphore(VkDevice Device, VkSemaphore Semaphore, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroySemaphore(VK_RESULT_MAX_ENUM, Device, Semaphore);
		VULKANAPINAMESPACE::vkDestroySemaphore(Device, Semaphore, Allocator);
		FWrapLayer::DestroySemaphore(VK_SUCCESS, Device, Semaphore);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateEvent(VkDevice Device, const VkEventCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkEvent* Event)
	{
		FWrapLayer::CreateEvent(VK_RESULT_MAX_ENUM, Device, CreateInfo, Event);
		VkResult Result = VULKANAPINAMESPACE::vkCreateEvent(Device, CreateInfo, Allocator, Event);
		FWrapLayer::CreateEvent(Result, Device, CreateInfo, Event);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyEvent(VkDevice Device, VkEvent Event, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyEvent(VK_RESULT_MAX_ENUM, Device, Event);
		VULKANAPINAMESPACE::vkDestroyEvent(Device, Event, Allocator);
		FWrapLayer::DestroyEvent(VK_SUCCESS, Device, Event);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkGetEventStatus(VkDevice Device, VkEvent Event)
	{
		FWrapLayer::GetEventStatus(VK_RESULT_MAX_ENUM, Device, Event);
		VkResult Result = VULKANAPINAMESPACE::vkGetEventStatus(Device, Event);
		FWrapLayer::GetEventStatus(Result, Device, Event);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkSetEvent(VkDevice Device, VkEvent Event)
	{
		FWrapLayer::SetEvent(VK_RESULT_MAX_ENUM, Device, Event);
		VkResult Result = VULKANAPINAMESPACE::vkSetEvent(Device, Event);
		FWrapLayer::SetEvent(Result, Device, Event);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkResetEvent(VkDevice Device, VkEvent Event)
	{
		FWrapLayer::ResetEvent(VK_RESULT_MAX_ENUM, Device, Event);
		VkResult Result = VULKANAPINAMESPACE::vkResetEvent(Device, Event);
		FWrapLayer::ResetEvent(Result, Device, Event);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateQueryPool(VkDevice Device, const VkQueryPoolCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkQueryPool* QueryPool)
	{
		FWrapLayer::CreateQueryPool(VK_RESULT_MAX_ENUM, Device, CreateInfo, QueryPool);
		VkResult Result = VULKANAPINAMESPACE::vkCreateQueryPool(Device, CreateInfo, Allocator, QueryPool);
		FWrapLayer::CreateQueryPool(Result, Device, CreateInfo, QueryPool);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyQueryPool(VkDevice Device, VkQueryPool QueryPool, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyQueryPool(VK_RESULT_MAX_ENUM, Device, QueryPool);
		VULKANAPINAMESPACE::vkDestroyQueryPool(Device, QueryPool, Allocator);
		FWrapLayer::DestroyQueryPool(VK_SUCCESS, Device, QueryPool);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkGetQueryPoolResults(VkDevice Device, VkQueryPool QueryPool,
		uint32 FirstQuery, uint32 QueryCount, size_t DataSize, void* Data, VkDeviceSize Stride, VkQueryResultFlags Flags)
	{
		FWrapLayer::GetQueryPoolResults(VK_RESULT_MAX_ENUM, Device, QueryPool, FirstQuery, QueryCount, DataSize, Data, Stride, Flags);
		VkResult Result = VULKANAPINAMESPACE::vkGetQueryPoolResults(Device, QueryPool, FirstQuery, QueryCount, DataSize, Data, Stride, Flags);
		FWrapLayer::GetQueryPoolResults(Result, Device, QueryPool, FirstQuery, QueryCount, DataSize, Data, Stride, Flags);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateBuffer(VkDevice Device, const VkBufferCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkBuffer* Buffer)
	{
		FWrapLayer::CreateBuffer(VK_RESULT_MAX_ENUM, Device, CreateInfo, Buffer);
		VkResult Result = VULKANAPINAMESPACE::vkCreateBuffer(Device, CreateInfo, Allocator, Buffer);
		FWrapLayer::CreateBuffer(Result, Device, CreateInfo, Buffer);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyBuffer(VkDevice Device, VkBuffer Buffer, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyBuffer(VK_RESULT_MAX_ENUM, Device, Buffer);
		VULKANAPINAMESPACE::vkDestroyBuffer(Device, Buffer, Allocator);
		FWrapLayer::DestroyBuffer(VK_SUCCESS, Device, Buffer);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateBufferView(VkDevice Device, const VkBufferViewCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkBufferView* View)
	{
		FWrapLayer::CreateBufferView(VK_RESULT_MAX_ENUM, Device, CreateInfo, View);
		VkResult Result = VULKANAPINAMESPACE::vkCreateBufferView(Device, CreateInfo, Allocator, View);
		FWrapLayer::CreateBufferView(Result, Device, CreateInfo, View);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyBufferView(VkDevice Device, VkBufferView BufferView, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyBufferView(VK_RESULT_MAX_ENUM, Device, BufferView);
		VULKANAPINAMESPACE::vkDestroyBufferView(Device, BufferView, Allocator);
		FWrapLayer::DestroyBufferView(VK_RESULT_MAX_ENUM, Device, BufferView);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateImage(VkDevice Device, const VkImageCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkImage* Image)
	{
		FWrapLayer::CreateImage(VK_RESULT_MAX_ENUM, Device, CreateInfo, Image);
		VkResult Result = VULKANAPINAMESPACE::vkCreateImage(Device, CreateInfo, Allocator, Image);
		FWrapLayer::CreateImage(Result, Device, CreateInfo, Image);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyImage(VkDevice Device, VkImage Image, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyImage(VK_RESULT_MAX_ENUM, Device, Image);
		VULKANAPINAMESPACE::vkDestroyImage(Device, Image, Allocator);
		FWrapLayer::DestroyImage(VK_SUCCESS, Device, Image);
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetImageSubresourceLayout(VkDevice Device, VkImage Image, const VkImageSubresource* Subresource, VkSubresourceLayout* Layout)
	{
		FWrapLayer::GetImageSubresourceLayout(VK_RESULT_MAX_ENUM, Device, Image, Subresource, Layout);
		VULKANAPINAMESPACE::vkGetImageSubresourceLayout(Device, Image, Subresource, Layout);
		FWrapLayer::GetImageSubresourceLayout(VK_SUCCESS, Device, Image, Subresource, Layout);
	}

	static FORCEINLINE_DEBUGGABLE void	vkGetImageSubresourceLayout2KHR(VkDevice Device, VkImage Image, const VkImageSubresource2KHR* Subresource, VkSubresourceLayout2KHR* Layout)
	{
		FWrapLayer::GetImageSubresourceLayout2(VK_RESULT_MAX_ENUM, Device, Image, Subresource, Layout);
		VULKANAPINAMESPACE::vkGetImageSubresourceLayout2KHR(Device, Image, Subresource, Layout);
		FWrapLayer::GetImageSubresourceLayout2(VK_SUCCESS, Device, Image, Subresource, Layout);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateImageView(VkDevice Device, const VkImageViewCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkImageView* View)
	{
		FWrapLayer::CreateImageView(VK_RESULT_MAX_ENUM, Device, CreateInfo, View);
		VkResult Result = VULKANAPINAMESPACE::vkCreateImageView(Device, CreateInfo, Allocator, View);
		FWrapLayer::CreateImageView(Result, Device, CreateInfo, View);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyImageView(VkDevice Device, VkImageView ImageView, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyImageView(VK_RESULT_MAX_ENUM, Device, ImageView);
		VULKANAPINAMESPACE::vkDestroyImageView(Device, ImageView, Allocator);
		FWrapLayer::DestroyImageView(VK_SUCCESS, Device, ImageView);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateShaderModule(VkDevice Device, const VkShaderModuleCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkShaderModule* ShaderModule)
	{
		FWrapLayer::CreateShaderModule(VK_RESULT_MAX_ENUM, Device, CreateInfo, ShaderModule);
		VkResult Result = VULKANAPINAMESPACE::vkCreateShaderModule(Device, CreateInfo, Allocator, ShaderModule);
		FWrapLayer::CreateShaderModule(Result, Device, CreateInfo, ShaderModule);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyShaderModule(VkDevice Device, VkShaderModule ShaderModule, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyShaderModule(VK_RESULT_MAX_ENUM, Device, ShaderModule);
		VULKANAPINAMESPACE::vkDestroyShaderModule(Device, ShaderModule, Allocator);
		FWrapLayer::DestroyShaderModule(VK_SUCCESS, Device, ShaderModule);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreatePipelineCache(VkDevice Device, const VkPipelineCacheCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkPipelineCache* PipelineCache)
	{
		FWrapLayer::CreatePipelineCache(VK_RESULT_MAX_ENUM, Device, CreateInfo, PipelineCache);
		VkResult Result = VULKANAPINAMESPACE::vkCreatePipelineCache(Device, CreateInfo, Allocator, PipelineCache);
		FWrapLayer::CreatePipelineCache(Result, Device, CreateInfo, PipelineCache);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyPipelineCache(VkDevice Device, VkPipelineCache PipelineCache, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyPipelineCache(VK_RESULT_MAX_ENUM, Device, PipelineCache);
		VULKANAPINAMESPACE::vkDestroyPipelineCache(Device, PipelineCache, Allocator);
		FWrapLayer::DestroyPipelineCache(VK_SUCCESS, Device, PipelineCache);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkGetPipelineCacheData(VkDevice Device, VkPipelineCache PipelineCache, size_t* DataSize, void* Data)
	{
		FWrapLayer::GetPipelineCacheData(VK_RESULT_MAX_ENUM, Device, PipelineCache, DataSize, Data);
		VkResult Result = VULKANAPINAMESPACE::vkGetPipelineCacheData(Device, PipelineCache, DataSize, Data);
		FWrapLayer::GetPipelineCacheData(Result, Device, PipelineCache, DataSize, Data);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkMergePipelineCaches(VkDevice Device, VkPipelineCache DestCache, uint32 SourceCacheCount, const VkPipelineCache* SrcCaches)
	{
		FWrapLayer::MergePipelineCaches(VK_RESULT_MAX_ENUM, Device, DestCache, SourceCacheCount, SrcCaches);
		VkResult Result = VULKANAPINAMESPACE::vkMergePipelineCaches(Device, DestCache, SourceCacheCount, SrcCaches);
		FWrapLayer::MergePipelineCaches(Result, Device, DestCache, SourceCacheCount, SrcCaches);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateGraphicsPipelines(VkDevice Device, VkPipelineCache PipelineCache, uint32 CreateInfoCount, const VkGraphicsPipelineCreateInfo* CreateInfos, const VkAllocationCallbacks* Allocator, VkPipeline* Pipelines)
	{
		FWrapLayer::CreateGraphicsPipelines(VK_RESULT_MAX_ENUM, Device, PipelineCache, CreateInfoCount, CreateInfos, Pipelines);
		VkResult Result = VULKANAPINAMESPACE::vkCreateGraphicsPipelines(Device, PipelineCache, CreateInfoCount, CreateInfos, Allocator, Pipelines);
		FWrapLayer::CreateGraphicsPipelines(Result, Device, PipelineCache, CreateInfoCount, CreateInfos, Pipelines);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateComputePipelines(VkDevice Device, VkPipelineCache PipelineCache, uint32 CreateInfoCount, const VkComputePipelineCreateInfo* CreateInfos, const VkAllocationCallbacks* Allocator, VkPipeline* Pipelines)
	{
		FWrapLayer::CreateComputePipelines(VK_RESULT_MAX_ENUM, Device, PipelineCache, CreateInfoCount, CreateInfos, Pipelines);
		VkResult Result = VULKANAPINAMESPACE::vkCreateComputePipelines(Device, PipelineCache, CreateInfoCount, CreateInfos, Allocator, Pipelines);
		FWrapLayer::CreateComputePipelines(Result, Device, PipelineCache, CreateInfoCount, CreateInfos, Pipelines);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyPipeline(VkDevice Device, VkPipeline Pipeline, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyPipeline(VK_RESULT_MAX_ENUM, Device, Pipeline);
		VULKANAPINAMESPACE::vkDestroyPipeline(Device, Pipeline, Allocator);
		FWrapLayer::DestroyPipeline(VK_SUCCESS, Device, Pipeline);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreatePipelineLayout(VkDevice Device, const VkPipelineLayoutCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkPipelineLayout* PipelineLayout)
	{
		FWrapLayer::CreatePipelineLayout(VK_RESULT_MAX_ENUM, Device, CreateInfo, PipelineLayout);
		VkResult Result = VULKANAPINAMESPACE::vkCreatePipelineLayout(Device, CreateInfo, Allocator, PipelineLayout);
		FWrapLayer::CreatePipelineLayout(Result, Device, CreateInfo, PipelineLayout);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyPipelineLayout(VkDevice Device, VkPipelineLayout PipelineLayout, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyPipelineLayout(VK_RESULT_MAX_ENUM, Device, PipelineLayout);
		VULKANAPINAMESPACE::vkDestroyPipelineLayout(Device, PipelineLayout, Allocator);
		FWrapLayer::DestroyPipelineLayout(VK_SUCCESS, Device, PipelineLayout);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateSampler(VkDevice Device, const VkSamplerCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkSampler* Sampler)
	{
		FWrapLayer::CreateSampler(VK_RESULT_MAX_ENUM, Device, CreateInfo, Sampler);
		VkResult Result = VULKANAPINAMESPACE::vkCreateSampler(Device, CreateInfo, Allocator, Sampler);
		FWrapLayer::CreateSampler(Result, Device, CreateInfo, Sampler);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroySampler(VkDevice Device, VkSampler Sampler, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroySampler(VK_RESULT_MAX_ENUM, Device, Sampler);
		VULKANAPINAMESPACE::vkDestroySampler(Device, Sampler, Allocator);
		FWrapLayer::DestroySampler(VK_RESULT_MAX_ENUM, Device, Sampler);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateDescriptorSetLayout(VkDevice Device, const VkDescriptorSetLayoutCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkDescriptorSetLayout* SetLayout)
	{
		FWrapLayer::CreateDescriptorSetLayout(VK_RESULT_MAX_ENUM, Device, CreateInfo, SetLayout);
		VkResult Result = VULKANAPINAMESPACE::vkCreateDescriptorSetLayout(Device, CreateInfo, Allocator, SetLayout);
		FWrapLayer::CreateDescriptorSetLayout(Result, Device, CreateInfo, SetLayout);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyDescriptorSetLayout(VkDevice Device, VkDescriptorSetLayout DescriptorSetLayout, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyDescriptorSetLayout(VK_RESULT_MAX_ENUM, Device, DescriptorSetLayout);
		VULKANAPINAMESPACE::vkDestroyDescriptorSetLayout(Device, DescriptorSetLayout, Allocator);
		FWrapLayer::DestroyDescriptorSetLayout(VK_SUCCESS, Device, DescriptorSetLayout);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateDescriptorPool(VkDevice Device, const VkDescriptorPoolCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkDescriptorPool* DescriptorPool)
	{
		FWrapLayer::CreateDescriptorPool(VK_RESULT_MAX_ENUM, Device, CreateInfo, DescriptorPool);
		VkResult Result = VULKANAPINAMESPACE::vkCreateDescriptorPool(Device, CreateInfo, Allocator, DescriptorPool);
		FWrapLayer::CreateDescriptorPool(Result, Device, CreateInfo, DescriptorPool);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyDescriptorPool(VkDevice Device, VkDescriptorPool DescriptorPool, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyDescriptorPool(VK_RESULT_MAX_ENUM, Device, DescriptorPool);
		VULKANAPINAMESPACE::vkDestroyDescriptorPool(Device, DescriptorPool, Allocator);
		FWrapLayer::DestroyDescriptorPool(VK_SUCCESS, Device, DescriptorPool);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkResetDescriptorPool(VkDevice Device, VkDescriptorPool DescriptorPool, VkDescriptorPoolResetFlags Flags)
	{
		FWrapLayer::ResetDescriptorPool(VK_RESULT_MAX_ENUM, Device, DescriptorPool, Flags);
		VkResult Result = VULKANAPINAMESPACE::vkResetDescriptorPool(Device, DescriptorPool, Flags);
		FWrapLayer::ResetDescriptorPool(Result, Device, DescriptorPool, Flags);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkAllocateDescriptorSets(VkDevice Device, const VkDescriptorSetAllocateInfo* AllocateInfo, VkDescriptorSet* DescriptorSets)
	{
		FWrapLayer::AllocateDescriptorSets(VK_RESULT_MAX_ENUM, Device, AllocateInfo, DescriptorSets);
		VkResult Result = VULKANAPINAMESPACE::vkAllocateDescriptorSets(Device, AllocateInfo, DescriptorSets);
		FWrapLayer::AllocateDescriptorSets(Result, Device, AllocateInfo, DescriptorSets);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkFreeDescriptorSets(VkDevice Device, VkDescriptorPool DescriptorPool, uint32 DescriptorSetCount, const VkDescriptorSet* DescriptorSets)
	{
		FWrapLayer::FreeDescriptorSets(VK_RESULT_MAX_ENUM, Device, DescriptorPool, DescriptorSetCount, DescriptorSets);
		VkResult Result = VULKANAPINAMESPACE::vkFreeDescriptorSets(Device, DescriptorPool, DescriptorSetCount, DescriptorSets);
		FWrapLayer::FreeDescriptorSets(Result, Device, DescriptorPool, DescriptorSetCount, DescriptorSets);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkUpdateDescriptorSets(VkDevice Device, uint32 DescriptorWriteCount, const VkWriteDescriptorSet* DescriptorWrites, uint32 DescriptorCopyCount, const VkCopyDescriptorSet* DescriptorCopies)
	{
		FWrapLayer::UpdateDescriptorSets(VK_RESULT_MAX_ENUM, Device, DescriptorWriteCount, DescriptorWrites, DescriptorCopyCount, DescriptorCopies);
		VULKANAPINAMESPACE::vkUpdateDescriptorSets(Device, DescriptorWriteCount, DescriptorWrites, DescriptorCopyCount, DescriptorCopies);
		FWrapLayer::UpdateDescriptorSets(VK_SUCCESS, Device, DescriptorWriteCount, DescriptorWrites, DescriptorCopyCount, DescriptorCopies);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateFramebuffer(VkDevice Device, const VkFramebufferCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkFramebuffer* Framebuffer)
	{
		FWrapLayer::CreateFramebuffer(VK_RESULT_MAX_ENUM, Device, CreateInfo, Framebuffer);
		VkResult Result = VULKANAPINAMESPACE::vkCreateFramebuffer(Device, CreateInfo, Allocator, Framebuffer);
		FWrapLayer::CreateFramebuffer(Result, Device, CreateInfo, Framebuffer);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyFramebuffer(VkDevice Device, VkFramebuffer Framebuffer, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyFramebuffer(VK_RESULT_MAX_ENUM, Device, Framebuffer);
		VULKANAPINAMESPACE::vkDestroyFramebuffer(Device, Framebuffer, Allocator);
		FWrapLayer::DestroyFramebuffer(VK_SUCCESS, Device, Framebuffer);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateRenderPass(VkDevice Device, const VkRenderPassCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkRenderPass* RenderPass)
	{
		FWrapLayer::CreateRenderPass(VK_RESULT_MAX_ENUM, Device, CreateInfo, RenderPass);
		VkResult Result = VULKANAPINAMESPACE::vkCreateRenderPass(Device, CreateInfo, Allocator, RenderPass);
		FWrapLayer::CreateRenderPass(Result, Device, CreateInfo, RenderPass);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkCreateRenderPass2KHR(VkDevice Device, const VkRenderPassCreateInfo2* CreateInfo, const VkAllocationCallbacks* Allocator, VkRenderPass* RenderPass)
	{
		FWrapLayer::CreateRenderPass2KHR(VK_RESULT_MAX_ENUM, Device, CreateInfo, RenderPass);
		VkResult Result = VULKANAPINAMESPACE::vkCreateRenderPass2KHR(Device, CreateInfo, Allocator, RenderPass);
		FWrapLayer::CreateRenderPass2KHR(Result, Device, CreateInfo, RenderPass);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyRenderPass(VkDevice Device, VkRenderPass RenderPass, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyRenderPass(VK_RESULT_MAX_ENUM, Device, RenderPass);
		VULKANAPINAMESPACE::vkDestroyRenderPass(Device, RenderPass, Allocator);
		FWrapLayer::DestroyRenderPass(VK_SUCCESS, Device, RenderPass);
	}

	static FORCEINLINE_DEBUGGABLE void  vkGetRenderAreaGranularity(VkDevice Device, VkRenderPass RenderPass, VkExtent2D* pGranularity)
	{
		FWrapLayer::GetRenderAreaGranularity(VK_RESULT_MAX_ENUM, Device, RenderPass, pGranularity);
		VULKANAPINAMESPACE::vkGetRenderAreaGranularity(Device, RenderPass, pGranularity);
		FWrapLayer::GetRenderAreaGranularity(VK_SUCCESS, Device, RenderPass, pGranularity);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkCreateCommandPool(VkDevice Device, const VkCommandPoolCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkCommandPool* CommandPool)
	{
		FWrapLayer::CreateCommandPool(VK_RESULT_MAX_ENUM, Device, CreateInfo, CommandPool);
		VkResult Result = VULKANAPINAMESPACE::vkCreateCommandPool(Device, CreateInfo, Allocator, CommandPool);
		FWrapLayer::CreateCommandPool(Result, Device, CreateInfo, CommandPool);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkDestroyCommandPool(VkDevice Device, VkCommandPool CommandPool, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyCommandPool(VK_RESULT_MAX_ENUM, Device, CommandPool);
		VULKANAPINAMESPACE::vkDestroyCommandPool(Device, CommandPool, Allocator);
		FWrapLayer::DestroyCommandPool(VK_SUCCESS, Device, CommandPool);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkResetCommandPool(VkDevice Device, VkCommandPool CommandPool, VkCommandPoolResetFlags Flags)
	{
		FWrapLayer::ResetCommandPool(VK_RESULT_MAX_ENUM, Device, CommandPool, Flags);
		VkResult Result = VULKANAPINAMESPACE::vkResetCommandPool(Device, CommandPool, Flags);
		FWrapLayer::ResetCommandPool(Result, Device, CommandPool, Flags);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkTrimCommandPool(VkDevice Device, VkCommandPool CommandPool, VkCommandPoolTrimFlags Flags)
	{
		FWrapLayer::TrimCommandPool(VK_RESULT_MAX_ENUM, Device, CommandPool, Flags);
		VULKANAPINAMESPACE::vkTrimCommandPool(Device, CommandPool, Flags);
		FWrapLayer::TrimCommandPool(VK_SUCCESS, Device, CommandPool, Flags);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkAllocateCommandBuffers(VkDevice Device, const VkCommandBufferAllocateInfo* AllocateInfo, VkCommandBuffer* CommandBuffers)
	{
		FWrapLayer::AllocateCommandBuffers(VK_RESULT_MAX_ENUM, Device, AllocateInfo, CommandBuffers);
		VkResult Result = VULKANAPINAMESPACE::vkAllocateCommandBuffers(Device, AllocateInfo, CommandBuffers);
		FWrapLayer::AllocateCommandBuffers(Result, Device, AllocateInfo, CommandBuffers);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkFreeCommandBuffers(VkDevice Device, VkCommandPool CommandPool, uint32 CommandBufferCount, const VkCommandBuffer* CommandBuffers)
	{
		FWrapLayer::FreeCommandBuffers(VK_RESULT_MAX_ENUM, Device, CommandPool, CommandBufferCount, CommandBuffers);
		VULKANAPINAMESPACE::vkFreeCommandBuffers(Device, CommandPool, CommandBufferCount, CommandBuffers);
		FWrapLayer::FreeCommandBuffers(VK_SUCCESS, Device, CommandPool, CommandBufferCount, CommandBuffers);
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkBeginCommandBuffer(VkCommandBuffer CommandBuffer, const VkCommandBufferBeginInfo* BeginInfo)
	{
		FWrapLayer::BeginCommandBuffer(VK_RESULT_MAX_ENUM, CommandBuffer, BeginInfo);
		VkResult Result = VULKANAPINAMESPACE::vkBeginCommandBuffer(CommandBuffer, BeginInfo);
		FWrapLayer::BeginCommandBuffer(Result, CommandBuffer, BeginInfo);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkEndCommandBuffer(VkCommandBuffer CommandBuffer)
	{
		FWrapLayer::EndCommandBuffer(VK_RESULT_MAX_ENUM, CommandBuffer);
		VkResult Result = VULKANAPINAMESPACE::vkEndCommandBuffer(CommandBuffer);
		FWrapLayer::EndCommandBuffer(Result, CommandBuffer);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult  vkResetCommandBuffer(VkCommandBuffer CommandBuffer, VkCommandBufferResetFlags Flags)
	{
		FWrapLayer::ResetCommandBuffer(VK_RESULT_MAX_ENUM, CommandBuffer, Flags);
		VkResult Result = VULKANAPINAMESPACE::vkResetCommandBuffer(CommandBuffer, Flags);
		FWrapLayer::ResetCommandBuffer(Result, CommandBuffer, Flags);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdBindPipeline(VkCommandBuffer CommandBuffer, VkPipelineBindPoint PipelineBindPoint, VkPipeline Pipeline)
	{
		FWrapLayer::CmdBindPipeline(VK_RESULT_MAX_ENUM, CommandBuffer, PipelineBindPoint, Pipeline);
		VULKANAPINAMESPACE::vkCmdBindPipeline(CommandBuffer, PipelineBindPoint, Pipeline);
		FWrapLayer::CmdBindPipeline(VK_SUCCESS, CommandBuffer, PipelineBindPoint, Pipeline);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdSetViewport(VkCommandBuffer CommandBuffer, uint32 FirstViewport, uint32 ViewportCount, const VkViewport* Viewports)
	{
		FWrapLayer::CmdSetViewport(VK_RESULT_MAX_ENUM, CommandBuffer, FirstViewport, ViewportCount, Viewports);
		VULKANAPINAMESPACE::vkCmdSetViewport(CommandBuffer, FirstViewport, ViewportCount, Viewports);
		FWrapLayer::CmdSetViewport(VK_SUCCESS, CommandBuffer, FirstViewport, ViewportCount, Viewports);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdSetScissor(VkCommandBuffer CommandBuffer, uint32 FirstScissor, uint32 ScissorCount, const VkRect2D* Scissors)
	{
		FWrapLayer::CmdSetScissor(VK_RESULT_MAX_ENUM, CommandBuffer, FirstScissor, ScissorCount, Scissors);
		VULKANAPINAMESPACE::vkCmdSetScissor(CommandBuffer, FirstScissor, ScissorCount, Scissors);
		FWrapLayer::CmdSetScissor(VK_SUCCESS, CommandBuffer, FirstScissor, ScissorCount, Scissors);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdSetLineWidth(VkCommandBuffer CommandBuffer, float LineWidth)
	{
		FWrapLayer::CmdSetLineWidth(VK_RESULT_MAX_ENUM, CommandBuffer, LineWidth);
		VULKANAPINAMESPACE::vkCmdSetLineWidth(CommandBuffer, LineWidth);
		FWrapLayer::CmdSetLineWidth(VK_SUCCESS, CommandBuffer, LineWidth);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdSetDepthBias(VkCommandBuffer CommandBuffer, float DepthBiasConstantFactor, float DepthBiasClamp, float DepthBiasSlopeFactor)
	{
		FWrapLayer::CmdSetDepthBias(VK_RESULT_MAX_ENUM, CommandBuffer, DepthBiasConstantFactor, DepthBiasClamp, DepthBiasSlopeFactor);
		VULKANAPINAMESPACE::vkCmdSetDepthBias(CommandBuffer, DepthBiasConstantFactor, DepthBiasClamp, DepthBiasSlopeFactor);
		FWrapLayer::CmdSetDepthBias(VK_SUCCESS, CommandBuffer, DepthBiasConstantFactor, DepthBiasClamp, DepthBiasSlopeFactor);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdSetBlendConstants(VkCommandBuffer CommandBuffer, const float BlendConstants[4])
	{
		FWrapLayer::CmdSetBlendConstants(VK_RESULT_MAX_ENUM, CommandBuffer, BlendConstants);
		VULKANAPINAMESPACE::vkCmdSetBlendConstants(CommandBuffer, BlendConstants);
		FWrapLayer::CmdSetBlendConstants(VK_SUCCESS, CommandBuffer, BlendConstants);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdSetDepthBounds(VkCommandBuffer CommandBuffer, float MinDepthBounds, float MaxDepthBounds)
	{
		FWrapLayer::CmdSetDepthBounds(VK_RESULT_MAX_ENUM, CommandBuffer, MinDepthBounds, MaxDepthBounds);
		VULKANAPINAMESPACE::vkCmdSetDepthBounds(CommandBuffer, MinDepthBounds, MaxDepthBounds);
		FWrapLayer::CmdSetDepthBounds(VK_SUCCESS, CommandBuffer, MinDepthBounds, MaxDepthBounds);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdSetStencilCompareMask(VkCommandBuffer CommandBuffer, VkStencilFaceFlags FaceMask, uint32 CompareMask)
	{
		FWrapLayer::CmdSetStencilCompareMask(VK_RESULT_MAX_ENUM, CommandBuffer, FaceMask, CompareMask);
		VULKANAPINAMESPACE::vkCmdSetStencilCompareMask(CommandBuffer, FaceMask, CompareMask);
		FWrapLayer::CmdSetStencilCompareMask(VK_SUCCESS, CommandBuffer, FaceMask, CompareMask);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdSetStencilWriteMask(VkCommandBuffer CommandBuffer, VkStencilFaceFlags FaceMask, uint32 WriteMask)
	{
		FWrapLayer::CmdSetStencilWriteMask(VK_RESULT_MAX_ENUM, CommandBuffer, FaceMask, WriteMask);
		VULKANAPINAMESPACE::vkCmdSetStencilWriteMask(CommandBuffer, FaceMask, WriteMask);
		FWrapLayer::CmdSetStencilWriteMask(VK_SUCCESS, CommandBuffer, FaceMask, WriteMask);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdSetStencilReference(VkCommandBuffer CommandBuffer, VkStencilFaceFlags FaceMask, uint32 Reference)
	{
		FWrapLayer::CmdSetStencilReference(VK_RESULT_MAX_ENUM, CommandBuffer, FaceMask, Reference);
		VULKANAPINAMESPACE::vkCmdSetStencilReference(CommandBuffer, FaceMask, Reference);
		FWrapLayer::CmdSetStencilReference(VK_SUCCESS, CommandBuffer, FaceMask, Reference);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdBindDescriptorSets(VkCommandBuffer CommandBuffer, VkPipelineBindPoint PipelineBindPoint, VkPipelineLayout Layout, uint32 FirstSet, uint32 DescriptorSetCount, const VkDescriptorSet* DescriptorSets, uint32 DynamicOffsetCount, const uint32* DynamicOffsets)
	{
		FWrapLayer::CmdBindDescriptorSets(VK_RESULT_MAX_ENUM, CommandBuffer, PipelineBindPoint, Layout, FirstSet, DescriptorSetCount, DescriptorSets, DynamicOffsetCount, DynamicOffsets);
		VULKANAPINAMESPACE::vkCmdBindDescriptorSets(CommandBuffer, PipelineBindPoint, Layout, FirstSet, DescriptorSetCount, DescriptorSets, DynamicOffsetCount, DynamicOffsets);
		FWrapLayer::CmdBindDescriptorSets(VK_SUCCESS, CommandBuffer, PipelineBindPoint, Layout, FirstSet, DescriptorSetCount, DescriptorSets, DynamicOffsetCount, DynamicOffsets);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdBindIndexBuffer(VkCommandBuffer CommandBuffer, VkBuffer Buffer, VkDeviceSize Offset, VkIndexType IndexType)
	{
		FWrapLayer::CmdBindIndexBuffer(VK_RESULT_MAX_ENUM, CommandBuffer, Buffer, Offset, IndexType);
		VULKANAPINAMESPACE::vkCmdBindIndexBuffer(CommandBuffer, Buffer, Offset, IndexType);
		FWrapLayer::CmdBindIndexBuffer(VK_SUCCESS, CommandBuffer, Buffer, Offset, IndexType);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdBindVertexBuffers(VkCommandBuffer CommandBuffer, uint32 FirstBinding, uint32 BindingCount, const VkBuffer* Buffers, const VkDeviceSize* Offsets)
	{
		FWrapLayer::CmdBindVertexBuffers(VK_RESULT_MAX_ENUM, CommandBuffer, FirstBinding, BindingCount, Buffers, Offsets);
		VULKANAPINAMESPACE::vkCmdBindVertexBuffers(CommandBuffer, FirstBinding, BindingCount, Buffers, Offsets);
		FWrapLayer::CmdBindVertexBuffers(VK_SUCCESS, CommandBuffer, FirstBinding, BindingCount, Buffers, Offsets);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdDraw(VkCommandBuffer CommandBuffer, uint32 VertexCount, uint32 InstanceCount, uint32 FirstVertex, uint32 FirstInstance)
	{
		FWrapLayer::CmdDraw(VK_RESULT_MAX_ENUM, CommandBuffer, VertexCount, InstanceCount, FirstVertex, FirstInstance);
		VULKANAPINAMESPACE::vkCmdDraw(CommandBuffer, VertexCount, InstanceCount, FirstVertex, FirstInstance);
		FWrapLayer::CmdDraw(VK_SUCCESS, CommandBuffer, VertexCount, InstanceCount, FirstVertex, FirstInstance);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdDrawIndexed(VkCommandBuffer CommandBuffer, uint32 IndexCount, uint32 InstanceCount, uint32 FirstIndex, int32_t VertexOffset, uint32 FirstInstance)
	{
		FWrapLayer::CmdDrawIndexed(VK_RESULT_MAX_ENUM, CommandBuffer, IndexCount, InstanceCount, FirstIndex, VertexOffset, FirstInstance);
		VULKANAPINAMESPACE::vkCmdDrawIndexed(CommandBuffer, IndexCount, InstanceCount, FirstIndex, VertexOffset, FirstInstance);
		FWrapLayer::CmdDrawIndexed(VK_SUCCESS, CommandBuffer, IndexCount, InstanceCount, FirstIndex, VertexOffset, FirstInstance);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdDrawIndirect(VkCommandBuffer CommandBuffer, VkBuffer Buffer, VkDeviceSize Offset, uint32 DrawCount, uint32 Stride)
	{
		FWrapLayer::CmdDrawIndirect(VK_RESULT_MAX_ENUM, CommandBuffer, Buffer, Offset, DrawCount, Stride);
		VULKANAPINAMESPACE::vkCmdDrawIndirect(CommandBuffer, Buffer, Offset, DrawCount, Stride);
		FWrapLayer::CmdDrawIndirect(VK_SUCCESS, CommandBuffer, Buffer, Offset, DrawCount, Stride);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdDrawIndexedIndirect(VkCommandBuffer CommandBuffer, VkBuffer Buffer, VkDeviceSize Offset, uint32 DrawCount, uint32 Stride)
	{
		FWrapLayer::CmdDrawIndexedIndirect(VK_RESULT_MAX_ENUM, CommandBuffer, Buffer, Offset, DrawCount, Stride);
		VULKANAPINAMESPACE::vkCmdDrawIndexedIndirect(CommandBuffer, Buffer, Offset, DrawCount, Stride);
		FWrapLayer::CmdDrawIndexedIndirect(VK_SUCCESS, CommandBuffer, Buffer, Offset, DrawCount, Stride);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdDispatch(VkCommandBuffer CommandBuffer, uint32 X, uint32 Y, uint32 Z)
	{
		FWrapLayer::CmdDispatch(VK_RESULT_MAX_ENUM, CommandBuffer, X, Y, Z);
		VULKANAPINAMESPACE::vkCmdDispatch(CommandBuffer, X, Y, Z);
		FWrapLayer::CmdDispatch(VK_SUCCESS, CommandBuffer, X, Y, Z);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdDispatchIndirect(VkCommandBuffer CommandBuffer, VkBuffer Buffer, VkDeviceSize Offset)
	{
		FWrapLayer::CmdDispatchIndirect(VK_RESULT_MAX_ENUM, CommandBuffer, Buffer, Offset);
		VULKANAPINAMESPACE::vkCmdDispatchIndirect(CommandBuffer, Buffer, Offset);
		FWrapLayer::CmdDispatchIndirect(VK_SUCCESS, CommandBuffer, Buffer, Offset);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdCopyBuffer(VkCommandBuffer CommandBuffer, VkBuffer SrcBuffer, VkBuffer DstBuffer, uint32 RegionCount, const VkBufferCopy* Regions)
	{
		FWrapLayer::CmdCopyBuffer(VK_RESULT_MAX_ENUM, CommandBuffer, SrcBuffer, DstBuffer, RegionCount, Regions);
		VULKANAPINAMESPACE::vkCmdCopyBuffer(CommandBuffer, SrcBuffer, DstBuffer, RegionCount, Regions);
		FWrapLayer::CmdCopyBuffer(VK_SUCCESS, CommandBuffer, SrcBuffer, DstBuffer, RegionCount, Regions);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdCopyImage(VkCommandBuffer CommandBuffer, VkImage SrcImage, VkImageLayout SrcImageLayout, VkImage DstImage, VkImageLayout DstImageLayout, uint32 RegionCount, const VkImageCopy* Regions)
	{
		FWrapLayer::CmdCopyImage(VK_RESULT_MAX_ENUM, CommandBuffer, SrcImage, SrcImageLayout, DstImage, DstImageLayout, RegionCount, Regions);
		VULKANAPINAMESPACE::vkCmdCopyImage(CommandBuffer, SrcImage, SrcImageLayout, DstImage, DstImageLayout, RegionCount, Regions);
		FWrapLayer::CmdCopyImage(VK_SUCCESS, CommandBuffer, SrcImage, SrcImageLayout, DstImage, DstImageLayout, RegionCount, Regions);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdBlitImage(VkCommandBuffer CommandBuffer, VkImage SrcImage, VkImageLayout SrcImageLayout, VkImage DstImage, VkImageLayout DstImageLayout, uint32 RegionCount, const VkImageBlit* Regions, VkFilter Filter)
	{
		FWrapLayer::CmdBlitImage(VK_RESULT_MAX_ENUM, CommandBuffer, SrcImage, SrcImageLayout, DstImage, DstImageLayout, RegionCount, Regions, Filter);
		VULKANAPINAMESPACE::vkCmdBlitImage(CommandBuffer, SrcImage, SrcImageLayout, DstImage, DstImageLayout, RegionCount, Regions, Filter);
		FWrapLayer::CmdBlitImage(VK_SUCCESS, CommandBuffer, SrcImage, SrcImageLayout, DstImage, DstImageLayout, RegionCount, Regions, Filter);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdCopyBufferToImage(VkCommandBuffer CommandBuffer, VkBuffer SrcBuffer, VkImage DstImage, VkImageLayout DstImageLayout, uint32 RegionCount, const VkBufferImageCopy* Regions)
	{
		FWrapLayer::CmdCopyBufferToImage(VK_RESULT_MAX_ENUM, CommandBuffer, SrcBuffer, DstImage, DstImageLayout, RegionCount, Regions);
		VULKANAPINAMESPACE::vkCmdCopyBufferToImage(CommandBuffer, SrcBuffer, DstImage, DstImageLayout, RegionCount, Regions);
		FWrapLayer::CmdCopyBufferToImage(VK_SUCCESS, CommandBuffer, SrcBuffer, DstImage, DstImageLayout, RegionCount, Regions);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdCopyImageToBuffer(VkCommandBuffer CommandBuffer, VkImage SrcImage, VkImageLayout SrcImageLayout, VkBuffer DstBuffer, uint32 RegionCount, const VkBufferImageCopy* Regions)
	{
		FWrapLayer::CmdCopyImageToBuffer(VK_RESULT_MAX_ENUM, CommandBuffer, SrcImage, SrcImageLayout, DstBuffer, RegionCount, Regions);
		VULKANAPINAMESPACE::vkCmdCopyImageToBuffer(CommandBuffer, SrcImage, SrcImageLayout, DstBuffer, RegionCount, Regions);
		FWrapLayer::CmdCopyImageToBuffer(VK_SUCCESS, CommandBuffer, SrcImage, SrcImageLayout, DstBuffer, RegionCount, Regions);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdUpdateBuffer(VkCommandBuffer CommandBuffer, VkBuffer DstBuffer, VkDeviceSize DstOffset, VkDeviceSize DataSize, const void* pData)
	{
		FWrapLayer::CmdUpdateBuffer(VK_RESULT_MAX_ENUM, CommandBuffer, DstBuffer, DstOffset, DataSize, pData);
		VULKANAPINAMESPACE::vkCmdUpdateBuffer(CommandBuffer, DstBuffer, DstOffset, DataSize, pData);
		FWrapLayer::CmdUpdateBuffer(VK_SUCCESS, CommandBuffer, DstBuffer, DstOffset, DataSize, pData);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdFillBuffer(VkCommandBuffer CommandBuffer, VkBuffer DstBuffer, VkDeviceSize DstOffset, VkDeviceSize Size, uint32 Data)
	{
		FWrapLayer::CmdFillBuffer(VK_RESULT_MAX_ENUM, CommandBuffer, DstBuffer, DstOffset, Size, Data);
		VULKANAPINAMESPACE::vkCmdFillBuffer(CommandBuffer, DstBuffer, DstOffset, Size, Data);
		FWrapLayer::CmdFillBuffer(VK_SUCCESS, CommandBuffer, DstBuffer, DstOffset, Size, Data);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdClearColorImage(VkCommandBuffer CommandBuffer, VkImage Image, VkImageLayout ImageLayout, const VkClearColorValue* Color, uint32 RangeCount, const VkImageSubresourceRange* Ranges)
	{
		FWrapLayer::CmdClearColorImage(VK_RESULT_MAX_ENUM, CommandBuffer, Image, ImageLayout, Color, RangeCount, Ranges);
		VULKANAPINAMESPACE::vkCmdClearColorImage(CommandBuffer, Image, ImageLayout, Color, RangeCount, Ranges);
		FWrapLayer::CmdClearColorImage(VK_SUCCESS, CommandBuffer, Image, ImageLayout, Color, RangeCount, Ranges);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdClearDepthStencilImage(VkCommandBuffer CommandBuffer, VkImage Image, VkImageLayout ImageLayout, const VkClearDepthStencilValue* DepthStencil, uint32 RangeCount, const VkImageSubresourceRange* Ranges)
	{
		FWrapLayer::CmdClearDepthStencilImage(VK_RESULT_MAX_ENUM, CommandBuffer, Image, ImageLayout, DepthStencil, RangeCount, Ranges);
		VULKANAPINAMESPACE::vkCmdClearDepthStencilImage(CommandBuffer, Image, ImageLayout, DepthStencil, RangeCount, Ranges);
		FWrapLayer::CmdClearDepthStencilImage(VK_SUCCESS, CommandBuffer, Image, ImageLayout, DepthStencil, RangeCount, Ranges);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdClearAttachments(VkCommandBuffer CommandBuffer, uint32 AttachmentCount, const VkClearAttachment* Attachments, uint32 RectCount, const VkClearRect* Rects)
	{
		FWrapLayer::CmdClearAttachments(VK_RESULT_MAX_ENUM, CommandBuffer, AttachmentCount, Attachments, RectCount, Rects);
		VULKANAPINAMESPACE::vkCmdClearAttachments(CommandBuffer, AttachmentCount, Attachments, RectCount, Rects);
		FWrapLayer::CmdClearAttachments(VK_SUCCESS, CommandBuffer, AttachmentCount, Attachments, RectCount, Rects);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdResolveImage(
		VkCommandBuffer CommandBuffer,
		VkImage SrcImage, VkImageLayout SrcImageLayout,
		VkImage DstImage, VkImageLayout DstImageLayout,
		uint32 RegionCount, const VkImageResolve* Regions)
	{
		FWrapLayer::CmdResolveImage(VK_RESULT_MAX_ENUM, CommandBuffer, SrcImage, SrcImageLayout, DstImage, DstImageLayout, RegionCount, Regions);
		VULKANAPINAMESPACE::vkCmdResolveImage(CommandBuffer, SrcImage, SrcImageLayout, DstImage, DstImageLayout, RegionCount, Regions);
		FWrapLayer::CmdResolveImage(VK_SUCCESS, CommandBuffer, SrcImage, SrcImageLayout, DstImage, DstImageLayout, RegionCount, Regions);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdSetEvent(VkCommandBuffer CommandBuffer, VkEvent Event, VkPipelineStageFlags StageMask)
	{
		FWrapLayer::CmdSetEvent(VK_RESULT_MAX_ENUM, CommandBuffer, Event, StageMask);
		VULKANAPINAMESPACE::vkCmdSetEvent(CommandBuffer, Event, StageMask);
		FWrapLayer::CmdSetEvent(VK_SUCCESS, CommandBuffer, Event, StageMask);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdResetEvent(VkCommandBuffer CommandBuffer, VkEvent Event, VkPipelineStageFlags StageMask)
	{
		FWrapLayer::CmdResetEvent(VK_RESULT_MAX_ENUM, CommandBuffer, Event, StageMask);
		VULKANAPINAMESPACE::vkCmdResetEvent(CommandBuffer, Event, StageMask);
		FWrapLayer::CmdResetEvent(VK_SUCCESS, CommandBuffer, Event, StageMask);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdWaitEvents(VkCommandBuffer CommandBuffer, uint32 EventCount, const VkEvent* Events,
		VkPipelineStageFlags SrcStageMask, VkPipelineStageFlags DstStageMask,
		uint32 MemoryBarrierCount, const VkMemoryBarrier* pMemoryBarriers,
		uint32 BufferMemoryBarrierCount, const VkBufferMemoryBarrier* pBufferMemoryBarriers,
		uint32 ImageMemoryBarrierCount, const VkImageMemoryBarrier* pImageMemoryBarriers)
	{
		FWrapLayer::CmdWaitEvents(VK_RESULT_MAX_ENUM, CommandBuffer, EventCount, Events, SrcStageMask, DstStageMask,
			MemoryBarrierCount, pMemoryBarriers, BufferMemoryBarrierCount, pBufferMemoryBarriers, ImageMemoryBarrierCount, pImageMemoryBarriers);
		VULKANAPINAMESPACE::vkCmdWaitEvents(CommandBuffer, EventCount, Events, SrcStageMask, DstStageMask, MemoryBarrierCount, pMemoryBarriers,
			BufferMemoryBarrierCount, pBufferMemoryBarriers, ImageMemoryBarrierCount, pImageMemoryBarriers);
		FWrapLayer::CmdWaitEvents(VK_SUCCESS, CommandBuffer, EventCount, Events, SrcStageMask, DstStageMask,
			MemoryBarrierCount, pMemoryBarriers, BufferMemoryBarrierCount, pBufferMemoryBarriers, ImageMemoryBarrierCount, pImageMemoryBarriers);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdPipelineBarrier(
		VkCommandBuffer CommandBuffer, VkPipelineStageFlags SrcStageMask, VkPipelineStageFlags DstStageMask, VkDependencyFlags DependencyFlags,
		uint32 MemoryBarrierCount, const VkMemoryBarrier* MemoryBarriers,
		uint32 BufferMemoryBarrierCount, const VkBufferMemoryBarrier* BufferMemoryBarriers,
		uint32 ImageMemoryBarrierCount, const VkImageMemoryBarrier* ImageMemoryBarriers)
	{
		FWrapLayer::CmdPipelineBarrier(VK_RESULT_MAX_ENUM, CommandBuffer, SrcStageMask, DstStageMask, DependencyFlags, MemoryBarrierCount, MemoryBarriers, BufferMemoryBarrierCount, BufferMemoryBarriers, ImageMemoryBarrierCount, ImageMemoryBarriers);
		VULKANAPINAMESPACE::vkCmdPipelineBarrier(CommandBuffer, SrcStageMask, DstStageMask, DependencyFlags, MemoryBarrierCount, MemoryBarriers, BufferMemoryBarrierCount, BufferMemoryBarriers, ImageMemoryBarrierCount, ImageMemoryBarriers);
		FWrapLayer::CmdPipelineBarrier(VK_SUCCESS, CommandBuffer, SrcStageMask, DstStageMask, DependencyFlags, MemoryBarrierCount, MemoryBarriers, BufferMemoryBarrierCount, BufferMemoryBarriers, ImageMemoryBarrierCount, ImageMemoryBarriers);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdBeginQuery(VkCommandBuffer CommandBuffer, VkQueryPool QueryPool, uint32 Query, VkQueryControlFlags Flags)
	{
		FWrapLayer::CmdBeginQuery(VK_RESULT_MAX_ENUM, CommandBuffer, QueryPool, Query, Flags);
		VULKANAPINAMESPACE::vkCmdBeginQuery(CommandBuffer, QueryPool, Query, Flags);
		FWrapLayer::CmdBeginQuery(VK_SUCCESS, CommandBuffer, QueryPool, Query, Flags);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdEndQuery(VkCommandBuffer CommandBuffer, VkQueryPool QueryPool, uint32 Query)
	{
		FWrapLayer::CmdEndQuery(VK_RESULT_MAX_ENUM, CommandBuffer, QueryPool, Query);
		VULKANAPINAMESPACE::vkCmdEndQuery(CommandBuffer, QueryPool, Query);
		FWrapLayer::CmdEndQuery(VK_SUCCESS, CommandBuffer, QueryPool, Query);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdResetQueryPool(VkCommandBuffer CommandBuffer, VkQueryPool QueryPool, uint32 FirstQuery, uint32 QueryCount)
	{
		FWrapLayer::CmdResetQueryPool(VK_RESULT_MAX_ENUM, CommandBuffer, QueryPool, FirstQuery, QueryCount);
		VULKANAPINAMESPACE::vkCmdResetQueryPool(CommandBuffer, QueryPool, FirstQuery, QueryCount);
		FWrapLayer::CmdResetQueryPool(VK_SUCCESS, CommandBuffer, QueryPool, FirstQuery, QueryCount);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdWriteTimestamp(VkCommandBuffer CommandBuffer, VkPipelineStageFlagBits PipelineStage, VkQueryPool QueryPool, uint32 Query)
	{
		FWrapLayer::CmdWriteTimestamp(VK_RESULT_MAX_ENUM, CommandBuffer, PipelineStage, QueryPool, Query);
		VULKANAPINAMESPACE::vkCmdWriteTimestamp(CommandBuffer, PipelineStage, QueryPool, Query);
		FWrapLayer::CmdWriteTimestamp(VK_SUCCESS, CommandBuffer, PipelineStage, QueryPool, Query);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdCopyQueryPoolResults(VkCommandBuffer CommandBuffer, VkQueryPool QueryPool, uint32 FirstQuery, uint32 QueryCount,
		VkBuffer DstBuffer, VkDeviceSize DstOffset, VkDeviceSize Stride, VkQueryResultFlags Flags)
	{
		FWrapLayer::CmdCopyQueryPoolResults(VK_RESULT_MAX_ENUM, CommandBuffer, QueryPool, FirstQuery, QueryCount, DstBuffer, DstOffset, Stride, Flags);
		VULKANAPINAMESPACE::vkCmdCopyQueryPoolResults(CommandBuffer, QueryPool, FirstQuery, QueryCount, DstBuffer, DstOffset, Stride, Flags);
		FWrapLayer::CmdCopyQueryPoolResults(VK_SUCCESS, CommandBuffer, QueryPool, FirstQuery, QueryCount, DstBuffer, DstOffset, Stride, Flags);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdPushConstants(VkCommandBuffer CommandBuffer, VkPipelineLayout Layout, VkShaderStageFlags StageFlags, uint32 Offset, uint32 Size, const void* pValues)
	{
		FWrapLayer::CmdPushConstants(VK_RESULT_MAX_ENUM, CommandBuffer, Layout, StageFlags, Offset, Size, pValues);
		VULKANAPINAMESPACE::vkCmdPushConstants(CommandBuffer, Layout, StageFlags, Offset, Size, pValues);
		FWrapLayer::CmdPushConstants(VK_SUCCESS, CommandBuffer, Layout, StageFlags, Offset, Size, pValues);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdPushDescriptorSetKHR(VkCommandBuffer CommandBuffer, VkPipelineBindPoint PipelineBindPoint, VkPipelineLayout Layout, uint32_t Set, uint32_t DescriptorWriteCount, const VkWriteDescriptorSet* DescriptorWrites)
	{
		FWrapLayer::CmdPushDescriptorSetKHR(VK_RESULT_MAX_ENUM, CommandBuffer, PipelineBindPoint, Layout, Set, DescriptorWriteCount, DescriptorWrites);
		VULKANAPINAMESPACE::vkCmdPushDescriptorSetKHR(CommandBuffer, PipelineBindPoint, Layout, Set, DescriptorWriteCount, DescriptorWrites);
		FWrapLayer::CmdPushDescriptorSetKHR(VK_SUCCESS, CommandBuffer, PipelineBindPoint, Layout, Set, DescriptorWriteCount, DescriptorWrites);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdPushDescriptorSet2KHR(VkCommandBuffer CommandBuffer, const VkPushDescriptorSetInfo* PushDescriptorSetInfo)
	{
		FWrapLayer::CmdPushDescriptorSet2KHR(VK_RESULT_MAX_ENUM, CommandBuffer, PushDescriptorSetInfo);
		VULKANAPINAMESPACE::vkCmdPushDescriptorSet2KHR(CommandBuffer, PushDescriptorSetInfo);
		FWrapLayer::CmdPushDescriptorSet2KHR(VK_SUCCESS, CommandBuffer, PushDescriptorSetInfo);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdBeginRenderPass(VkCommandBuffer CommandBuffer, const VkRenderPassBeginInfo* RenderPassBegin, VkSubpassContents Contents)
	{
		FWrapLayer::CmdBeginRenderPass(VK_RESULT_MAX_ENUM, CommandBuffer, RenderPassBegin, Contents);
		VULKANAPINAMESPACE::vkCmdBeginRenderPass(CommandBuffer, RenderPassBegin, Contents);
		FWrapLayer::CmdBeginRenderPass(VK_SUCCESS, CommandBuffer, RenderPassBegin, Contents);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdBeginRenderPass2KHR(VkCommandBuffer CommandBuffer, const VkRenderPassBeginInfo* RenderPassBegin, const VkSubpassBeginInfo* SubpassBeginInfo)
	{
		FWrapLayer::CmdBeginRenderPass2KHR(VK_RESULT_MAX_ENUM, CommandBuffer, RenderPassBegin, SubpassBeginInfo);
		VULKANAPINAMESPACE::vkCmdBeginRenderPass2KHR(CommandBuffer, RenderPassBegin, SubpassBeginInfo);
		FWrapLayer::CmdBeginRenderPass2KHR(VK_SUCCESS, CommandBuffer, RenderPassBegin, SubpassBeginInfo);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdNextSubpass(VkCommandBuffer CommandBuffer, VkSubpassContents Contents)
	{
		FWrapLayer::CmdNextSubpass(VK_RESULT_MAX_ENUM, CommandBuffer, Contents);
		VULKANAPINAMESPACE::vkCmdNextSubpass(CommandBuffer, Contents);
		FWrapLayer::CmdNextSubpass(VK_SUCCESS, CommandBuffer, Contents);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdEndRenderPass(VkCommandBuffer CommandBuffer)
	{
		FWrapLayer::CmdEndRenderPass(VK_RESULT_MAX_ENUM, CommandBuffer);
		VULKANAPINAMESPACE::vkCmdEndRenderPass(CommandBuffer);
		FWrapLayer::CmdEndRenderPass(VK_SUCCESS, CommandBuffer);
	}
	// The VK_EXT_fragment_density_map_offset extension requires EndRenderPass2KHR so that we can pack extra info into the struct chain to specify the offsets
 	static FORCEINLINE_DEBUGGABLE void vkCmdEndRenderPass2KHR(VkCommandBuffer CommandBuffer, const VkSubpassEndInfo* SubpassEndInfo)
 	{
		FWrapLayer::CmdEndRenderPass2KHR(VK_RESULT_MAX_ENUM, CommandBuffer, SubpassEndInfo);
 		VULKANAPINAMESPACE::vkCmdEndRenderPass2KHR(CommandBuffer, SubpassEndInfo);
 		FWrapLayer::CmdEndRenderPass2KHR(VK_SUCCESS, CommandBuffer, SubpassEndInfo);
 	}

	static FORCEINLINE_DEBUGGABLE void vkCmdBeginRenderingKHR(VkCommandBuffer CommandBuffer, const VkRenderingInfo* RenderingInfo)
	{
		FWrapLayer::CmdBeginRenderingKHR(VK_RESULT_MAX_ENUM, CommandBuffer, RenderingInfo);
		VULKANAPINAMESPACE::vkCmdBeginRenderingKHR(CommandBuffer, RenderingInfo);
		FWrapLayer::CmdBeginRenderingKHR(VK_SUCCESS, CommandBuffer, RenderingInfo);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdEndRenderingKHR(VkCommandBuffer CommandBuffer)
	{
		FWrapLayer::CmdEndRenderingKHR(VK_RESULT_MAX_ENUM, CommandBuffer);
		VULKANAPINAMESPACE::vkCmdEndRenderingKHR(CommandBuffer);
		FWrapLayer::CmdEndRenderingKHR(VK_SUCCESS, CommandBuffer);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdEndRendering2KHR(VkCommandBuffer CommandBuffer, const VkRenderingEndInfoKHR* RenderingEndInfo)
	{
		FWrapLayer::CmdEndRendering2KHR(VK_RESULT_MAX_ENUM, CommandBuffer, RenderingEndInfo);
		VULKANAPINAMESPACE::vkCmdEndRendering2KHR(CommandBuffer, RenderingEndInfo);
		FWrapLayer::CmdEndRendering2KHR(VK_SUCCESS, CommandBuffer, RenderingEndInfo);
	}

	static FORCEINLINE_DEBUGGABLE void  vkCmdExecuteCommands(VkCommandBuffer CommandBuffer, uint32 CommandBufferCount, const VkCommandBuffer* pCommandBuffers)
	{
		FWrapLayer::CmdExecuteCommands(VK_RESULT_MAX_ENUM, CommandBuffer, CommandBufferCount, pCommandBuffers);
		VULKANAPINAMESPACE::vkCmdExecuteCommands(CommandBuffer, CommandBufferCount, pCommandBuffers);
		FWrapLayer::CmdExecuteCommands(VK_SUCCESS, CommandBuffer, CommandBufferCount, pCommandBuffers);
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkCreateSwapchainKHR(VkDevice Device, const VkSwapchainCreateInfoKHR* CreateInfo, const VkAllocationCallbacks* Allocator, VkSwapchainKHR* Swapchain)
	{
		FWrapLayer::CreateSwapchainKHR(VK_RESULT_MAX_ENUM, Device, CreateInfo, Swapchain);
		VkResult Result = VULKANAPINAMESPACE::vkCreateSwapchainKHR(Device, CreateInfo, Allocator, Swapchain);
		FWrapLayer::CreateSwapchainKHR(Result, Device, CreateInfo, Swapchain);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void vkDestroySwapchainKHR(VkDevice Device, VkSwapchainKHR Swapchain, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroySwapchainKHR(VK_RESULT_MAX_ENUM, Device, Swapchain);
		VULKANAPINAMESPACE::vkDestroySwapchainKHR(Device, Swapchain, Allocator);
		FWrapLayer::DestroySwapchainKHR(VK_SUCCESS, Device, Swapchain);
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetSwapchainImagesKHR(VkDevice Device, VkSwapchainKHR Swapchain, uint32_t* SwapchainImageCount, VkImage* SwapchainImages)
	{
		FWrapLayer::GetSwapChainImagesKHR(VK_RESULT_MAX_ENUM, Device, Swapchain, SwapchainImageCount, SwapchainImages);
		VkResult Result = VULKANAPINAMESPACE::vkGetSwapchainImagesKHR(Device, Swapchain, SwapchainImageCount, SwapchainImages);
		FWrapLayer::GetSwapChainImagesKHR(Result, Device, Swapchain, SwapchainImageCount, SwapchainImages);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkAcquireNextImageKHR(VkDevice Device, VkSwapchainKHR Swapchain, uint64_t Timeout, VkSemaphore Semaphore, VkFence Fence, uint32_t* ImageIndex)
	{
		FWrapLayer::AcquireNextImageKHR(VK_RESULT_MAX_ENUM, Device, Swapchain, Timeout, Semaphore, Fence, ImageIndex);
		VkResult Result = VULKANAPINAMESPACE::vkAcquireNextImageKHR(Device, Swapchain, Timeout, Semaphore, Fence, ImageIndex);
		FWrapLayer::AcquireNextImageKHR(Result, Device, Swapchain, Timeout, Semaphore, Fence, ImageIndex);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkQueuePresentKHR(VkQueue Queue, const VkPresentInfoKHR* PresentInfo)
	{
		FWrapLayer::QueuePresent(VK_RESULT_MAX_ENUM, Queue, PresentInfo);
		VkResult Result = VULKANAPINAMESPACE::vkQueuePresentKHR(Queue, PresentInfo);
		FWrapLayer::QueuePresent(Result, Queue, PresentInfo);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetPhysicalDeviceSurfaceCapabilitiesKHR(VkPhysicalDevice PhysicalDevice, VkSurfaceKHR Surface, VkSurfaceCapabilitiesKHR* SurfaceCapabilities)
	{
		FWrapLayer::GetPhysicalDeviceSurfaceCapabilitiesKHR(VK_RESULT_MAX_ENUM, PhysicalDevice, Surface, SurfaceCapabilities);
		VkResult Result = VULKANAPINAMESPACE::vkGetPhysicalDeviceSurfaceCapabilitiesKHR(PhysicalDevice, Surface, SurfaceCapabilities);
		FWrapLayer::GetPhysicalDeviceSurfaceCapabilitiesKHR(VK_SUCCESS, PhysicalDevice, Surface, SurfaceCapabilities);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetPhysicalDeviceSurfaceFormatsKHR(VkPhysicalDevice PhysicalDevice, VkSurfaceKHR Surface, uint32_t* SurfaceFormatCountPtr, VkSurfaceFormatKHR* SurfaceFormats)
	{
		FWrapLayer::GetPhysicalDeviceSurfaceFormatsKHR(VK_RESULT_MAX_ENUM, PhysicalDevice, Surface, SurfaceFormatCountPtr, SurfaceFormats);
		VkResult Result = VULKANAPINAMESPACE::vkGetPhysicalDeviceSurfaceFormatsKHR(PhysicalDevice, Surface, SurfaceFormatCountPtr, SurfaceFormats);
		FWrapLayer::GetPhysicalDeviceSurfaceFormatsKHR(VK_SUCCESS, PhysicalDevice, Surface, SurfaceFormatCountPtr, SurfaceFormats);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetPhysicalDeviceSurfaceSupportKHR(VkPhysicalDevice PhysicalDevice, uint32_t QueueFamilyIndex,	VkSurfaceKHR Surface, VkBool32* SupportedPtr)
	{
		FWrapLayer::GetPhysicalDeviceSurfaceSupportKHR(VK_RESULT_MAX_ENUM, PhysicalDevice, QueueFamilyIndex, Surface, SupportedPtr);
		VkResult Result = VULKANAPINAMESPACE::vkGetPhysicalDeviceSurfaceSupportKHR(PhysicalDevice, QueueFamilyIndex, Surface, SupportedPtr);
		FWrapLayer::GetPhysicalDeviceSurfaceSupportKHR(VK_SUCCESS, PhysicalDevice, QueueFamilyIndex, Surface, SupportedPtr);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetPhysicalDeviceSurfacePresentModesKHR(VkPhysicalDevice PhysicalDevice, VkSurfaceKHR Surface, uint32_t* PresentModeCountPtr, VkPresentModeKHR* PresentModesPtr)
	{
		FWrapLayer::GetPhysicalDeviceSurfacePresentModesKHR(VK_RESULT_MAX_ENUM, PhysicalDevice, Surface, PresentModeCountPtr, PresentModesPtr);
		VkResult Result = VULKANAPINAMESPACE::vkGetPhysicalDeviceSurfacePresentModesKHR(PhysicalDevice, Surface, PresentModeCountPtr, PresentModesPtr);
		FWrapLayer::GetPhysicalDeviceSurfacePresentModesKHR(VK_SUCCESS, PhysicalDevice, Surface, PresentModeCountPtr, PresentModesPtr);
		return Result;
	}

#if VULKAN_USE_CREATE_WIN32_SURFACE
	FORCEINLINE_DEBUGGABLE VkResult vkCreateWin32SurfaceKHR(VkInstance Instance, const VkWin32SurfaceCreateInfoKHR* CreateInfo, const VkAllocationCallbacks* Allocator, VkSurfaceKHR* Surface)
	{
		FWrapLayer::CreateWin32SurfaceKHR(VK_RESULT_MAX_ENUM, Instance, CreateInfo, Surface);
		VkResult Result = VULKANAPINAMESPACE::vkCreateWin32SurfaceKHR(Instance, CreateInfo, Allocator, Surface);
		FWrapLayer::CreateWin32SurfaceKHR(Result, Instance, CreateInfo, Surface);
		return Result;
	}
#endif

#if VULKAN_USE_CREATE_ANDROID_SURFACE
	static FORCEINLINE_DEBUGGABLE VkResult vkCreateAndroidSurfaceKHR(VkInstance Instance, const VkAndroidSurfaceCreateInfoKHR* CreateInfo, const VkAllocationCallbacks* Allocator, VkSurfaceKHR* Surface)
	{
		FWrapLayer::CreateAndroidSurfaceKHR(VK_RESULT_MAX_ENUM, Instance, CreateInfo, Surface);
		VkResult Result = VULKANAPINAMESPACE::vkCreateAndroidSurfaceKHR(Instance, CreateInfo, Allocator, Surface);
		FWrapLayer::CreateAndroidSurfaceKHR(Result, Instance, CreateInfo, Surface);
		return Result;
	}
#endif

	static FORCEINLINE_DEBUGGABLE void vkDestroySurfaceKHR(VkInstance Instance, VkSurfaceKHR Surface, const VkAllocationCallbacks* pAllocator)
	{
		FWrapLayer::DestroySurfaceKHR(VK_RESULT_MAX_ENUM, Instance, Surface);
		VULKANAPINAMESPACE::vkDestroySurfaceKHR(Instance, Surface, pAllocator);
		FWrapLayer::DestroySurfaceKHR(VK_SUCCESS, Instance, Surface);
	}

	static FORCEINLINE_DEBUGGABLE void vkGetImageMemoryRequirements2(VkDevice Device, const VkImageMemoryRequirementsInfo2* Info, VkMemoryRequirements2* MemoryRequirements)
	{
		FWrapLayer::GetImageMemoryRequirements2(VK_RESULT_MAX_ENUM, Device, Info, MemoryRequirements);
		VULKANAPINAMESPACE::vkGetImageMemoryRequirements2(Device, Info, MemoryRequirements);
		FWrapLayer::GetImageMemoryRequirements2(VK_SUCCESS, Device, Info, MemoryRequirements);
	}

	static FORCEINLINE_DEBUGGABLE void vkGetBufferMemoryRequirements2(VkDevice Device, const VkBufferMemoryRequirementsInfo2* Info, VkMemoryRequirements2* MemoryRequirements)
	{
		FWrapLayer::GetBufferMemoryRequirements2(VK_RESULT_MAX_ENUM, Device, Info, MemoryRequirements);
		VULKANAPINAMESPACE::vkGetBufferMemoryRequirements2(Device, Info, MemoryRequirements);
		FWrapLayer::GetBufferMemoryRequirements2(VK_SUCCESS, Device, Info, MemoryRequirements);
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkCreateAccelerationStructureKHR(VkDevice Device, const VkAccelerationStructureCreateInfoKHR* CreateInfo, const VkAllocationCallbacks* Allocator, VkAccelerationStructureKHR* AccelerationStructure)
	{
		FWrapLayer::CreateAccelerationStructureKHR(VK_RESULT_MAX_ENUM, Device, CreateInfo, Allocator, AccelerationStructure);
		const VkResult Result = VULKANAPINAMESPACE::vkCreateAccelerationStructureKHR(Device, CreateInfo, Allocator, AccelerationStructure);
		FWrapLayer::CreateAccelerationStructureKHR(Result, Device, CreateInfo, Allocator, AccelerationStructure);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void vkDestroyAccelerationStructureKHR(VkDevice Device, VkAccelerationStructureKHR AccelerationStructure, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyAccelerationStructureKHR(VK_RESULT_MAX_ENUM, Device, AccelerationStructure, Allocator);
		VULKANAPINAMESPACE::vkDestroyAccelerationStructureKHR(Device, AccelerationStructure, Allocator);
		FWrapLayer::DestroyAccelerationStructureKHR(VK_SUCCESS, Device, AccelerationStructure, Allocator);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdBuildAccelerationStructuresKHR(VkCommandBuffer CommandBuffer, uint32 InfoCount, const VkAccelerationStructureBuildGeometryInfoKHR* Infos, const VkAccelerationStructureBuildRangeInfoKHR* const* BuildRangeInfos)
	{
		FWrapLayer::CmdBuildAccelerationStructuresKHR(VK_RESULT_MAX_ENUM, CommandBuffer, InfoCount, Infos, BuildRangeInfos);
		VULKANAPINAMESPACE::vkCmdBuildAccelerationStructuresKHR(CommandBuffer, InfoCount, Infos, BuildRangeInfos);
		FWrapLayer::CmdBuildAccelerationStructuresKHR(VK_SUCCESS, CommandBuffer, InfoCount, Infos, BuildRangeInfos);
	}

	static FORCEINLINE_DEBUGGABLE void vkGetAccelerationStructureBuildSizesKHR(VkDevice Device, VkAccelerationStructureBuildTypeKHR BuildType, const VkAccelerationStructureBuildGeometryInfoKHR* BuildInfo, const uint32* MaxPrimitiveCounts, VkAccelerationStructureBuildSizesInfoKHR* SizeInfo)
	{
		FWrapLayer::GetAccelerationStructureBuildSizesKHR(VK_RESULT_MAX_ENUM, Device, BuildType, BuildInfo, MaxPrimitiveCounts, SizeInfo);
		VULKANAPINAMESPACE::vkGetAccelerationStructureBuildSizesKHR(Device, BuildType, BuildInfo, MaxPrimitiveCounts, SizeInfo);
		FWrapLayer::GetAccelerationStructureBuildSizesKHR(VK_SUCCESS, Device, BuildType, BuildInfo, MaxPrimitiveCounts, SizeInfo);
	}

	static FORCEINLINE_DEBUGGABLE VkDeviceAddress vkGetAccelerationStructureDeviceAddressKHR(VkDevice Device, const VkAccelerationStructureDeviceAddressInfoKHR* Info)
	{
		FWrapLayer::GetAccelerationStructureDeviceAddressKHR(VK_RESULT_MAX_ENUM, Device, Info);
		VkDeviceAddress Result = VULKANAPINAMESPACE::vkGetAccelerationStructureDeviceAddressKHR(Device, Info);
		FWrapLayer::GetAccelerationStructureDeviceAddressKHR(VK_SUCCESS, Device, Info);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdTraceRaysKHR(VkCommandBuffer CommandBuffer, const VkStridedDeviceAddressRegionKHR* RaygenShaderBindingTable, const VkStridedDeviceAddressRegionKHR* MissShaderBindingTable, const VkStridedDeviceAddressRegionKHR* HitShaderBindingTable, const VkStridedDeviceAddressRegionKHR* CallableShaderBindingTable, uint32 width, uint32 height, uint32 depth)
	{
		FWrapLayer::CmdTraceRaysKHR(VK_RESULT_MAX_ENUM, CommandBuffer, RaygenShaderBindingTable, MissShaderBindingTable, HitShaderBindingTable, CallableShaderBindingTable, width, height, depth);
		VULKANAPINAMESPACE::vkCmdTraceRaysKHR(CommandBuffer, RaygenShaderBindingTable, MissShaderBindingTable, HitShaderBindingTable, CallableShaderBindingTable, width, height, depth);
		FWrapLayer::CmdTraceRaysKHR(VK_SUCCESS, CommandBuffer, RaygenShaderBindingTable, MissShaderBindingTable, HitShaderBindingTable, CallableShaderBindingTable, width, height, depth);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdTraceRaysIndirectKHR(VkCommandBuffer CommandBuffer, const VkStridedDeviceAddressRegionKHR* RaygenShaderBindingTable, const VkStridedDeviceAddressRegionKHR* MissShaderBindingTable, const VkStridedDeviceAddressRegionKHR* HitShaderBindingTable, const VkStridedDeviceAddressRegionKHR* CallableShaderBindingTable, VkDeviceAddress IndirectDeviceAddress)
	{
		FWrapLayer::CmdTraceRaysIndirectKHR(VK_RESULT_MAX_ENUM, CommandBuffer, RaygenShaderBindingTable, MissShaderBindingTable, HitShaderBindingTable, CallableShaderBindingTable, IndirectDeviceAddress);
		VULKANAPINAMESPACE::vkCmdTraceRaysIndirectKHR(CommandBuffer, RaygenShaderBindingTable, MissShaderBindingTable, HitShaderBindingTable, CallableShaderBindingTable, IndirectDeviceAddress);
		FWrapLayer::CmdTraceRaysIndirectKHR(VK_SUCCESS, CommandBuffer, RaygenShaderBindingTable, MissShaderBindingTable, HitShaderBindingTable, CallableShaderBindingTable, IndirectDeviceAddress);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdTraceRaysIndirect2KHR(VkCommandBuffer CommandBuffer, VkDeviceAddress IndirectDeviceAddress)
	{
		FWrapLayer::CmdTraceRaysIndirect2KHR(VK_RESULT_MAX_ENUM, CommandBuffer, IndirectDeviceAddress);
		VULKANAPINAMESPACE::vkCmdTraceRaysIndirect2KHR(CommandBuffer, IndirectDeviceAddress);
		FWrapLayer::CmdTraceRaysIndirect2KHR(VK_SUCCESS, CommandBuffer, IndirectDeviceAddress);
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkCreateRayTracingPipelinesKHR(VkDevice Device, VkDeferredOperationKHR DeferredOperation, VkPipelineCache PipelineCache, uint32_t CreateInfoCount, const VkRayTracingPipelineCreateInfoKHR* CreateInfos, const VkAllocationCallbacks* Allocator, VkPipeline* Pipelines)
	{
		FWrapLayer::CreateRayTracingPipelinesKHR(VK_RESULT_MAX_ENUM, Device, DeferredOperation, PipelineCache, CreateInfoCount, CreateInfos, Allocator, Pipelines);
		const VkResult Result = VULKANAPINAMESPACE::vkCreateRayTracingPipelinesKHR(Device, DeferredOperation, PipelineCache, CreateInfoCount, CreateInfos, Allocator, Pipelines);
		FWrapLayer::CreateRayTracingPipelinesKHR(Result, Device, DeferredOperation, PipelineCache, CreateInfoCount, CreateInfos, Allocator, Pipelines);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetRayTracingShaderGroupHandlesKHR(VkDevice Device, VkPipeline Pipeline, uint32_t FirstGroup, uint32_t GroupCount, size_t DataSize, void* Data)
	{
		FWrapLayer::GetRayTracingShaderGroupHandlesKHR(VK_RESULT_MAX_ENUM, Device, Pipeline, FirstGroup, GroupCount, DataSize, Data);
		const VkResult Result = VULKANAPINAMESPACE::vkGetRayTracingShaderGroupHandlesKHR(Device, Pipeline, FirstGroup, GroupCount, DataSize, Data);
		FWrapLayer::GetRayTracingShaderGroupHandlesKHR(Result, Device, Pipeline, FirstGroup, GroupCount, DataSize, Data);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdWriteAccelerationStructuresPropertiesKHR(VkCommandBuffer CommandBuffer,	uint32_t AccelerationStructureCount, const VkAccelerationStructureKHR* AccelerationStructures, VkQueryType QueryType, VkQueryPool QueryPool, uint32_t FirstQuery)
	{
		FWrapLayer::CmdWriteAccelerationStructuresPropertiesKHR(VK_RESULT_MAX_ENUM, CommandBuffer, AccelerationStructureCount, AccelerationStructures, QueryType, QueryPool, FirstQuery);
		VULKANAPINAMESPACE::vkCmdWriteAccelerationStructuresPropertiesKHR(CommandBuffer, AccelerationStructureCount, AccelerationStructures, QueryType, QueryPool, FirstQuery);
		FWrapLayer::CmdWriteAccelerationStructuresPropertiesKHR(VK_SUCCESS, CommandBuffer, AccelerationStructureCount, AccelerationStructures, QueryType, QueryPool, FirstQuery);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdCopyAccelerationStructureKHR(VkCommandBuffer CommandBuffer, const VkCopyAccelerationStructureInfoKHR* Info)
	{
		FWrapLayer::CmdCopyAccelerationStructureKHR(VK_RESULT_MAX_ENUM, CommandBuffer, Info);
		VULKANAPINAMESPACE::vkCmdCopyAccelerationStructureKHR(CommandBuffer, Info);
		FWrapLayer::CmdCopyAccelerationStructureKHR(VK_SUCCESS, CommandBuffer, Info);
	}

	static FORCEINLINE_DEBUGGABLE VkDeviceAddress vkGetBufferDeviceAddressKHR(VkDevice Device, const VkBufferDeviceAddressInfo* Info)
	{
		FWrapLayer::GetBufferDeviceAddressKHR(VK_RESULT_MAX_ENUM, Device, Info);
		const VkDeviceAddress Result = VULKANAPINAMESPACE::vkGetBufferDeviceAddressKHR(Device, Info);
		FWrapLayer::GetBufferDeviceAddressKHR(VK_SUCCESS, Device, Info);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void vkGetDeviceImageMemoryRequirementsKHR(VkDevice Device, const VkDeviceImageMemoryRequirements* Info, VkMemoryRequirements2* MemoryRequirements)
	{
		FWrapLayer::GetDeviceImageMemoryRequirementsKHR(VK_RESULT_MAX_ENUM, Device, Info, MemoryRequirements);
		VULKANAPINAMESPACE::vkGetDeviceImageMemoryRequirementsKHR(Device, Info, MemoryRequirements);
		FWrapLayer::GetDeviceImageMemoryRequirementsKHR(VK_SUCCESS, Device, Info, MemoryRequirements);
	}

	static FORCEINLINE_DEBUGGABLE void vkGetDeviceBufferMemoryRequirementsKHR(VkDevice Device, const VkDeviceBufferMemoryRequirements* Info, VkMemoryRequirements2* MemoryRequirements)
	{
		FWrapLayer::GetDeviceBufferMemoryRequirementsKHR(VK_RESULT_MAX_ENUM, Device, Info, MemoryRequirements);
		VULKANAPINAMESPACE::vkGetDeviceBufferMemoryRequirementsKHR(Device, Info, MemoryRequirements);
		FWrapLayer::GetDeviceBufferMemoryRequirementsKHR(VK_SUCCESS, Device, Info, MemoryRequirements);
	}

	static FORCEINLINE_DEBUGGABLE void vkResetQueryPoolEXT(VkDevice Device, VkQueryPool QueryPool, uint32_t FirstQuery, uint32_t QueryCount)
	{
		FWrapLayer::ResetQueryPoolEXT(VK_RESULT_MAX_ENUM, Device, QueryPool, FirstQuery, QueryCount);
		VULKANAPINAMESPACE::vkResetQueryPoolEXT(Device, QueryPool, FirstQuery, QueryCount);
		FWrapLayer::ResetQueryPoolEXT(VK_SUCCESS, Device, QueryPool, FirstQuery, QueryCount);
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetPhysicalDeviceCalibrateableTimeDomainsEXT(VkPhysicalDevice PhysicalDevice, uint32_t* TimeDomainCount, VkTimeDomainEXT* TimeDomains)
	{
		FWrapLayer::GetPhysicalDeviceCalibrateableTimeDomainsEXT(VK_RESULT_MAX_ENUM, PhysicalDevice, TimeDomainCount, TimeDomains);
		const VkResult Result = VULKANAPINAMESPACE::vkGetPhysicalDeviceCalibrateableTimeDomainsEXT(PhysicalDevice, TimeDomainCount, TimeDomains);
		FWrapLayer::GetPhysicalDeviceCalibrateableTimeDomainsEXT(Result, PhysicalDevice, TimeDomainCount, TimeDomains);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetCalibratedTimestampsEXT(VkDevice Device, uint32_t TimestampCount, const VkCalibratedTimestampInfoEXT* TimestampInfos, uint64_t* Timestamps, uint64_t* MaxDeviation)
	{
		FWrapLayer::GetCalibratedTimestampsEXT(VK_RESULT_MAX_ENUM, Device, TimestampCount, TimestampInfos, Timestamps, MaxDeviation);
		const VkResult Result = VULKANAPINAMESPACE::vkGetCalibratedTimestampsEXT(Device, TimestampCount, TimestampInfos, Timestamps, MaxDeviation);
		FWrapLayer::GetCalibratedTimestampsEXT(Result, Device, TimestampCount, TimestampInfos, Timestamps, MaxDeviation);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkBindBufferMemory2(VkDevice Device, uint32_t BindInfoCount, const VkBindBufferMemoryInfo* BindInfos)
	{
		FWrapLayer::BindBufferMemory2(VK_RESULT_MAX_ENUM, Device, BindInfoCount, BindInfos);
		const VkResult Result = VULKANAPINAMESPACE::vkBindBufferMemory2(Device, BindInfoCount, BindInfos);
		FWrapLayer::BindBufferMemory2(Result, Device, BindInfoCount, BindInfos);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkBindImageMemory2(VkDevice Device, uint32_t BindInfoCount, const VkBindImageMemoryInfo* BindInfos)
	{
		FWrapLayer::BindImageMemory2(VK_RESULT_MAX_ENUM, Device, BindInfoCount, BindInfos);
		const VkResult Result = VULKANAPINAMESPACE::vkBindImageMemory2(Device, BindInfoCount, BindInfos);
		FWrapLayer::BindImageMemory2(Result, Device, BindInfoCount, BindInfos);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdPipelineBarrier2KHR(VkCommandBuffer CommandBuffer, const VkDependencyInfo* DependencyInfo)
	{
		FWrapLayer::CmdPipelineBarrier2KHR(VK_RESULT_MAX_ENUM, CommandBuffer, DependencyInfo);
		VULKANAPINAMESPACE::vkCmdPipelineBarrier2KHR(CommandBuffer, DependencyInfo);
		FWrapLayer::CmdPipelineBarrier2KHR(VK_SUCCESS, CommandBuffer, DependencyInfo);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdResetEvent2KHR(VkCommandBuffer CommandBuffer, VkEvent Event, VkPipelineStageFlags2 StageMask)
	{
		FWrapLayer::CmdResetEvent2KHR(VK_RESULT_MAX_ENUM, CommandBuffer, Event, StageMask);
		VULKANAPINAMESPACE::vkCmdResetEvent2KHR(CommandBuffer, Event, StageMask);
		FWrapLayer::CmdResetEvent2KHR(VK_SUCCESS, CommandBuffer, Event, StageMask);
	}
	
	static FORCEINLINE_DEBUGGABLE void vkCmdSetEvent2KHR(VkCommandBuffer CommandBuffer, VkEvent Event, const VkDependencyInfo* DependencyInfo)
	{
		FWrapLayer::CmdSetEvent2KHR(VK_RESULT_MAX_ENUM, CommandBuffer, Event, DependencyInfo);
		VULKANAPINAMESPACE::vkCmdSetEvent2KHR(CommandBuffer, Event, DependencyInfo);
		FWrapLayer::CmdSetEvent2KHR(VK_SUCCESS, CommandBuffer, Event, DependencyInfo);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdWaitEvents2KHR(VkCommandBuffer CommandBuffer, uint32_t EventCount, const VkEvent* Events, const VkDependencyInfo* DependencyInfos)
	{
		FWrapLayer::CmdWaitEvents2KHR(VK_RESULT_MAX_ENUM, CommandBuffer, EventCount, Events, DependencyInfos);
		VULKANAPINAMESPACE::vkCmdWaitEvents2KHR(CommandBuffer, EventCount, Events, DependencyInfos);
		FWrapLayer::CmdWaitEvents2KHR(VK_SUCCESS, CommandBuffer, EventCount, Events, DependencyInfos);
	}

	static FORCEINLINE_DEBUGGABLE void vkQueueSubmit2KHR(VkQueue Queue, uint32_t SubmitCount, const VkSubmitInfo2* Submits, VkFence Fence)
	{
		FWrapLayer::QueueSubmit2KHR(VK_RESULT_MAX_ENUM, Queue, SubmitCount, Submits, Fence);
		VULKANAPINAMESPACE::vkQueueSubmit2KHR(Queue, SubmitCount, Submits, Fence);
		FWrapLayer::QueueSubmit2KHR(VK_SUCCESS, Queue, SubmitCount, Submits, Fence);
	}

	static FORCEINLINE_DEBUGGABLE void vkGetDescriptorSetLayoutSizeEXT(VkDevice Device, VkDescriptorSetLayout Layout, VkDeviceSize* OutLayoutSizeInBytes)
	{
		FWrapLayer::GetDescriptorSetLayoutSizeEXT(VK_RESULT_MAX_ENUM, Device, Layout, OutLayoutSizeInBytes);
		VULKANAPINAMESPACE::vkGetDescriptorSetLayoutSizeEXT(Device, Layout, OutLayoutSizeInBytes);
		FWrapLayer::GetDescriptorSetLayoutSizeEXT(VK_SUCCESS, Device, Layout, OutLayoutSizeInBytes);
	}

	static FORCEINLINE_DEBUGGABLE void vkGetDescriptorSetLayoutBindingOffsetEXT(VkDevice Device, VkDescriptorSetLayout Layout, uint32_t Binding, VkDeviceSize* Offset)
	{
		FWrapLayer::GetDescriptorSetLayoutBindingOffsetEXT(VK_RESULT_MAX_ENUM, Device, Layout, Binding, Offset);
		VULKANAPINAMESPACE::vkGetDescriptorSetLayoutBindingOffsetEXT(Device, Layout, Binding, Offset);
		FWrapLayer::GetDescriptorSetLayoutBindingOffsetEXT(VK_SUCCESS, Device, Layout, Binding, Offset);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdBindDescriptorBuffersEXT(VkCommandBuffer CommandBuffer, uint32_t BufferCount, const VkDescriptorBufferBindingInfoEXT* BindingInfos)
	{
		FWrapLayer::CmdBindDescriptorBuffersEXT(VK_RESULT_MAX_ENUM, CommandBuffer, BufferCount, BindingInfos);
		VULKANAPINAMESPACE::vkCmdBindDescriptorBuffersEXT(CommandBuffer, BufferCount, BindingInfos);
		FWrapLayer::CmdBindDescriptorBuffersEXT(VK_SUCCESS, CommandBuffer, BufferCount, BindingInfos);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetDescriptorBufferOffsetsEXT(VkCommandBuffer CommandBuffer, VkPipelineBindPoint PipelineBindPoint, VkPipelineLayout Layout, uint32_t FirstSet, uint32_t SetCount, const uint32_t* BufferIndices, const VkDeviceSize* Offsets)
	{
		FWrapLayer::CmdSetDescriptorBufferOffsetsEXT(VK_RESULT_MAX_ENUM, CommandBuffer, PipelineBindPoint, Layout, FirstSet, SetCount, BufferIndices, Offsets);
		VULKANAPINAMESPACE::vkCmdSetDescriptorBufferOffsetsEXT(CommandBuffer, PipelineBindPoint, Layout, FirstSet, SetCount, BufferIndices, Offsets);
		FWrapLayer::CmdSetDescriptorBufferOffsetsEXT(VK_SUCCESS, CommandBuffer, PipelineBindPoint, Layout, FirstSet, SetCount, BufferIndices, Offsets);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetDescriptorBufferOffsets2EXT(VkCommandBuffer CommandBuffer, const VkSetDescriptorBufferOffsetsInfoEXT* SetDescriptorBufferOffsetsInfo)
	{
		FWrapLayer::CmdSetDescriptorBufferOffsets2EXT(VK_RESULT_MAX_ENUM, CommandBuffer, SetDescriptorBufferOffsetsInfo);
		VULKANAPINAMESPACE::vkCmdSetDescriptorBufferOffsets2EXT(CommandBuffer, SetDescriptorBufferOffsetsInfo);
		FWrapLayer::CmdSetDescriptorBufferOffsets2EXT(VK_SUCCESS, CommandBuffer, SetDescriptorBufferOffsetsInfo);
	}

	static FORCEINLINE_DEBUGGABLE void vkGetDescriptorEXT(VkDevice Device, const VkDescriptorGetInfoEXT* DescriptorInfo, size_t DataSize, void* Descriptor)
	{
		FWrapLayer::GetDescriptorEXT(VK_RESULT_MAX_ENUM, Device, DescriptorInfo, DataSize, Descriptor);
		VULKANAPINAMESPACE::vkGetDescriptorEXT(Device, DescriptorInfo,  DataSize, Descriptor);
		FWrapLayer::GetDescriptorEXT(VK_SUCCESS, Device, DescriptorInfo,  DataSize, Descriptor);
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkCreateDeferredOperationKHR(VkDevice Device, const VkAllocationCallbacks* Allocator, VkDeferredOperationKHR* DeferredOperation)
	{
		FWrapLayer::CreateDeferredOperationKHR(VK_RESULT_MAX_ENUM, Device, Allocator, DeferredOperation);
		VkResult Result = VULKANAPINAMESPACE::vkCreateDeferredOperationKHR(Device, Allocator, DeferredOperation);
		FWrapLayer::CreateDeferredOperationKHR(Result, Device, Allocator, DeferredOperation);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void vkDestroyDeferredOperationKHR(VkDevice Device, VkDeferredOperationKHR DeferredOperation, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyDeferredOperationKHR(VK_RESULT_MAX_ENUM, Device, DeferredOperation, Allocator);
		VULKANAPINAMESPACE::vkDestroyDeferredOperationKHR(Device, DeferredOperation, Allocator);
		FWrapLayer::DestroyDeferredOperationKHR(VK_SUCCESS, Device, DeferredOperation, Allocator);
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkDeferredOperationJoinKHR(VkDevice Device, VkDeferredOperationKHR DeferredOperation)
	{
		FWrapLayer::DeferredOperationJoinKHR(VK_RESULT_MAX_ENUM, Device, DeferredOperation);
		VkResult Result = VULKANAPINAMESPACE::vkDeferredOperationJoinKHR(Device, DeferredOperation);
		FWrapLayer::DeferredOperationJoinKHR(Result, Device, DeferredOperation);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE uint32_t vkGetDeferredOperationMaxConcurrencyKHR(VkDevice Device, VkDeferredOperationKHR DeferredOperation)
	{
		FWrapLayer::GetDeferredOperationMaxConcurrencyKHR(VK_RESULT_MAX_ENUM, Device, DeferredOperation);
		uint32_t Result = VULKANAPINAMESPACE::vkGetDeferredOperationMaxConcurrencyKHR(Device, DeferredOperation);
		FWrapLayer::GetDeferredOperationMaxConcurrencyKHR(VK_SUCCESS, Device, DeferredOperation);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetDeferredOperationResultKHR(VkDevice Device, VkDeferredOperationKHR DeferredOperation)
	{
		FWrapLayer::GetDeferredOperationResultKHR(VK_RESULT_MAX_ENUM, Device, DeferredOperation);
		VkResult Result = VULKANAPINAMESPACE::vkGetDeferredOperationResultKHR(Device, DeferredOperation);
		FWrapLayer::GetDeferredOperationResultKHR(Result, Device, DeferredOperation);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetDeviceFaultInfoEXT(VkDevice Device, VkDeviceFaultCountsEXT* FaultCounts, VkDeviceFaultInfoEXT* FaultInfo)
	{
		FWrapLayer::GetDeviceFaultInfoEXT(VK_RESULT_MAX_ENUM, Device, FaultCounts, FaultInfo);
		VkResult Result = VULKANAPINAMESPACE::vkGetDeviceFaultInfoEXT(Device, FaultCounts, FaultInfo);
		FWrapLayer::GetDeviceFaultInfoEXT(Result, Device, FaultCounts, FaultInfo);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetPhysicalDeviceToolPropertiesEXT(VkPhysicalDevice PhysicalDevice, uint32_t* ToolCount, VkPhysicalDeviceToolPropertiesEXT* ToolProperties)
	{
		FWrapLayer::GetPhysicalDeviceToolPropertiesEXT(VK_RESULT_MAX_ENUM, PhysicalDevice, ToolCount, ToolProperties);
		VkResult Result = VULKANAPINAMESPACE::vkGetPhysicalDeviceToolPropertiesEXT(PhysicalDevice, ToolCount, ToolProperties);
		FWrapLayer::GetPhysicalDeviceToolPropertiesEXT(Result, PhysicalDevice, ToolCount, ToolProperties);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkCreateSamplerYcbcrConversion(VkDevice Device, const VkSamplerYcbcrConversionCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkSamplerYcbcrConversion* YcbcrConversion)
	{
		FWrapLayer::CreateSamplerYcbcrConversion(VK_RESULT_MAX_ENUM, Device, CreateInfo, Allocator, YcbcrConversion);
		VkResult Result = VULKANAPINAMESPACE::vkCreateSamplerYcbcrConversion(Device, CreateInfo, Allocator, YcbcrConversion);
		FWrapLayer::CreateSamplerYcbcrConversion(Result, Device, CreateInfo, Allocator, YcbcrConversion);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void vkDestroySamplerYcbcrConversion(VkDevice Device, VkSamplerYcbcrConversion YcbcrConversion, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroySamplerYcbcrConversion(VK_RESULT_MAX_ENUM, Device, YcbcrConversion, Allocator);
		VULKANAPINAMESPACE::vkDestroySamplerYcbcrConversion(Device, YcbcrConversion, Allocator);
		FWrapLayer::DestroySamplerYcbcrConversion(VK_SUCCESS, Device, YcbcrConversion, Allocator);
	}

#if PLATFORM_ANDROID
	static FORCEINLINE_DEBUGGABLE VkResult vkGetAndroidHardwareBufferPropertiesANDROID(VkDevice Device, const struct AHardwareBuffer* Buffer, VkAndroidHardwareBufferPropertiesANDROID* Properties)
	{
		FWrapLayer::GetAndroidHardwareBufferPropertiesANDROID(VK_RESULT_MAX_ENUM, Device, Buffer, Properties);
		VkResult Result = VULKANAPINAMESPACE::vkGetAndroidHardwareBufferPropertiesANDROID(Device, Buffer, Properties);
		FWrapLayer::GetAndroidHardwareBufferPropertiesANDROID(Result, Device, Buffer, Properties);
		return Result;
	}
#endif // PLATFORM_ANDROID

	static FORCEINLINE_DEBUGGABLE VkResult vkGetSemaphoreCounterValueKHR(VkDevice Device, VkSemaphore Semaphore, uint64_t* Value)
	{
		FWrapLayer::GetSemaphoreCounterValueKHR(VK_RESULT_MAX_ENUM, Device, Semaphore, Value);
		const VkResult Result = VULKANAPINAMESPACE::vkGetSemaphoreCounterValueKHR(Device, Semaphore, Value);
		FWrapLayer::GetSemaphoreCounterValueKHR(Result, Device, Semaphore, Value);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkWaitSemaphoresKHR(VkDevice Device, const VkSemaphoreWaitInfo* WaitInfo, uint64_t Timeout)
	{
		FWrapLayer::WaitSemaphoresKHR(VK_RESULT_MAX_ENUM, Device, WaitInfo, Timeout);
		const VkResult Result = VULKANAPINAMESPACE::vkWaitSemaphoresKHR(Device, WaitInfo, Timeout);
		FWrapLayer::WaitSemaphoresKHR(Result, Device, WaitInfo, Timeout);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkSignalSemaphoreKHR(VkDevice Device, const VkSemaphoreSignalInfo* SignalInfo)
	{
		FWrapLayer::SignalSemaphoreKHR(VK_RESULT_MAX_ENUM, Device, SignalInfo);
		const VkResult Result = VULKANAPINAMESPACE::vkSignalSemaphoreKHR(Device, SignalInfo);
		FWrapLayer::SignalSemaphoreKHR(Result, Device, SignalInfo);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkCreateShadersEXT(VkDevice Device, uint32_t CreateInfoCount, const VkShaderCreateInfoEXT* CreateInfos, const VkAllocationCallbacks* Allocator, VkShaderEXT* Shaders)
	{
		FWrapLayer::CreateShadersEXT(VK_RESULT_MAX_ENUM, Device, CreateInfoCount, CreateInfos, Allocator, Shaders);
		const VkResult Result = VULKANAPINAMESPACE::vkCreateShadersEXT(Device, CreateInfoCount, CreateInfos, Allocator, Shaders);
		FWrapLayer::CreateShadersEXT(Result, Device, CreateInfoCount, CreateInfos, Allocator, Shaders);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void vkDestroyShaderEXT(VkDevice Device, VkShaderEXT Shader, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyShaderEXT(VK_RESULT_MAX_ENUM, Device, Shader, Allocator);
		VULKANAPINAMESPACE::vkDestroyShaderEXT(Device, Shader, Allocator);
		FWrapLayer::DestroyShaderEXT(VK_SUCCESS, Device, Shader, Allocator);
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetShaderBinaryDataEXT(VkDevice Device, VkShaderEXT Shader, size_t* DataSize, void* Data)
	{
		FWrapLayer::GetShaderBinaryDataEXT(VK_RESULT_MAX_ENUM, Device, Shader, DataSize, Data);
		const VkResult Result = VULKANAPINAMESPACE::vkGetShaderBinaryDataEXT(Device, Shader, DataSize, Data);
		FWrapLayer::GetShaderBinaryDataEXT(Result, Device, Shader, DataSize, Data);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdBindShadersEXT(VkCommandBuffer CommandBuffer, uint32_t StageCount, const VkShaderStageFlagBits* Stages, const VkShaderEXT* Shaders)
	{
		FWrapLayer::CmdBindShadersEXT(VK_RESULT_MAX_ENUM, CommandBuffer, StageCount, Stages, Shaders);
		VULKANAPINAMESPACE::vkCmdBindShadersEXT(CommandBuffer, StageCount, Stages, Shaders);
		FWrapLayer::CmdBindShadersEXT(VK_SUCCESS, CommandBuffer, StageCount, Stages, Shaders);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdBindVertexBuffers2EXT(VkCommandBuffer CommandBuffer, uint32_t FirstBinding, uint32_t BindingCount, const VkBuffer* Buffers, const VkDeviceSize* Offsets, const VkDeviceSize* Sizes, const VkDeviceSize* Strides)
	{
		FWrapLayer::CmdBindVertexBuffers2EXT(VK_RESULT_MAX_ENUM, CommandBuffer, FirstBinding, BindingCount, Buffers, Offsets, Sizes, Strides);
		VULKANAPINAMESPACE::vkCmdBindVertexBuffers2EXT(CommandBuffer, FirstBinding, BindingCount, Buffers, Offsets, Sizes, Strides);
		FWrapLayer::CmdBindVertexBuffers2EXT(VK_SUCCESS, CommandBuffer, FirstBinding, BindingCount, Buffers, Offsets, Sizes, Strides);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetCullModeEXT(VkCommandBuffer CommandBuffer, VkCullModeFlags CullMode)
	{
		FWrapLayer::CmdSetCullModeEXT(VK_RESULT_MAX_ENUM, CommandBuffer, CullMode);
		VULKANAPINAMESPACE::vkCmdSetCullModeEXT(CommandBuffer, CullMode);
		FWrapLayer::CmdSetCullModeEXT(VK_SUCCESS, CommandBuffer, CullMode);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetColorBlendEnableEXT(VkCommandBuffer CommandBuffer, uint32_t FirstAttachment, uint32_t AttachmentCount, const VkBool32* ColorBlendEnables)
	{
		FWrapLayer::CmdSetColorBlendEnableEXT(VK_RESULT_MAX_ENUM, CommandBuffer, FirstAttachment, AttachmentCount, ColorBlendEnables);
		VULKANAPINAMESPACE::vkCmdSetColorBlendEnableEXT(CommandBuffer, FirstAttachment, AttachmentCount, ColorBlendEnables);
		FWrapLayer::CmdSetColorBlendEnableEXT(VK_SUCCESS, CommandBuffer, FirstAttachment, AttachmentCount, ColorBlendEnables);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetColorBlendEquationEXT(VkCommandBuffer CommandBuffer, uint32_t FirstAttachment, uint32_t AttachmentCount, const VkColorBlendEquationEXT* ColorBlendEquations)
	{
		FWrapLayer::CmdSetColorBlendEquationEXT(VK_RESULT_MAX_ENUM, CommandBuffer, FirstAttachment, AttachmentCount, ColorBlendEquations);
		VULKANAPINAMESPACE::vkCmdSetColorBlendEquationEXT(CommandBuffer, FirstAttachment, AttachmentCount, ColorBlendEquations);
		FWrapLayer::CmdSetColorBlendEquationEXT(VK_SUCCESS, CommandBuffer, FirstAttachment, AttachmentCount, ColorBlendEquations);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetColorWriteMaskEXT(VkCommandBuffer CommandBuffer, uint32_t FirstAttachment, uint32_t AttachmentCount, const VkColorComponentFlags* ColorWriteMasks)
	{
		FWrapLayer::CmdSetColorWriteMaskEXT(VK_RESULT_MAX_ENUM, CommandBuffer, FirstAttachment, AttachmentCount, ColorWriteMasks);
		VULKANAPINAMESPACE::vkCmdSetColorWriteMaskEXT(CommandBuffer, FirstAttachment, AttachmentCount, ColorWriteMasks);
		FWrapLayer::CmdSetColorWriteMaskEXT(VK_SUCCESS, CommandBuffer, FirstAttachment, AttachmentCount, ColorWriteMasks);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetFrontFaceEXT(VkCommandBuffer CommandBuffer, VkFrontFace FrontFace)
	{
		FWrapLayer::CmdSetFrontFaceEXT(VK_RESULT_MAX_ENUM, CommandBuffer, FrontFace);
		VULKANAPINAMESPACE::vkCmdSetFrontFaceEXT(CommandBuffer, FrontFace);
		FWrapLayer::CmdSetFrontFaceEXT(VK_SUCCESS, CommandBuffer, FrontFace);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetPolygonModeEXT(VkCommandBuffer CommandBuffer, VkPolygonMode PolygonMode)
	{
		FWrapLayer::CmdSetPolygonModeEXT(VK_RESULT_MAX_ENUM, CommandBuffer, PolygonMode);
		VULKANAPINAMESPACE::vkCmdSetPolygonModeEXT(CommandBuffer, PolygonMode);
		FWrapLayer::CmdSetPolygonModeEXT(VK_SUCCESS, CommandBuffer, PolygonMode);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetPrimitiveTopologyEXT(VkCommandBuffer CommandBuffer, VkPrimitiveTopology PrimitiveTopology)
	{
		FWrapLayer::CmdSetPrimitiveTopologyEXT(VK_RESULT_MAX_ENUM, CommandBuffer, PrimitiveTopology);
		VULKANAPINAMESPACE::vkCmdSetPrimitiveTopologyEXT(CommandBuffer, PrimitiveTopology);
		FWrapLayer::CmdSetPrimitiveTopologyEXT(VK_SUCCESS, CommandBuffer, PrimitiveTopology);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetRasterizationSamplesEXT(VkCommandBuffer CommandBuffer, VkSampleCountFlagBits RasterizationSamples)
	{
		FWrapLayer::CmdSetRasterizationSamplesEXT(VK_RESULT_MAX_ENUM, CommandBuffer, RasterizationSamples);
		VULKANAPINAMESPACE::vkCmdSetRasterizationSamplesEXT(CommandBuffer, RasterizationSamples);
		FWrapLayer::CmdSetRasterizationSamplesEXT(VK_SUCCESS, CommandBuffer, RasterizationSamples);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetScissorWithCountEXT(VkCommandBuffer CommandBuffer, uint32_t ScissorCount, const VkRect2D* Scissors)
	{
		FWrapLayer::CmdSetScissorWithCountEXT(VK_RESULT_MAX_ENUM, CommandBuffer, ScissorCount, Scissors);
		VULKANAPINAMESPACE::vkCmdSetScissorWithCountEXT(CommandBuffer, ScissorCount, Scissors);
		FWrapLayer::CmdSetScissorWithCountEXT(VK_SUCCESS, CommandBuffer, ScissorCount, Scissors);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetStencilOpEXT(VkCommandBuffer CommandBuffer, VkStencilFaceFlags FaceMask, VkStencilOp FailOp, VkStencilOp PassOp, VkStencilOp DepthFailOp, VkCompareOp CompareOp)
	{
		FWrapLayer::CmdSetStencilOpEXT(VK_RESULT_MAX_ENUM, CommandBuffer, FaceMask, FailOp, PassOp, DepthFailOp, CompareOp);
		VULKANAPINAMESPACE::vkCmdSetStencilOpEXT(CommandBuffer, FaceMask, FailOp, PassOp, DepthFailOp, CompareOp);
		FWrapLayer::CmdSetStencilOpEXT(VK_SUCCESS, CommandBuffer, FaceMask, FailOp, PassOp, DepthFailOp, CompareOp);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetStencilTestEnableEXT(VkCommandBuffer CommandBuffer, VkBool32 StencilTestEnable)
	{
		FWrapLayer::CmdSetStencilTestEnableEXT(VK_RESULT_MAX_ENUM, CommandBuffer, StencilTestEnable);
		VULKANAPINAMESPACE::vkCmdSetStencilTestEnableEXT(CommandBuffer, StencilTestEnable);
		FWrapLayer::CmdSetStencilTestEnableEXT(VK_SUCCESS, CommandBuffer, StencilTestEnable);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetVertexInputEXT(VkCommandBuffer CommandBuffer, uint32_t VertexBindingDescriptionCount, const VkVertexInputBindingDescription2EXT* VertexBindingDescriptions, uint32_t VertexAttributeDescriptionCount, const VkVertexInputAttributeDescription2EXT* VertexAttributeDescriptions)
	{
		FWrapLayer::CmdSetVertexInputEXT(VK_RESULT_MAX_ENUM, CommandBuffer, VertexBindingDescriptionCount, VertexBindingDescriptions, VertexAttributeDescriptionCount, VertexAttributeDescriptions);
		VULKANAPINAMESPACE::vkCmdSetVertexInputEXT(CommandBuffer, VertexBindingDescriptionCount, VertexBindingDescriptions, VertexAttributeDescriptionCount, VertexAttributeDescriptions);
		FWrapLayer::CmdSetVertexInputEXT(VK_SUCCESS, CommandBuffer, VertexBindingDescriptionCount, VertexBindingDescriptions, VertexAttributeDescriptionCount, VertexAttributeDescriptions);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetViewportWithCountEXT(VkCommandBuffer CommandBuffer, uint32_t ViewportCount, const VkViewport* Viewports)
	{
		FWrapLayer::CmdSetViewportWithCountEXT(VK_RESULT_MAX_ENUM, CommandBuffer, ViewportCount, Viewports);
		VULKANAPINAMESPACE::vkCmdSetViewportWithCountEXT(CommandBuffer, ViewportCount, Viewports);
		FWrapLayer::CmdSetViewportWithCountEXT(VK_SUCCESS, CommandBuffer, ViewportCount, Viewports);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetDepthBiasEnableEXT(VkCommandBuffer CommandBuffer, VkBool32 DepthBiasEnable)
	{
		FWrapLayer::CmdSetDepthBiasEnableEXT(VK_RESULT_MAX_ENUM, CommandBuffer, DepthBiasEnable);
		VULKANAPINAMESPACE::vkCmdSetDepthBiasEnableEXT(CommandBuffer, DepthBiasEnable);
		FWrapLayer::CmdSetDepthBiasEnableEXT(VK_SUCCESS, CommandBuffer, DepthBiasEnable);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetDepthBoundsTestEnableEXT(VkCommandBuffer CommandBuffer, VkBool32 DepthBoundsTestEnable)
	{
		FWrapLayer::CmdSetDepthBoundsTestEnableEXT(VK_RESULT_MAX_ENUM, CommandBuffer, DepthBoundsTestEnable);
		VULKANAPINAMESPACE::vkCmdSetDepthBoundsTestEnableEXT(CommandBuffer, DepthBoundsTestEnable);
		FWrapLayer::CmdSetDepthBoundsTestEnableEXT(VK_SUCCESS, CommandBuffer, DepthBoundsTestEnable);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetDepthClampEnableEXT(VkCommandBuffer CommandBuffer, VkBool32 DepthClampEnable)
	{
		FWrapLayer::CmdSetDepthClampEnableEXT(VK_RESULT_MAX_ENUM, CommandBuffer, DepthClampEnable);
		VULKANAPINAMESPACE::vkCmdSetDepthClampEnableEXT(CommandBuffer, DepthClampEnable);
		FWrapLayer::CmdSetDepthClampEnableEXT(VK_SUCCESS, CommandBuffer, DepthClampEnable);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetDepthCompareOpEXT(VkCommandBuffer CommandBuffer, VkCompareOp DepthCompareOp)
	{
		FWrapLayer::CmdSetDepthCompareOpEXT(VK_RESULT_MAX_ENUM, CommandBuffer, DepthCompareOp);
		VULKANAPINAMESPACE::vkCmdSetDepthCompareOpEXT(CommandBuffer, DepthCompareOp);
		FWrapLayer::CmdSetDepthCompareOpEXT(VK_SUCCESS, CommandBuffer, DepthCompareOp);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetDepthTestEnableEXT(VkCommandBuffer CommandBuffer, VkBool32 DepthTestEnable)
	{
		FWrapLayer::CmdSetDepthTestEnableEXT(VK_RESULT_MAX_ENUM, CommandBuffer, DepthTestEnable);
		VULKANAPINAMESPACE::vkCmdSetDepthTestEnableEXT(CommandBuffer, DepthTestEnable);
		FWrapLayer::CmdSetDepthTestEnableEXT(VK_SUCCESS, CommandBuffer, DepthTestEnable);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetDepthWriteEnableEXT(VkCommandBuffer CommandBuffer, VkBool32 DepthWriteEnable)
	{
		FWrapLayer::CmdSetDepthWriteEnableEXT(VK_RESULT_MAX_ENUM, CommandBuffer, DepthWriteEnable);
		VULKANAPINAMESPACE::vkCmdSetDepthWriteEnableEXT(CommandBuffer, DepthWriteEnable);
		FWrapLayer::CmdSetDepthWriteEnableEXT(VK_SUCCESS, CommandBuffer, DepthWriteEnable);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetRasterizerDiscardEnableEXT(VkCommandBuffer CommandBuffer, VkBool32 RasterizerDiscardEnable)
	{
		FWrapLayer::CmdSetRasterizerDiscardEnableEXT(VK_RESULT_MAX_ENUM, CommandBuffer, RasterizerDiscardEnable);
		VULKANAPINAMESPACE::vkCmdSetRasterizerDiscardEnableEXT(CommandBuffer, RasterizerDiscardEnable);
		FWrapLayer::CmdSetRasterizerDiscardEnableEXT(VK_SUCCESS, CommandBuffer, RasterizerDiscardEnable);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetPrimitiveRestartEnableEXT(VkCommandBuffer CommandBuffer, VkBool32 PrimitiveRestartEnable)
	{
		FWrapLayer::CmdSetPrimitiveRestartEnableEXT(VK_RESULT_MAX_ENUM, CommandBuffer, PrimitiveRestartEnable);
		VULKANAPINAMESPACE::vkCmdSetPrimitiveRestartEnableEXT(CommandBuffer, PrimitiveRestartEnable);
		FWrapLayer::CmdSetPrimitiveRestartEnableEXT(VK_SUCCESS, CommandBuffer, PrimitiveRestartEnable);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetLogicOpEnableEXT(VkCommandBuffer CommandBuffer, VkBool32 LogicOpEnable)
	{
		FWrapLayer::CmdSetLogicOpEnableEXT(VK_RESULT_MAX_ENUM, CommandBuffer, LogicOpEnable);
		VULKANAPINAMESPACE::vkCmdSetLogicOpEnableEXT(CommandBuffer, LogicOpEnable);
		FWrapLayer::CmdSetLogicOpEnableEXT(VK_SUCCESS, CommandBuffer, LogicOpEnable);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetAlphaToCoverageEnableEXT(VkCommandBuffer CommandBuffer, VkBool32 AlphaToCoverageEnable)
	{
		FWrapLayer::CmdSetAlphaToCoverageEnableEXT(VK_RESULT_MAX_ENUM, CommandBuffer, AlphaToCoverageEnable);
		VULKANAPINAMESPACE::vkCmdSetAlphaToCoverageEnableEXT(CommandBuffer, AlphaToCoverageEnable);
		FWrapLayer::CmdSetAlphaToCoverageEnableEXT(VK_SUCCESS, CommandBuffer, AlphaToCoverageEnable);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetAlphaToOneEnableEXT(VkCommandBuffer CommandBuffer, VkBool32 AlphaToOneEnable)
	{
		FWrapLayer::CmdSetAlphaToOneEnableEXT(VK_RESULT_MAX_ENUM, CommandBuffer, AlphaToOneEnable);
		VULKANAPINAMESPACE::vkCmdSetAlphaToOneEnableEXT(CommandBuffer, AlphaToOneEnable);
		FWrapLayer::CmdSetAlphaToOneEnableEXT(VK_SUCCESS, CommandBuffer, AlphaToOneEnable);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetSampleMaskEXT(VkCommandBuffer CommandBuffer, VkSampleCountFlagBits Samples, const VkSampleMask* SampleMask)
	{
		FWrapLayer::CmdSetSampleMaskEXT(VK_RESULT_MAX_ENUM, CommandBuffer, Samples, SampleMask);
		VULKANAPINAMESPACE::vkCmdSetSampleMaskEXT(CommandBuffer, Samples, SampleMask);
		FWrapLayer::CmdSetSampleMaskEXT(VK_SUCCESS, CommandBuffer, Samples, SampleMask);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetFragmentShadingRateKHR(VkCommandBuffer CommandBuffer, const VkExtent2D* FragmentSize, const VkFragmentShadingRateCombinerOpKHR combinerOps[2])
	{
		FWrapLayer::CmdSetFragmentShadingRateKHR(VK_RESULT_MAX_ENUM, CommandBuffer, FragmentSize, combinerOps);
		VULKANAPINAMESPACE::vkCmdSetFragmentShadingRateKHR(CommandBuffer, FragmentSize, combinerOps);
		FWrapLayer::CmdSetFragmentShadingRateKHR(VK_SUCCESS, CommandBuffer, FragmentSize, combinerOps);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdExecuteGeneratedCommandsEXT(VkCommandBuffer CommandBuffer, VkBool32 IsPreprocessed, const VkGeneratedCommandsInfoEXT* GeneratedCommandsInfo)
	{
		FWrapLayer::CmdExecuteGeneratedCommandsEXT(VK_RESULT_MAX_ENUM, CommandBuffer, IsPreprocessed, GeneratedCommandsInfo);
		VULKANAPINAMESPACE::vkCmdExecuteGeneratedCommandsEXT(CommandBuffer, IsPreprocessed, GeneratedCommandsInfo);
		FWrapLayer::CmdExecuteGeneratedCommandsEXT(VK_SUCCESS, CommandBuffer, IsPreprocessed, GeneratedCommandsInfo);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdPreprocessGeneratedCommandsEXT(VkCommandBuffer CommandBuffer, const VkGeneratedCommandsInfoEXT* GeneratedCommandsInfo, VkCommandBuffer StateCommandBuffer)
	{
		FWrapLayer::CmdPreprocessGeneratedCommandsEXT(VK_RESULT_MAX_ENUM, CommandBuffer, GeneratedCommandsInfo, StateCommandBuffer);
		VULKANAPINAMESPACE::vkCmdPreprocessGeneratedCommandsEXT(CommandBuffer, GeneratedCommandsInfo, StateCommandBuffer);
		FWrapLayer::CmdPreprocessGeneratedCommandsEXT(VK_SUCCESS, CommandBuffer, GeneratedCommandsInfo, StateCommandBuffer);
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkCreateIndirectCommandsLayoutEXT(VkDevice Device, const VkIndirectCommandsLayoutCreateInfoEXT* CreateInfo, const VkAllocationCallbacks* Allocator, VkIndirectCommandsLayoutEXT* IndirectCommandsLayout)
	{
		FWrapLayer::CreateIndirectCommandsLayoutEXT(VK_RESULT_MAX_ENUM, Device, CreateInfo, Allocator, IndirectCommandsLayout);
		VkResult Result = VULKANAPINAMESPACE::vkCreateIndirectCommandsLayoutEXT(Device, CreateInfo, Allocator, IndirectCommandsLayout);
		FWrapLayer::CreateIndirectCommandsLayoutEXT(VK_SUCCESS, Device, CreateInfo, Allocator, IndirectCommandsLayout);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkCreateIndirectExecutionSetEXT(VkDevice Device, const VkIndirectExecutionSetCreateInfoEXT* CreateInfo, const VkAllocationCallbacks* Allocator, VkIndirectExecutionSetEXT* IndirectExecutionSet)
	{
		FWrapLayer::CreateIndirectExecutionSetEXT(VK_RESULT_MAX_ENUM, Device, CreateInfo, Allocator, IndirectExecutionSet);
		VkResult Result = VULKANAPINAMESPACE::vkCreateIndirectExecutionSetEXT(Device, CreateInfo, Allocator, IndirectExecutionSet);
		FWrapLayer::CreateIndirectExecutionSetEXT(VK_SUCCESS, Device, CreateInfo, Allocator, IndirectExecutionSet);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void vkDestroyIndirectCommandsLayoutEXT(VkDevice Device, VkIndirectCommandsLayoutEXT IndirectCommandsLayout, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyIndirectCommandsLayoutEXT(VK_RESULT_MAX_ENUM, Device, IndirectCommandsLayout, Allocator);
		VULKANAPINAMESPACE::vkDestroyIndirectCommandsLayoutEXT(Device, IndirectCommandsLayout, Allocator);
		FWrapLayer::DestroyIndirectCommandsLayoutEXT(VK_SUCCESS, Device, IndirectCommandsLayout, Allocator);
	}

	static FORCEINLINE_DEBUGGABLE void vkDestroyIndirectExecutionSetEXT(VkDevice Device, VkIndirectExecutionSetEXT IndirectExecutionSet, const VkAllocationCallbacks* Allocator)
	{
		FWrapLayer::DestroyIndirectExecutionSetEXT(VK_RESULT_MAX_ENUM, Device, IndirectExecutionSet, Allocator);
		VULKANAPINAMESPACE::vkDestroyIndirectExecutionSetEXT(Device, IndirectExecutionSet, Allocator);
		FWrapLayer::DestroyIndirectExecutionSetEXT(VK_SUCCESS, Device, IndirectExecutionSet, Allocator);
	}

	static FORCEINLINE_DEBUGGABLE void vkGetGeneratedCommandsMemoryRequirementsEXT(VkDevice Device, const VkGeneratedCommandsMemoryRequirementsInfoEXT* Info, VkMemoryRequirements2* MemoryRequirements)
	{
		FWrapLayer::GetGeneratedCommandsMemoryRequirementsEXT(VK_RESULT_MAX_ENUM, Device, Info, MemoryRequirements);
		VULKANAPINAMESPACE::vkGetGeneratedCommandsMemoryRequirementsEXT(Device, Info, MemoryRequirements);
		FWrapLayer::GetGeneratedCommandsMemoryRequirementsEXT(VK_SUCCESS, Device, Info, MemoryRequirements);
	}

	static FORCEINLINE_DEBUGGABLE void vkUpdateIndirectExecutionSetPipelineEXT(VkDevice Device, VkIndirectExecutionSetEXT IndirectExecutionSet, uint32_t ExecutionSetWriteCount, const VkWriteIndirectExecutionSetPipelineEXT* ExecutionSetWrites)
	{
		FWrapLayer::UpdateIndirectExecutionSetPipelineEXT(VK_RESULT_MAX_ENUM, Device, IndirectExecutionSet, ExecutionSetWriteCount, ExecutionSetWrites);
		VULKANAPINAMESPACE::vkUpdateIndirectExecutionSetPipelineEXT(Device, IndirectExecutionSet, ExecutionSetWriteCount, ExecutionSetWrites);
		FWrapLayer::UpdateIndirectExecutionSetPipelineEXT(VK_SUCCESS, Device, IndirectExecutionSet, ExecutionSetWriteCount, ExecutionSetWrites);
	}

	static FORCEINLINE_DEBUGGABLE void vkUpdateIndirectExecutionSetShaderEXT(VkDevice Device, VkIndirectExecutionSetEXT IndirectExecutionSet, uint32_t ExecutionSetWriteCount, const VkWriteIndirectExecutionSetShaderEXT* ExecutionSetWrites)
	{
		FWrapLayer::UpdateIndirectExecutionSetShaderEXT(VK_RESULT_MAX_ENUM, Device, IndirectExecutionSet, ExecutionSetWriteCount, ExecutionSetWrites);
		VULKANAPINAMESPACE::vkUpdateIndirectExecutionSetShaderEXT(Device, IndirectExecutionSet, ExecutionSetWriteCount, ExecutionSetWrites);
		FWrapLayer::UpdateIndirectExecutionSetShaderEXT(VK_SUCCESS, Device, IndirectExecutionSet, ExecutionSetWriteCount, ExecutionSetWrites);
	}


	static FORCEINLINE_DEBUGGABLE VkResult vkCopyMemoryToImageEXT(VkDevice Device, const VkCopyMemoryToImageInfo* CopyMemoryToImageInfo)
	{
		FWrapLayer::CopyMemoryToImageEXT(VK_RESULT_MAX_ENUM, Device, CopyMemoryToImageInfo);
		const VkResult Result = VULKANAPINAMESPACE::vkCopyMemoryToImageEXT(Device, CopyMemoryToImageInfo);
		FWrapLayer::CopyMemoryToImageEXT(Result, Device, CopyMemoryToImageInfo);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkTransitionImageLayoutEXT(VkDevice Device, uint32_t TransitionCount, const VkHostImageLayoutTransitionInfo* Transitions)
	{
		FWrapLayer::TransitionImageLayoutEXT(VK_RESULT_MAX_ENUM, Device, TransitionCount, Transitions);
		const VkResult Result = VULKANAPINAMESPACE::vkTransitionImageLayoutEXT(Device, TransitionCount, Transitions);
		FWrapLayer::TransitionImageLayoutEXT(Result, Device, TransitionCount, Transitions);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetPastPresentationTimingEXT(VkDevice Device, const VkPastPresentationTimingInfoEXT* PastPresentationTimingInfo, VkPastPresentationTimingPropertiesEXT* PastPresentationTimingProperties)
	{
		FWrapLayer::GetPastPresentationTimingEXT(VK_RESULT_MAX_ENUM, Device, PastPresentationTimingInfo, PastPresentationTimingProperties);
		const VkResult Result = VULKANAPINAMESPACE::vkGetPastPresentationTimingEXT(Device, PastPresentationTimingInfo, PastPresentationTimingProperties);
		FWrapLayer::GetPastPresentationTimingEXT(Result, Device, PastPresentationTimingInfo, PastPresentationTimingProperties);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetSwapchainTimeDomainPropertiesEXT(VkDevice Device, VkSwapchainKHR Swapchain, VkSwapchainTimeDomainPropertiesEXT* SwapchainTimeDomainProperties, uint64_t* TimeDomainsCounter)
	{
		FWrapLayer::GetSwapchainTimeDomainPropertiesEXT(VK_RESULT_MAX_ENUM, Device, Swapchain, SwapchainTimeDomainProperties, TimeDomainsCounter);
		const VkResult Result = VULKANAPINAMESPACE::vkGetSwapchainTimeDomainPropertiesEXT(Device, Swapchain, SwapchainTimeDomainProperties, TimeDomainsCounter);
		FWrapLayer::GetSwapchainTimeDomainPropertiesEXT(Result, Device, Swapchain, SwapchainTimeDomainProperties, TimeDomainsCounter);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkGetSwapchainTimingPropertiesEXT(VkDevice Device, VkSwapchainKHR Swapchain, VkSwapchainTimingPropertiesEXT* SwapchainTimingProperties, uint64_t* SwapchainTimingPropertiesCounter)
	{
		FWrapLayer::GetSwapchainTimingPropertiesEXT(VK_RESULT_MAX_ENUM, Device, Swapchain, SwapchainTimingProperties, SwapchainTimingPropertiesCounter);
		const VkResult Result = VULKANAPINAMESPACE::vkGetSwapchainTimingPropertiesEXT(Device, Swapchain, SwapchainTimingProperties, SwapchainTimingPropertiesCounter);
		FWrapLayer::GetSwapchainTimingPropertiesEXT(Result, Device, Swapchain, SwapchainTimingProperties, SwapchainTimingPropertiesCounter);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkSetSwapchainPresentTimingQueueSizeEXT(VkDevice Device, VkSwapchainKHR Swapchain, uint32_t Size)
	{
		FWrapLayer::SetSwapchainPresentTimingQueueSizeEXT(VK_RESULT_MAX_ENUM, Device, Swapchain, Size);
		const VkResult Result = VULKANAPINAMESPACE::vkSetSwapchainPresentTimingQueueSizeEXT(Device, Swapchain, Size);
		FWrapLayer::SetSwapchainPresentTimingQueueSizeEXT(Result, Device, Swapchain, Size);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdBindResourceHeapEXT(VkCommandBuffer CommandBuffer, const VkBindHeapInfoEXT* BindInfo)
	{
		FWrapLayer::CmdBindResourceHeapEXT(VK_RESULT_MAX_ENUM, CommandBuffer, BindInfo);
		VULKANAPINAMESPACE::vkCmdBindResourceHeapEXT(CommandBuffer, BindInfo);
		FWrapLayer::CmdBindResourceHeapEXT(VK_SUCCESS, CommandBuffer, BindInfo);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdBindSamplerHeapEXT(VkCommandBuffer CommandBuffer, const VkBindHeapInfoEXT* BindInfo)
	{
		FWrapLayer::CmdBindSamplerHeapEXT(VK_RESULT_MAX_ENUM, CommandBuffer, BindInfo);
		VULKANAPINAMESPACE::vkCmdBindSamplerHeapEXT(CommandBuffer, BindInfo);
		FWrapLayer::CmdBindSamplerHeapEXT(VK_SUCCESS, CommandBuffer, BindInfo);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdPushDataEXT(VkCommandBuffer CommandBuffer, const VkPushDataInfoEXT* PushDataInfo)
	{
		FWrapLayer::CmdPushDataEXT(VK_RESULT_MAX_ENUM, CommandBuffer, PushDataInfo);
		VULKANAPINAMESPACE::vkCmdPushDataEXT(CommandBuffer, PushDataInfo);
		FWrapLayer::CmdPushDataEXT(VK_SUCCESS, CommandBuffer, PushDataInfo);
	}

	static FORCEINLINE_DEBUGGABLE VkDeviceSize vkGetPhysicalDeviceDescriptorSizeEXT(VkPhysicalDevice PhysicalDevice, VkDescriptorType DescriptorType)
	{
		FWrapLayer::GetPhysicalDeviceDescriptorSizeEXT(0, PhysicalDevice, DescriptorType);
		const VkDeviceSize OutSize = VULKANAPINAMESPACE::vkGetPhysicalDeviceDescriptorSizeEXT(PhysicalDevice, DescriptorType);
		FWrapLayer::GetPhysicalDeviceDescriptorSizeEXT(OutSize, PhysicalDevice, DescriptorType);
		return OutSize;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkWriteResourceDescriptorsEXT(VkDevice Device, uint32_t ResourceCount, const VkResourceDescriptorInfoEXT* Resources, const VkHostAddressRangeEXT* Descriptors)
	{
		FWrapLayer::WriteResourceDescriptorsEXT(VK_RESULT_MAX_ENUM, Device, ResourceCount, Resources, Descriptors);
		const VkResult Result = VULKANAPINAMESPACE::vkWriteResourceDescriptorsEXT(Device, ResourceCount, Resources, Descriptors);
		FWrapLayer::WriteResourceDescriptorsEXT(Result, Device, ResourceCount, Resources, Descriptors);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE VkResult vkWriteSamplerDescriptorsEXT(VkDevice Device, uint32_t SamplerCount, const VkSamplerCreateInfo* Samplers, const VkHostAddressRangeEXT* Descriptors)
	{
		FWrapLayer::WriteSamplerDescriptorsEXT(VK_RESULT_MAX_ENUM, Device, SamplerCount, Samplers, Descriptors);
		const VkResult Result = VULKANAPINAMESPACE::vkWriteSamplerDescriptorsEXT(Device, SamplerCount, Samplers, Descriptors);
		FWrapLayer::WriteSamplerDescriptorsEXT(Result, Device, SamplerCount, Samplers, Descriptors);
		return Result;
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetRenderingAttachmentLocationsKHR(VkCommandBuffer CommandBuffer, const VkRenderingAttachmentLocationInfo* LocationInfo)
	{
		FWrapLayer::CmdSetRenderingAttachmentLocationsKHR(VK_RESULT_MAX_ENUM, CommandBuffer, LocationInfo);
		VULKANAPINAMESPACE::vkCmdSetRenderingAttachmentLocationsKHR(CommandBuffer, LocationInfo);
		FWrapLayer::CmdSetRenderingAttachmentLocationsKHR(VK_SUCCESS, CommandBuffer, LocationInfo);
	}

	static FORCEINLINE_DEBUGGABLE void vkCmdSetRenderingInputAttachmentIndicesKHR(VkCommandBuffer CommandBuffer, const VkRenderingInputAttachmentIndexInfo* InputAttachmentIndexInfo)
	{
		FWrapLayer::CmdSetRenderingInputAttachmentIndicesKHR(VK_RESULT_MAX_ENUM, CommandBuffer, InputAttachmentIndexInfo);
		VULKANAPINAMESPACE::vkCmdSetRenderingInputAttachmentIndicesKHR(CommandBuffer, InputAttachmentIndexInfo);
		FWrapLayer::CmdSetRenderingInputAttachmentIndicesKHR(VK_SUCCESS, CommandBuffer, InputAttachmentIndexInfo);
	}

	static FORCEINLINE_DEBUGGABLE void vkGetDescriptorSetLayoutSupport(VkDevice Device, const VkDescriptorSetLayoutCreateInfo* CreateInfo, VkDescriptorSetLayoutSupport* OutSupport)
	{
		FWrapLayer::GetDescriptorSetLayoutSupport(VK_RESULT_MAX_ENUM, Device, CreateInfo, OutSupport);
		VULKANAPINAMESPACE::vkGetDescriptorSetLayoutSupport(Device, CreateInfo, OutSupport);
		FWrapLayer::GetDescriptorSetLayoutSupport(VK_SUCCESS, Device, CreateInfo, OutSupport);
	}


#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER
	void BindDebugLabelName(VkImage Image, const TCHAR* Name);
#endif

#if VULKAN_ENABLE_DUMP_LAYER
	void DumpLayerPushMarker(const TCHAR* Name);
	void DumpLayerPopMarker();
	void PrintfBegin(const FString& String);
#endif
}

#endif // VULKAN_COMMANDWRAPPERS_ENABLE
