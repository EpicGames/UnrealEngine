// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanCommands.cpp: Vulkan RHI commands implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanPendingState.h"
#include "VulkanContext.h"
#include "EngineGlobals.h"
#include "VulkanLLM.h"
#include "RenderUtils.h"
#include "GlobalRenderResources.h"
#include "RHIShaderParametersShared.h"
#include "RHIUtilities.h"
#include "VulkanFramebuffer.h"
#include "VulkanRenderpass.h"
#include "VulkanResourceCollection.h"
#include "RHICoreShader.h"
#include "RHIUniformBufferUtilities.h"

// make sure what the hardware expects matches what we give it for indirect arguments
static_assert(sizeof(FRHIDrawIndirectParameters) == sizeof(VkDrawIndirectCommand), "FRHIDrawIndirectParameters size is wrong.");
static_assert(STRUCT_OFFSET(FRHIDrawIndirectParameters, VertexCountPerInstance) == STRUCT_OFFSET(VkDrawIndirectCommand, vertexCount), "Wrong offset of FRHIDrawIndirectParameters::VertexCountPerInstance.");
static_assert(STRUCT_OFFSET(FRHIDrawIndirectParameters, InstanceCount) == STRUCT_OFFSET(VkDrawIndirectCommand, instanceCount), "Wrong offset of FRHIDrawIndirectParameters::InstanceCount.");
static_assert(STRUCT_OFFSET(FRHIDrawIndirectParameters, StartVertexLocation) == STRUCT_OFFSET(VkDrawIndirectCommand, firstVertex), "Wrong offset of FRHIDrawIndirectParameters::StartVertexLocation.");
static_assert(STRUCT_OFFSET(FRHIDrawIndirectParameters, StartInstanceLocation) == STRUCT_OFFSET(VkDrawIndirectCommand, firstInstance), "Wrong offset of FRHIDrawIndirectParameters::StartInstanceLocation.");

static_assert(sizeof(FRHIDrawIndexedIndirectParameters) == sizeof(VkDrawIndexedIndirectCommand), "FRHIDrawIndexedIndirectParameters size is wrong.");
static_assert(STRUCT_OFFSET(FRHIDrawIndexedIndirectParameters, IndexCountPerInstance) == STRUCT_OFFSET(VkDrawIndexedIndirectCommand, indexCount), "Wrong offset of FRHIDrawIndexedIndirectParameters::IndexCountPerInstance.");
static_assert(STRUCT_OFFSET(FRHIDrawIndexedIndirectParameters, InstanceCount) == STRUCT_OFFSET(VkDrawIndexedIndirectCommand, instanceCount), "Wrong offset of FRHIDrawIndexedIndirectParameters::InstanceCount.");
static_assert(STRUCT_OFFSET(FRHIDrawIndexedIndirectParameters, StartIndexLocation) == STRUCT_OFFSET(VkDrawIndexedIndirectCommand, firstIndex), "Wrong offset of FRHIDrawIndexedIndirectParameters::StartIndexLocation.");
static_assert(STRUCT_OFFSET(FRHIDrawIndexedIndirectParameters, BaseVertexLocation) == STRUCT_OFFSET(VkDrawIndexedIndirectCommand, vertexOffset), "Wrong offset of FRHIDrawIndexedIndirectParameters::BaseVertexLocation.");
static_assert(STRUCT_OFFSET(FRHIDrawIndexedIndirectParameters, StartInstanceLocation) == STRUCT_OFFSET(VkDrawIndexedIndirectCommand, firstInstance), "Wrong offset of FRHIDrawIndexedIndirectParameters::StartInstanceLocation.");

static_assert(sizeof(FRHIDispatchIndirectParameters) == sizeof(VkDispatchIndirectCommand), "FRHIDispatchIndirectParameters size is wrong.");
static_assert(STRUCT_OFFSET(FRHIDispatchIndirectParameters, ThreadGroupCountX) == STRUCT_OFFSET(VkDispatchIndirectCommand, x), "FRHIDispatchIndirectParameters X dimension is wrong.");
static_assert(STRUCT_OFFSET(FRHIDispatchIndirectParameters, ThreadGroupCountY) == STRUCT_OFFSET(VkDispatchIndirectCommand, y), "FRHIDispatchIndirectParameters Y dimension is wrong.");
static_assert(STRUCT_OFFSET(FRHIDispatchIndirectParameters, ThreadGroupCountZ) == STRUCT_OFFSET(VkDispatchIndirectCommand, z), "FRHIDispatchIndirectParameters Z dimension is wrong.");

inline const FRHIShader* RHIShaderFromVulkanShader(const FVulkanShader* InShader, EShaderFrequency InFrequency)
{
	switch (InFrequency)
	{
	case SF_Vertex:
		return static_cast<const FVulkanVertexShader*>(InShader);
#if PLATFORM_SUPPORTS_MESH_SHADERS
	case SF_Mesh:
		return static_cast<const FVulkanMeshShader*>(InShader);
	case SF_Amplification:
		return static_cast<const FVulkanTaskShader*>(InShader);
#endif
	case SF_Pixel:
		return static_cast<const FVulkanPixelShader*>(InShader);
#if VULKAN_SUPPORTS_GEOMETRY_SHADERS
	case SF_Geometry:
		return static_cast<const FVulkanGeometryShader*>(InShader);
#endif
	case SF_Compute:
		return static_cast<const FVulkanComputeShader*>(InShader);

	default:
		checkf(0, TEXT("Undefined FRHIShader Frequency %d!"), (int32)InFrequency);
		return nullptr;
	}
}

static FORCEINLINE ShaderStage::EStage GetAndVerifyShaderStage(FRHIGraphicsShader* ShaderRHI, FVulkanPendingGfxState* PendingGfxState)
{
	switch (ShaderRHI->GetFrequency())
	{
	case SF_Vertex:
		check(PendingGfxState->GetCurrentShaderKey(ShaderStage::Vertex) == GetShaderKey<FVulkanVertexShader>(ShaderRHI));
		return ShaderStage::Vertex;
#if PLATFORM_SUPPORTS_MESH_SHADERS
	case SF_Mesh:
		check(PendingGfxState->GetCurrentShaderKey(ShaderStage::Mesh) == GetShaderKey<FVulkanMeshShader>(ShaderRHI));
		return ShaderStage::Mesh;
	case SF_Amplification:
		check(PendingGfxState->GetCurrentShaderKey(ShaderStage::Task) == GetShaderKey<FVulkanTaskShader>(ShaderRHI));
		return ShaderStage::Task;
#endif
	case SF_Geometry:
#if VULKAN_SUPPORTS_GEOMETRY_SHADERS
		check(PendingGfxState->GetCurrentShaderKey(ShaderStage::Geometry) == GetShaderKey<FVulkanGeometryShader>(ShaderRHI));
		return ShaderStage::Geometry;
#else
		checkf(0, TEXT("Geometry shaders not supported on this platform!"));
		UE_LOGF(LogVulkanRHI, Fatal, "Geometry shaders not supported on this platform!");
		break;
#endif
	case SF_Pixel:
		check(PendingGfxState->GetCurrentShaderKey(ShaderStage::Pixel) == GetShaderKey<FVulkanPixelShader>(ShaderRHI));
		return ShaderStage::Pixel;
	default:
		checkf(0, TEXT("Undefined FRHIShader Frequency %d!"), (int32)ShaderRHI->GetFrequency());
		break;
	}

	return ShaderStage::Invalid;
}

static FORCEINLINE ShaderStage::EStage GetAndVerifyShaderStageAndVulkanShader(FRHIGraphicsShader* ShaderRHI, FVulkanPendingGfxState* PendingGfxState, FVulkanShader*& OutShader)
{
	switch (ShaderRHI->GetFrequency())
	{
	case SF_Vertex:
		//check(PendingGfxState->GetCurrentShaderKey(ShaderStage::Vertex) == GetShaderKey<FVulkanVertexShader>(ShaderRHI));
		OutShader = static_cast<FVulkanVertexShader*>(static_cast<FRHIVertexShader*>(ShaderRHI));
		return ShaderStage::Vertex;
#if PLATFORM_SUPPORTS_MESH_SHADERS
	case SF_Mesh:
		OutShader = static_cast<FVulkanMeshShader*>(static_cast<FRHIMeshShader*>(ShaderRHI));
		return ShaderStage::Mesh;
	case SF_Amplification:
		OutShader = static_cast<FVulkanTaskShader*>(static_cast<FVulkanTaskShader*>(ShaderRHI));
		return ShaderStage::Task;
#endif
	case SF_Geometry:
#if VULKAN_SUPPORTS_GEOMETRY_SHADERS
		//check(PendingGfxState->GetCurrentShaderKey(ShaderStage::Geometry) == GetShaderKey<FVulkanGeometryShader>(ShaderRHI));
		OutShader = static_cast<FVulkanGeometryShader*>(static_cast<FRHIGeometryShader*>(ShaderRHI));
		return ShaderStage::Geometry;
#else
		checkf(0, TEXT("Geometry shaders not supported on this platform!"));
		UE_LOGF(LogVulkanRHI, Fatal, "Geometry shaders not supported on this platform!");
		break;
#endif
	case SF_Pixel:
		//check(PendingGfxState->GetCurrentShaderKey(ShaderStage::Pixel) == GetShaderKey<FVulkanPixelShader>(ShaderRHI));
		OutShader = static_cast<FVulkanPixelShader*>(static_cast<FRHIPixelShader*>(ShaderRHI));
		return ShaderStage::Pixel;
	default:
		checkf(0, TEXT("Undefined FRHIShader Frequency %d!"), (int32)ShaderRHI->GetFrequency());
		break;
	}

	OutShader = nullptr;
	return ShaderStage::Invalid;
}

void FVulkanCommandListContext::RHISetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBufferRHI, uint32 Offset)
{
	FVulkanBuffer* VertexBuffer = ResourceCast(VertexBufferRHI);
	if (VertexBuffer != nullptr)
	{
		PendingGfxState->SetStreamSource(StreamIndex, VertexBuffer->GetHandle(), Offset + VertexBuffer->GetOffset());
	}
}

struct FVulkanResourceBinder
{
	FVulkanCommandListContext& Context;
	const FVulkanShader* Shader;
	const EShaderFrequency Frequency;
	const ShaderStage::EStage Stage;
	FVulkanCommonPipelineDescriptorState& State;
	FVulkanPackedUniformBufferState& UniformBuffer;

	// Even if no resource are bound when bindless is enabled, we still need to process the Resource Table to go through proper validation
	const bool bBindless;

	template <class PendingStateType>
	FVulkanResourceBinder(FVulkanCommandListContext& InContext, PendingStateType* InPendingState, EShaderFrequency InFrequency, const FVulkanShader* InShader)
		: Context(InContext)
		, Shader(InShader)
		, Frequency(InFrequency)
		, Stage(ShaderStage::GetStageForFrequency(Frequency))
		, State(InPendingState->GetCurrentState())
		, UniformBuffer(InPendingState->GetUniformBuffers(Stage))
		, bBindless(InShader->UsesBindless())
	{
	}

	void SetParameter(uint32 BaseIndex, uint32 NumBytes, const void* NewValue)
	{
		UniformBuffer.SetParameter(BaseIndex, NumBytes, NewValue);
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	void SetBindlessHandle(const FRHIDescriptorHandle& Handle, uint32 Offset)
	{
		if (Handle.IsValid())
		{
			const uint32 BindlessIndex = Handle.GetIndex();
			SetParameter(Offset, 4, reinterpret_cast<const uint8*>(&BindlessIndex));
		}
	}
#endif

	void SetUAV(FRHIUnorderedAccessView* UAV, uint16 Index, bool bClearResources = false)
	{
		if (bClearResources)
		{
			//Context.ClearShaderResources(UAV);
		}

		if (!bBindless)
		{
			State.SetUAV(Frequency == SF_Compute, Stage, Index, ResourceCast(UAV));
		}
	}

	void SetSRV(FRHIShaderResourceView* SRV, uint16 Index)
	{
		if (!bBindless)
		{
			State.SetSRV(Frequency == SF_Compute, Stage, Index, ResourceCast(SRV));
		}
	}

	void SetTexture(FRHITexture* TextureRHI, uint16 Index)
	{
		if (!bBindless)
		{
			FVulkanTexture* VulkanTexture = ResourceCast(TextureRHI);
			const ERHIAccess RHIAccess = (Frequency == SF_Compute) ? ERHIAccess::SRVCompute : ERHIAccess::SRVGraphics;
			const VkImageLayout ExpectedLayout = FVulkanPipelineBarrier::GetDefaultLayout(*VulkanTexture, RHIAccess);
			State.SetTexture(Stage, Index, VulkanTexture, ExpectedLayout);
		}
	}

	void SetSampler(FRHISamplerState* Sampler, uint16 Index)
	{
		if (!bBindless)
		{
			State.SetSamplerState(Stage, Index, ResourceCast(Sampler));
		}
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	void SetResourceCollection(FRHIResourceCollection* ResourceCollection, uint32 Index)
	{
		FVulkanResourceCollection* VulkanResourceCollection = ResourceCast(ResourceCollection);
		SetSRV(VulkanResourceCollection->GetShaderResourceView(), Index);
	}
#endif
};

template <class ShaderType> 
void FVulkanCommandListContext::SetResourcesFromTables(const ShaderType* Shader)
{
	checkSlow(Shader);

	static constexpr EShaderFrequency Frequency = static_cast<EShaderFrequency>(ShaderType::StaticFrequency);

	if (Frequency == SF_Compute)
	{
		FVulkanResourceBinder Binder(*this, PendingComputeState, SF_Compute, Shader);
		UE::RHI::Private::SetUniformBufferResourcesFromTables(
			Binder
			, *Shader
			, DirtyUniformBuffers[Frequency]
			, BoundUniformBuffers[Frequency]
#if ENABLE_RHI_VALIDATION
			, Tracker
#endif
		);
	}
	else
	{
		FVulkanResourceBinder Binder(*this, PendingGfxState, Frequency, Shader);
		UE::RHI::Private::SetUniformBufferResourcesFromTables(
			Binder
			, *Shader
			, DirtyUniformBuffers[Frequency]
			, BoundUniformBuffers[Frequency]
#if ENABLE_RHI_VALIDATION
			, Tracker
#endif
		);
	}
}

static void BindUniformBuffer(
	FVulkanCommandListContext& Context,
	FVulkanCommonPipelineDescriptorState& State,
	const FVulkanShader* Shader,
	EShaderFrequency Frequency,
	ShaderStage::EStage Stage,
	uint32 BufferIndex,
	FVulkanUniformBuffer* UniformBuffer
)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanSetUniformBufferTime);
#endif

	const FVulkanShaderHeader& CodeHeader = Shader->GetCodeHeader();
	checkfSlow(!CodeHeader.UniformBufferInfos[BufferIndex].LayoutHash || (CodeHeader.UniformBufferInfos[BufferIndex].LayoutHash == UniformBuffer->GetLayout().GetHash()),
		TEXT("Mismatched UB layout! Got hash 0x%x, expected 0x%x!"), UniformBuffer->GetLayout().GetHash(), CodeHeader.UniformBufferInfos[BufferIndex].LayoutHash);

	if (UniformBuffer->IsUniformView())
	{
		UniformBuffer->SetupUniformBufferView();
	}

	bool bHasResources = false;
	if (BufferIndex < CodeHeader.NumBoundUniformBuffers)
	{
		checkSlow(UniformBuffer->GetLayout().ConstantBufferSize > 0);

		const VkDescriptorType DescriptorType = State.GetDescriptorType(Stage, BufferIndex);
		const bool bDynamic = (DescriptorType == VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC);

		State.SetUniformBuffer(Stage, BufferIndex, UniformBuffer, bDynamic);

		bHasResources = (CodeHeader.UniformBufferInfos[BufferIndex].bHasResources != 0);
	}
	else
	{
		// If the buffer has no bindings, then it is as resource only ub
		bHasResources = true;
	}

	if (bHasResources)
	{
		checkSlow(Frequency < SF_NumStandardFrequencies);
		check(BufferIndex < FVulkanCommandListContext::MAX_UNIFORM_BUFFERS_PER_SHADER_STAGE);
		Context.BoundUniformBuffers[Frequency][BufferIndex] = UniformBuffer;
		Context.DirtyUniformBuffers[Frequency] |= (1 << BufferIndex);
	}
}

static void SetShaderParametersOnBinder(
	FVulkanResourceBinder& Binder
	, TConstArrayView<uint8> InParametersData
	, TConstArrayView<FRHIShaderParameter> InParameters
	, TConstArrayView<FRHIShaderParameterResource> InResourceParameters
	, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters)
{
	const EShaderFrequency Frequency = Binder.Frequency;

	for (const FRHIShaderParameter& Parameter : InParameters)
	{
		checkSlow(Parameter.BufferIndex == 0);
		Binder.SetParameter(Parameter.BaseIndex, Parameter.ByteSize, &InParametersData[Parameter.ByteOffset]);
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	for (const FRHIShaderParameterResource& Parameter : InBindlessParameters)
	{
		if (FRHIResource* Resource = Parameter.Resource)
		{
			if (Parameter.Type == FRHIShaderParameterResource::EType::UniformBuffer)
			{
				UE::RHI::Private::SetAllUniformBufferResourcesForBindless(Binder, static_cast<FRHIUniformBuffer*>(Resource));
			}
			else
			{
				FRHIDescriptorHandle Handle;

				switch (Parameter.Type)
				{
				case FRHIShaderParameterResource::EType::Texture:
					Handle = static_cast<FRHITexture*>(Resource)->GetDefaultBindlessHandle();
					Binder.SetTexture(static_cast<FRHITexture*>(Resource), Parameter.Index);
					break;
				case FRHIShaderParameterResource::EType::ResourceView:
					Handle = static_cast<FRHIShaderResourceView*>(Resource)->GetBindlessHandle();
					Binder.SetSRV(static_cast<FRHIShaderResourceView*>(Resource), Parameter.Index);
					break;
				case FRHIShaderParameterResource::EType::UnorderedAccessView:
					Handle = static_cast<FRHIUnorderedAccessView*>(Resource)->GetBindlessHandle();
					Binder.SetUAV(static_cast<FRHIUnorderedAccessView*>(Resource), Parameter.Index, true);
					break;
				case FRHIShaderParameterResource::EType::Sampler:
					Handle = static_cast<FRHISamplerState*>(Resource)->GetBindlessHandle();
					Binder.SetSampler(static_cast<FRHISamplerState*>(Resource), Parameter.Index);
					break;
				case FRHIShaderParameterResource::EType::ResourceCollection:
					Handle = static_cast<FRHIResourceCollection*>(Resource)->GetBindlessHandle();
					Binder.SetResourceCollection(static_cast<FRHIResourceCollection*>(Resource), Parameter.Index);
					break;
				}

				checkf(Handle.IsValid(), TEXT("Vulkan resource did not provide a valid descriptor handle. Please validate that all Vulkan types can provide this or that the resource is still valid."));
				Binder.SetBindlessHandle(Handle, Parameter.Index);
			}
		}
	}
#endif

	for (const FRHIShaderParameterResource& Parameter : InResourceParameters)
	{
		if (Parameter.Type == FRHIShaderParameterResource::EType::UnorderedAccessView)
		{
			if (Binder.Frequency == SF_Pixel || Binder.Frequency == SF_Vertex || Binder.Frequency == SF_Compute)
			{
				Binder.SetUAV(static_cast<FRHIUnorderedAccessView*>(Parameter.Resource), Parameter.Index, true);
			}
			else
			{
				checkf(false, TEXT("TShaderRHI Can't have compute shader to be set. UAVs are not supported tessellation and geometry shaders."));
			}
		}
	}

	for (const FRHIShaderParameterResource& Parameter : InResourceParameters)
	{
		switch (Parameter.Type)
		{
		case FRHIShaderParameterResource::EType::Texture:
			Binder.SetTexture(static_cast<FRHITexture*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::ResourceView:
			Binder.SetSRV(static_cast<FRHIShaderResourceView*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::UnorderedAccessView:
			break;
		case FRHIShaderParameterResource::EType::Sampler:
			Binder.SetSampler(static_cast<FRHISamplerState*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::UniformBuffer:
			BindUniformBuffer(Binder.Context, Binder.State, Binder.Shader, Binder.Frequency, Binder.Stage, Parameter.Index, ResourceCast(static_cast<FRHIUniformBuffer*>(Parameter.Resource)));
			break;
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		case FRHIShaderParameterResource::EType::ResourceCollection:
			Binder.SetResourceCollection(static_cast<FRHIResourceCollection*>(Parameter.Resource), Parameter.Index);
			break;
#endif
		default:
			checkf(false, TEXT("Unhandled resource type?"));
			break;
		}
	}
}

void FVulkanCommandListContext::CommitGraphicsResourceTables()
{
	checkfSlow(Queue.GetQueueType() == EVulkanQueueType::Graphics, TEXT("Recording a graphic command on a non-graphic queue."));
	checkSlow(PendingGfxState);

	if (const FVulkanShader* Shader = PendingGfxState->GetCurrentShader(SF_Vertex))
	{
		checkSlow(Shader->Frequency == SF_Vertex);
		const FVulkanVertexShader* VertexShader = static_cast<const FVulkanVertexShader*>(Shader);
		SetResourcesFromTables(VertexShader);
	}

	if (const FVulkanShader* Shader = PendingGfxState->GetCurrentShader(SF_Pixel))
	{
		checkSlow(Shader->Frequency == SF_Pixel);
		const FVulkanPixelShader* PixelShader = static_cast<const FVulkanPixelShader*>(Shader);
		SetResourcesFromTables(PixelShader);
	}

#if PLATFORM_SUPPORTS_MESH_SHADERS
	if (const FVulkanShader* Shader = PendingGfxState->GetCurrentShader(SF_Mesh))
	{
		checkSlow(Shader->Frequency == SF_Mesh);
		const FVulkanMeshShader* MeshShader = static_cast<const FVulkanMeshShader*>(Shader);
		SetResourcesFromTables(MeshShader);
	}

	if (const FVulkanShader* Shader = PendingGfxState->GetCurrentShader(SF_Amplification))
	{
		checkSlow(Shader->Frequency == SF_Amplification);
		const FVulkanTaskShader* AmplificationShader = static_cast<const FVulkanTaskShader*>(Shader);
		SetResourcesFromTables(AmplificationShader);
	}
#endif

#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	if (const FVulkanShader* Shader = PendingGfxState->GetCurrentShader(SF_Geometry))
	{
		checkSlow(Shader->Frequency == SF_Geometry);
		const FVulkanGeometryShader* GeometryShader = static_cast<const FVulkanGeometryShader*>(Shader);
		SetResourcesFromTables(GeometryShader);
	}
#endif
}

void FVulkanCommandListContext::CommitComputeResourceTables()
{
	SetResourcesFromTables(PendingComputeState->GetCurrentShader());
}


void FVulkanCommandListContext::RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ)
{
	RHI_DISPATCH_CALL_INC();

#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanDispatchCallTime);
#endif

	CommitComputeResourceTables();

	PendingComputeState->PrepareForDispatch(*this);

	ensure(GetCommandBuffer().IsOutsideRenderPass());
	const VkCommandBuffer CommandBufferHandle = GetCommandBuffer().GetHandle();

	VulkanRHI::vkCmdDispatch(CommandBufferHandle, ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);

	VulkanRHI::DebugHeavyWeightBarrier(CommandBufferHandle, 2);
}

void FVulkanCommandListContext::RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
	RHI_DISPATCH_CALL_INC();

	static_assert(sizeof(FRHIDispatchIndirectParameters) == sizeof(VkDispatchIndirectCommand), "Dispatch indirect doesn't match!");
	FVulkanBuffer* ArgumentBuffer = ResourceCast(ArgumentBufferRHI);

	CommitComputeResourceTables();

	PendingComputeState->PrepareForDispatch(*this);

	ensure(GetCommandBuffer().IsOutsideRenderPass());
	const VkCommandBuffer CommandBufferHandle = GetCommandBuffer().GetHandle();

	VulkanRHI::vkCmdDispatchIndirect(CommandBufferHandle, ArgumentBuffer->GetHandle(), ArgumentBuffer->GetOffset() + ArgumentOffset);

	VulkanRHI::DebugHeavyWeightBarrier(CommandBufferHandle, 2);
}

void FVulkanCommandListContext::RHISetShaderParameters(FRHIGraphicsShader* ShaderRHI, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters)
{
	FVulkanShader* Shader = nullptr;
	const ShaderStage::EStage Stage = GetAndVerifyShaderStageAndVulkanShader(ShaderRHI, PendingGfxState, Shader);

	FVulkanResourceBinder Binder(*this, PendingGfxState, Shader->Frequency, Shader);

	SetShaderParametersOnBinder(
		Binder
		, InParametersData
		, InParameters
		, InResourceParameters
		, InBindlessParameters);
}

void FVulkanCommandListContext::RHISetShaderParameters(FRHIComputeShader* ShaderRHI, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters)
{
	check(PendingComputeState->GetCurrentShader() == ResourceCast(ShaderRHI));
	FVulkanResourceBinder Binder(*this, PendingComputeState, SF_Compute, ResourceCast(ShaderRHI));

	SetShaderParametersOnBinder(
		Binder
		, InParametersData
		, InParameters
		, InResourceParameters
		, InBindlessParameters);
}

void FVulkanCommandListContext::RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers)
{
	FMemory::Memzero(StaticUniformBuffers.GetData(), StaticUniformBuffers.Num() * sizeof(FRHIUniformBuffer*));

	if (const FRHIShaderBindingLayout* Layout = InUniformBuffers.GetShaderBindingLayout())
	{
		check(InUniformBuffers.GetUniformBufferCount() == Layout->GetNumUniformBufferEntries());

		for (int32 Index = 0; Index < InUniformBuffers.GetUniformBufferCount(); ++Index)
		{
			StaticUniformBuffers[Index] = InUniformBuffers.GetUniformBuffer(Index);
			checkf(StaticUniformBuffers[Index], TEXT("Static uniform buffer at index %d is referenced in the shader binding layout but is not provided"), Index);
		}

		ShaderBindingLayout = Layout;
	}
	else
	{
		for (int32 Index = 0; Index < InUniformBuffers.GetUniformBufferCount(); ++Index)
		{
			StaticUniformBuffers[InUniformBuffers.GetSlot(Index)] = InUniformBuffers.GetUniformBuffer(Index);
		}

		ShaderBindingLayout = nullptr;
	}
}

void FVulkanCommandListContext::RHISetStaticUniformBuffer(FUniformBufferStaticSlot InSlot, FRHIUniformBuffer* InBuffer)
{
	StaticUniformBuffers[InSlot] = InBuffer;
}

void ApplyStaticUniformBuffersOnContext(
	FVulkanCommandListContext& Context,
	FVulkanCommonPipelineDescriptorState& State,
	const FRHIShader* ShaderRHI,
	const FVulkanShader* Shader,
	EShaderFrequency ShaderFrequency,
	ShaderStage::EStage Stage)
{
	if (Shader)
	{
		UE::RHICore::ApplyStaticUniformBuffers(
			ShaderRHI,
			Context.GetStaticUniformBuffers(),
			[&Context, &State, Shader, ShaderFrequency, Stage](int32 BufferIndex, FRHIUniformBuffer* Buffer)
			{
				BindUniformBuffer(Context, State, Shader, ShaderFrequency, Stage, BufferIndex, ResourceCast(Buffer));
			}
		);
	}
}

void FVulkanCommandListContext::RHISetGraphicsPipelineState(FRHIGraphicsPipelineState* GraphicsState, uint32 StencilRef, bool bApplyAdditionalState)
{
	FVulkanGraphicsPipelineState* Pipeline = ResourceCast(GraphicsState);

	FVulkanPipelineStateCacheManager* PipelineStateCache = Device.GetPipelineStateCache();
	PipelineStateCache->LRUTouch(Pipeline);

	Pipeline->FrameCounter.Set(GFrameNumberRenderThread);

	FVulkanCommandBuffer& CommandBuffer = GetCommandBuffer();
	bool bForceResetPipeline = !CommandBuffer.bHasPipeline;

	if (PendingGfxState->SetGfxPipeline(Pipeline, bForceResetPipeline))
	{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
		SCOPE_CYCLE_COUNTER(STAT_VulkanPipelineBind);
#endif
		PendingGfxState->Bind(CommandBuffer);
		CommandBuffer.bHasPipeline = true;
	}

	PendingGfxState->SetStencilRef(StencilRef);

	if (bApplyAdditionalState)
	{
		auto ApplyStaticUniformBuffers = [this, Pipeline](ShaderStage::EStage Stage)
		{
			const EShaderFrequency Frequency = ShaderStage::GetFrequencyForGfxStage(Stage);
			const FVulkanShader* Shader = Pipeline->VulkanShaders[Stage];
			const FRHIShader* ShaderRHI = RHIShaderFromVulkanShader(Shader, Frequency);
			ApplyStaticUniformBuffersOnContext(*this, PendingGfxState->GetCurrentState(), ShaderRHI, Shader, Frequency, Stage);
		};

		ApplyStaticUniformBuffers(ShaderStage::Vertex);
#if PLATFORM_SUPPORTS_MESH_SHADERS
		ApplyStaticUniformBuffers(ShaderStage::Mesh);
		ApplyStaticUniformBuffers(ShaderStage::Task);
#endif
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		ApplyStaticUniformBuffers(ShaderStage::Geometry);
#endif
		ApplyStaticUniformBuffers(ShaderStage::Pixel);
	}
}

void FVulkanCommandListContext::RHISetComputePipelineState(FRHIComputePipelineState* ComputePipelineState)
{
	AcquirePoolSetContainer();

	//#todo-rco: Set PendingGfx to null
	FVulkanComputePipeline* ComputePipeline = ResourceCast(ComputePipelineState);
	PendingComputeState->SetComputePipeline(ComputePipeline);

	ComputePipeline->FrameCounter.Set(GFrameNumberRenderThread);

	ApplyStaticUniformBuffersOnContext(
		*this,
		PendingComputeState->GetCurrentState(),
		ComputePipelineState->GetComputeShader(),
		ResourceCast(ComputePipeline->GetComputeShader()),
		SF_Compute,
		ShaderStage::Compute
	);
}

void FVulkanCommandListContext::RHISetUniformBufferDynamicOffset(FUniformBufferStaticSlot InSlot, uint32 InOffset)
{
	check(IsAligned(InOffset, Device.GetLimits().minUniformBufferOffsetAlignment));

	FVulkanUniformBuffer* UniformBuffer = ResourceCast(StaticUniformBuffers[InSlot]);

	static const ShaderStage::EStage Stages[2] = 
	{
		ShaderStage::Vertex,
		ShaderStage::Pixel
	};

	for (int32 i = 0; i < UE_ARRAY_COUNT(Stages); i++)
	{
		ShaderStage::EStage Stage = Stages[i];
		FVulkanShader* Shader = PendingGfxState->CurrentPipeline->VulkanShaders[Stage];
		if (Shader == nullptr)
		{
			continue;
		}

		const auto& StaticSlots = Shader->StaticSlots;

		for (int32 BufferIndex = 0; BufferIndex < StaticSlots.Num(); ++BufferIndex)
		{
			const FUniformBufferStaticSlot Slot = StaticSlots[BufferIndex];
			if (Slot == InSlot)
			{
				// Uniform views always bind max supported range, so make sure Offset+Range is within buffer allocation
				check((InOffset + PLATFORM_MAX_UNIFORM_BUFFER_RANGE) <= UniformBuffer->GetAllocation().Size);
				uint32 DynamicOffset = InOffset + UniformBuffer->GetOffset();
				PendingGfxState->CurrentState->SetUniformBufferDynamicOffset(Stage, BufferIndex, DynamicOffset);

				break;
			}
		}
	}
}

void FVulkanCommandListContext::RHISetStencilRef(uint32 StencilRef)
{
	PendingGfxState->SetStencilRef(StencilRef);
}

void FVulkanCommandListContext::RHIDrawPrimitive(uint32 BaseVertexIndex, uint32 NumPrimitives, uint32 NumInstances)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanDrawCallTime);
#endif
	NumInstances = FMath::Max(1U, NumInstances);

	CommitGraphicsResourceTables();

	PendingGfxState->PrepareForDraw(*this);

	const uint32 NumVertices = GetVertexCountForPrimitiveCount(NumPrimitives, PendingGfxState->PrimitiveType);

	RHI_DRAW_CALL_STATS(PendingGfxState->PrimitiveType, NumVertices, NumPrimitives, NumInstances);

	VulkanRHI::vkCmdDraw(GetCommandBuffer().GetHandle(), NumVertices, NumInstances, BaseVertexIndex, 0);
}

void FVulkanCommandListContext::RHIDrawPrimitiveIndirect(FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
	static_assert(sizeof(FRHIDrawIndirectParameters) == sizeof(VkDrawIndirectCommand), "Draw indirect doesn't match!");

#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanDrawCallTime);
#endif
	RHI_DRAW_CALL_INC();

	CommitGraphicsResourceTables();

	PendingGfxState->PrepareForDraw(*this);

	const VkCommandBuffer CommandBufferHandle = GetCommandBuffer().GetHandle();

	FVulkanBuffer* ArgumentBuffer = ResourceCast(ArgumentBufferRHI);

	VulkanRHI::vkCmdDrawIndirect(CommandBufferHandle, ArgumentBuffer->GetHandle(), ArgumentBuffer->GetOffset() + ArgumentOffset, 1, sizeof(VkDrawIndirectCommand));
}

void FVulkanCommandListContext::RHIDrawIndexedPrimitive(FRHIBuffer* IndexBufferRHI, int32 BaseVertexIndex, uint32 FirstInstance,
	uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanDrawCallTime);
#endif
	NumInstances = FMath::Max(1U, NumInstances);
	RHI_DRAW_CALL_STATS(PendingGfxState->PrimitiveType, NumVertices, NumPrimitives, NumInstances);
	checkf(GRHISupportsFirstInstance || FirstInstance == 0, TEXT("FirstInstance must be 0, see GRHISupportsFirstInstance"));

	CommitGraphicsResourceTables();

	PendingGfxState->PrepareForDraw(*this);

	FVulkanBuffer* IndexBuffer = ResourceCast(IndexBufferRHI);
	const VkCommandBuffer CommandBufferHandle = GetCommandBuffer().GetHandle();
	VulkanRHI::vkCmdBindIndexBuffer(CommandBufferHandle, IndexBuffer->GetHandle(), IndexBuffer->GetOffset(), IndexBuffer->GetIndexType());

	uint32 NumIndices = GetVertexCountForPrimitiveCount(NumPrimitives, PendingGfxState->PrimitiveType);
	VulkanRHI::vkCmdDrawIndexed(CommandBufferHandle, NumIndices, NumInstances, StartIndex, BaseVertexIndex, FirstInstance);
}

void FVulkanCommandListContext::RHIDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBufferRHI, FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanDrawCallTime);
#endif
	RHI_DRAW_CALL_INC();

	CommitGraphicsResourceTables();

	PendingGfxState->PrepareForDraw(*this);

	FVulkanBuffer* IndexBuffer = ResourceCast(IndexBufferRHI);
	const VkCommandBuffer CommandBufferHandle = GetCommandBuffer().GetHandle();
	VulkanRHI::vkCmdBindIndexBuffer(CommandBufferHandle, IndexBuffer->GetHandle(), IndexBuffer->GetOffset(), IndexBuffer->GetIndexType());

	FVulkanBuffer* ArgumentBuffer = ResourceCast(ArgumentBufferRHI);

	VulkanRHI::vkCmdDrawIndexedIndirect(CommandBufferHandle, ArgumentBuffer->GetHandle(), ArgumentBuffer->GetOffset() + ArgumentOffset, 1, sizeof(VkDrawIndexedIndirectCommand));
}

#if PLATFORM_SUPPORTS_MESH_SHADERS
void FVulkanCommandListContext::RHIDispatchMeshShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ)
{
	RHI_DRAW_CALL_INC();

	CommitGraphicsResourceTables();

	PendingGfxState->PrepareForDraw(*this);

	VulkanRHI::vkCmdDrawMeshTasksEXT(GetCommandBuffer().GetHandle(), ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
}

void FVulkanCommandListContext::RHIDispatchIndirectMeshShader(FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
	RHI_DRAW_CALL_INC();

	CommitGraphicsResourceTables();

	PendingGfxState->PrepareForDraw(*this);

	FVulkanBuffer* ArgumentBuffer = ResourceCast(ArgumentBufferRHI);

	VulkanRHI::vkCmdDrawMeshTasksIndirectEXT(GetCommandBuffer().GetHandle(), ArgumentBuffer->GetHandle(), ArgumentBuffer->GetOffset() + ArgumentOffset, 1, sizeof(VkDrawMeshTasksIndirectCommandEXT));
}
#endif

void FVulkanCommandListContext::RHIClearMRT(bool bClearColor, int32 NumClearColors, const FLinearColor* ClearColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil)
{
	if (!(bClearColor || bClearDepth || bClearStencil))
	{
		return;
	}

	check(bClearColor ? NumClearColors > 0 : true);

	FVulkanCommandBuffer& CommandBuffer = GetCommandBuffer();
	//FRCLog::Printf(TEXT("RHIClearMRT"));

	const uint32 NumColorAttachments = CurrentFramebuffer->GetNumColorAttachments();
	check(!bClearColor || (uint32)NumClearColors <= NumColorAttachments);
	InternalClearMRT(CommandBuffer, bClearColor, bClearColor ? NumClearColors : 0, ClearColorArray, bClearDepth, Depth, bClearStencil, Stencil);
}

void FVulkanCommandListContext::InternalClearMRT(FVulkanCommandBuffer& CommandBuffer, bool bClearColor, int32 NumClearColors, const FLinearColor* ClearColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil)
{
	if (CurrentRenderPass)
	{
		const VkExtent2D& Extents = CurrentRenderPass->GetLayout().GetExtent2D();
		VkClearRect Rect;
		FMemory::Memzero(Rect);
		Rect.rect.offset.x = 0;
		Rect.rect.offset.y = 0;
		Rect.rect.extent = Extents;

		VkClearAttachment Attachments[MaxSimultaneousRenderTargets + 1];
		FMemory::Memzero(Attachments);

		uint32 NumAttachments = NumClearColors;
		if (bClearColor)
		{
			for (int32 i = 0; i < NumClearColors; ++i)
			{
				Attachments[i].aspectMask = VK_IMAGE_ASPECT_COLOR_BIT;
				Attachments[i].colorAttachment = i;
				Attachments[i].clearValue.color.float32[0] = ClearColorArray[i].R;
				Attachments[i].clearValue.color.float32[1] = ClearColorArray[i].G;
				Attachments[i].clearValue.color.float32[2] = ClearColorArray[i].B;
				Attachments[i].clearValue.color.float32[3] = ClearColorArray[i].A;
			}
		}

		if (bClearDepth || bClearStencil)
		{
			Attachments[NumClearColors].aspectMask = bClearDepth ? VK_IMAGE_ASPECT_DEPTH_BIT : 0;
			Attachments[NumClearColors].aspectMask |= bClearStencil ? VK_IMAGE_ASPECT_STENCIL_BIT : 0;
			Attachments[NumClearColors].colorAttachment = 0;
			Attachments[NumClearColors].clearValue.depthStencil.depth = Depth;
			Attachments[NumClearColors].clearValue.depthStencil.stencil = Stencil;
			++NumAttachments;
		}

		VulkanRHI::vkCmdClearAttachments(CommandBuffer.GetHandle(), NumAttachments, Attachments, 1, &Rect);
	}
	else
	{
		ensure(0);
		//VulkanRHI::vkCmdClearColorImage(CommandBuffer.GetHandle(), )
	}
}

void FVulkanDynamicRHI::RHISuspendRendering()
{
}

void FVulkanDynamicRHI::RHIResumeRendering()
{
}

bool FVulkanDynamicRHI::RHIIsRenderingSuspended()
{
	return false;
}

void FVulkanCommandListContext::RHISetDepthBounds(float MinDepth, float MaxDepth)
{
	FVulkanCommandBuffer& CommandBuffer = GetCommandBuffer();
	VulkanRHI::vkCmdSetDepthBounds(CommandBuffer.GetHandle(), MinDepth, MaxDepth);
}

void FVulkanCommandListContext::RHICopyToStagingBuffer(FRHIBuffer* SourceBufferRHI, FRHIStagingBuffer* StagingBufferRHI, uint32 Offset, uint32 NumBytes)
{
	FVulkanCommandBuffer& CommandBuffer = GetCommandBuffer();
	FVulkanBuffer* VertexBuffer = ResourceCast(SourceBufferRHI);

	ensure(CommandBuffer.IsOutsideRenderPass());

	FVulkanStagingBuffer* StagingBuffer = ResourceCast(StagingBufferRHI);
	if (!StagingBuffer->StagingBuffer || StagingBuffer->StagingBuffer->GetSize() < NumBytes) //-V1051
	{
		if (StagingBuffer->StagingBuffer)
		{
			Device.GetStagingManager().ReleaseBuffer(nullptr, StagingBuffer->StagingBuffer);
		}

		VulkanRHI::FStagingBuffer* ReadbackStagingBuffer = Device.GetStagingManager().AcquireBuffer(NumBytes, VK_BUFFER_USAGE_TRANSFER_DST_BIT, VK_MEMORY_PROPERTY_HOST_CACHED_BIT);
		StagingBuffer->StagingBuffer = ReadbackStagingBuffer;
		StagingBuffer->Device = &Device;
	}

	StagingBuffer->QueuedNumBytes = NumBytes;

	const VkBufferCopy Region =
	{
		.srcOffset = Offset + VertexBuffer->GetOffset(),
		.dstOffset = 0,
		.size = NumBytes
	};
	VulkanRHI::vkCmdCopyBuffer(CommandBuffer.GetHandle(), VertexBuffer->GetHandle(), StagingBuffer->StagingBuffer->GetHandle(), 1, &Region);

	// Make the transfer write available/visible to host reads (Lock() will handle InvalidateMappedMemory())
	const VkMemoryBarrier BarrierAfter = { VK_STRUCTURE_TYPE_MEMORY_BARRIER, nullptr, VK_ACCESS_TRANSFER_WRITE_BIT, VK_ACCESS_HOST_READ_BIT };
	VulkanRHI::vkCmdPipelineBarrier(CommandBuffer.GetHandle(), VK_PIPELINE_STAGE_TRANSFER_BIT, VK_PIPELINE_STAGE_HOST_BIT, 0, 1, &BarrierAfter, 0, nullptr, 0, nullptr);
}

void FVulkanCommandListContext::RHIWriteGPUFence(FRHIGPUFence* FenceRHI)
{
	checkNoEntry(); // Should never be called
}

IRHIComputeContext* FVulkanDynamicRHI::RHIGetCommandContext(ERHIPipeline Pipeline)
{
	check(IsSingleRHIPipeline(Pipeline));
	return new FVulkanCommandListContext(*Device, Pipeline, false);
}
