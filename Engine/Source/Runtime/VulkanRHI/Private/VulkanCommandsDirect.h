// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanCommandsDirect.h: Enumerate the VulkanDynamicAPI.  Doing this in a header to avoid static analysis warnings.
=============================================================================*/

#pragma once

#include "VulkanCommandWrappers.h"
