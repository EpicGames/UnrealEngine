// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanContext.h"
#include "VulkanCommandBuffer.h"
#include "VulkanFramebuffer.h"
#include "VulkanMemory.h"
#include "VulkanRenderpass.h"
#include "VulkanResources.h"
#include "VulkanRHIPrivate.h"
#include "VulkanDevice.h"
#include "VulkanPendingState.h"
#include "VulkanQuery.h"
#include "VulkanBindlessDescriptorManager.h"
#include "DynamicRHI.h"



FVulkanContextCommon::FVulkanContextCommon(FVulkanDevice& InDevice, FVulkanQueue& InQueue, EVulkanCommandBufferType InCommandBufferType)
	: Device(InDevice)
	, Queue(InQueue)
	, Pool(*InQueue.AcquireCommandBufferPool(InCommandBufferType))
{
}

FVulkanContextCommon::~FVulkanContextCommon()
{
	Queue.ReleaseCommandBufferPool(&Pool);
}

void FVulkanContextCommon::NewPayload()
{
	EndPayload();
	Payloads.Add(new FVulkanPayload(Queue));
	CurrentPhase = EPhase::Wait;
}

void FVulkanContextCommon::EndPayload()
{
	if (Payloads.Num() > 0)
	{
		FlushPendingSyncPoints();

		FVulkanPayload* Payload = Payloads.Last();
		if (Payload->CommandBuffers.Num() > 0)
		{
			FVulkanCommandBuffer* CommandBuffer = Payload->CommandBuffers.Last();
			if (!CommandBuffer->HasEnded())
			{
				checkSlow(!CommandBuffer->IsSubmitted() && CommandBuffer->HasBegun());

				const bool bIsPrimaryCommandBuffer = (CommandBuffer->GetCommandBufferType() == EVulkanCommandBufferType::Primary);
				const bool bIsParallelCommandBuffer = (CommandBuffer->GetCommandBufferType() == EVulkanCommandBufferType::Parallel);

				// Only record begin/end timestamps on primary command buffers
				FVulkanQueryPool* OptionalTimestampQueryPool = GRHIGlobals.SupportsTimestampRenderQueries && (bIsPrimaryCommandBuffer || bIsParallelCommandBuffer)
					? GetCurrentTimestampQueryPool(*Payload)
					: nullptr;

				if (!CommandBuffer->IsOutsideRenderPass() && (bIsPrimaryCommandBuffer || bIsParallelCommandBuffer))
				{
					if (Device.GetOptionalExtensions().HasKHRDynamicRendering)
					{
						UE_LOGF(LogVulkanRHI, Warning, "Forcing EndDynamicRendering() for submission");
						CommandBuffer->EndDynamicRendering(bIsParallelCommandBuffer ? OptionalTimestampQueryPool : nullptr);
					}
					else
					{
						UE_LOGF(LogVulkanRHI, Warning, "Forcing EndRenderPass() for submission");
						CommandBuffer->EndRenderPass();
					}
				}

				CommandBuffer->End(bIsPrimaryCommandBuffer ? OptionalTimestampQueryPool : nullptr);
			}
		}
	}
}

void FVulkanContextCommon::AppendParallelRenderPayload(FVulkanPayload* ParallelRenderingPayload, VkRenderingInfo* NeedsResumeInfo)
{
	check(Device.GetOptionalExtensions().HasKHRDynamicRendering);
	check(ParallelRenderingPayload->CommandBuffers.IsEmpty() ||
		(ParallelRenderingPayload->CommandBuffers[0]->GetCommandBufferType() == EVulkanCommandBufferType::Parallel));

	EndPayload();
	Payloads.Add(ParallelRenderingPayload);

	// Close the last suspended render pass with an empty call, do not use any timestamps or work tracking
	if (NeedsResumeInfo)
	{
		PrepareNewCommandBuffer(GetPayload(EPhase::Execute), false);
		GetCommandBuffer().BeginDynamicRendering(*NeedsResumeInfo, nullptr);
		GetCommandBuffer().EndDynamicRendering(nullptr);
		GetCommandBuffer().End(nullptr);
	}

	PrepareNewCommandBuffer(GetPayload(EPhase::Execute));
	CurrentPhase = EPhase::Execute;
}

// Complete recording of the current command list set, and appends the resulting
// payloads to the given array. Resets the context so new commands can be recorded.
void FVulkanContextCommon::Finalize(TArray<FVulkanPayload*>& OutPayloads)
{
	FlushPendingSyncPoints();

	if (ContextSyncPoint.IsValid())
	{
		SignalSyncPoint(ContextSyncPoint);
		ContextSyncPoint = nullptr;
	}

	EndPayload();

	OutPayloads.Append(MoveTemp(Payloads));
}

void FVulkanContextCommon::FlushCommands(EVulkanFlushFlags FlushFlags)
{
	FVulkanSyncPointRef SyncPoint;
	if (EnumHasAnyFlags(FlushFlags, EVulkanFlushFlags::WaitForCompletion))
	{
		SyncPoint = GetContextSyncPoint();
	}

	FGraphEventRef SubmissionEvent;
	if (EnumHasAnyFlags(FlushFlags, EVulkanFlushFlags::WaitForSubmission))
	{
		SubmissionEvent = FGraphEvent::CreateGraphEvent();
		AddSubmissionEvent(SubmissionEvent);
	}

	FVulkanPlatformCommandList* FinalizedPayloads = new FVulkanPlatformCommandList;
	Finalize(*FinalizedPayloads);

	FDynamicRHI::FRHISubmitCommandListsArgs Args;
	Args.CommandLists.Add(FinalizedPayloads);
	FVulkanDynamicRHI::Get().RHISubmitCommandLists(MoveTemp(Args));

	if (SyncPoint)
	{
		FVulkanDynamicRHI::Get().ProcessInterruptQueueUntil(SyncPoint);
	}

	if (Device.UseMinimalSubmits())
	{
		FVulkanDynamicRHI::Get().KickSubmissionThread(true);
	}

	if (SubmissionEvent && !SubmissionEvent->IsComplete())
	{
		SCOPED_NAMED_EVENT_TEXT("Submission_Wait", FColor::Turquoise);
		SubmissionEvent->Wait();
	}
}

#if VULKAN_DELETE_STALE_CMDBUFFERS
struct FRHICommandFreeUnusedCmdBuffers final : public FRHICommand<FRHICommandFreeUnusedCmdBuffers>
{
	FVulkanCommandBufferPool* Pool;
	FVulkanQueue* Queue;
	bool bTrimMemory;

	FRHICommandFreeUnusedCmdBuffers(FVulkanCommandBufferPool* InPool, FVulkanQueue* InQueue, bool bInTrimMemory)
		: Pool(InPool)
		, Queue(InQueue)
		, bTrimMemory(bInTrimMemory)
	{
	}

	void Execute(FRHICommandListBase& CmdList)
	{
		Pool->FreeUnusedCmdBuffers(Queue, bTrimMemory);
	}
};
#endif

void FVulkanContextCommon::FreeUnusedCmdBuffers(bool bTrimMemory)
{
#if VULKAN_DELETE_STALE_CMDBUFFERS
	FRHICommandList& RHICmdList = FRHICommandListExecutor::GetImmediateCommandList();
	if (!IsInRenderingThread() || (RHICmdList.Bypass() || !IsRunningRHIInSeparateThread()))
	{
		Pool.FreeUnusedCmdBuffers(&Queue, bTrimMemory);
	}
	else
	{
		check(IsInRenderingThread());
		ALLOC_COMMAND_CL(RHICmdList, FRHICommandFreeUnusedCmdBuffers)(&Pool, &Queue, bTrimMemory);
	}
#endif
}

void FVulkanContextCommon::PrepareNewCommandBuffer(FVulkanPayload& Payload, bool bAllowTimestamp)
{
	FScopeLock ScopeLock(&Pool.CS);

	FVulkanCommandBuffer* NewCommandBuffer = nullptr;

	for (int32 Index = 0; Index < Pool.CmdBuffers.Num(); ++Index)
	{
		FVulkanCommandBuffer* CmdBuffer = Pool.CmdBuffers[Index];
		if (CmdBuffer->State == FVulkanCommandBuffer::EState::ReadyForBegin || CmdBuffer->State == FVulkanCommandBuffer::EState::NeedReset)
		{
			NewCommandBuffer = CmdBuffer;
			break;
		}
		else
		{
			check(CmdBuffer->IsSubmitted() || CmdBuffer->HasEnded());
		}
	}

	// All cmd buffers are being executed still, create a new one
	if (!NewCommandBuffer)
	{
		NewCommandBuffer = Pool.Create();
	}

	Payload.CommandBuffers.Add(NewCommandBuffer);

	// Only record begin/end timestamps on primary command buffers
	const bool bIsPrimaryCommandBuffer = (NewCommandBuffer->GetCommandBufferType() == EVulkanCommandBufferType::Primary);
	
	FVulkanQueryPool* OptionalTimestampQueryPool = bAllowTimestamp && GRHIGlobals.SupportsTimestampRenderQueries && bIsPrimaryCommandBuffer
		? GetCurrentTimestampQueryPool(Payload)
		: nullptr;

	VkRenderPass RenderPassHandle = GetParallelRenderPassInfo()
		? GetParallelRenderPassInfo()->RenderPassHandle
		: VK_NULL_HANDLE;

	NewCommandBuffer->Begin(OptionalTimestampQueryPool, RenderPassHandle);
}

void FVulkanContextCommon::HandleReservedResourceCommits(TArrayView<const FRHITransition*> Transitions)
{
	TArray<FVulkanCommitReservedResourceDesc> ReservedResourcesToCommit;
	for (const FRHITransition* Transition : Transitions)
	{
		const FVulkanTransitionData* TransitionData = Transition->GetPrivateData<FVulkanTransitionData>();
		for (const FRHITransitionInfo& Info : TransitionData->TransitionInfos)
		{
			if (const FRHICommitResourceInfo* CommitInfo = Info.CommitInfo.GetPtrOrNull())
			{
				if (Info.Type == FRHITransitionInfo::EType::Buffer)
				{
					FVulkanCommitReservedResourceDesc CommitDesc;
					CommitDesc.Resource = Info.Buffer;
					CommitDesc.CommitSizeInBytes = CommitInfo->SizeInBytes;

					checkf(CommitDesc.Resource, TEXT("FVulkanCommitReservedResourceDesc::Resource must be set"));

					ReservedResourcesToCommit.Add(CommitDesc);
				}
				else
				{
					checkNoEntry();
				}
			}
		}

	}

	if (!ReservedResourcesToCommit.IsEmpty())
	{
		GetPayload(EPhase::UpdateReservedResources).ReservedResourcesToCommit.Append(ReservedResourcesToCommit);
	}
}





FVulkanCommandListContext::FVulkanCommandListContext(FVulkanDevice& InDevice, ERHIPipeline InPipeline, bool bInImmediate)
	: FVulkanContextCommon(InDevice, *InDevice.GetQueue(InPipeline), EVulkanCommandBufferType::Primary)
	, RHIPipeline(InPipeline)
	, bSupportsBreadcrumbs(GRHIGlobals.SupportsTimestampRenderQueries)
	, bImmediate(bInImmediate)
{
	// Create Pending state, contains pipeline states such as current shader and etc..
	PendingGfxState = new FVulkanPendingGfxState(Device);
	PendingComputeState = new FVulkanPendingComputeState(Device);

	StaticUniformBuffers.AddZeroed(FUniformBufferStaticSlotRegistry::Get().GetSlotCount());

	if (InDevice.GetOptionalExtensions().HasKHRDynamicRendering)
	{
		CurrentDynamicRenderingInfo = new FVulkanDynamicRenderingInfo;
	}
}

FVulkanCommandListContext::FVulkanCommandListContext(FVulkanDevice& InDevice, FVulkanParallelRenderPassInfo* InParallelRenderPassInfo)
	: FVulkanContextCommon(InDevice, *InDevice.GetQueue(ERHIPipeline::Graphics), InDevice.GetOptionalExtensions().HasKHRDynamicRendering ? EVulkanCommandBufferType::Parallel : EVulkanCommandBufferType::Secondary)
	, RHIPipeline(ERHIPipeline::Graphics)
	, bSupportsBreadcrumbs(false)
	, bImmediate(false)
	, CurrentParallelRenderPassInfo(InParallelRenderPassInfo)
{
	checkf(CurrentParallelRenderPassInfo, TEXT("Secondary command buffers should be created with a FVulkanParallelRenderPassInfo."));

	// Only graphic commands can be used
	PendingGfxState = new FVulkanPendingGfxState(Device);

	StaticUniformBuffers.AddZeroed(FUniformBufferStaticSlotRegistry::Get().GetSlotCount());
}

void FVulkanCommandListContext::NotifyDeletedRenderTarget(VkImage Image)
{
	if (CurrentFramebuffer && CurrentFramebuffer->ContainsRenderTarget(Image))
	{
		CurrentFramebuffer = nullptr;
	}
}


void FVulkanCommandListContext::ReleasePendingState()
{
	delete PendingGfxState;
	PendingGfxState = nullptr;

	delete PendingComputeState;
	PendingComputeState = nullptr;

	if (CurrentDynamicRenderingInfo)
	{
		delete CurrentDynamicRenderingInfo;
		CurrentDynamicRenderingInfo = nullptr;
	}
}

FVulkanCommandListContext::~FVulkanCommandListContext()
{
	ReleasePendingState();
}

void FVulkanCommandListContext::FillDynamicRenderingInfo(const FRHIRenderPassInfo& InRenderPassInfo, FVulkanDynamicRenderingInfo& OutRenderingInfo)
{
	VkRenderingInfo& RenderingInfo = OutRenderingInfo.RenderingInfo;
	ZeroVulkanStruct(RenderingInfo, VK_STRUCTURE_TYPE_RENDERING_INFO);

	TStaticArray<VkRenderingAttachmentInfo, MaxSimultaneousRenderTargets>& ColorAttachmentInfos = OutRenderingInfo.ColorAttachmentInfos;
	VkRenderingAttachmentInfo& DepthAttachmentInfo = OutRenderingInfo.DepthAttachmentInfo;
	VkRenderingAttachmentInfo& StencilAttachmentInfo = OutRenderingInfo.StencilAttachmentInfo;
	VkRenderingFragmentShadingRateAttachmentInfoKHR& FragmentShadingRateAttachmentInfo = OutRenderingInfo.FragmentShadingRateAttachmentInfo;

	uint32 NumSamples = 0;
	uint32 MipIndex = 0;
	uint32 NumLayers = 0;

	FMemory::Memzero(OutRenderingInfo.ColorAttachmentViews);
	OutRenderingInfo.PartialDepthAttachmentView = nullptr;

	// CustomResolveSubpass can have targets with a different NumSamples
	// Also with a CustomResolveSubpass last color attachment is a resolve target
	const bool bHasCustomResolveSubpass = (InRenderPassInfo.SubpassHint == ESubpassHint::CustomResolveSubpass);

	const int32 NumColorRenderTargets = InRenderPassInfo.GetNumColorRenderTargets();
	if (NumColorRenderTargets)
	{
		RenderingInfo.colorAttachmentCount = NumColorRenderTargets;
		RenderingInfo.pColorAttachments = ColorAttachmentInfos.GetData();

		for (int32 Index = 0; Index < NumColorRenderTargets; ++Index)
		{
			const FRHIRenderPassInfo::FColorEntry& ColorEntry = InRenderPassInfo.ColorRenderTargets[Index];

			VkRenderingAttachmentInfo& AttachmentInfo = ColorAttachmentInfos[Index];
			ZeroVulkanStruct(AttachmentInfo, VK_STRUCTURE_TYPE_RENDERING_ATTACHMENT_INFO);

			FVulkanTexture* Texture = ResourceCast(ColorEntry.RenderTarget);
			check(Texture);
			const FRHITextureDesc& TextureDesc = Texture->GetDesc();

			VkExtent2D& Extent = RenderingInfo.renderArea.extent;
			const uint32 ExtentWidth = FMath::Max(1, TextureDesc.Extent.X >> ColorEntry.MipIndex);
			ensure((Extent.width == 0) || (Extent.width == ExtentWidth));
			Extent.width = ExtentWidth;
			const uint32 ExtentHeight = FMath::Max(1, TextureDesc.Extent.Y >> ColorEntry.MipIndex);
			ensure((Extent.height == 0) || (Extent.height == ExtentHeight));
			Extent.height = ExtentHeight;

			NumLayers = TextureDesc.Depth;

			AttachmentInfo.imageLayout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;
			AttachmentInfo.loadOp = RenderTargetLoadActionToVulkan(GetLoadAction(ColorEntry.Action));
			AttachmentInfo.storeOp = RenderTargetStoreActionToVulkan(GetStoreAction(ColorEntry.Action));
			ensure(!EnumHasAnyFlags(TextureDesc.Flags, TexCreate_Memoryless) || (AttachmentInfo.storeOp == VK_ATTACHMENT_STORE_OP_DONT_CARE));

			if (AttachmentInfo.loadOp == VK_ATTACHMENT_LOAD_OP_CLEAR)
			{
				const FLinearColor ClearColor = Texture->HasClearValue() ? Texture->GetClearColor() : FLinearColor::Black;
				AttachmentInfo.clearValue.color.float32[0] = ClearColor.R;
				AttachmentInfo.clearValue.color.float32[1] = ClearColor.G;
				AttachmentInfo.clearValue.color.float32[2] = ClearColor.B;
				AttachmentInfo.clearValue.color.float32[3] = ClearColor.A;
			}

			MipIndex = ColorEntry.MipIndex;
			FVulkanView* View = FVulkanFramebuffer::GetColorRenderTargetViewDesc(Texture, MipIndex, ColorEntry.ArraySlice, InRenderPassInfo.MultiViewCount, NumLayers);
			AttachmentInfo.imageView = View->GetTextureView().View;

			ensure(!NumSamples || (NumSamples == ColorEntry.RenderTarget->GetNumSamples()) || bHasCustomResolveSubpass);
			NumSamples = ColorEntry.RenderTarget->GetNumSamples();

			const bool bCustomResolveAttachment = (Index == (NumColorRenderTargets - 1)) && bHasCustomResolveSubpass;
			const VkSampleCountFlagBits Samples = bCustomResolveAttachment ? VK_SAMPLE_COUNT_1_BIT : static_cast<VkSampleCountFlagBits>(NumSamples);
			if ((Samples > VK_SAMPLE_COUNT_1_BIT) && ColorEntry.ResolveTarget)
			{
				FVulkanTexture* ResolveTexture = ResourceCast(ColorEntry.ResolveTarget);
				if (FVulkanView* ResolveView = FVulkanFramebuffer::GetColorResolveTargetViewDesc(ResolveTexture, MipIndex, ColorEntry.ArraySlice))
				{
					AttachmentInfo.resolveMode = IsInteger(TextureDesc.Format) ? VK_RESOLVE_MODE_SAMPLE_ZERO_BIT : VK_RESOLVE_MODE_AVERAGE_BIT;
					AttachmentInfo.resolveImageLayout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;
					AttachmentInfo.resolveImageView = ResolveView->GetTextureView().View;
				}
			}

			OutRenderingInfo.ColorAttachmentViews[Index] = View;
		}
	}

	if (InRenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget)
	{
		VkImageLayout CurrentDepthLayout = VK_IMAGE_LAYOUT_UNDEFINED;
		VkImageLayout CurrentStencilLayout = VK_IMAGE_LAYOUT_UNDEFINED;
		ExtractDepthStencilLayouts(InRenderPassInfo, CurrentDepthLayout, CurrentStencilLayout);

		FVulkanTexture* Texture = ResourceCast(InRenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget);
		check(Texture);
		const FRHITextureDesc& TextureDesc = Texture->GetDesc();
		check(IsDepthOrStencilFormat(TextureDesc.Format));

		OutRenderingInfo.PartialDepthAttachmentView = Texture->PartialView;

		VkExtent2D& Extent = RenderingInfo.renderArea.extent;
		Extent.width = (Extent.width == 0) ? TextureDesc.Extent.X : FMath::Min<int32>(Extent.width, TextureDesc.Extent.X);
		Extent.height = (Extent.height == 0) ? TextureDesc.Extent.Y : FMath::Min<int32>(Extent.height, TextureDesc.Extent.Y);
		NumLayers = (NumLayers == 0) ? Texture->GetNumberOfArrayLevels() : NumLayers;

		FVulkanView* DepthView = nullptr;
		bool bIsDepthStore = false;
		if (InRenderPassInfo.DepthStencilRenderTarget.ExclusiveDepthStencil.IsUsingDepth())
		{
			ZeroVulkanStruct(DepthAttachmentInfo, VK_STRUCTURE_TYPE_RENDERING_ATTACHMENT_INFO);
			RenderingInfo.pDepthAttachment = &DepthAttachmentInfo;
			if (Texture->HasClearValue())
			{
				Texture->GetDepthStencilClearValue(DepthAttachmentInfo.clearValue.depthStencil.depth, DepthAttachmentInfo.clearValue.depthStencil.stencil);
			}

			// We can't have the final layout be UNDEFINED, but it's possible that we get here from a transient texture
			// where the stencil was never used yet.  We can set the layout to whatever we want, the next transition will
			// happen from UNDEFINED anyhow.
			if (CurrentDepthLayout != VK_IMAGE_LAYOUT_UNDEFINED)
			{
				DepthAttachmentInfo.imageLayout = CurrentDepthLayout;
				DepthAttachmentInfo.loadOp = RenderTargetLoadActionToVulkan(GetLoadAction(GetDepthActions(InRenderPassInfo.DepthStencilRenderTarget.Action)));
				DepthAttachmentInfo.storeOp = RenderTargetStoreActionToVulkan(GetStoreAction(GetDepthActions(InRenderPassInfo.DepthStencilRenderTarget.Action)));
				ensure(!EnumHasAnyFlags(TextureDesc.Flags, TexCreate_Memoryless) || (DepthAttachmentInfo.storeOp == VK_ATTACHMENT_STORE_OP_DONT_CARE));

				bIsDepthStore = (DepthAttachmentInfo.storeOp == VK_ATTACHMENT_STORE_OP_STORE);
			}
			else
			{
				// Unused image aspects with undefined layout should just remain untouched
				DepthAttachmentInfo.imageLayout = VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL;
				DepthAttachmentInfo.loadOp = VK_ATTACHMENT_LOAD_OP_NONE;
				DepthAttachmentInfo.storeOp = VK_ATTACHMENT_STORE_OP_NONE;
			}

			DepthView = FVulkanFramebuffer::GetDepthStencilTargetViewDesc(Texture, NumColorRenderTargets, MipIndex, InRenderPassInfo.MultiViewCount, NumLayers);
			DepthAttachmentInfo.imageView = DepthView->GetTextureView().View;
		}

		bool bIsStencilStore = false;
		if (InRenderPassInfo.DepthStencilRenderTarget.ExclusiveDepthStencil.IsUsingStencil() && IsStencilFormat(TextureDesc.Format))
		{
			ZeroVulkanStruct(StencilAttachmentInfo, VK_STRUCTURE_TYPE_RENDERING_ATTACHMENT_INFO);
			RenderingInfo.pStencilAttachment = &StencilAttachmentInfo;
			if (Texture->HasClearValue())
			{
				Texture->GetDepthStencilClearValue(StencilAttachmentInfo.clearValue.depthStencil.depth, StencilAttachmentInfo.clearValue.depthStencil.stencil);
			}

			if (CurrentStencilLayout != VK_IMAGE_LAYOUT_UNDEFINED)
			{
				StencilAttachmentInfo.imageLayout = CurrentStencilLayout;
				StencilAttachmentInfo.loadOp = RenderTargetLoadActionToVulkan(GetLoadAction(GetStencilActions(InRenderPassInfo.DepthStencilRenderTarget.Action)));
				StencilAttachmentInfo.storeOp = RenderTargetStoreActionToVulkan(GetStoreAction(GetStencilActions(InRenderPassInfo.DepthStencilRenderTarget.Action)));

				bIsStencilStore = (StencilAttachmentInfo.storeOp == VK_ATTACHMENT_STORE_OP_STORE);
			}
			else
			{
				// Unused image aspects with undefined layout should just remain untouched
				StencilAttachmentInfo.imageLayout = VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL;
				StencilAttachmentInfo.loadOp = VK_ATTACHMENT_LOAD_OP_NONE;
				StencilAttachmentInfo.storeOp = VK_ATTACHMENT_STORE_OP_NONE;
			}

			if (!DepthView)
			{
				DepthView = FVulkanFramebuffer::GetDepthStencilTargetViewDesc(Texture, NumColorRenderTargets, MipIndex, InRenderPassInfo.MultiViewCount, NumLayers);
			}
			StencilAttachmentInfo.imageView = DepthView->GetTextureView().View;
		}

		if (FVulkanPlatform::RequiresDepthStencilFullWrite() &&
			VKHasAllFlags(Texture->GetFullAspectMask(), (VK_IMAGE_ASPECT_DEPTH_BIT | VK_IMAGE_ASPECT_STENCIL_BIT)) &&
			(bIsDepthStore || bIsStencilStore))
		{
			// Workaround for old mali drivers: writing not all of the image aspects to compressed render-target could cause gpu-hang
			DepthAttachmentInfo.storeOp = VK_ATTACHMENT_STORE_OP_STORE;
			StencilAttachmentInfo.storeOp = VK_ATTACHMENT_STORE_OP_STORE;
		}

		// CustomResolveSubpass can have targets with a different NumSamples
		ensure(!NumSamples || (NumSamples == InRenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget->GetNumSamples()) || bHasCustomResolveSubpass);
		NumSamples = InRenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget->GetNumSamples();
		const VkSampleCountFlagBits Samples = static_cast<VkSampleCountFlagBits>(NumSamples);

		if (GRHISupportsDepthStencilResolve && (Samples > VK_SAMPLE_COUNT_1_BIT) && InRenderPassInfo.DepthStencilRenderTarget.ResolveTarget)
		{
			FVulkanTexture* ResolveTexture = ResourceCast(InRenderPassInfo.DepthStencilRenderTarget.ResolveTarget);

			if (FVulkanView* ResolveView = FVulkanFramebuffer::GetDepthStencilResolveTargetViewDesc(ResolveTexture, MipIndex))
			{
				DepthAttachmentInfo.resolveMode = VK_RESOLVE_MODE_SAMPLE_ZERO_BIT;
				DepthAttachmentInfo.resolveImageLayout = CurrentDepthLayout;
				DepthAttachmentInfo.imageView = ResolveView->GetTextureView().View;
				StencilAttachmentInfo.resolveMode = VK_RESOLVE_MODE_SAMPLE_ZERO_BIT;
				StencilAttachmentInfo.resolveImageLayout = CurrentStencilLayout;
				StencilAttachmentInfo.imageView = ResolveView->GetTextureView().View;
			}
		}
	}

	if ((NumColorRenderTargets == 0) && !RenderingInfo.pDepthAttachment && !RenderingInfo.pStencilAttachment)
	{
		// No Depth and no color, it's a raster-only pass so make sure the renderArea will be set up properly
		VkExtent2D& Extent = RenderingInfo.renderArea.extent;
		Extent.width = InRenderPassInfo.ResolveRect.X2 - InRenderPassInfo.ResolveRect.X1;
		Extent.height = InRenderPassInfo.ResolveRect.Y2 - InRenderPassInfo.ResolveRect.Y1;
		NumLayers = 1;

		VkOffset2D& Offset = RenderingInfo.renderArea.offset;
		Offset.x = InRenderPassInfo.ResolveRect.X1;
		Offset.y = InRenderPassInfo.ResolveRect.Y1;
	}

	if (GRHISupportsAttachmentVariableRateShading && InRenderPassInfo.ShadingRateTexture)
	{
		ZeroVulkanStruct(FragmentShadingRateAttachmentInfo, VK_STRUCTURE_TYPE_RENDERING_FRAGMENT_DENSITY_MAP_ATTACHMENT_INFO_EXT);
		RenderingInfo.pNext = &FragmentShadingRateAttachmentInfo;

		FVulkanTexture* Texture = ResourceCast(InRenderPassInfo.ShadingRateTexture);
		check(Texture->GetFormat() == GRHIVariableRateShadingImageFormat);
		FVulkanView* View = FVulkanFramebuffer::GetFragmentDensityAttachmentViewDesc(Texture, MipIndex);
		FragmentShadingRateAttachmentInfo.imageView = View->GetTextureView().View;

		if (ValidateShadingRateDataType())
		{
			if (GRHIVariableRateShadingImageDataType == VRSImage_Palette)
			{
				FragmentShadingRateAttachmentInfo.imageLayout = VK_IMAGE_LAYOUT_FRAGMENT_SHADING_RATE_ATTACHMENT_OPTIMAL_KHR;
			}
			if (GRHIVariableRateShadingImageDataType == VRSImage_Fractional)
			{
				FragmentShadingRateAttachmentInfo.imageLayout = VK_IMAGE_LAYOUT_FRAGMENT_DENSITY_MAP_OPTIMAL_EXT;
			}
		}
	}

	RenderingInfo.viewMask = (0b1 << InRenderPassInfo.MultiViewCount) - 1;
	RenderingInfo.layerCount = NumLayers;
}


void FVulkanCommandListContext::RHIBeginParallelRenderPass(TSharedPtr<FRHIParallelRenderPassInfo> InInfo, const TCHAR* InName)
{
	checkf(CurrentParallelRenderPassInfo == nullptr, TEXT("There is already a parallel render pass in progress!"));
	CurrentParallelRenderPassInfo = new FVulkanParallelRenderPassInfo();

	if (Device.GetOptionalExtensions().HasKHRDynamicRendering)
	{
		check(InInfo->NumOcclusionQueries == 0);
		RenderPassInfo = *InInfo;

		CurrentParallelRenderPassInfo->DynamicRenderingInfo = new FVulkanDynamicRenderingInfo();
		FillDynamicRenderingInfo(*InInfo.Get(), *CurrentParallelRenderPassInfo->DynamicRenderingInfo);

		// Set the flags for the parallel contexts to use directly
		CurrentParallelRenderPassInfo->DynamicRenderingInfo->RenderingInfo.flags = VK_RENDERING_RESUMING_BIT | VK_RENDERING_SUSPENDING_BIT;
	}
	else
	{
		RHIBeginRenderPass(*InInfo.Get(), InName);
		CurrentParallelRenderPassInfo->RenderPassHandle = CurrentRenderPass->GetHandle();
	}

	InInfo->RHIPlatformData = CurrentParallelRenderPassInfo;
}

void FVulkanCommandListContext::RHIEndParallelRenderPass()
{
	if (!CurrentParallelRenderPassInfo->SecondaryPayloads.IsEmpty())
	{
		// Merge all the payloads into a single one
		FVulkanPayload* MergedPayload = CurrentParallelRenderPassInfo->SecondaryPayloads[0];
		for (int32 Index = 1; Index < CurrentParallelRenderPassInfo->SecondaryPayloads.Num(); ++Index)
		{
			FVulkanPayload* OtherPayload = CurrentParallelRenderPassInfo->SecondaryPayloads[Index];
			MergedPayload->Merge(*OtherPayload);
			delete OtherPayload;
		}
		CurrentParallelRenderPassInfo->SecondaryPayloads.Reset();

		if (Device.GetOptionalExtensions().HasKHRDynamicRendering)
		{
			// Make sure the first command buffer is the one that only suspends (should rarely happen)
			if (VKHasAllFlags(MergedPayload->CommandBuffers[0]->LastDynamicRenderingFlags, VK_RENDERING_RESUMING_BIT))
			{
				for (int32 Index = 1; Index < MergedPayload->CommandBuffers.Num(); ++Index)
				{
					if (!VKHasAllFlags(MergedPayload->CommandBuffers[Index]->LastDynamicRenderingFlags, VK_RENDERING_RESUMING_BIT))
					{
						MergedPayload->CommandBuffers.Swap(0, Index);
						break;
					}
				}
			}

			// If the parallel pass has a single payload, it's possible we leave things in a suspended state
			const bool bNeedsResume = VKHasAllFlags(MergedPayload->CommandBuffers.Last()->LastDynamicRenderingFlags, VK_RENDERING_SUSPENDING_BIT);
			CurrentParallelRenderPassInfo->DynamicRenderingInfo->RenderingInfo.flags = VK_RENDERING_RESUMING_BIT;
			VkRenderingInfo* NeedsResumeInfo = bNeedsResume ? &CurrentParallelRenderPassInfo->DynamicRenderingInfo->RenderingInfo : nullptr;

			// Insert all the parallel payloads
			AppendParallelRenderPayload(MergedPayload, NeedsResumeInfo);
		}
		else
		{
			FVulkanPayload& ParentPayload = GetPayload(EPhase::Execute);
			FVulkanCommandBuffer& ParentCommandBuffer = GetCommandBuffer();

			TArray<VkCommandBuffer> CommandBufferHandles;
			CommandBufferHandles.Reserve(MergedPayload->CommandBuffers.Num());
			for (FVulkanCommandBuffer* SecondaryCommandBuffer : MergedPayload->CommandBuffers)
			{
				CommandBufferHandles.Add(SecondaryCommandBuffer->GetHandle());
			}
			ParentCommandBuffer.ExecutedSecondaryCommandBuffers.Append(MoveTemp(MergedPayload->CommandBuffers));

			VulkanRHI::vkCmdExecuteCommands(ParentCommandBuffer.GetHandle(), CommandBufferHandles.Num(), CommandBufferHandles.GetData());

			ParentPayload.Merge(*MergedPayload);
			delete MergedPayload;
		}
	}

	if (!Device.GetOptionalExtensions().HasKHRDynamicRendering)
	{
		RHIEndRenderPass();
	}

	delete CurrentParallelRenderPassInfo;
	CurrentParallelRenderPassInfo = nullptr;
}

void FVulkanCommandListContext::Finalize(TArray<FVulkanPayload*>& OutPayloads)
{
	FlushProfilerStats();

	if (CurrentDescriptorPoolSetContainer)
	{
		GetPayload(EPhase::Signal).DescriptorPoolSetContainers.Add(CurrentDescriptorPoolSetContainer);
		CurrentDescriptorPoolSetContainer = nullptr;
		TypedDescriptorPoolSets.Reset();
	}
	else
	{
		check(TypedDescriptorPoolSets.Num() == 0);
	}

	FVulkanContextCommon::Finalize(OutPayloads);
}



void FVulkanCommandListContext::AcquirePoolSetContainer()
{
	if (!CurrentDescriptorPoolSetContainer)
	{
		CurrentDescriptorPoolSetContainer = &Device.GetDescriptorPoolsManager().AcquirePoolSetContainer();
		ensure(TypedDescriptorPoolSets.Num() == 0);
	}
}

bool FVulkanCommandListContext::AcquirePoolSetAndDescriptorsIfNeeded(const FVulkanDescriptorSetsLayout& Layout, bool bNeedDescriptors, VkDescriptorSet* OutDescriptors)
{
	AcquirePoolSetContainer();

	const uint32 Hash = VULKAN_HASH_POOLS_WITH_TYPES_USAGE_ID ? Layout.GetTypesUsageID() : GetTypeHash(Layout);
	FVulkanTypedDescriptorPoolSet*& FoundTypedSet = TypedDescriptorPoolSets.FindOrAdd(Hash);

	if (!FoundTypedSet)
	{
		FoundTypedSet = CurrentDescriptorPoolSetContainer->AcquireTypedPoolSet(Layout);
		bNeedDescriptors = true;
	}

	if (bNeedDescriptors)
	{
		return FoundTypedSet->AllocateDescriptorSets(Layout, OutDescriptors);
	}

	return false;
}

void FVulkanCommandListContext::ApplyShaderBindingLayout()
{
	if (ShaderBindingLayout && ShaderBindingLayout->GetNumUniformBufferEntries())
	{
		checkf(Device.GetBindlessDescriptorManager()->IsSupported(), TEXT("Static uniform buffer require bindless."));

		TArray<uint32> UBBindlessIndices;
		UBBindlessIndices.SetNum(ShaderBindingLayout->GetNumUniformBufferEntries());
		for (uint32 Index = 0; Index < ShaderBindingLayout->GetNumUniformBufferEntries(); ++Index)
		{
			checkf(StaticUniformBuffers[Index], TEXT("Missing static uniform buffer for shader binding layout (%s)."),
				*ShaderBindingLayout->GetUniformBufferEntry(Index).LayoutName);

			FVulkanUniformBuffer* StaticUniformBuffer = ResourceCast(StaticUniformBuffers[Index]);
			const FRHIDescriptorHandle BindlessHandle = StaticUniformBuffer->GetBindlessHandle();
			check(BindlessHandle.IsValid());

			UBBindlessIndices[Index] = BindlessHandle.GetIndex();
		}

		GetCommandBuffer().SetStaticUniformBufferPushConstants(UBBindlessIndices);
	}
}

// Inserts a Vulkan-specific end frame marker, which can be used by tools like RenderDoc to
// identify when a frame is done when there is no native Present (e.g. with a standalone XR
// device rendering to an offscreen buffer
void FVulkanCommandListContext::InsertEndFrameDebugLabel()
{
#if VULKAN_ENABLE_DRAW_MARKERS
	if (auto CmdDbgInsertLabel = Device.GetCmdInsertDebugLabel())
	{
		FTCHARToUTF8 Converter(TEXT("vr-marker,frame_end,type,application"));
		VkDebugUtilsLabelEXT Label;
		ZeroVulkanStruct(Label, VK_STRUCTURE_TYPE_DEBUG_UTILS_LABEL_EXT);
		Label.pLabelName = Converter.Get();
		const FLinearColor LColor = FLinearColor::Blue;
		Label.color[0] = LColor.R;
		Label.color[1] = LColor.G;
		Label.color[2] = LColor.B;
		Label.color[3] = LColor.A;
		CmdDbgInsertLabel(GetCommandBuffer().GetHandle(), &Label);
	}
#endif //VULKAN_ENABLE_DRAW_MARKERS
}


TLockFreePointerListUnordered<FVulkanUploadContext, PLATFORM_CACHE_LINE_SIZE> FVulkanUploadContext::Pool;
void FVulkanUploadContext::DestroyPool()
{
	while (FVulkanUploadContext* Context = FVulkanUploadContext::Pool.Pop())
	{
		delete Context;
	}
}


FVulkanUploadContext::FVulkanUploadContext(FVulkanDevice& InDevice, FVulkanQueue& InQueue)
	: FVulkanContextCommon(InDevice, InQueue, EVulkanCommandBufferType::Primary)
{

}
FVulkanUploadContext::~FVulkanUploadContext()
{

}

IRHIUploadContext* FVulkanDynamicRHI::RHIGetUploadContext()
{
	FVulkanUploadContext* Context = FVulkanUploadContext::Pool.Pop();
	if (!Context)
	{
		// :todo-jn: locked to graphics queue for now
		Context = new FVulkanUploadContext(*Device, *Device->GetGraphicsQueue());
	}
	return Context;
}