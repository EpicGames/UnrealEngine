// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanContext.h: Class to generate Vulkan command buffers from RHI CommandLists
=============================================================================*/

#pragma once 

#include "VulkanResources.h"
#include "VulkanRHIPrivate.h"
#include "VulkanSubmission.h"
#include "VulkanSynchronization.h"

class FVulkanDevice;
class FVulkanDynamicRHI;
class FVulkanPendingGfxState;
class FVulkanPendingComputeState;
class FVulkanQueue;
class FVulkanSwapChain;


enum class EVulkanFlushFlags
{
	None = 0,

	// Block the calling thread until the submission thread has dispatched all work.
	WaitForSubmission = 1,

	// Both the calling thread until the GPU has signaled completion of all dispatched work.
	WaitForCompletion = 2
};
ENUM_CLASS_FLAGS(EVulkanFlushFlags)


struct FVulkanDynamicRenderingInfo
{
	// Vulkan structures sent to API
	TStaticArray<VkRenderingAttachmentInfo, MaxSimultaneousRenderTargets> ColorAttachmentInfos;
	VkRenderingAttachmentInfo DepthAttachmentInfo;
	VkRenderingAttachmentInfo StencilAttachmentInfo;
	VkRenderingFragmentShadingRateAttachmentInfoKHR FragmentShadingRateAttachmentInfo;
	VkRenderingInfo RenderingInfo;

	// View tracking used for input attachments
	TStaticArray<FVulkanView*, MaxSimultaneousRenderTargets> ColorAttachmentViews;
	FVulkanView* PartialDepthAttachmentView;
};

struct FVulkanParallelRenderPassInfo
{
	VkRenderPass RenderPassHandle = VK_NULL_HANDLE;
	FVulkanDynamicRenderingInfo* DynamicRenderingInfo = nullptr;
	TArray<FVulkanPayload*> SecondaryPayloads;
	std::atomic<int32> NumParallelContexts = 0;
};


class FVulkanContextCommon
{
protected:
	// Phases are used to track where we are at in filling the current payload.
	// Phases always move forward in the order declared here, never backwards.
	// Always use GetPayload(EPhase) to fetch the payload for the phase you want,
	// it will take care of creating a new payload when necessary.
	enum class EPhase
	{
		Wait,
		UpdateReservedResources,
		Execute,
		Signal
	};

public:
	FVulkanContextCommon(FVulkanDevice& InDevice, FVulkanQueue& InQueue, EVulkanCommandBufferType InCommandBufferType);
	virtual ~FVulkanContextCommon();

	FVulkanPayload& GetPayload(EPhase Phase)
	{
		if (Payloads.Num() == 0 || Phase < CurrentPhase)
		{
			NewPayload();
		}

		CurrentPhase = Phase;
		return *Payloads.Last();
	}

	// NOTE: This call is getting phased out, use GetCommandBuffer()
	FVulkanCommandBuffer* GetActiveCmdBuffer()
	{
		return &GetCommandBuffer();
	}

	FVulkanCommandBuffer& GetCommandBuffer()
	{
		FVulkanPayload& Payload = GetPayload(EPhase::Execute);
		checkSlow(Payload.SignalSemaphores.Num() == 0);

		if (Payload.CommandBuffers.Num() == 0)
		{
			PrepareNewCommandBuffer(Payload);
		}

		return *Payload.CommandBuffers.Last();
	}

	void AddWaitSemaphore(VkPipelineStageFlags InWaitFlags, FVulkanSemaphore* InWaitSemaphore)
	{
		AddWaitSemaphores(InWaitFlags, MakeArrayView<FVulkanSemaphore*>(&InWaitSemaphore, 1));
	}
	void AddWaitSemaphores(VkPipelineStageFlags InWaitFlags, TArrayView<FVulkanSemaphore*> InWaitSemaphores)
	{
		if (InWaitSemaphores.Num())
		{
			FVulkanPayload& Payload = GetPayload(EPhase::Wait);
			checkSlow((Payload.CommandBuffers.Num() == 0) && (Payload.SignalSemaphores.Num() == 0));

			Payload.WaitFlags.Reserve(Payload.WaitFlags.Num() + InWaitSemaphores.Num());
			for (int32 Index = 0; Index < InWaitSemaphores.Num(); ++Index)
			{
				Payload.WaitFlags.Add(InWaitFlags);
			}
			Payload.WaitSemaphores.Append(InWaitSemaphores);
		}
	}

	void AddSignalSemaphore(FVulkanSemaphore* InSignalSemaphore)
	{
		AddSignalSemaphores(MakeArrayView<FVulkanSemaphore*>(&InSignalSemaphore, 1));
	}
	void AddSignalSemaphores(TArrayView<FVulkanSemaphore*> InSignalSemaphores)
	{
		if (InSignalSemaphores.Num())
		{
			FVulkanPayload& Payload = GetPayload(EPhase::Signal);
			Payload.SignalSemaphores.Append(InSignalSemaphores);
		}
	}

	// Complete recording of the current command list set, and appends the resulting
	// payloads to the given array. Resets the context so new commands can be recorded.
	virtual void Finalize(TArray<FVulkanPayload*>& OutPayloads);

	VkCommandPool GetHandle() const
	{
		return Pool.GetHandle();
	}

	void FreeUnusedCmdBuffers(bool bTrimMemory);

	// Should only ne used when we are certain there are no other pending contexts (like UploadContext)
	void FlushCommands(EVulkanFlushFlags FlushFlags = EVulkanFlushFlags::None);

	// Public immutable references, access them directly
	FVulkanDevice& Device;
	FVulkanQueue& Queue;

	// Add a provided sync point that will be signaled at the end of the current payload's accumulated work
	void SignalSyncPoint(const FVulkanSyncPointRef& InSync)
	{
		FVulkanPayload& Payload = GetPayload(EPhase::Signal);
		Payload.SyncPoints.Add(InSync);
	}

	// Add provided sync points that will be appended next time a payload reaches the Signal phase
	void AddPendingSyncPoint(const FVulkanSyncPointRef& InSyncs)
	{
		PendingSyncPoints.Add(InSyncs);
	}

	// Add an event to be signaled with the current payload is submitted (does not alter phase)
	void AddSubmissionEvent(const FGraphEventRef& InEvent)
	{
		FVulkanPayload& Payload = GetPayload(CurrentPhase);
		Payload.SubmissionEvents.Add(InEvent);
	}

	// Force pending syncs to be sent to a payload
	void FlushPendingSyncPoints()
	{
		if (PendingSyncPoints.Num())
		{
			FVulkanPayload& Payload = GetPayload(EPhase::Signal);
			Payload.SyncPoints.Append(PendingSyncPoints);
			PendingSyncPoints.Empty();
		}
	}

	// Returns a single sync point for the context that will be inserted when it is finalized
	FVulkanSyncPointRef GetContextSyncPoint()
	{
		if (!ContextSyncPoint)
		{
			ContextSyncPoint = FVulkanSyncPoint::Create(EVulkanSyncPointType::Context, TEXT("VulkanContextSyncPoint"));
		}
		return ContextSyncPoint;
	}

	TArray<FVulkanQueryPool*>& GetQueryPoolArray(EVulkanQueryPoolType Type)
	{
		FVulkanPayload& Payload = GetPayload(EPhase::Execute);
		return Payload.QueryPools[(int32)Type];
	}

	FVulkanQueryPool* GetCurrentTimestampQueryPool()
	{
		return GetCurrentTimestampQueryPool(GetPayload(EPhase::Execute));
	}

	virtual FVulkanParallelRenderPassInfo* GetParallelRenderPassInfo()
	{
		return nullptr;
	}

	void HandleReservedResourceCommits(TArrayView<const FRHITransition*> Transitions);

protected:

	// End the current payload, insert the parallel rendering work, and leave context with a valid payload
	void AppendParallelRenderPayload(FVulkanPayload* ParallelRenderingPayload, VkRenderingInfo* NeedsResumeInfo);

private:

	FVulkanQueryPool* GetCurrentTimestampQueryPool(FVulkanPayload& Payload);
	void PrepareNewCommandBuffer(FVulkanPayload& Payload, bool bAllowTimestamp=true);
	void NewPayload();
	void EndPayload();

	FVulkanCommandBufferPool& Pool;

	TArray<FVulkanPayload*> Payloads;
	EPhase CurrentPhase = EPhase::Wait;

	// Sync points signaled at the next Signal phase (will not force a phase change) or when context is finalized
	TArray<FVulkanSyncPointRef> PendingSyncPoints;

	// Sync point signaled when the current context is finalized
	FVulkanSyncPointRef ContextSyncPoint;

	friend class FVulkanDynamicRHI;
};



class FVulkanCommandListContext : public FVulkanContextCommon, public IRHICommandContext
{
public:
	FVulkanCommandListContext(FVulkanDevice& InDevice, ERHIPipeline InPipeline, bool bInImmediate);
	virtual ~FVulkanCommandListContext();

	// Constructor for parallel render contexts that use secondary command buffers
	FVulkanCommandListContext(FVulkanDevice& InDevice, FVulkanParallelRenderPassInfo* InParallelRenderPassInfo);

	static inline FVulkanCommandListContext& Get(FRHICommandListBase& RHICmdList)
	{
		return static_cast<FVulkanCommandListContext&>(RHICmdList.GetContext().GetLowestLevelContext());
	}

	inline bool IsImmediate() const
	{
		return bImmediate;
	}

	virtual ERHIPipeline GetPipeline() const override
	{
		return RHIPipeline;
	}

	template <class ShaderType> void SetResourcesFromTables(const ShaderType* RESTRICT);
	void CommitGraphicsResourceTables();
	void CommitComputeResourceTables();

	virtual void RHISetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBuffer, uint32 Offset) final override;
	virtual void RHISetViewport(float MinX, float MinY, float MinZ, float MaxX, float MaxY, float MaxZ) final override;
	virtual void RHISetStereoViewport(float LeftMinX, float RightMinX, float LeftMinY, float RightMinY, float MinZ, float LeftMaxX, float RightMaxX, float LeftMaxY, float RightMaxY, float MaxZ) override;
	virtual void RHISetScissorRect(bool bEnable, uint32 MinX, uint32 MinY, uint32 MaxX, uint32 MaxY) final override;
	virtual void RHISetGraphicsPipelineState(FRHIGraphicsPipelineState* GraphicsState, uint32 StencilRef, bool bApplyAdditionalState) final override;
	virtual void RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers) final override;
	virtual void RHISetStaticUniformBuffer(FUniformBufferStaticSlot Slot, FRHIUniformBuffer* Buffer) final override;
	virtual void RHISetUniformBufferDynamicOffset(FUniformBufferStaticSlot Slot, uint32 InOffset) final override;
	virtual void RHISetShaderParameters(FRHIGraphicsShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override;
	virtual void RHISetShaderParameters(FRHIComputeShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override;
	virtual void RHISetStencilRef(uint32 StencilRef) final override;
	virtual void RHIDrawPrimitive(uint32 BaseVertexIndex, uint32 NumPrimitives, uint32 NumInstances) final override;
	virtual void RHIDrawPrimitiveIndirect(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
	virtual void RHIDrawIndexedPrimitive(FRHIBuffer* IndexBuffer, int32 BaseVertexIndex, uint32 FirstInstance, uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances) final override;
	virtual void RHIDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
#if PLATFORM_SUPPORTS_MESH_SHADERS
	virtual void RHIDispatchMeshShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) final override;
	virtual void RHIDispatchIndirectMeshShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
#endif
	virtual void RHISetDepthBounds(float MinDepth, float MaxDepth) final override;
#if WITH_RHI_BREADCRUMBS
	virtual void RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) final override;
	virtual void RHIEndBreadcrumbGPU  (FRHIBreadcrumbNode* Breadcrumb) final override;
#endif
	virtual void RHISetComputePipelineState(FRHIComputePipelineState* ComputePipelineState) final override;
	virtual void RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) final override;
	virtual void RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;

	virtual void RHISetMultipleViewports(uint32 Count, const FViewportBounds* Data) final override;
	virtual void RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values) final override;
	virtual void RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values) final override;
	virtual void RHICopyTexture(FRHITexture* SourceTexture, FRHITexture* DestTexture, const FRHICopyTextureInfo& CopyInfo) final override;
	virtual void RHICopyBufferRegion(FRHIBuffer* DstBuffer, uint64 DstOffset, FRHIBuffer* SrcBuffer, uint64 SrcOffset, uint64 NumBytes) final override;
	virtual void RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions) override final;
	virtual void RHIEndTransitions(TArrayView<const FRHITransition*> Transitions) override final;
	virtual void RHICopyToStagingBuffer(FRHIBuffer* SourceBuffer, FRHIStagingBuffer* DestinationStagingBuffer, uint32 Offset, uint32 NumBytes) final override;
	virtual void RHIWriteGPUFence(FRHIGPUFence* Fence) final override;

	// Render time measurement
	virtual void RHIBeginRenderQuery(FRHIRenderQuery* RenderQuery) final override;
	virtual void RHIEndRenderQuery(FRHIRenderQuery* RenderQuery) final override;

	void FlushProfilerStats()
	{
		// Flush accumulated draw stats (if breadcrumbs are available to attach them to)
		if (StatEvent && bSupportsBreadcrumbs)
		{
			GetCommandBuffer().EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FStats>() = StatEvent;
			StatEvent = {};
		}
	}

	virtual void Finalize(TArray<FVulkanPayload*>& OutPayloads) override;

	virtual void RHIBeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* InName) final override;
	virtual void RHIEndRenderPass() final override;
	virtual void RHINextSubpass() final override;

	virtual void RHIBeginParallelRenderPass(TSharedPtr<FRHIParallelRenderPassInfo> InInfo, const TCHAR* InName) final override;
	virtual void RHIEndParallelRenderPass() final override;

	virtual void RHICommitShaderBindingTable(FRHIShaderBindingTable* SBT, FRHIBuffer* InlineBindingDataBuffer) final override;
	virtual void RHIBindAccelerationStructureMemory(FRHIRayTracingScene* Scene, FRHIBuffer* Buffer, uint32 BufferOffset) final override;
	virtual void RHIBuildAccelerationStructures(TConstArrayView<FRayTracingGeometryBuildParams> Params, const FRHIBufferRange& ScratchBufferRange) final override;
	virtual void RHIBuildAccelerationStructures(TConstArrayView<FRayTracingSceneBuildParams> Params) final override;

	virtual void RHIRayTraceDispatch(FRHIRayTracingPipelineState* RayTracingPipelineState, FRHIRayTracingShader* RayGenShader,
		FRHIShaderBindingTable* SBT, const FRayTracingShaderBindings& GlobalResourceBindings,
		uint32 Width, uint32 Height) final override;
	virtual void RHIRayTraceDispatchIndirect(FRHIRayTracingPipelineState* RayTracingPipelineState, FRHIRayTracingShader* RayGenShader,
		FRHIShaderBindingTable* SBT, const FRayTracingShaderBindings& GlobalResourceBindings,
		FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;

	virtual void RHISetBindingsOnShaderBindingTable(FRHIShaderBindingTable* SBT,
		FRHIRayTracingPipelineState* Pipeline,
		uint32 NumBindings, const FRayTracingLocalShaderBindings* Bindings,
		ERayTracingBindingType BindingType) final override;

	void InsertEndFrameDebugLabel();

	inline FVulkanPendingGfxState* GetPendingGfxState()
	{
		return PendingGfxState;
	}

	inline FVulkanPendingComputeState* GetPendingComputeState()
	{
		return PendingComputeState;
	}

	void NotifyDeletedRenderTarget(VkImage Image);

	inline FVulkanRenderPass* GetCurrentRenderPass()
	{
		return CurrentRenderPass;
	}

	inline FVulkanFramebuffer* GetCurrentFramebuffer()
	{
		return CurrentFramebuffer;
	}

	void BeginRecursiveCommand()
	{
		// Nothing to do
	}

	void ReleasePendingState();

	FVulkanQueryPool* GetCurrentOcclusionQueryPool();

	virtual FVulkanParallelRenderPassInfo* GetParallelRenderPassInfo()
	{
		return CurrentParallelRenderPassInfo;
	}
	
	void SetParallelRenderPassInfo(FVulkanParallelRenderPassInfo* ParallelRenderPassInfo)
	{
		CurrentParallelRenderPassInfo = ParallelRenderPassInfo;
	}

	bool AcquirePoolSetAndDescriptorsIfNeeded(const class FVulkanDescriptorSetsLayout& Layout, bool bNeedDescriptors, VkDescriptorSet* OutDescriptors);

	bool IsSwapchainImage(FRHITexture* InTexture) const;
	FVulkanSwapChain* GetSwapChain() const;

	FVulkanRenderPass* PrepareRenderPassForPSOCreation(const FGraphicsPipelineStateInitializer& Initializer);
	FVulkanRenderPass* PrepareRenderPassForPSOCreation(const FVulkanRenderTargetLayout& Initializer);

	void ApplyShaderBindingLayout();

	void ExtractDepthStencilLayouts(const FRHIRenderPassInfo& InInfo, VkImageLayout& OutDepthLayout, VkImageLayout& OutStencilLayout);
	void FillDynamicRenderingInfo(const FRHIRenderPassInfo& RenderPassInfo, FVulkanDynamicRenderingInfo& OutRenderingInfo);

	const ERHIPipeline RHIPipeline;
	const bool bSupportsBreadcrumbs;
	const bool bImmediate;

	void BeginOcclusionQueryBatch(uint32 NumQueriesInBatch, uint32 MultiViewCount);

	TArray<FString> EventStack;

	FVulkanRenderPass* CurrentRenderPass = nullptr;
	FVulkanFramebuffer* CurrentFramebuffer = nullptr;

	FVulkanParallelRenderPassInfo* CurrentParallelRenderPassInfo = nullptr;
	FVulkanDynamicRenderingInfo* CurrentDynamicRenderingInfo = nullptr;

	FVulkanPendingGfxState* PendingGfxState = nullptr;
	FVulkanPendingComputeState* PendingComputeState = nullptr;

	// Match the D3D12 maximum of 16 constant buffers per shader stage.
	enum { MAX_UNIFORM_BUFFERS_PER_SHADER_STAGE = 16 };

	// Track the currently bound uniform buffers.
	FVulkanUniformBuffer* BoundUniformBuffers[SF_NumStandardFrequencies][MAX_UNIFORM_BUFFERS_PER_SHADER_STAGE] = {};

	// Bit array to track which uniform buffers have changed since the last draw call.
	uint16 DirtyUniformBuffers[SF_NumStandardFrequencies] = {};

	void InternalClearMRT(FVulkanCommandBuffer& CommandBuffer, bool bClearColor, int32 NumClearColors, const FLinearColor* ColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil);

	FVulkanDescriptorPoolSetContainer* CurrentDescriptorPoolSetContainer = nullptr;
	TMap<uint32, class FVulkanTypedDescriptorPoolSet*> TypedDescriptorPoolSets;
	void AcquirePoolSetContainer();

	inline const TArray<FRHIUniformBuffer*>& GetStaticUniformBuffers() const
	{
		return StaticUniformBuffers;
	}

private:
	void RHIClearMRT(bool bClearColor, int32 NumClearColors, const FLinearColor* ColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil);

	friend class FVulkanDevice;
	friend class FVulkanDynamicRHI;

	TArray<FRHIUniformBuffer*> StaticUniformBuffers;
	const FRHIShaderBindingLayout* ShaderBindingLayout = nullptr;
};

class FVulkanUploadContext : public FVulkanContextCommon, public IRHIUploadContext
{
public:

	static inline FVulkanUploadContext& Get(FRHICommandListBase& RHICmdList)
	{
		return static_cast<FVulkanUploadContext&>(RHICmdList.GetUploadContext());
	}

	FVulkanUploadContext(FVulkanDevice& Device, FVulkanQueue& InQueue);
	virtual ~FVulkanUploadContext();

	static TLockFreePointerListUnordered<FVulkanUploadContext, PLATFORM_CACHE_LINE_SIZE> Pool;
	static void DestroyPool();
};

template<>
struct TVulkanResourceTraits<IRHIUploadContext>
{
	typedef FVulkanUploadContext TConcreteType;
};


struct FVulkanContextArray : public TRHIPipelineArray<FVulkanCommandListContext*>
{
	FVulkanContextArray()
		: TRHIPipelineArray(InPlace, nullptr)
	{
	}

	FVulkanContextArray(FRHIContextArray const& Contexts)
		: TRHIPipelineArray(InPlace, nullptr)
	{
		for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
		{
			IRHIComputeContext* Context = Contexts[Pipeline];
			(*this)[Pipeline] = Context ? static_cast<FVulkanCommandListContext*>(&Context->GetLowestLevelContext()) : nullptr;
		}
	}
};
