// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanDebug.cpp: Vulkan device RHI implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"

static FAutoConsoleVariable GCVarUniqueValidationMessages(
	TEXT("r.Vulkan.UniqueValidationMessages"),
	1,
	TEXT("Filter out validation errors with the same code (only when r.Vulkan.EnableValidation is non zero)")
);

#define VULKAN_ENABLE_TRACKING_CALLSTACK			1

DEFINE_LOG_CATEGORY(LogVulkanRHI);

#if VULKAN_HAS_DEBUGGING_ENABLED

#if PLATFORM_ANDROID
	#define VULKAN_REPORT_LOG(Format, ...)	UE_LOG(LogVulkanRHI, Warning, Format, __VA_ARGS__)
#else
	#define VULKAN_REPORT_LOG(Format, ...)	do { if (FPlatformMisc::IsDebuggerPresent()) \
											{ \
												FPlatformMisc::LowLevelOutputDebugStringf(Format, __VA_ARGS__); FPlatformMisc::LowLevelOutputDebugString(TEXT("\n")); \
											} \
											UE_LOG(LogVulkanRHI, Warning, Format, __VA_ARGS__); } while (0)
#endif // PLATFORM_ANDROID

extern TAutoConsoleVariable<int32> GValidationCvar;

static VKAPI_ATTR VkBool32 VKAPI_CALL DebugUtilsCallback(VkDebugUtilsMessageSeverityFlagBitsEXT MsgSeverity, 
	VkDebugUtilsMessageTypeFlagsEXT MsgType, const VkDebugUtilsMessengerCallbackDataEXT* CallbackData, void* UserData)
{
	const bool bError = (MsgSeverity & VK_DEBUG_UTILS_MESSAGE_SEVERITY_ERROR_BIT_EXT) != 0;
	const bool bWarning = (MsgSeverity & VK_DEBUG_UTILS_MESSAGE_SEVERITY_WARNING_BIT_EXT) != 0;

	const TCHAR* Severity = TEXT("");
	if (bError)
	{
		ensure((MsgSeverity & ~VK_DEBUG_UTILS_MESSAGE_SEVERITY_ERROR_BIT_EXT) == 0);
		Severity = TEXT("Error");
	}
	else if (bWarning)
	{
		ensure((MsgSeverity & ~VK_DEBUG_UTILS_MESSAGE_SEVERITY_WARNING_BIT_EXT) == 0);
		Severity = TEXT("Warning");
	}
	else if (MsgSeverity & VK_DEBUG_UTILS_MESSAGE_SEVERITY_INFO_BIT_EXT)
	{
		ensure((MsgSeverity & ~VK_DEBUG_UTILS_MESSAGE_SEVERITY_INFO_BIT_EXT) == 0);
		Severity = TEXT("Info");
	}
	else if (MsgSeverity & VK_DEBUG_UTILS_MESSAGE_SEVERITY_VERBOSE_BIT_EXT)
	{
		ensure((MsgSeverity & ~VK_DEBUG_UTILS_MESSAGE_SEVERITY_VERBOSE_BIT_EXT) == 0);
		Severity = TEXT("Verbose");
	}

	enum class EMsgBucket
	{
		General,
		GeneralValidation,
		PerfValidation,
		Validation,
		Perf,
		Count,
	};
	EMsgBucket MsgBucket = EMsgBucket::Count;
	const TCHAR* Type = TEXT("");
	if (MsgType & VK_DEBUG_UTILS_MESSAGE_TYPE_GENERAL_BIT_EXT)
	{
		if (MsgType & VK_DEBUG_UTILS_MESSAGE_TYPE_VALIDATION_BIT_EXT)
		{
			ensure((MsgType & ~(VK_DEBUG_UTILS_MESSAGE_TYPE_GENERAL_BIT_EXT | VK_DEBUG_UTILS_MESSAGE_TYPE_VALIDATION_BIT_EXT)) == 0);
			Type = TEXT(" General/Validation");
			MsgBucket = EMsgBucket::GeneralValidation;
		}
		else
		{
			ensure((MsgType & ~VK_DEBUG_UTILS_MESSAGE_TYPE_GENERAL_BIT_EXT) == 0);
			Type = TEXT(" General");
			MsgBucket = EMsgBucket::General;
		}
	}
	else if (MsgType & VK_DEBUG_UTILS_MESSAGE_TYPE_VALIDATION_BIT_EXT)
	{
		if (MsgType & VK_DEBUG_UTILS_MESSAGE_TYPE_PERFORMANCE_BIT_EXT)
		{
			ensure((MsgType & ~(VK_DEBUG_UTILS_MESSAGE_TYPE_VALIDATION_BIT_EXT | VK_DEBUG_UTILS_MESSAGE_TYPE_PERFORMANCE_BIT_EXT)) == 0);
			Type = TEXT("Perf/Validation");
			MsgBucket = EMsgBucket::PerfValidation;
		}
		else
		{
			ensure((MsgType & ~VK_DEBUG_UTILS_MESSAGE_TYPE_VALIDATION_BIT_EXT) == 0);
			Type = TEXT("Validation");
			MsgBucket = EMsgBucket::Validation;
		}
	}
	else if (MsgType & VK_DEBUG_UTILS_MESSAGE_TYPE_PERFORMANCE_BIT_EXT)
	{
		ensure((MsgType & ~VK_DEBUG_UTILS_MESSAGE_TYPE_PERFORMANCE_BIT_EXT) == 0);
		Type = TEXT("Perf");
		MsgBucket = EMsgBucket::Perf;
	}

	if (!CallbackData->pMessageIdName)
	{
		VULKAN_REPORT_LOG(TEXT("*** [%s:%s(NULL)%d] %s"), Severity, Type, CallbackData->messageIdNumber, ANSI_TO_TCHAR(CallbackData->pMessage));
		ensure(0);
		return VK_FALSE;
	}

	if (!FCStringAnsi::Strcmp(CallbackData->pMessageIdName, "UNASSIGNED-CoreValidation-Shader-OutputNotConsumed") ||
		!FCStringAnsi::Strcmp(CallbackData->pMessageIdName, "Undefined-Value-ShaderOutputNotConsumed") ||
		!FCStringAnsi::Strcmp(CallbackData->pMessageIdName, "Undefined-Value-ShaderOutputNotConsumed-DynamicRendering"))
	{
		// Warning: *** [Warning:Validation-1(UNASSIGNED-CoreValidation-Shader-OutputNotConsumed)] fragment shader writes to output location 0 with no matching attachment
		return VK_FALSE;
	}
	else if (!FCStringAnsi::Strcmp(CallbackData->pMessageIdName, "VUID-VkSwapchainCreateInfoKHR-imageExtent-01274")||
		     !FCStringAnsi::Strcmp(CallbackData->pMessageIdName, "VUID-VkSwapchainCreateInfoKHR-pNext-07781"))
	{
		// Warning: *** [Error:Validation341838324(VUID-VkSwapchainCreateInfoKHR-imageExtent-01274)] vkCreateSwapChainKHR() called with imageExtent = (8,8), which is outside the bounds returned by vkGetPhysicalDeviceSurfaceCapabilitiesKHR(): currentExtent = (0,0), minImageExtent = (0,0), maxImageExtent = (0,0).
		return VK_FALSE;
	}
	else if (!FCStringAnsi::Strcmp(CallbackData->pMessageIdName, "UNASSIGNED-GPU-Assisted Validation Setup Error."))
	{
		// *** [Error:Validation(UNASSIGNED-GPU-Assisted Validation Setup Error.)] Unable to reserve descriptor binding slot on a device with only one slot.
		return VK_FALSE;
	}
	else if (!FCStringAnsi::Strcmp(CallbackData->pMessageIdName, "VUID-RuntimeSpirv-Fragment-06427"))
	{
		// Warning: *** [Error:Validation(VUID-RuntimeSpirv-Fragment-06427)]
		// False-positive in validation layers 1.3.250.1, known to be fixed in 1.3.268.0
		return VK_FALSE;
	}
	else if (!FCStringAnsi::Strcmp(CallbackData->pMessageIdName, "UNASSIGNED-BestPractices-vkCreateDevice-deprecated-extension"))
	{
		// *** CreateDevice(): Attempting to enable deprecated extension VK_KHR_get_memory_requirements2, but this extension has been promoted to VK_VERSION_1_1.
		return VK_FALSE;
	}
	else if (FCStringAnsi::Strstr(CallbackData->pMessage, "SPV_GOOGLE_hlsl_functionality1") != nullptr)
	{
		// *** [Error:Validation(VUID-VkShaderModuleCreateInfo-pCode-04147)] vkCreateShaderModule(): The SPIR-V Extension (SPV_GOOGLE_hlsl_functionality1) was declared, but none of the requirements were met to use it.
		return VK_FALSE;
	 }
	else if (FCStringAnsi::Strstr(CallbackData->pMessage, "SPV_GOOGLE_user_type") != nullptr)
	{
		// *** [Error:Validation(VUID-VkShaderModuleCreateInfo-pCode-04147)] vkCreateShaderModule(): The SPIR-V Extension (SPV_GOOGLE_user_type) was declared, but none of the requirements were met to use it.
		return VK_FALSE;
	}
	else if (FCStringAnsi::Strstr(CallbackData->pMessage, "Cannot store to OpTypeImage, OpTypeSampler, OpTypeSampledImage, or OpTypeAccelerationStructure") != nullptr)
	{
		// *** [VUID-StandaloneSpirv-OpTypeImage-06924] Cannot store to OpTypeImage, OpTypeSampler, OpTypeSampledImage, or OpTypeAccelerationStructureKHR objects
		// OpStore is called on OpTypeAccelerationStructure when they are used in bindless.  Currently no side effects from it, skip until compiler fix is integrated.
		return VK_FALSE;
	}

	static TStaticArray<TSet<uint32>, (int32)EMsgBucket::Count> SeenCodes;
	bool bUniqueMessages = GCVarUniqueValidationMessages->GetInt() != 0;
	bool bPrintMessage = true;
	uint32 CRC = 0;
	if (bUniqueMessages)
	{
		CRC = FCrc::StrCrc32(CallbackData->pMessageIdName);
		bPrintMessage = !SeenCodes[(int32)MsgBucket].Contains(CRC);
	}

	if (bPrintMessage)
	{
		VULKAN_REPORT_LOG(TEXT("*** [%s:%s(%s)] %s"), Severity, Type, ANSI_TO_TCHAR(CallbackData->pMessageIdName), ANSI_TO_TCHAR(CallbackData->pMessage));
		if (bUniqueMessages)
		{
			SeenCodes[(int32)MsgBucket].Add(CRC);
		}
	}

	return VK_FALSE;
}

void FVulkanDynamicRHI::SetupDebugCallbacks()
{
	if (ActiveDebugLayerExtension == EActiveDebugLayerExtension::DebugUtilsExtension)
	{
		PFN_vkCreateDebugUtilsMessengerEXT CreateDebugUtilsMessengerEXT = (PFN_vkCreateDebugUtilsMessengerEXT)(void*)VulkanRHI::vkGetInstanceProcAddr(Instance, "vkCreateDebugUtilsMessengerEXT");
		if (CreateDebugUtilsMessengerEXT)
		{
			VkDebugUtilsMessengerCreateInfoEXT CreateInfo;
			ZeroVulkanStruct(CreateInfo, VK_STRUCTURE_TYPE_DEBUG_UTILS_MESSENGER_CREATE_INFO_EXT);

			const int32 CVar = GValidationCvar.GetValueOnRenderThread();
			CreateInfo.messageSeverity = (CVar >= 1 ? VK_DEBUG_UTILS_MESSAGE_SEVERITY_ERROR_BIT_EXT : 0) |
				(CVar >= 2 ? VK_DEBUG_UTILS_MESSAGE_SEVERITY_WARNING_BIT_EXT : 0) | (CVar >= 3 ? VK_DEBUG_UTILS_MESSAGE_SEVERITY_INFO_BIT_EXT : 0);
			CreateInfo.messageType = (CVar >= 1 ? (VK_DEBUG_UTILS_MESSAGE_TYPE_GENERAL_BIT_EXT | VK_DEBUG_UTILS_MESSAGE_TYPE_VALIDATION_BIT_EXT) : 0) |
				(CVar >= 3 ? VK_DEBUG_UTILS_MESSAGE_TYPE_PERFORMANCE_BIT_EXT : 0);
			CreateInfo.pfnUserCallback = DebugUtilsCallback;
			VkResult Result = (*CreateDebugUtilsMessengerEXT)(Instance, &CreateInfo, nullptr, &Messenger);
			ensure(Result == VK_SUCCESS);
		}
	}
}

void FVulkanDynamicRHI::RemoveDebugCallbacks()
{
	if (Messenger != VK_NULL_HANDLE)
	{
		PFN_vkDestroyDebugUtilsMessengerEXT DestroyDebugUtilsMessengerEXT = (PFN_vkDestroyDebugUtilsMessengerEXT)(void*)VulkanRHI::vkGetInstanceProcAddr(Instance, "vkDestroyDebugUtilsMessengerEXT");
		if (DestroyDebugUtilsMessengerEXT)
		{
			(*DestroyDebugUtilsMessengerEXT)(Instance, Messenger, nullptr);
		}
	}
}


#if VULKAN_ENABLE_TRACKING_LAYER || VULKAN_ENABLE_DUMP_LAYER
template <typename TResourceCreateInfoType>
struct TTrackingResource
{
	FString DebugName = "null";
	TResourceCreateInfoType CreateInfo;

#if VULKAN_ENABLE_TRACKING_CALLSTACK
	FString CreateCallstack;
#endif
};
static FCriticalSection GTrackingCS;

#if VULKAN_ENABLE_TRACKING_CALLSTACK
#include "HAL/PlatformStackWalk.h"

static FCriticalSection GStackTraceMutex;
static char GStackTrace[65536];
static void CaptureCallStack(FString& OutCallstack, uint32 Delta)
{
	FScopeLock ScopeLock(&GStackTraceMutex);
	GStackTrace[0] = 0;
	FPlatformStackWalk::StackWalkAndDump(GStackTrace, 65535, 3 + Delta);
	OutCallstack = ANSI_TO_TCHAR(GStackTrace);
}
#endif
#endif

#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER || VULKAN_ENABLE_DUMP_LAYER
struct FTrackingImage
{
	struct
	{
		FString DebugName = "null";
		VkImageCreateInfo CreateInfo;
		bool bFromSwapchain = false;
	} Info;

	// ArrayLayers[Mips]
	TArray<TArray<VkImageLayout>> ArrayLayouts;

#if VULKAN_ENABLE_TRACKING_CALLSTACK
	struct FHistoryEntry
	{
		TArray<TArray<VkImageLayout>> ArrayLayouts;
		FString Callstack;
	};
	TArray<FHistoryEntry> History;
	FString CreateCallstack;
#endif

	void Setup(uint32 NumArrayLayers, uint32 NumMips, VkImageLayout Layout, bool bFromSwapchain)
	{
		if (bFromSwapchain)
		{
			Info.CreateInfo.arrayLayers = 1;
			Info.CreateInfo.imageType = VK_IMAGE_TYPE_2D;
			Info.CreateInfo.mipLevels = 1;
			Info.bFromSwapchain = bFromSwapchain;
		}
		ensure(NumArrayLayers > 0);
		ensure(NumMips > 0);
		ArrayLayouts.Empty(0);
		ArrayLayouts.AddDefaulted(NumArrayLayers);
		for (uint32 Index = 0; Index < NumArrayLayers; ++Index)
		{
			TArray<VkImageLayout>& MipLayouts = ArrayLayouts[Index];
			MipLayouts.Empty(0);
			for (uint32 MIndex = 0; MIndex < NumMips; ++MIndex)
			{
				MipLayouts.Add(Layout);
			}
		}
	}

	void Setup(const VkImageCreateInfo& CreateInfo, bool bFromSwapchain)
	{
		Info.CreateInfo = CreateInfo;
		Info.bFromSwapchain = bFromSwapchain;
		Setup(CreateInfo.arrayLayers, CreateInfo.mipLevels, CreateInfo.initialLayout, bFromSwapchain);
	}
};
static TMap<VkImage, FTrackingImage> GVulkanTrackingImageLayouts;
static TMap<VkImageView, TTrackingResource<VkImageViewCreateInfo>> GVulkanTrackingImageViews;
static VkImage GBreakOnTrackImage = VK_NULL_HANDLE;

static VkImage GDumpTrackImage = VK_NULL_HANDLE;

static FORCEINLINE void BreakOnTrackingImage(VkImage InImage)
{
	if (GBreakOnTrackImage != VK_NULL_HANDLE)
	{
		ensureAlways(InImage != GBreakOnTrackImage);
	}
}
static FORCEINLINE bool DumpTrackImage(VkImage InImage)
{
	return (GDumpTrackImage != VK_NULL_HANDLE) && (InImage == GDumpTrackImage);
}


static VkImage FindTrackingImage(VkImageView InView)
{
	const auto& Found = GVulkanTrackingImageViews.FindChecked(InView);
	return Found.CreateInfo.image;
}

static FORCEINLINE void BreakOnTrackingImageView(VkImageView InView)
{
	BreakOnTrackingImage(FindTrackingImage(InView));
}
#endif

#if VULKAN_ENABLE_BUFFER_TRACKING_LAYER || VULKAN_ENABLE_DUMP_LAYER || VULKAN_ENABLE_IMAGE_TRACKING_LAYER 
static TMap<VkBuffer, TTrackingResource<VkBufferCreateInfo>> GVulkanTrackingBuffers;
static TMap<VkBuffer, TArray<VkBufferView>> GVulkanTrackingBufferToBufferViews;
static TMap<VkBufferView, TTrackingResource<VkBufferViewCreateInfo>> GVulkanTrackingBufferViews;

static VkBuffer FindTrackingBuffer(VkBufferView InView)
{
	const auto& Found = GVulkanTrackingBufferViews.FindChecked(InView);
	return Found.CreateInfo.buffer;
}
#endif

static void ValidationFail()
{
	ensure(0);
}


#if VULKAN_ENABLE_DUMP_LAYER || VULKAN_ENABLE_IMAGE_TRACKING_LAYER
#include "Misc/OutputDeviceRedirector.h"
namespace VulkanRHI
{
	static FCriticalSection CS;
	struct FMutexString
	{
		FString Inner;

		FMutexString& operator += (const TCHAR* S)
		{
			FScopeLock Lock(&CS);
			Inner += S;
			return *this;
		}

		FMutexString& operator += (const char* S)
		{
			FScopeLock Lock(&CS);
			Inner += ANSI_TO_TCHAR(S);
			return *this;
		}

		FMutexString& operator += (const char S)
		{
			FScopeLock Lock(&CS);
			char T[2] = "\0";
			T[0] = S;
			Inner += ANSI_TO_TCHAR(T);
			return *this;
		}

		FMutexString& operator = (TCHAR T)
		{
			FScopeLock Lock(&CS);
			Inner = TEXT("");
			Inner += T;
			return *this;
		}

		FMutexString& operator = (const TCHAR* S)
		{
			FScopeLock Lock(&CS);
			Inner = S;
			return *this;
		}

		FMutexString& operator += (const FString& S)
		{
			FScopeLock Lock(&CS);
			Inner += S;
			return *this;
		}

		int32 Len() const
		{
			FScopeLock Lock(&CS);
			return Inner.Len();
		}
	};

	static FMutexString DebugLog;
	static int32 DebugLine = 1;

	static const TCHAR* Tabs = TEXT("\t\t\t\t\t\t\t\t\t");

	struct FRenderPassInfo
	{
		TArray<VkAttachmentDescription> Descriptions;
		VkRenderPassCreateInfo Info;
	};
	static TMap<VkRenderPass, FRenderPassInfo> GRenderPassInfo;
	struct FFBInfo
	{
		TArray<VkImageView> Attachments;
		VkFramebufferCreateInfo Info;
	};
	static TMap<VkFramebuffer, FFBInfo> GFramebufferInfo;

	void FlushDebugWrapperLog()
	{
		if (DebugLog.Len() > 0)
		{
			VULKAN_REPORT_LOG(TEXT("VULKANRHI: %s"), *DebugLog.Inner);
			//GLog->Flush();
			//UE_LOGF(LogVulkanRHI, Display, "Vulkan Wrapper Log:\n%ls", *DebugLog);
			//GLog->Flush();
			DebugLog = TEXT("");
		}
	}

	static void HandleFlushWrapperLog(const TArray<FString>& Args)
	{
		FlushDebugWrapperLog();
	}

	static FAutoConsoleCommand CVarVulkanFlushLog(
		TEXT("r.Vulkan.FlushLog"),
		TEXT("\n"),
		FConsoleCommandWithArgsDelegate::CreateStatic(&HandleFlushWrapperLog)
		);

	static FString GetPipelineBindPointString(VkPipelineBindPoint BindPoint)
	{
		switch (BindPoint)
		{
		case VK_PIPELINE_BIND_POINT_GRAPHICS:
			return TEXT("GFX");
		case VK_PIPELINE_BIND_POINT_COMPUTE:
			return TEXT("COMPUTE");
		default:
			break;
		}
		return FString::Printf(TEXT("Unknown VkPipelineBindPoint %d"), (int32)BindPoint);
	}

	static FString GetVkFormatString(VkFormat Format)
	{
		switch (Format)
		{
			// + 10 to skip "VK_FORMAT"
#define VKSWITCHCASE(x)	case x: return &TEXT(#x)[10];
			VKSWITCHCASE(VK_FORMAT_UNDEFINED)
			VKSWITCHCASE(VK_FORMAT_R4G4_UNORM_PACK8)
			VKSWITCHCASE(VK_FORMAT_R4G4B4A4_UNORM_PACK16)
			VKSWITCHCASE(VK_FORMAT_B4G4R4A4_UNORM_PACK16)
			VKSWITCHCASE(VK_FORMAT_R5G6B5_UNORM_PACK16)
			VKSWITCHCASE(VK_FORMAT_B5G6R5_UNORM_PACK16)
			VKSWITCHCASE(VK_FORMAT_R5G5B5A1_UNORM_PACK16)
			VKSWITCHCASE(VK_FORMAT_B5G5R5A1_UNORM_PACK16)
			VKSWITCHCASE(VK_FORMAT_A1R5G5B5_UNORM_PACK16)
			VKSWITCHCASE(VK_FORMAT_R8_UNORM)
			VKSWITCHCASE(VK_FORMAT_R8_SNORM)
			VKSWITCHCASE(VK_FORMAT_R8_USCALED)
			VKSWITCHCASE(VK_FORMAT_R8_SSCALED)
			VKSWITCHCASE(VK_FORMAT_R8_UINT)
			VKSWITCHCASE(VK_FORMAT_R8_SINT)
			VKSWITCHCASE(VK_FORMAT_R8_SRGB)
			VKSWITCHCASE(VK_FORMAT_R8G8_UNORM)
			VKSWITCHCASE(VK_FORMAT_R8G8_SNORM)
			VKSWITCHCASE(VK_FORMAT_R8G8_USCALED)
			VKSWITCHCASE(VK_FORMAT_R8G8_SSCALED)
			VKSWITCHCASE(VK_FORMAT_R8G8_UINT)
			VKSWITCHCASE(VK_FORMAT_R8G8_SINT)
			VKSWITCHCASE(VK_FORMAT_R8G8_SRGB)
			VKSWITCHCASE(VK_FORMAT_R8G8B8_UNORM)
			VKSWITCHCASE(VK_FORMAT_R8G8B8_SNORM)
			VKSWITCHCASE(VK_FORMAT_R8G8B8_USCALED)
			VKSWITCHCASE(VK_FORMAT_R8G8B8_SSCALED)
			VKSWITCHCASE(VK_FORMAT_R8G8B8_UINT)
			VKSWITCHCASE(VK_FORMAT_R8G8B8_SINT)
			VKSWITCHCASE(VK_FORMAT_R8G8B8_SRGB)
			VKSWITCHCASE(VK_FORMAT_B8G8R8_UNORM)
			VKSWITCHCASE(VK_FORMAT_B8G8R8_SNORM)
			VKSWITCHCASE(VK_FORMAT_B8G8R8_USCALED)
			VKSWITCHCASE(VK_FORMAT_B8G8R8_SSCALED)
			VKSWITCHCASE(VK_FORMAT_B8G8R8_UINT)
			VKSWITCHCASE(VK_FORMAT_B8G8R8_SINT)
			VKSWITCHCASE(VK_FORMAT_B8G8R8_SRGB)
			VKSWITCHCASE(VK_FORMAT_R8G8B8A8_UNORM)
			VKSWITCHCASE(VK_FORMAT_R8G8B8A8_SNORM)
			VKSWITCHCASE(VK_FORMAT_R8G8B8A8_USCALED)
			VKSWITCHCASE(VK_FORMAT_R8G8B8A8_SSCALED)
			VKSWITCHCASE(VK_FORMAT_R8G8B8A8_UINT)
			VKSWITCHCASE(VK_FORMAT_R8G8B8A8_SINT)
			VKSWITCHCASE(VK_FORMAT_R8G8B8A8_SRGB)
			VKSWITCHCASE(VK_FORMAT_B8G8R8A8_UNORM)
			VKSWITCHCASE(VK_FORMAT_B8G8R8A8_SNORM)
			VKSWITCHCASE(VK_FORMAT_B8G8R8A8_USCALED)
			VKSWITCHCASE(VK_FORMAT_B8G8R8A8_SSCALED)
			VKSWITCHCASE(VK_FORMAT_B8G8R8A8_UINT)
			VKSWITCHCASE(VK_FORMAT_B8G8R8A8_SINT)
			VKSWITCHCASE(VK_FORMAT_B8G8R8A8_SRGB)
			VKSWITCHCASE(VK_FORMAT_A8B8G8R8_UNORM_PACK32)
			VKSWITCHCASE(VK_FORMAT_A8B8G8R8_SNORM_PACK32)
			VKSWITCHCASE(VK_FORMAT_A8B8G8R8_USCALED_PACK32)
			VKSWITCHCASE(VK_FORMAT_A8B8G8R8_SSCALED_PACK32)
			VKSWITCHCASE(VK_FORMAT_A8B8G8R8_UINT_PACK32)
			VKSWITCHCASE(VK_FORMAT_A8B8G8R8_SINT_PACK32)
			VKSWITCHCASE(VK_FORMAT_A8B8G8R8_SRGB_PACK32)
			VKSWITCHCASE(VK_FORMAT_A2R10G10B10_UNORM_PACK32)
			VKSWITCHCASE(VK_FORMAT_A2R10G10B10_SNORM_PACK32)
			VKSWITCHCASE(VK_FORMAT_A2R10G10B10_USCALED_PACK32)
			VKSWITCHCASE(VK_FORMAT_A2R10G10B10_SSCALED_PACK32)
			VKSWITCHCASE(VK_FORMAT_A2R10G10B10_UINT_PACK32)
			VKSWITCHCASE(VK_FORMAT_A2R10G10B10_SINT_PACK32)
			VKSWITCHCASE(VK_FORMAT_A2B10G10R10_UNORM_PACK32)
			VKSWITCHCASE(VK_FORMAT_A2B10G10R10_SNORM_PACK32)
			VKSWITCHCASE(VK_FORMAT_A2B10G10R10_USCALED_PACK32)
			VKSWITCHCASE(VK_FORMAT_A2B10G10R10_SSCALED_PACK32)
			VKSWITCHCASE(VK_FORMAT_A2B10G10R10_UINT_PACK32)
			VKSWITCHCASE(VK_FORMAT_A2B10G10R10_SINT_PACK32)
			VKSWITCHCASE(VK_FORMAT_R16_UNORM)
			VKSWITCHCASE(VK_FORMAT_R16_SNORM)
			VKSWITCHCASE(VK_FORMAT_R16_USCALED)
			VKSWITCHCASE(VK_FORMAT_R16_SSCALED)
			VKSWITCHCASE(VK_FORMAT_R16_UINT)
			VKSWITCHCASE(VK_FORMAT_R16_SINT)
			VKSWITCHCASE(VK_FORMAT_R16_SFLOAT)
			VKSWITCHCASE(VK_FORMAT_R16G16_UNORM)
			VKSWITCHCASE(VK_FORMAT_R16G16_SNORM)
			VKSWITCHCASE(VK_FORMAT_R16G16_USCALED)
			VKSWITCHCASE(VK_FORMAT_R16G16_SSCALED)
			VKSWITCHCASE(VK_FORMAT_R16G16_UINT)
			VKSWITCHCASE(VK_FORMAT_R16G16_SINT)
			VKSWITCHCASE(VK_FORMAT_R16G16_SFLOAT)
			VKSWITCHCASE(VK_FORMAT_R16G16B16_UNORM)
			VKSWITCHCASE(VK_FORMAT_R16G16B16_SNORM)
			VKSWITCHCASE(VK_FORMAT_R16G16B16_USCALED)
			VKSWITCHCASE(VK_FORMAT_R16G16B16_SSCALED)
			VKSWITCHCASE(VK_FORMAT_R16G16B16_UINT)
			VKSWITCHCASE(VK_FORMAT_R16G16B16_SINT)
			VKSWITCHCASE(VK_FORMAT_R16G16B16_SFLOAT)
			VKSWITCHCASE(VK_FORMAT_R16G16B16A16_UNORM)
			VKSWITCHCASE(VK_FORMAT_R16G16B16A16_SNORM)
			VKSWITCHCASE(VK_FORMAT_R16G16B16A16_USCALED)
			VKSWITCHCASE(VK_FORMAT_R16G16B16A16_SSCALED)
			VKSWITCHCASE(VK_FORMAT_R16G16B16A16_UINT)
			VKSWITCHCASE(VK_FORMAT_R16G16B16A16_SINT)
			VKSWITCHCASE(VK_FORMAT_R16G16B16A16_SFLOAT)
			VKSWITCHCASE(VK_FORMAT_R32_UINT)
			VKSWITCHCASE(VK_FORMAT_R32_SINT)
			VKSWITCHCASE(VK_FORMAT_R32_SFLOAT)
			VKSWITCHCASE(VK_FORMAT_R32G32_UINT)
			VKSWITCHCASE(VK_FORMAT_R32G32_SINT)
			VKSWITCHCASE(VK_FORMAT_R32G32_SFLOAT)
			VKSWITCHCASE(VK_FORMAT_R32G32B32_UINT)
			VKSWITCHCASE(VK_FORMAT_R32G32B32_SINT)
			VKSWITCHCASE(VK_FORMAT_R32G32B32_SFLOAT)
			VKSWITCHCASE(VK_FORMAT_R32G32B32A32_UINT)
			VKSWITCHCASE(VK_FORMAT_R32G32B32A32_SINT)
			VKSWITCHCASE(VK_FORMAT_R32G32B32A32_SFLOAT)
			VKSWITCHCASE(VK_FORMAT_R64_UINT)
			VKSWITCHCASE(VK_FORMAT_R64_SINT)
			VKSWITCHCASE(VK_FORMAT_R64_SFLOAT)
			VKSWITCHCASE(VK_FORMAT_R64G64_UINT)
			VKSWITCHCASE(VK_FORMAT_R64G64_SINT)
			VKSWITCHCASE(VK_FORMAT_R64G64_SFLOAT)
			VKSWITCHCASE(VK_FORMAT_R64G64B64_UINT)
			VKSWITCHCASE(VK_FORMAT_R64G64B64_SINT)
			VKSWITCHCASE(VK_FORMAT_R64G64B64_SFLOAT)
			VKSWITCHCASE(VK_FORMAT_R64G64B64A64_UINT)
			VKSWITCHCASE(VK_FORMAT_R64G64B64A64_SINT)
			VKSWITCHCASE(VK_FORMAT_R64G64B64A64_SFLOAT)
			VKSWITCHCASE(VK_FORMAT_B10G11R11_UFLOAT_PACK32)
			VKSWITCHCASE(VK_FORMAT_E5B9G9R9_UFLOAT_PACK32)
			VKSWITCHCASE(VK_FORMAT_D16_UNORM)
			VKSWITCHCASE(VK_FORMAT_X8_D24_UNORM_PACK32)
			VKSWITCHCASE(VK_FORMAT_D32_SFLOAT)
			VKSWITCHCASE(VK_FORMAT_S8_UINT)
			VKSWITCHCASE(VK_FORMAT_D16_UNORM_S8_UINT)
			VKSWITCHCASE(VK_FORMAT_D24_UNORM_S8_UINT)
			VKSWITCHCASE(VK_FORMAT_D32_SFLOAT_S8_UINT)
			VKSWITCHCASE(VK_FORMAT_BC1_RGB_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC1_RGB_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC1_RGBA_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC1_RGBA_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC2_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC2_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC3_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC3_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC4_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC4_SNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC5_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC5_SNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC6H_UFLOAT_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC6H_SFLOAT_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC7_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_BC7_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ETC2_R8G8B8_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ETC2_R8G8B8_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ETC2_R8G8B8A1_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ETC2_R8G8B8A1_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ETC2_R8G8B8A8_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ETC2_R8G8B8A8_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_EAC_R11_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_EAC_R11_SNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_EAC_R11G11_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_EAC_R11G11_SNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_4x4_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_4x4_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_5x4_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_5x4_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_5x5_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_5x5_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_6x5_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_6x5_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_6x6_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_6x6_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_8x5_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_8x5_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_8x6_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_8x6_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_8x8_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_8x8_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_10x5_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_10x5_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_10x6_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_10x6_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_10x8_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_10x8_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_10x10_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_10x10_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_12x10_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_12x10_SRGB_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_12x12_UNORM_BLOCK)
			VKSWITCHCASE(VK_FORMAT_ASTC_12x12_SRGB_BLOCK)
#undef VKSWITCHCASE
		default:
			break;
		}
		return FString::Printf(TEXT("Unknown VkFormat %d"), (int32)Format);
	}

	static FString GetVkResultErrorString(VkResult Result)
	{
		switch (Result)
		{
			// + 3 to skip "VK_"
#define VKSWITCHCASE(x)	case x: return &TEXT(#x)[3];
			VKSWITCHCASE(VK_SUCCESS)
			VKSWITCHCASE(VK_NOT_READY)
			VKSWITCHCASE(VK_TIMEOUT)
			VKSWITCHCASE(VK_EVENT_SET)
			VKSWITCHCASE(VK_EVENT_RESET)
			VKSWITCHCASE(VK_INCOMPLETE)
			VKSWITCHCASE(VK_ERROR_OUT_OF_HOST_MEMORY)
			VKSWITCHCASE(VK_ERROR_OUT_OF_DEVICE_MEMORY)
			VKSWITCHCASE(VK_ERROR_INITIALIZATION_FAILED)
			VKSWITCHCASE(VK_ERROR_DEVICE_LOST)
			VKSWITCHCASE(VK_ERROR_MEMORY_MAP_FAILED)
			VKSWITCHCASE(VK_ERROR_LAYER_NOT_PRESENT)
			VKSWITCHCASE(VK_ERROR_EXTENSION_NOT_PRESENT)
			VKSWITCHCASE(VK_ERROR_FEATURE_NOT_PRESENT)
			VKSWITCHCASE(VK_ERROR_INCOMPATIBLE_DRIVER)
			VKSWITCHCASE(VK_ERROR_TOO_MANY_OBJECTS)
			VKSWITCHCASE(VK_ERROR_FORMAT_NOT_SUPPORTED)
			VKSWITCHCASE(VK_ERROR_SURFACE_LOST_KHR)
			VKSWITCHCASE(VK_ERROR_NATIVE_WINDOW_IN_USE_KHR)
			VKSWITCHCASE(VK_SUBOPTIMAL_KHR)
			VKSWITCHCASE(VK_ERROR_OUT_OF_DATE_KHR)
			VKSWITCHCASE(VK_ERROR_INCOMPATIBLE_DISPLAY_KHR)
			VKSWITCHCASE(VK_ERROR_VALIDATION_FAILED_EXT)
			VKSWITCHCASE(VK_ERROR_INVALID_SHADER_NV)
			VKSWITCHCASE(VK_ERROR_FRAGMENTED_POOL)
			VKSWITCHCASE(VK_ERROR_OUT_OF_POOL_MEMORY_KHR)
			VKSWITCHCASE(VK_ERROR_INVALID_EXTERNAL_HANDLE_KHR)
			VKSWITCHCASE(VK_ERROR_NOT_PERMITTED_EXT)
#undef VKSWITCHCASE
		default:
			break;
		}

		return FString::Printf(TEXT("Unknown VkResult %d"), (int32)Result);
	}

	static FString GetVkImageTilingString(VkImageTiling Tiling)
	{
		switch (Tiling)
		{
			// + 16 to skip "VK_IMAGE_TILING_"
#define VKSWITCHCASE(x)	case x: return &TEXT(#x)[16];
			VKSWITCHCASE(VK_IMAGE_TILING_OPTIMAL)
			VKSWITCHCASE(VK_IMAGE_TILING_LINEAR)
#undef VKSWITCHCASE
		default:
			break;
		}

		return FString::Printf(TEXT("Unknown VkImageTiling %d"), (int32)Tiling);
	}

	static FString GetVkImageLayoutString(VkImageLayout Layout)
	{
		return FString(VK_TYPE_TO_STRING(VkImageLayout, Layout)).RightChop(16);
	}

	static FString GetVkImageViewTypeString(VkImageViewType Type)
	{
		return FString(VK_TYPE_TO_STRING(VkImageViewType, Type)).RightChop(19);
	}

	static FString GetVkImageTypeString(VkImageType Type)
	{
		return FString(VK_TYPE_TO_STRING(VkImageType, Type)).RightChop(14);
	}

	static FString GetVkDescriptorTypeString(VkDescriptorType Type)
	{
		switch (Type)
		{
			// + 19 to skip "VK_DESCRIPTOR_TYPE_"
#define VKSWITCHCASE(x)	case x: return &TEXT(#x)[19];
		VKSWITCHCASE(VK_DESCRIPTOR_TYPE_SAMPLER)
		VKSWITCHCASE(VK_DESCRIPTOR_TYPE_COMBINED_IMAGE_SAMPLER)
		VKSWITCHCASE(VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE)
		VKSWITCHCASE(VK_DESCRIPTOR_TYPE_STORAGE_IMAGE)
		VKSWITCHCASE(VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER)
		VKSWITCHCASE(VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER)
		VKSWITCHCASE(VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER)
		VKSWITCHCASE(VK_DESCRIPTOR_TYPE_STORAGE_BUFFER)
		VKSWITCHCASE(VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC)
		VKSWITCHCASE(VK_DESCRIPTOR_TYPE_STORAGE_BUFFER_DYNAMIC)
		VKSWITCHCASE(VK_DESCRIPTOR_TYPE_INPUT_ATTACHMENT)
#undef VKSWITCHCASE
		default:
			break;
		}

		return FString::Printf(TEXT("Unknown VkDescriptorType %d"), (int32)Type);
	}

	static FString GetVkStencilOpString(VkStencilOp Op)
	{
		switch (Op)
		{
			// + 14 to skip "VK_STENCIL_OP_"
#define VKSWITCHCASE(x)	case x: return &TEXT(#x)[14];
			VKSWITCHCASE(VK_STENCIL_OP_KEEP)
			VKSWITCHCASE(VK_STENCIL_OP_ZERO)
			VKSWITCHCASE(VK_STENCIL_OP_REPLACE)
			VKSWITCHCASE(VK_STENCIL_OP_INCREMENT_AND_CLAMP)
			VKSWITCHCASE(VK_STENCIL_OP_DECREMENT_AND_CLAMP)
			VKSWITCHCASE(VK_STENCIL_OP_INVERT)
			VKSWITCHCASE(VK_STENCIL_OP_INCREMENT_AND_WRAP)
			VKSWITCHCASE(VK_STENCIL_OP_DECREMENT_AND_WRAP)
#undef VKSWITCHCASE
		default:
			break;
		}

		return FString::Printf(TEXT("Unknown VkStencilOp %d"), (int32)Op);
	}

	static FString GetVkCompareOpString(VkCompareOp Op)
	{
		switch (Op)
		{
			// + 14 to skip "VK_COMPARE_OP_"
#define VKSWITCHCASE(x)	case x: return &TEXT(#x)[14];
			VKSWITCHCASE(VK_COMPARE_OP_NEVER)
			VKSWITCHCASE(VK_COMPARE_OP_LESS)
			VKSWITCHCASE(VK_COMPARE_OP_EQUAL)
			VKSWITCHCASE(VK_COMPARE_OP_LESS_OR_EQUAL)
			VKSWITCHCASE(VK_COMPARE_OP_GREATER)
			VKSWITCHCASE(VK_COMPARE_OP_NOT_EQUAL)
			VKSWITCHCASE(VK_COMPARE_OP_GREATER_OR_EQUAL)
			VKSWITCHCASE(VK_COMPARE_OP_ALWAYS)
#undef VKSWITCHCASE
		default:
			break;
		}

		return FString::Printf(TEXT("Unknown VkStencilOp %d"), (int32)Op);
	}

	static FString GetVkComponentMappingString(const VkComponentMapping& Mapping)
	{
		auto GetSwizzle = [](VkComponentSwizzle Swizzle) -> const TCHAR*
			{
				switch (Swizzle)
				{
				case VK_COMPONENT_SWIZZLE_IDENTITY: return TEXT("ID");
				case VK_COMPONENT_SWIZZLE_ZERO:		return TEXT("0");
				case VK_COMPONENT_SWIZZLE_ONE:		return TEXT("1");
				case VK_COMPONENT_SWIZZLE_R:		return TEXT("R");
				case VK_COMPONENT_SWIZZLE_G:		return TEXT("G");
				case VK_COMPONENT_SWIZZLE_B:		return TEXT("B");
				case VK_COMPONENT_SWIZZLE_A:		return TEXT("A");
				default:
					check(0);
					return TEXT("-");
				}
			};
		return FString::Printf(TEXT("(r=%s, g=%s, b=%s, a=%s)"), GetSwizzle(Mapping.r), GetSwizzle(Mapping.g), GetSwizzle(Mapping.b), GetSwizzle(Mapping.a));
	}


#define AppendBitFieldName(BitField, Name) \
	if ((Flags & BitField) == BitField)\
	{\
		Flags &= ~BitField;\
		if (String.Len() > 0)\
		{\
			String += TEXT("|");\
		}\
		String += Name;\
	}

	static FString GetAspectMaskString(VkImageAspectFlags Flags)
	{
		if (Flags == 0)
		{
			return TEXT("0");
		}
		FString String;
		AppendBitFieldName(VK_IMAGE_ASPECT_COLOR_BIT, TEXT("COLOR"));
		AppendBitFieldName(VK_IMAGE_ASPECT_DEPTH_BIT, TEXT("DEPTH"));
		AppendBitFieldName(VK_IMAGE_ASPECT_STENCIL_BIT, TEXT("STENCIL"));
		AppendBitFieldName(VK_IMAGE_ASPECT_METADATA_BIT, TEXT("METADATA"));
		if (Flags != 0)
		{
			FString Unknown = FString::Printf(TEXT("%d"), Flags);
			AppendBitFieldName(Flags, Unknown);
		}
		return String;
	}

	FString GetSampleCountString(VkSampleCountFlags Flags)
	{
		if (Flags == 0)
		{
			return TEXT("0");
		}
		FString String;
		AppendBitFieldName(VK_SAMPLE_COUNT_1_BIT, TEXT("1"));
		AppendBitFieldName(VK_SAMPLE_COUNT_2_BIT, TEXT("2"));
		AppendBitFieldName(VK_SAMPLE_COUNT_4_BIT, TEXT("4"));
		AppendBitFieldName(VK_SAMPLE_COUNT_8_BIT, TEXT("8"));
		AppendBitFieldName(VK_SAMPLE_COUNT_16_BIT, TEXT("16"));
		AppendBitFieldName(VK_SAMPLE_COUNT_32_BIT, TEXT("32"));
		AppendBitFieldName(VK_SAMPLE_COUNT_64_BIT, TEXT("64"));
		if (Flags != 0)
		{
			FString Unknown = FString::Printf(TEXT("%d"), Flags);
			AppendBitFieldName(Flags, Unknown);
		}
		return String;
	}

	FString GetImageUsageString(VkImageUsageFlags Flags)
	{
		if (Flags == 0)
		{
			return TEXT("0");
		}
		FString String;
		AppendBitFieldName(VK_IMAGE_USAGE_TRANSFER_SRC_BIT, TEXT("XFER_SRC"));
		AppendBitFieldName(VK_IMAGE_USAGE_TRANSFER_DST_BIT, TEXT("XFER_DST"));
		AppendBitFieldName(VK_IMAGE_USAGE_SAMPLED_BIT, TEXT("SAMPLED"));
		AppendBitFieldName(VK_IMAGE_USAGE_STORAGE_BIT, TEXT("STORAGE"));
		AppendBitFieldName(VK_IMAGE_USAGE_COLOR_ATTACHMENT_BIT, TEXT("COLOR_ATT"));
		AppendBitFieldName(VK_IMAGE_USAGE_DEPTH_STENCIL_ATTACHMENT_BIT, TEXT("DS_ATT"));
		AppendBitFieldName(VK_IMAGE_USAGE_TRANSIENT_ATTACHMENT_BIT, TEXT("TRANS_ATT"));
		AppendBitFieldName(VK_IMAGE_USAGE_INPUT_ATTACHMENT_BIT, TEXT("IN_ATT"));
		if (Flags != 0)
		{
			FString Unknown = FString::Printf(TEXT("%d"), Flags);
			AppendBitFieldName(Flags, Unknown);
		}
		return String;
	}
#undef  AppendBitFieldName

	static FString GetImageSubresourceLayers(const VkImageSubresourceLayers& Layer)
	{
		return FString::Printf(TEXT("Aspect=%s MipLvl=%d BaseArray=%d NumLayers=%d"), *GetAspectMaskString(Layer.aspectMask), Layer.mipLevel, Layer.baseArrayLayer, Layer.layerCount);
	}

	FString GetExtentString(const VkExtent3D& Extent)
	{
		return FString::Printf(TEXT("w:%d h:%d d:%d"), Extent.width, Extent.height, Extent.depth);
	}

	static FString GetExtentString(const VkExtent2D& Extent)
	{
		return FString::Printf(TEXT("w:%d h:%d"), Extent.width, Extent.height);
	}

	static FString GetOffsetString(const VkOffset3D& Offset)
	{
		return FString::Printf(TEXT("x:%d y:%d z:%d"), Offset.x, Offset.y, Offset.z);
	}

	static FString GetOffsetString(const VkOffset2D& Offset)
	{
		return FString::Printf(TEXT("x:%d y:%d"), Offset.x, Offset.y);
	}

	static FString GetRectString(const VkRect2D& Rect)
	{
		return FString::Printf(TEXT("%s %s"), *GetOffsetString(Rect.offset), *GetExtentString(Rect.extent));
	}

	FString GetBufferImageCopyString(const VkBufferImageCopy& Region)
	{
		return FString::Printf(TEXT("BufOff=%d BufRow=%d BufImgHeight=%d ImgSubR=[%s] ImgOff=[%s] ImgExt=[%s]"),
			Region.bufferOffset, Region.bufferRowLength, Region.bufferImageHeight, *GetImageSubresourceLayers(Region.imageSubresource),
			*GetOffsetString(Region.imageOffset), *GetExtentString(Region.imageExtent));
	}

	static FString GetImageSubResourceRangeString(const VkImageSubresourceRange& Range)
	{
		return FString::Printf(TEXT("AspectMask=%s, BaseMip=%d, NumLevels=%d, BaseArrayLayer=%d, NumLayers=%d"), *GetAspectMaskString(Range.aspectMask), Range.baseMipLevel, Range.levelCount, Range.baseArrayLayer, Range.layerCount);
	}

	static FString GetClearColorValueString(const VkClearColorValue& Value)
	{
		return FString::Printf(TEXT("0x%x(%f), 0x%x(%f), 0x%x(%f), 0x%x(%f)"),
			Value.uint32[0], Value.float32[0],
			Value.uint32[1], Value.float32[1],
			Value.uint32[2], Value.float32[2],
			Value.uint32[3], Value.float32[3]);
	}

	static FString GetClearDepthStencilValueString(const VkClearDepthStencilValue& Value)
	{
		return FString::Printf(TEXT("d:%f s:%d"), Value.depth, Value.stencil);
	}

	static FString GetClearValueString(const VkClearValue& Value)
	{
		return FString::Printf(TEXT("(%s/%s)"), *GetClearColorValueString(Value.color), *GetClearDepthStencilValueString(Value.depthStencil));
	}

	void PrintfBeginResult(const FString& String)
	{
		DebugLog += FString::Printf(TEXT("[GLOBAL METHOD]     %8d: %s"), DebugLine++, *String);
		FlushDebugWrapperLog();
	}

	void PrintfBegin(const FString& String)
	{
		DebugLog += FString::Printf(TEXT("[GLOBAL METHOD]     %8d: %s\n"), DebugLine++, *String);
		FlushDebugWrapperLog();
	}

	void DevicePrintfBeginResult(VkDevice Device, const FString& String)
	{
		DebugLog += FString::Printf(TEXT("[D:0x%p]%8d: %s"), Device, DebugLine++, *String);
		FlushDebugWrapperLog();
	}

	void DevicePrintfBegin(VkDevice Device, const FString& String)
	{
		DebugLog += FString::Printf(TEXT("[D:0x%p]%8d: %s\n"), Device, DebugLine++, *String);
		FlushDebugWrapperLog();
	}

	void CmdPrintfBegin(VkCommandBuffer CmdBuffer, const FString& String)
	{
		DebugLog += FString::Printf(TEXT("[C:0x%p]%8d: %s\n"), CmdBuffer, DebugLine++, *String);
		FlushDebugWrapperLog();
	}

	void CmdPrintfBeginResult(VkCommandBuffer CmdBuffer, const FString& String)
	{
		DebugLog += FString::Printf(TEXT("[C:0x%p]%8d: %s"), CmdBuffer, DebugLine++, *String);
		FlushDebugWrapperLog();
	}

	void PrintResult(VkResult Result)
	{
		DebugLog += FString::Printf(TEXT(" -> %s\n"), *GetVkResultErrorString(Result));
		FlushDebugWrapperLog();
	}

	void PrintResultAndPointer(VkResult Result, void* Handle)
	{
		DebugLog += FString::Printf(TEXT(" -> %s => 0x%p\n"), *GetVkResultErrorString(Result), Handle);
		FlushDebugWrapperLog();
	}

	void PrintResultAndNamedHandle(VkResult Result, const TCHAR* HandleName, void* Handle)
	{
		DebugLog += FString::Printf(TEXT(" -> %s => %s=0x%p\n"), *GetVkResultErrorString(Result), HandleName, Handle);
		FlushDebugWrapperLog();
	}

	void PrintResultAndNamedHandles(VkResult Result, const TCHAR* HandleName, uint32 NumHandles, uint64* Handles)
	{
		DebugLog += FString::Printf(TEXT(" -> %s => %s\n"), *GetVkResultErrorString(Result), HandleName);
		for (uint32 Index = 0; Index < NumHandles; ++Index)
		{
			DebugLog += FString::Printf(TEXT(" [%d]=0x%p"), Index, (void*)Handles[Index]);
		}
		DebugLog += TEXT("\n");
		FlushDebugWrapperLog();
	}

	void PrintResultAndPointer(VkResult Result, uint64 Handle)
	{
		DebugLog += FString::Printf(TEXT(" -> %s => %llu\n"), *GetVkResultErrorString(Result), Handle);
		FlushDebugWrapperLog();
	}

	void PrintResultAndNamedHandle(VkResult Result, const TCHAR* HandleName, uint64 Handle)
	{
		DebugLog += FString::Printf(TEXT(" -> %s => %s=%llu\n"), *GetVkResultErrorString(Result), HandleName, Handle);
		FlushDebugWrapperLog();
	}
}
#endif

#if VULKAN_ENABLE_WRAP_LAYER
void FWrapLayer::GetPhysicalDeviceMemoryProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceMemoryProperties* Properties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
			PrintfBegin(FString::Printf(TEXT("vkGetPhysicalDeviceMemoryProperties(OutProp=0x%p)[...]"), Properties));
			FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::QueueWaitIdle(VkResult Result, VkQueue Queue)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkQueueWaitIdle(Queue=0x%p)"), Queue));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::AllocateMemory(VkResult Result, VkDevice Device, const VkMemoryAllocateInfo* AllocateInfo, VkDeviceMemory* Memory)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkAllocateMemory(AllocateInfo=0x%p, OutMem=0x%p): Size=%d, MemTypeIndex=%d"), AllocateInfo, Memory, (uint32)AllocateInfo->allocationSize, AllocateInfo->memoryTypeIndex));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("DevMem"), *Memory);
		FlushDebugWrapperLog();
#endif
	}
}

#if VULKAN_ENABLE_DUMP_LAYER
static void DumpErrorInvalidPointer(const FString& PointerTypename)
{
	DebugLog += FString::Printf(TEXT(" -> ERROR: invalid pointer to %s\n"), *PointerTypename);
}
#endif

#if VULKAN_ENABLE_DUMP_LAYER
static void DumpMemoryRequirements(VkMemoryRequirements* pMemRequirements)
{
	if (pMemRequirements)
	{
		DebugLog += FString::Printf(TEXT(" -> Size=%llu Align=%llu MemTypeBits=0x%x\n"), pMemRequirements->size, pMemRequirements->alignment, pMemRequirements->memoryTypeBits);
	}
	else
	{
		DumpErrorInvalidPointer(TEXT("VkMemoryRequirements"));
	}
	FlushDebugWrapperLog();
}
#endif

void FWrapLayer::GetBufferMemoryRequirements(VkResult Result, VkDevice Device, VkBuffer Buffer, VkMemoryRequirements* MemoryRequirements)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkGetBufferMemoryRequirements(Buffer=0x%p, OutReq=0x%p)"), Buffer, MemoryRequirements));
	}
	else
	{
		DumpMemoryRequirements(MemoryRequirements);
	}
#endif
}

void FWrapLayer::GetImageMemoryRequirements(VkResult Result, VkDevice Device, VkImage Image, VkMemoryRequirements* MemoryRequirements)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkGetImageMemoryRequirements(Image=0x%p, OutReq=0x%p)"), Image, MemoryRequirements));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DumpMemoryRequirements(MemoryRequirements);
#endif
	}
}


void FWrapLayer::CreateBuffer(VkResult Result, VkDevice Device, const VkBufferCreateInfo* CreateInfo, VkBuffer* Buffer)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkCreateBuffer(Info=0x%p, OutBuffer=0x%p)[...]"), CreateInfo, Buffer));
		DebugLog += FString::Printf(TEXT("%sVkBufferCreateInfo: Flags=%d, Size=%d, Usage=%d"), Tabs, CreateInfo->flags, (uint32)CreateInfo->size, (uint32)CreateInfo->usage);
		FlushDebugWrapperLog();
	/*
	VkSharingMode          sharingMode;
	uint32_t               queueFamilyIndexCount;
	const uint32_t*        pQueueFamilyIndices;
	*/
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("Buffer"), *Buffer);
#endif
#if VULKAN_ENABLE_BUFFER_TRACKING_LAYER
		FScopeLock ScopeLock(&GTrackingCS);
		if (Buffer && CreateInfo)
		{
			auto& TrackingBuffer = GVulkanTrackingBuffers.FindOrAdd(*Buffer);
			TrackingBuffer.CreateInfo = *CreateInfo;

#if VULKAN_ENABLE_TRACKING_CALLSTACK
			CaptureCallStack(TrackingBuffer.CreateCallstack, 3);
#endif
		}
#endif
	}
}


void FWrapLayer::CreateBufferView(VkResult Result, VkDevice Device, const VkBufferViewCreateInfo* CreateInfo, VkBufferView* BufferView)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkCreateBufferView(Info=0x%p, OutBufferView=0x%p)\n"), CreateInfo, BufferView));
		DebugLog += FString::Printf(TEXT("%sVkBufferCreateInfo: Flags=%d, Buffer=0x%p, Format=%s, Offset=%d, Range=%d\n"), Tabs, CreateInfo->flags, CreateInfo->buffer,
					*GetVkFormatString(CreateInfo->format), CreateInfo->offset, CreateInfo->range);
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("BufferView"), *BufferView);
		FlushDebugWrapperLog();
#endif
#if VULKAN_ENABLE_BUFFER_TRACKING_LAYER
		FScopeLock ScopeLock(&GTrackingCS);
		if (BufferView && CreateInfo)
		{
			auto& TrackingBuffer = GVulkanTrackingBufferViews.FindOrAdd(*BufferView);
			TrackingBuffer.CreateInfo = *CreateInfo;
#if VULKAN_ENABLE_TRACKING_CALLSTACK
			CaptureCallStack(TrackingBuffer.CreateCallstack, 3);
#endif
			GVulkanTrackingBufferToBufferViews.FindOrAdd(CreateInfo->buffer).Add(*BufferView);
		}
#endif
	}
}

void FWrapLayer::CreateImage(VkResult Result, VkDevice Device, const VkImageCreateInfo* CreateInfo, VkImage* Image)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkCreateImage(CreateInfo=0x%p, OutImage=0x%p)"), CreateInfo, Image));
		if (CreateInfo)
		{
			DebugLog += FString::Printf(TEXT("%sVkImageCreateInfo: Flags=%d, ImageType=%s, Format=%s, MipLevels=%d, ArrayLayers=%d, Samples=%s\n"), Tabs, CreateInfo->flags, *GetVkImageTypeString(CreateInfo->imageType),
				*GetVkFormatString(CreateInfo->format), CreateInfo->mipLevels, CreateInfo->arrayLayers, *GetSampleCountString(CreateInfo->samples));
			DebugLog += FString::Printf(TEXT("%s\tExtent=(%s) Tiling=%s, Usage=%s, Initial=%s\n"), Tabs, *GetExtentString(CreateInfo->extent),
				*GetVkImageTilingString(CreateInfo->tiling), *GetImageUsageString(CreateInfo->usage), *GetVkImageLayoutString(CreateInfo->initialLayout));
		}
		FlushDebugWrapperLog();
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("Image"), *Image);
		FlushDebugWrapperLog();
#endif

#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER
		{
			FScopeLock ScopeLock(&GTrackingCS);
			if (Image && CreateInfo)
			{
				FTrackingImage& TrackingImage = GVulkanTrackingImageLayouts.FindOrAdd(*Image);
				TrackingImage.Setup(*CreateInfo, false);

#if VULKAN_ENABLE_TRACKING_CALLSTACK
				CaptureCallStack(TrackingImage.CreateCallstack, 3);
#endif
			}
		}
#endif
	}
}

void FWrapLayer::DestroyImage(VkResult Result, VkDevice Device, VkImage Image)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyImage(Image=0x%p)"), Image));
#endif
	}
	else
	{
#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER
		BreakOnTrackingImage(Image);
		{
			FScopeLock ScopeLock(&GTrackingCS);
			int32 NumRemoved = GVulkanTrackingImageLayouts.Remove(Image);
			ensure(NumRemoved > 0);
		}
#endif
	}
}

void FWrapLayer::CreateImageView(VkResult Result, VkDevice Device, const VkImageViewCreateInfo* CreateInfo, VkImageView* ImageView)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkCreateImageView(Info=0x%p, OutImageView=0x%p)"), CreateInfo, ImageView));
		DebugLog += FString::Printf(TEXT("%sVkImageViewCreateInfo: Flags=%d, Image=0x%p, ViewType=%s, Format=%s, Components=%s\n"), Tabs, CreateInfo->flags, CreateInfo->image,
			*GetVkImageViewTypeString(CreateInfo->viewType), *GetVkFormatString(CreateInfo->format), *GetVkComponentMappingString(CreateInfo->components));
		DebugLog += FString::Printf(TEXT("%s\tSubresourceRange=(%s)"), Tabs, *GetImageSubResourceRangeString(CreateInfo->subresourceRange));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("ImageView"), *ImageView);
		FlushDebugWrapperLog();
#endif
#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER
		{
			FScopeLock ScopeLock(&GTrackingCS);
			auto& Found = GVulkanTrackingImageViews.FindOrAdd(*ImageView);
			Found.CreateInfo = *CreateInfo;
#if VULKAN_ENABLE_TRACKING_CALLSTACK
			CaptureCallStack(Found.CreateCallstack, 3);
#endif
		}
		BreakOnTrackingImageView(*ImageView);
#endif
	}
}


void FWrapLayer::CreateFence(VkResult Result, VkDevice Device, const VkFenceCreateInfo* CreateInfo, VkFence* Fence)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkCreateFence(CreateInfo=0x%p%s, OutFence=0x%p)"),
			CreateInfo, (CreateInfo->flags == VK_FENCE_CREATE_SIGNALED_BIT) ? TEXT("(SIGNALED)") : TEXT(""), Fence));
	}
	else
	{
		PrintResultAndNamedHandle(Result, TEXT("Fence"), *Fence);
	}
	FlushDebugWrapperLog();
#endif
}

#if VULKAN_ENABLE_DUMP_LAYER
static void DumpFenceList(uint32 FenceCount, const VkFence* Fences)
{
	for (uint32 Index = 0; Index < FenceCount; ++Index)
	{
		DebugLog += Tabs;
		DebugLog += '\t';
		DebugLog += FString::Printf(TEXT("Fence[%d]=0x%p"), Index, Fences[Index]);
		if (Index < FenceCount - 1)
		{
			DebugLog += TEXT("\n");
		}
	}
	FlushDebugWrapperLog();
}
#endif

void FWrapLayer::ResetFences(VkResult Result, VkDevice Device, uint32 FenceCount, const VkFence* Fences)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkResetFences(Count=%d, Fences=0x%p)"), FenceCount, Fences));
		DumpFenceList(FenceCount, Fences);
	}
	else
	{
		PrintResult(Result);
	}
#endif
}

void FWrapLayer::WaitForFences(VkResult Result, VkDevice Device, uint32 FenceCount, const VkFence* Fences, VkBool32 bWaitAll, uint64_t Timeout)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkWaitForFences(Count=%d, Fences=0x%p, WaitAll=%d, Timeout=0x%llx)"), FenceCount, Fences, (uint32)bWaitAll, Timeout));
		DumpFenceList(FenceCount, Fences);
	}
	else
	{
		PrintResult(Result);
	}
#endif
}

void FWrapLayer::CreateSemaphore(VkResult Result, VkDevice Device, const VkSemaphoreCreateInfo* CreateInfo, VkSemaphore* Semaphore)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkCreateSemaphore(CreateInfo=0x%p, OutSemaphore=0x%p)"), CreateInfo, Semaphore));
	}
	else
	{
		PrintResultAndNamedHandle(Result, TEXT("Semaphore"), *Semaphore);
	}
	FlushDebugWrapperLog();
#endif
}


#if VULKAN_ENABLE_DUMP_LAYER
static void DumpMappedMemoryRanges(VkResult Result, uint32 MemoryRangeCount, const VkMappedMemoryRange* MemoryRanges)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
		for (uint32 Index = 0; Index < MemoryRangeCount; ++Index)
		{
			const VkMappedMemoryRange& Range = MemoryRanges[Index];
/*
			typedef struct VkMappedMemoryRange {
				VkStructureType    sType;
				const void*        pNext;
				VkDeviceMemory     memory;
				VkDeviceSize       offset;
				VkDeviceSize       size;
			} VkMappedMemoryRange;
*/
			DebugLog += FString::Printf(TEXT("%s%d Memory=0x%p Offset=%d Size=%d\n"), Tabs, Index,
				(void*)Range.memory, (uint64)Range.offset, (uint64)Range.size);
		}
		FlushDebugWrapperLog();
	}
}
#endif

void FWrapLayer::InvalidateMappedMemoryRanges(VkResult Result, VkDevice Device, uint32 MemoryRangeCount, const VkMappedMemoryRange* MemoryRanges)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkInvalidateMappedMemoryRanges(Count=%d, Ranges=0x%p)"), MemoryRangeCount, MemoryRanges));
		DumpMappedMemoryRanges(Result, MemoryRangeCount, MemoryRanges);
	}
	else
	{
		PrintResult(Result);
	}
#endif
}

void FWrapLayer::FlushMappedMemoryRanges(VkResult Result, VkDevice Device, uint32 MemoryRangeCount, const VkMappedMemoryRange* MemoryRanges)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkFlushMappedMemoryRanges(Count=%d, Ranges=0x%p)"), MemoryRangeCount, MemoryRanges));
		DumpMappedMemoryRanges(Result, MemoryRangeCount, MemoryRanges);
	}
	else
	{
		PrintResult(Result);
	}
#endif
}


void FWrapLayer::CmdResolveImage(VkResult Result, VkCommandBuffer CommandBuffer, VkImage SrcImage, VkImageLayout SrcImageLayout, VkImage DstImage, VkImageLayout DstImageLayout, uint32 RegionCount, const VkImageResolve* Regions)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdResolveImage(SrcImage=0x%p, SrcImageLayout=%s, DestImage=0x%p, DestImageLayout=%s, NumRegions=%d, Regions=0x%p)[...]"),
			CommandBuffer, SrcImage, *GetVkImageLayoutString(SrcImageLayout), DstImage, *GetVkImageLayoutString(DstImageLayout), RegionCount, Regions));
		for (uint32 Index = 0; Index < RegionCount; ++Index)
		{
			DebugLog += Tabs;
			DebugLog += FString::Printf(TEXT("Region %d: "), Index);
			/*
						typedef struct VkImageResolve {
							VkImageSubresourceLayers    srcSubresource;
							VkOffset3D                  srcOffset;
							VkImageSubresourceLayers    dstSubresource;
							VkOffset3D                  dstOffset;
							VkExtent3D                  extent;

			*/
		}
		FlushDebugWrapperLog();
#endif

	}
}

void FWrapLayer::FreeDescriptorSets(VkResult Result, VkDevice Device, VkDescriptorPool DescriptorPool, uint32 DescriptorSetCount, const VkDescriptorSet* DescriptorSets)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
			DevicePrintfBegin(Device, FString::Printf(TEXT("vkFreeDescriptorSets(Pool=0x%p, NumSets=%d, Sets=0x%p)"), DescriptorPool, DescriptorSetCount, DescriptorSets));
			for (uint32 Index = 0; Index < DescriptorSetCount; ++Index)
			{
				DebugLog += Tabs;
				DebugLog += FString::Printf(TEXT("Set %d: 0x%p\n"), Index, DescriptorSets[Index]);
			}
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::CreateInstance(VkResult Result, const VkInstanceCreateInfo* CreateInfo, VkInstance* Instance)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBegin(FString::Printf(TEXT("vkCreateInstance(Info=0x%p, OutInstance=0x%p)[...]"), CreateInfo, Instance));
		/*
				typedef struct VkInstanceCreateInfo {
					VkInstanceCreateFlags       flags;
					const VkApplicationInfo*    pApplicationInfo;
					uint32_t                    enabledLayerCount;
					const char* const*          ppEnabledLayerNames;
					uint32_t                    enabledExtensionCount;
					const char* const*          ppEnabledExtensionNames;
				} VkInstanceCreateInfo;
		*/
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("Instance"), *Instance);
		FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::EnumeratePhysicalDevices(VkResult Result, uint32_t* ApiVersion)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBegin(FString::Printf(TEXT("vkEnumeratePhysicalDevices()")));
#endif
	}
}

void FWrapLayer::EnumerateInstanceVersion(VkResult Result, VkInstance Instance, uint32* PhysicalDeviceCount, VkPhysicalDevice* PhysicalDevices)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBegin(FString::Printf(TEXT("vkEnumeratePhysicalDevices(Instance=0x%p, Count=0x%p, Devices=0x%p)"), Instance, PhysicalDeviceCount, PhysicalDevices));
		if (PhysicalDeviceCount)
		{
			DebugLog += Tabs;
			DebugLog += FString::Printf(TEXT("OutCount=%d\n"), *PhysicalDeviceCount);
			if (PhysicalDevices)
			{
				for (uint32 Index = 0; Index < *PhysicalDeviceCount; ++Index)
				{
					DebugLog += Tabs;
					DebugLog += FString::Printf(TEXT("\tOutDevice[%d]=0x%p\n"), Index, PhysicalDevices[Index]);
				}
			}
			FlushDebugWrapperLog();
		}
#endif
	}
}

#if VULKAN_ENABLE_DUMP_LAYER || VULKAN_ENABLE_IMAGE_TRACKING_LAYER
static void DumpImageMemoryBarriers(uint32 ImageMemoryBarrierCount, const VkImageMemoryBarrier* ImageMemoryBarriers)
{
	for (uint32 Index = 0; Index < ImageMemoryBarrierCount; ++Index)
	{
		const VkImageMemoryBarrier& ImageBarrier = ImageMemoryBarriers[Index];
		DebugLog += FString::Printf(TEXT("%s\tImageBarrier[%d]: srcAccess=%s, oldLayout=%s, srcQueueFamilyIndex=%d\n"), Tabs, Index, VK_FLAGS_TO_STRING(VkAccessFlags, ImageBarrier.srcAccessMask), VK_TYPE_TO_STRING(VkImageLayout, ImageBarrier.oldLayout), ImageBarrier.srcQueueFamilyIndex);
		DebugLog += FString::Printf(TEXT("%s\t\tdstAccess=%s, newLayout=%s, dstQueueFamilyIndex=%d\n"), Tabs, VK_FLAGS_TO_STRING(VkAccessFlags, ImageBarrier.dstAccessMask), VK_TYPE_TO_STRING(VkImageLayout, ImageBarrier.newLayout), ImageBarrier.dstQueueFamilyIndex);
		DebugLog += FString::Printf(TEXT("%s\t\tImage=0x%p, subresourceRange=(%s)\n"), Tabs, ImageBarrier.image, *GetImageSubResourceRangeString(ImageBarrier.subresourceRange));
	}
}
static void DumpImageMemoryBarriers(uint32 ImageMemoryBarrierCount, const VkImageMemoryBarrier2* ImageMemoryBarriers)
{
	for (uint32 Index = 0; Index < ImageMemoryBarrierCount; ++Index)
	{
		const VkImageMemoryBarrier2& ImageBarrier = ImageMemoryBarriers[Index];
		DebugLog += FString::Printf(TEXT("%s\tImageBarrier[%d]: srcStage=%s, srcAccess=%s, oldLayout=%s, srcQueueFamilyIndex=%d\n"), Tabs, Index, VK_FLAGS_TO_STRING(VkPipelineStageFlags2, ImageBarrier.srcStageMask), VK_FLAGS_TO_STRING(VkAccessFlags2, ImageBarrier.srcAccessMask), VK_TYPE_TO_STRING(VkImageLayout, ImageBarrier.oldLayout), ImageBarrier.srcQueueFamilyIndex);
		DebugLog += FString::Printf(TEXT("%s\t\tdstStage=%s, dstAccess=%s, newLayout=%s, dstQueueFamilyIndex=%d\n"), Tabs, VK_FLAGS_TO_STRING(VkPipelineStageFlags2, ImageBarrier.dstStageMask), VK_FLAGS_TO_STRING(VkAccessFlags2, ImageBarrier.dstAccessMask), VK_TYPE_TO_STRING(VkImageLayout, ImageBarrier.newLayout), ImageBarrier.dstQueueFamilyIndex);
		DebugLog += FString::Printf(TEXT("%s\t\tImage=0x%p, subresourceRange=(%s)\n"), Tabs, ImageBarrier.image, *GetImageSubResourceRangeString(ImageBarrier.subresourceRange));
	}
}
#endif

void FWrapLayer::CmdPipelineBarrier(VkResult Result, VkCommandBuffer CommandBuffer, VkPipelineStageFlags SrcStageMask, VkPipelineStageFlags DstStageMask, VkDependencyFlags DependencyFlags, uint32 MemoryBarrierCount, const VkMemoryBarrier* MemoryBarriers, uint32 BufferMemoryBarrierCount, const VkBufferMemoryBarrier* BufferMemoryBarriers, uint32 ImageMemoryBarrierCount, const VkImageMemoryBarrier* ImageMemoryBarriers)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdPipelineBarrier(srcStageMask=%s, destStageMask=%s, Flags=%d, NumMemB=%d, MemB=0x%p,"), VK_FLAGS_TO_STRING(VkPipelineStageFlags, SrcStageMask), VK_FLAGS_TO_STRING(VkPipelineStageFlags, DstStageMask), (uint32)DependencyFlags, MemoryBarrierCount, MemoryBarriers));
		DebugLog += FString::Printf(TEXT("%s\tNumBufferB=%d, BufferB=0x%p, NumImageB=%d, ImageB=0x%p)[...]\n"), Tabs, BufferMemoryBarrierCount, BufferMemoryBarriers, ImageMemoryBarrierCount, ImageMemoryBarriers);
		DumpImageMemoryBarriers(ImageMemoryBarrierCount, ImageMemoryBarriers);
		FlushDebugWrapperLog();
#endif

#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER
		{
			FScopeLock ScopeLock(&GTrackingCS);
			for (uint32 Index = 0; Index < ImageMemoryBarrierCount; ++Index)
			{
				BreakOnTrackingImage(ImageMemoryBarriers[Index].image);
				if(DumpTrackImage(ImageMemoryBarriers[Index].image))
				{
					CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdPipelineBarrier(SrcStageMask=%s, DstStageMask=%s, Flags=%d, NumMemB=%d, MemB=0x%p,"), VK_FLAGS_TO_STRING(VkPipelineStageFlags, SrcStageMask), VK_FLAGS_TO_STRING(VkPipelineStageFlags, DstStageMask), (uint32)DependencyFlags, MemoryBarrierCount, MemoryBarriers));
					DebugLog += FString::Printf(TEXT("%s\tNumBufferB=%d, BufferB=0x%p, NumImageB=%d, ImageB=0x%p)[...]\n"), Tabs, BufferMemoryBarrierCount, BufferMemoryBarriers, ImageMemoryBarrierCount, ImageMemoryBarriers);
					DumpImageMemoryBarriers(ImageMemoryBarrierCount, ImageMemoryBarriers);
					FlushDebugWrapperLog();
				}
				FTrackingImage* TrackingImage = GVulkanTrackingImageLayouts.Find(ImageMemoryBarriers[Index].image);
				check(TrackingImage);
#if VULKAN_ENABLE_TRACKING_CALLSTACK
				FTrackingImage::FHistoryEntry* HistoryEntry = new (TrackingImage->History) FTrackingImage::FHistoryEntry;
				HistoryEntry->ArrayLayouts = TrackingImage->ArrayLayouts;
#endif
				const VkImageSubresourceRange& Range = ImageMemoryBarriers[Index].subresourceRange;
				uint32 NumLayers = (Range.layerCount == VK_REMAINING_ARRAY_LAYERS) ? (TrackingImage->Info.CreateInfo.arrayLayers - Range.baseArrayLayer) : Range.layerCount;
				for (uint32 LIndex = Range.baseArrayLayer; LIndex < Range.baseArrayLayer + NumLayers; ++LIndex)
				{
					TArray<VkImageLayout>& MipLayouts = TrackingImage->ArrayLayouts[LIndex];
					uint32 NumLevels = (Range.levelCount == VK_REMAINING_MIP_LEVELS ) ? (TrackingImage->Info.CreateInfo.mipLevels - Range.baseMipLevel) : Range.levelCount;
					for (uint32 MIndex = Range.baseMipLevel; MIndex < Range.baseMipLevel + NumLevels; ++MIndex)
					{
						if (ImageMemoryBarriers[Index].oldLayout != VK_IMAGE_LAYOUT_UNDEFINED && MipLayouts[MIndex] != ImageMemoryBarriers[Index].oldLayout)
						{
							ensure(0);
						}
						MipLayouts[MIndex] = ImageMemoryBarriers[Index].newLayout;
					}
				}
#if VULKAN_ENABLE_TRACKING_CALLSTACK
				CaptureCallStack(HistoryEntry->Callstack, 2);
#endif
			}
		}
#endif
	}
}

void FWrapLayer::CmdWaitEvents(VkResult Result, VkCommandBuffer CommandBuffer, uint32 EventCount, const VkEvent* Events, VkPipelineStageFlags SrcStageMask, VkPipelineStageFlags DstStageMask, uint32 MemoryBarrierCount,
		const VkMemoryBarrier* MemoryBarriers, uint32 BufferMemoryBarrierCount, const VkBufferMemoryBarrier* BufferMemoryBarriers, uint32 ImageMemoryBarrierCount, const VkImageMemoryBarrier* ImageMemoryBarriers)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdWaitEvents(NumEvents=%d, Events=0x%p, SrcStageMask=%s, DestStageMask=%s, NumMemB=%d, MemB=0x%p,"), EventCount, Events, VK_FLAGS_TO_STRING(VkPipelineStageFlags, SrcStageMask), VK_FLAGS_TO_STRING(VkPipelineStageFlags, DstStageMask), MemoryBarrierCount, MemoryBarriers));
		DebugLog += FString::Printf(TEXT("%s\tNumBufferB=%d, BufferB=0x%p, NumImageB=%d, ImageB=0x%p)[...]\n"), Tabs, BufferMemoryBarrierCount, BufferMemoryBarriers, ImageMemoryBarrierCount, ImageMemoryBarriers);
		for (uint32 Index = 0; Index < EventCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%s\tEvents[%d]=0x%p)\n"), Tabs, Index, Events[Index]);
		}
		DumpImageMemoryBarriers(ImageMemoryBarrierCount, ImageMemoryBarriers);
		FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::CmdBindDescriptorSets(VkResult Result, VkCommandBuffer CommandBuffer, VkPipelineBindPoint PipelineBindPoint, VkPipelineLayout Layout, uint32 FirstSet, uint32 DescriptorSetCount, const VkDescriptorSet* DescriptorSets, uint32 DynamicOffsetCount, const uint32* DynamicOffsets)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdBindDescriptorSets(BindPoint=%s, Layout=0x%p, FirstSet=%d, NumDS=%d, DS=0x%p, NumDynamicOffset=%d, DynamicOffsets=0x%p)"), *GetPipelineBindPointString(PipelineBindPoint), Layout, FirstSet, DescriptorSetCount, DescriptorSets, DynamicOffsetCount, DynamicOffsets));
		for (uint32 Index = 0; Index < DescriptorSetCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%s\tDS[%d]=0x%p\n"), Tabs, Index, DescriptorSets[Index]);
		}
		for (uint32 Index = 0; Index < DynamicOffsetCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%s\tDynamicOffset[%d]=%d (0x%x)\n"), Tabs, Index, DynamicOffsets[Index], DynamicOffsets[Index]);
		}
		FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::CreateDescriptorSetLayout(VkResult Result, VkDevice Device, const VkDescriptorSetLayoutCreateInfo* CreateInfo, VkDescriptorSetLayout* SetLayout)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkCreateDescriptorSetLayout(Info=0x%p, OutLayout=0x%p)[...]"), CreateInfo, SetLayout));
		DebugLog += FString::Printf(TEXT("%sNumBindings=%d, Bindings=0x%p\n"), Tabs, CreateInfo->bindingCount, CreateInfo->pBindings);
		for (uint32 Index = 0; Index < CreateInfo->bindingCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%s\tBinding[%d]= binding=%d DescType=%s NumDesc=%d StageFlags=%x\n"), Tabs, Index,
				CreateInfo->pBindings[Index].binding, *GetVkDescriptorTypeString(CreateInfo->pBindings[Index].descriptorType), CreateInfo->pBindings[Index].descriptorCount, (uint32)CreateInfo->pBindings[Index].stageFlags);
		}
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("DescriptorSetLayout"), *SetLayout);
#endif
	}
}

void FWrapLayer::AllocateDescriptorSets(VkResult Result, VkDevice Device, const VkDescriptorSetAllocateInfo* AllocateInfo, VkDescriptorSet* DescriptorSets)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkAllocateDescriptorSets(Info=0x%p, OutSets=0x%p)"), AllocateInfo, DescriptorSets));
		DebugLog += FString::Printf(TEXT("%s\tVkDescriptorSetAllocateInfo: Pool=0x%p, NumSetLayouts=%d:"), Tabs, AllocateInfo->descriptorPool, AllocateInfo->descriptorSetCount);
		for (uint32 Index = 0; Index < AllocateInfo->descriptorSetCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT(" [%d]=0x%p"), Index, (void*)AllocateInfo->pSetLayouts[Index]);
			FlushDebugWrapperLog();
		}
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandles(Result, TEXT("DescriptorSet"), AllocateInfo->descriptorSetCount, (uint64*)DescriptorSets);
#endif
	}
}

void FWrapLayer::UpdateDescriptorSets(VkResult Result, VkDevice Device, uint32 DescriptorWriteCount, const VkWriteDescriptorSet* DescriptorWrites, uint32 DescriptorCopyCount, const VkCopyDescriptorSet* DescriptorCopies)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkUpdateDescriptorSets(NumWrites=%d, Writes=0x%p, NumCopies=%d, Copies=0x%p)"), DescriptorWriteCount, DescriptorWrites, DescriptorCopyCount, DescriptorCopies));
#endif
		for (uint32 Index = 0; Index < DescriptorWriteCount; ++Index)
		{
#if VULKAN_ENABLE_DUMP_LAYER
			DebugLog += FString::Printf(TEXT("%sWrite[%d]: Set=0x%p Binding=%d DstArrayElem=%d NumDesc=%d DescType=%s "), Tabs, Index,
				DescriptorWrites[Index].dstSet, DescriptorWrites[Index].dstBinding, DescriptorWrites[Index].dstArrayElement, DescriptorWrites[Index].descriptorCount, *GetVkDescriptorTypeString(DescriptorWrites[Index].descriptorType));
#endif
			switch (DescriptorWrites[Index].descriptorType)
			{
			case VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER:
			case VK_DESCRIPTOR_TYPE_STORAGE_BUFFER:
			case VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC:
			case VK_DESCRIPTOR_TYPE_STORAGE_BUFFER_DYNAMIC:
#if VULKAN_ENABLE_DUMP_LAYER
				DebugLog += FString::Printf(TEXT("pBufferInfo=0x%p\n"), DescriptorWrites[Index].pBufferInfo);
#endif
				if (DescriptorWrites[Index].pBufferInfo)
				{
					for (uint32 SubIndex = 0; SubIndex < DescriptorWrites[Index].descriptorCount; ++SubIndex)
					{
#if VULKAN_ENABLE_DUMP_LAYER
						DebugLog += FString::Printf(TEXT("%s\tpBufferInfo[%d]: buffer=0x%p, offset=%d, range=%d\n"), Tabs, SubIndex,
							DescriptorWrites[Index].pBufferInfo->buffer, (int32)DescriptorWrites[Index].pBufferInfo->offset, (int32)DescriptorWrites[Index].pBufferInfo->range);
#endif
					}
				}
				else
				{
					ValidationFail();
				}
				break;

			case VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER:
			case VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER:
#if VULKAN_ENABLE_DUMP_LAYER
				DebugLog += FString::Printf(TEXT("pTexelBufferView=0x%p\n"), DescriptorWrites[Index].pTexelBufferView);
#endif
				if (DescriptorWrites[Index].pTexelBufferView)
				{
					for (uint32 SubIndex = 0; SubIndex < DescriptorWrites[Index].descriptorCount; ++SubIndex)
					{
#if VULKAN_ENABLE_DUMP_LAYER

#if VULKAN_ENABLE_BUFFER_TRACKING_LAYER
						DebugLog += FString::Printf(TEXT("%s\tpTexelBufferView[%d]=0x%p(B:0x%p)\n"), Tabs, SubIndex, DescriptorWrites[Index].pTexelBufferView[SubIndex], FindTrackingBuffer(DescriptorWrites[Index].pTexelBufferView[SubIndex]));
#else
						DebugLog += FString::Printf(TEXT("%s\tpTexelBufferView[%d]=0x%p\n"), Tabs, SubIndex, DescriptorWrites[Index].pTexelBufferView[SubIndex]);
#endif
#endif
					}
				}
				else
				{
					ValidationFail();
				}
				break;

			case VK_DESCRIPTOR_TYPE_SAMPLER:
			case VK_DESCRIPTOR_TYPE_COMBINED_IMAGE_SAMPLER:
			case VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE:
			case VK_DESCRIPTOR_TYPE_STORAGE_IMAGE:
			case VK_DESCRIPTOR_TYPE_INPUT_ATTACHMENT:
			default:
#if VULKAN_ENABLE_DUMP_LAYER
				DebugLog += FString::Printf(TEXT("pImageInfo=0x%p\n"), DescriptorWrites[Index].pImageInfo);
#endif
				if (DescriptorWrites[Index].pImageInfo)
				{
					for (uint32 SubIndex = 0; SubIndex < DescriptorWrites[Index].descriptorCount; ++SubIndex)
					{
#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER
						BreakOnTrackingImageView(DescriptorWrites[Index].pImageInfo->imageView);
#endif
#if VULKAN_ENABLE_DUMP_LAYER
						auto* FoundImageInfo = GVulkanTrackingImageViews.Find(DescriptorWrites[Index].pImageInfo->imageView);
						VkImage *Image = FoundImageInfo ? &FoundImageInfo->CreateInfo.image : nullptr;
						DebugLog += FString::Printf(TEXT("%s\tpImageInfo[%d]: Sampler=0x%p, ImageView=0x%p(I:0x%p), imageLayout=%s\n"), Tabs, SubIndex,
							DescriptorWrites[Index].pImageInfo->sampler, DescriptorWrites[Index].pImageInfo->imageView, Image ? *Image : nullptr, *GetVkImageLayoutString(DescriptorWrites[Index].pImageInfo->imageLayout));
#endif
					}
				}
				else
				{
					ValidationFail();
				}
				break;
			}
		}
#if VULKAN_ENABLE_DUMP_LAYER
		FlushDebugWrapperLog();
#endif
	}
	/*
	typedef struct VkWriteDescriptorSet {
		VkDescriptorSet                  dstSet;
		uint32_t                         dstBinding;
		uint32_t                         dstArrayElement;
		uint32_t                         descriptorCount;
		VkDescriptorType                 descriptorType;
		const VkDescriptorImageInfo*     pImageInfo;
		const VkDescriptorBufferInfo*    pBufferInfo;
		const VkBufferView*              pTexelBufferView;
	} VkWriteDescriptorSet;
*/
}

void FWrapLayer::CreateFramebuffer(VkResult Result, VkDevice Device, const VkFramebufferCreateInfo* CreateInfo, VkFramebuffer* Framebuffer)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkCreateFramebuffer(Info=0x%p, OutFramebuffer=0x%p)"), CreateInfo, Framebuffer));
		DebugLog += FString::Printf(TEXT("%sVkFramebufferCreateInfo: Flags=%d, RenderPass=0x%p, NumAttachments=%d\n"), Tabs, CreateInfo->flags, CreateInfo->renderPass, CreateInfo->attachmentCount);
		for (uint32 Index = 0; Index < CreateInfo->attachmentCount; ++Index)
		{
			auto* FoundImageInfo = GVulkanTrackingImageViews.Find(CreateInfo->pAttachments[Index]);
			VkImage *Image = FoundImageInfo ? &FoundImageInfo->CreateInfo.image : nullptr;

			DebugLog += FString::Printf(TEXT("%s\tAttachment[%d]: ImageView=0x%p(I:0x%p)\n"), Tabs, Index, CreateInfo->pAttachments[Index], Image ? *Image : nullptr);
		}
		DebugLog += FString::Printf(TEXT("%s\twidth=%d, height=%d, layers=%d\n"), Tabs, CreateInfo->width, CreateInfo->height, CreateInfo->layers);
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("Framebuffer"), *Framebuffer);
		if (Result == VK_SUCCESS)
		{
			FFBInfo Info;
			Info.Info = *CreateInfo;
			if (CreateInfo->attachmentCount)
			{
				Info.Attachments.AddUninitialized(CreateInfo->attachmentCount);
				FMemory::Memcpy(&Info.Attachments[0], CreateInfo->pAttachments, CreateInfo->attachmentCount * sizeof(VkImageView));
				Info.Info.pAttachments = &Info.Attachments[0];
			}
			GFramebufferInfo.Add(*Framebuffer, Info);
		}
#endif
	}
}

void FWrapLayer::CreateRenderPass(VkResult Result, VkDevice Device, const VkRenderPassCreateInfo* CreateInfo, VkRenderPass* RenderPass)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkCreateRenderPass(Info=0x%p, OutRenderPass=0x%p)[...]"), CreateInfo, RenderPass));
		DebugLog += FString::Printf(TEXT("%s\tVkRenderPassCreateInfo: NumAttachments=%d, Attachments=0x%p, NumSubPasses=%d, SubPasses=0x%p\n"), Tabs, CreateInfo->attachmentCount, CreateInfo->pAttachments, CreateInfo->subpassCount, CreateInfo->pSubpasses);
		for (uint32 Index = 0; Index < CreateInfo->attachmentCount; ++Index)
		{
			auto GetLoadOpString = [](VkAttachmentLoadOp Op) -> FString
				{
					switch (Op)
					{
					case VK_ATTACHMENT_LOAD_OP_LOAD: return TEXT("LOAD");
					case VK_ATTACHMENT_LOAD_OP_CLEAR: return TEXT("CLEAR");
					case VK_ATTACHMENT_LOAD_OP_DONT_CARE: return TEXT("DONT_CARE");
					default: return FString::Printf(TEXT("Invalid(%d)"), (uint32)Op);

					}
				};
			auto GetStoreOpString = [](VkAttachmentStoreOp Op) -> FString
				{
					switch(Op)
					{
					case VK_ATTACHMENT_STORE_OP_STORE: return TEXT("STORE");
					case VK_ATTACHMENT_STORE_OP_DONT_CARE: return TEXT("DONT_CARE");
					default: return FString::Printf(TEXT("Invalid(%d)"), (uint32)Op);

					}
				};

			const VkAttachmentDescription& Desc = CreateInfo->pAttachments[Index];
			DebugLog += FString::Printf(TEXT("%s\t\tAttachment[%d]: Flags=%s, Format=%s, Samples=%s, Load=%s, Store=%s\n"), Tabs, Index,
				(Desc.flags == VK_ATTACHMENT_DESCRIPTION_MAY_ALIAS_BIT ? TEXT("MAY_ALIAS") : TEXT("0")),
				*GetVkFormatString(Desc.format), *GetSampleCountString(Desc.samples), *GetLoadOpString(Desc.loadOp), *GetStoreOpString(Desc.storeOp));
			DebugLog += FString::Printf(TEXT("%s\t\t\tLoadStencil=%s, StoreStencil=%s, Initial=%s, Final=%s\n"), Tabs,
				*GetLoadOpString(Desc.stencilLoadOp), *GetStoreOpString(Desc.stencilStoreOp), *VulkanRHI::GetVkImageLayoutString(Desc.initialLayout), *VulkanRHI::GetVkImageLayoutString(Desc.finalLayout));
		}

		for (uint32 Index = 0; Index < CreateInfo->subpassCount; ++Index)
		{
			const VkSubpassDescription& Desc = CreateInfo->pSubpasses[Index];
			DebugLog += FString::Printf(TEXT("%s\t\tSubpass[%d]: Flags=%d, Bind=%s, NumInputAttach=%d, InputAttach=0x%p, NumColorAttach=%d, ColorAttach=0x%p, DSAttch=0x%p\n"), Tabs, Index,
				Desc.flags,
				Desc.pipelineBindPoint == VK_PIPELINE_BIND_POINT_COMPUTE ? TEXT("Compute") : TEXT("Gfx"),
				Desc.inputAttachmentCount, Desc.pInputAttachments, Desc.colorAttachmentCount, Desc.pColorAttachments, Desc.pDepthStencilAttachment);
			for (uint32 SubIndex = 0; SubIndex < Desc.inputAttachmentCount; ++SubIndex)
			{
				DebugLog += FString::Printf(TEXT("%s\t\t\tInputAttach[%d]: Attach=%d, Layout=%s\n"), Tabs, Index,
					Desc.pInputAttachments[SubIndex].attachment, *GetVkImageLayoutString(Desc.pInputAttachments[SubIndex].layout));
			}
			for (uint32 SubIndex = 0; SubIndex < Desc.colorAttachmentCount; ++SubIndex)
			{
				DebugLog += FString::Printf(TEXT("%s\t\t\tColorAttach[%d]: Attach=%d, Layout=%s\n"), Tabs, Index,
					Desc.pColorAttachments[SubIndex].attachment, *GetVkImageLayoutString(Desc.pColorAttachments[SubIndex].layout));
			}
			if (Desc.pDepthStencilAttachment)
			{
				DebugLog += FString::Printf(TEXT("%s\t\t\tDSAttach: Attach=%d, Layout=%s\n"), Tabs, Desc.pDepthStencilAttachment->attachment, *GetVkImageLayoutString(Desc.pDepthStencilAttachment->layout));
			}
			/*
			typedef struct VkSubpassDescription {
				const VkAttachmentReference*    pResolveAttachments;
				uint32_t                        preserveAttachmentCount;
				const uint32_t*                 pPreserveAttachments;
			} VkSubpassDescription;*/
		}
/*
		typedef struct VkRenderPassCreateInfo {
			uint32_t                          dependencyCount;
			const VkSubpassDependency*        pDependencies;
		} VkRenderPassCreateInfo;
*/
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("RenderPass"), *RenderPass);
		if (Result == VK_SUCCESS)
		{
			FRenderPassInfo Info;
			Info.Info = *CreateInfo;
			Info.Info.pAttachments = nullptr;
			Info.Info.pSubpasses = nullptr;
			Info.Info.pDependencies = nullptr;
			Info.Descriptions.AddUninitialized(CreateInfo->attachmentCount);
			if (CreateInfo->attachmentCount)
			{
				FMemory::Memcpy(&Info.Descriptions[0], CreateInfo->pAttachments, CreateInfo->attachmentCount * sizeof(VkAttachmentDescription));
			}
			GRenderPassInfo.Add(*RenderPass, Info);
		}
		FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::CreateRenderPass2KHR(VkResult Result, VkDevice Device, const VkRenderPassCreateInfo2* CreateInfo, VkRenderPass* RenderPass)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkCreateRenderPass2KHR(Info=0x%p, OutRenderPass=0x%p)[...]"), CreateInfo, RenderPass));
		DebugLog += FString::Printf(TEXT("%s\tVkRenderPassCreateInfo2: Flags=%d, NumAttachments=%d, Attachments=0x%p, NumSubPasses=%d, SubPasses=0x%p\n"), Tabs, CreateInfo->attachmentCount, CreateInfo->pAttachments, CreateInfo->subpassCount, CreateInfo->pSubpasses);
		for (uint32 Index = 0; Index < CreateInfo->attachmentCount; ++Index)
		{
			auto GetLoadOpString = [](VkAttachmentLoadOp Op) -> FString
			{
				switch (Op)
				{
				case VK_ATTACHMENT_LOAD_OP_LOAD: return TEXT("LOAD");
				case VK_ATTACHMENT_LOAD_OP_CLEAR: return TEXT("CLEAR");
				case VK_ATTACHMENT_LOAD_OP_DONT_CARE: return TEXT("DONT_CARE");
				default: return FString::Printf(TEXT("Invalid(%d)"), (uint32)Op);

				}
			};
			auto GetStoreOpString = [](VkAttachmentStoreOp Op) -> FString
			{
				switch (Op)
				{
				case VK_ATTACHMENT_STORE_OP_STORE: return TEXT("STORE");
				case VK_ATTACHMENT_STORE_OP_DONT_CARE: return TEXT("DONT_CARE");
				default: return FString::Printf(TEXT("Invalid(%d)"), (uint32)Op);

				}
			};

			const VkAttachmentDescription2& Desc = CreateInfo->pAttachments[Index];
			DebugLog += FString::Printf(TEXT("%s\t\tAttachment[%d]: Flags=%s, Format=%s, Samples=%s, Load=%s, Store=%s\n"), Tabs, Index,
										(Desc.flags == VK_ATTACHMENT_DESCRIPTION_MAY_ALIAS_BIT ? TEXT("MAY_ALIAS") : TEXT("0")),
										*GetVkFormatString(Desc.format), *GetSampleCountString(Desc.samples), *GetLoadOpString(Desc.loadOp), *GetStoreOpString(Desc.storeOp));
			DebugLog += FString::Printf(TEXT("%s\t\t\tLoadStencil=%s, StoreStencil=%s, Initial=%s, Final=%s\n"), Tabs,
										*GetLoadOpString(Desc.stencilLoadOp), *GetStoreOpString(Desc.stencilStoreOp), *VulkanRHI::GetVkImageLayoutString(Desc.initialLayout), *VulkanRHI::GetVkImageLayoutString(Desc.finalLayout));
		}

		for (uint32 Index = 0; Index < CreateInfo->subpassCount; ++Index)
		{
			const VkSubpassDescription2& Desc = CreateInfo->pSubpasses[Index];
			DebugLog += FString::Printf(TEXT("%s\t\tSubpass[%d]: Flags=%d, Bind=%s, NumInputAttach=%d, InputAttach=0x%p, NumColorAttach=%d, ColorAttach=0x%p, DSAttch=0x%p\n"), Tabs, Index,
										Desc.flags,
										Desc.pipelineBindPoint == VK_PIPELINE_BIND_POINT_COMPUTE ? TEXT("Compute") : TEXT("Gfx"),
										Desc.inputAttachmentCount, Desc.pInputAttachments, Desc.colorAttachmentCount, Desc.pColorAttachments, Desc.pDepthStencilAttachment);
			for (uint32 SubIndex = 0; SubIndex < Desc.inputAttachmentCount; ++SubIndex)
			{
				DebugLog += FString::Printf(TEXT("%s\t\t\tInputAttach[%d]: Attach=%d, Layout=%s\n"), Tabs, Index,
											Desc.pInputAttachments[SubIndex].attachment, *GetVkImageLayoutString(Desc.pInputAttachments[SubIndex].layout));
			}
			for (uint32 SubIndex = 0; SubIndex < Desc.colorAttachmentCount; ++SubIndex)
			{
				DebugLog += FString::Printf(TEXT("%s\t\t\tColorAttach[%d]: Attach=%d, Layout=%s\n"), Tabs, Index,
											Desc.pColorAttachments[SubIndex].attachment, *GetVkImageLayoutString(Desc.pColorAttachments[SubIndex].layout));
			}
			if (Desc.pDepthStencilAttachment)
			{
				DebugLog += FString::Printf(TEXT("%s\t\t\tDSAttach: Attach=%d, Layout=%s\n"), Tabs, Desc.pDepthStencilAttachment->attachment, *GetVkImageLayoutString(Desc.pDepthStencilAttachment->layout));
			}
			DebugLog += FString::Printf(TEXT("%s\t\t\tviewMask=%d\n"), Tabs, Desc.viewMask);
		}
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("RenderPass"), *RenderPass);
		if (Result == VK_SUCCESS)
		{
			FRenderPassInfo Info;
			Info.Info.sType = CreateInfo->sType;
			Info.Info.attachmentCount = CreateInfo->attachmentCount;
			Info.Info.dependencyCount = CreateInfo->dependencyCount;
			Info.Info.subpassCount = CreateInfo->subpassCount;
			Info.Info.flags = CreateInfo->flags;
			Info.Info.pAttachments = nullptr;
			Info.Info.pSubpasses = nullptr;
			Info.Info.pDependencies = nullptr;
			Info.Descriptions.AddUninitialized(CreateInfo->attachmentCount);
			for (uint32 Index = 0; Index < CreateInfo->attachmentCount; ++Index)
			{
				Info.Descriptions[Index].flags = CreateInfo->pAttachments[Index].flags;
				Info.Descriptions[Index].format = CreateInfo->pAttachments[Index].format;
				Info.Descriptions[Index].samples = CreateInfo->pAttachments[Index].samples;
				Info.Descriptions[Index].loadOp = CreateInfo->pAttachments[Index].loadOp;
				Info.Descriptions[Index].storeOp = CreateInfo->pAttachments[Index].storeOp;
				Info.Descriptions[Index].stencilLoadOp = CreateInfo->pAttachments[Index].stencilLoadOp;
				Info.Descriptions[Index].stencilStoreOp = CreateInfo->pAttachments[Index].stencilStoreOp;
				Info.Descriptions[Index].initialLayout = CreateInfo->pAttachments[Index].initialLayout;
				Info.Descriptions[Index].finalLayout = CreateInfo->pAttachments[Index].finalLayout;
			}
			GRenderPassInfo.Add(*RenderPass, Info);
		}
		FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::QueueSubmit(VkResult Result, VkQueue Queue, uint32 SubmitCount, const VkSubmitInfo* Submits, VkFence Fence)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkQueueSubmit(Queue=0x%p, Count=%d, Submits=0x%p, Fence=0x%p)"), Queue, SubmitCount, Submits, Fence));
		for (uint32 Index = 0; Index < SubmitCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("\n%sSubmit[%d]:"), Tabs, Index);
			if (Submits[Index].waitSemaphoreCount > 0)
			{
				DebugLog += FString::Printf(TEXT("\n%s\tWaitSemaphores(Mask): "), Tabs, Index);
				for (uint32 SubIndex = 0; SubIndex < Submits[Index].waitSemaphoreCount; ++SubIndex)
				{
					DebugLog += FString::Printf(TEXT("0x%p(%d) "), Submits[Index].pWaitSemaphores[SubIndex], (int32)Submits[Index].pWaitDstStageMask[SubIndex]);
				}
			}
			if (Submits[Index].commandBufferCount > 0)
			{
				DebugLog += FString::Printf(TEXT("\n%s\tCommandBuffers: "), Tabs, Index);
				for (uint32 SubIndex = 0; SubIndex < Submits[Index].commandBufferCount; ++SubIndex)
				{
					DebugLog += FString::Printf(TEXT("0x%p "), Submits[Index].pCommandBuffers[SubIndex]);
				}
			}
			if (Submits[Index].signalSemaphoreCount > 0)
			{
				DebugLog += FString::Printf(TEXT("\n%s\tSignalSemaphore: "), Tabs, Index);
				for (uint32 SubIndex = 0; SubIndex < Submits[Index].signalSemaphoreCount; ++SubIndex)
				{
					DebugLog += FString::Printf(TEXT("0x%p "), Submits[Index].pSignalSemaphores[SubIndex]);
				}
			}
		}
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::CreateShaderModule(VkResult Result, VkDevice Device, const VkShaderModuleCreateInfo* CreateInfo, VkShaderModule* ShaderModule)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkCreateShaderModule(CreateInfo=0x%p, OutShaderModule=0x%p)[...]"), CreateInfo, ShaderModule));
		/*
			typedef struct VkShaderModuleCreateInfo {
				VkStructureType              sType;
				const void*                  pNext;
				VkShaderModuleCreateFlags    flags;
				size_t                       codeSize;
				const uint32_t*              pCode;
			} VkShaderModuleCreateInfo;
		*/
	}
	else
	{
		PrintResultAndNamedHandle(Result, TEXT("ShaderModule"), *ShaderModule);
	}
	FlushDebugWrapperLog();
#endif
}

void FWrapLayer::CreatePipelineCache(VkResult Result, VkDevice Device, const VkPipelineCacheCreateInfo* CreateInfo, VkPipelineCache* PipelineCache)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkCreatePipelineCache(CreateInfo=0x%p, OutPipelineCache=0x%p) InitialSize=%d Data=0x%p "), CreateInfo, PipelineCache, (uint32)CreateInfo->initialDataSize, CreateInfo->pInitialData));
	}
	else
	{
		PrintResultAndNamedHandle(Result, TEXT("PipelineCache"), *PipelineCache);
	}
	FlushDebugWrapperLog();
#endif
}

void FWrapLayer::CreateCommandPool(VkResult Result, VkDevice Device, const VkCommandPoolCreateInfo* CreateInfo, VkCommandPool* CommandPool)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkCreateCommandPool(CreateInfo=0x%p, OutCommandPool=0x%p)[...]"), CreateInfo, CommandPool));
	}
	else
	{
		PrintResultAndNamedHandle(Result, TEXT("CommandPool"), *CommandPool);
	}
	FlushDebugWrapperLog();
#endif
}

void FWrapLayer::CreateQueryPool(VkResult Result, VkDevice Device, const VkQueryPoolCreateInfo* CreateInfo, VkQueryPool* QueryPool)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkCreateQueryPool(CreateInfo=0x%p, OutQueryPool=0x%p)[...]"), CreateInfo, QueryPool));
	}
	else
	{
		PrintResultAndNamedHandle(Result, TEXT("QueryPool"), *QueryPool);
	}
	FlushDebugWrapperLog();
#endif
}

void FWrapLayer::CreatePipelineLayout(VkResult Result, VkDevice Device, const VkPipelineLayoutCreateInfo* CreateInfo, VkPipelineLayout* PipelineLayout)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkCreatePipelineLayout(CreateInfo=0x%p, OutPipelineLayout=0x%p) NumLayouts=%d"), CreateInfo, PipelineLayout, CreateInfo->setLayoutCount));
		DebugLog += FString::Printf(TEXT("%sLayouts: "), Tabs);
		for (uint32 Index = 0; Index < CreateInfo->setLayoutCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%d=0x%p "), Index, CreateInfo->pSetLayouts[Index]);
		}
		DebugLog += '\n';
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("PipelineLayout"), *PipelineLayout);
#endif
	}
	/*
	typedef struct VkPipelineLayoutCreateInfo {
	VkPipelineLayoutCreateFlags     flags;
	uint32_t                        pushConstantRangeCount;
	const VkPushConstantRange*      pPushConstantRanges;
	} VkPipelineLayoutCreateInfo;
	*/
}

void FWrapLayer::CreateDescriptorPool(VkResult Result, VkDevice Device, const VkDescriptorPoolCreateInfo* CreateInfo, VkDescriptorPool* DescriptorPool)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkCreateDescriptorPool(CreateInfo=0x%p, OutDescriptorPool=0x%p)[...]"), CreateInfo, DescriptorPool));
	}
	else
	{
		PrintResultAndNamedHandle(Result, TEXT("DescriptorPool"), *DescriptorPool);
	}
	FlushDebugWrapperLog();
#endif
}


void FWrapLayer::CreateSampler(VkResult Result, VkDevice Device, const VkSamplerCreateInfo* CreateInfo, VkSampler* Sampler)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkCreateSampler(CreateInfo=0x%p, OutSampler=0x%p)[...]"), CreateInfo, Sampler));
	}
	else
	{
		PrintResultAndNamedHandle(Result, TEXT("Sampler"), *Sampler);
	}
	FlushDebugWrapperLog();
#endif
}


void FWrapLayer::CreateDevice(VkResult Result, VkPhysicalDevice PhysicalDevice, const VkDeviceCreateInfo* CreateInfo, VkDevice* Device)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		PrintfBeginResult(FString::Printf(TEXT("vkCreateDevice(PhysicalDevice=0x%p, CreateInfo=0x%p, OutDevice=0x%p)[...]"), PhysicalDevice, CreateInfo, Device));
	}
	else
	{
		PrintResultAndNamedHandle(Result, TEXT("Device"), *Device);
	}
	FlushDebugWrapperLog();
#endif
}

void FWrapLayer::GetPhysicalDeviceFeatures(VkResult Result, VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceFeatures* Features)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetPhysicalDeviceFeatures(PhysicalDevice=0x%p, Features=0x%p)[...]"), PhysicalDevice, Features));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DebugLog += FString::Printf(TEXT("VkPhysicalDeviceFeatures [...]\n"));
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceFormatProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, VkFormat Format, VkFormatProperties* FormatProperties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBegin(FString::Printf(TEXT("vkGetPhysicalDeviceFormatProperties(PhysicalDevice=0x%p, Format=%d, FormatProperties=0x%p)[...]"), PhysicalDevice, (int32)Format, FormatProperties));
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceFormatProperties2(VkResult Result, VkPhysicalDevice PhysicalDevice, VkFormat Format, VkFormatProperties2* FormatProperties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBegin(FString::Printf(TEXT("vkGetPhysicalDeviceFormatProperties2(PhysicalDevice=0x%p, Format=%d, FormatProperties2=0x%p)[...]"), PhysicalDevice, (int32)Format, FormatProperties));
#endif
	}
}

#if VULKAN_ENABLE_DUMP_LAYER
static void DumpImageFormatProperties(const VkImageFormatProperties* pProps)
{
	if (pProps)
	{
		DebugLog += FString::Printf(
			TEXT(" -> MaxExtent={ %u, %u, %u } MaxMipLevels=%u MaxArrayLayers=%u SampleCounts=0x%X MaxResourceSize=%llu\n"),
			pProps->maxExtent.width, pProps->maxExtent.height, pProps->maxMipLevels, pProps->maxArrayLayers, pProps->sampleCounts, pProps->maxResourceSize
		);
	}
	else
	{
		DumpErrorInvalidPointer(TEXT("VkImageFormatProperties"));
	}
	FlushDebugWrapperLog();
}
#endif

void FWrapLayer::GetPhysicalDeviceImageFormatProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, VkFormat Format, VkImageType Type, VkImageTiling Tiling, VkImageUsageFlags Usage, VkImageCreateFlags Flags, VkImageFormatProperties* pImageFormatProperties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(
			TEXT("vkGetPhysicalDeviceImageFormatProperties(PhysicalDevice=0x%p, Format=%s, Type=%d, Tiling=%s, Usage=%d, Flags=%d, pImageFormatProperties=0x%p)[...]"),
			PhysicalDevice, *GetVkFormatString(Format), (int32)Type, *GetVkImageTilingString(Tiling), (int32)Usage, (int32)Flags, pImageFormatProperties
		));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DumpImageFormatProperties(pImageFormatProperties);
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceImageFormatProperties2(VkResult Result, VkPhysicalDevice PhysicalDevice, const VkPhysicalDeviceImageFormatInfo2* pImageFormatInfo , VkImageFormatProperties2* pImageFormatProperties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(
			TEXT("vkGetPhysicalDeviceImageFormatProperties(PhysicalDevice=0x%p, pImageFormatInfo=0x%p, pImageFormatProperties=0x%p)[...]"),
			PhysicalDevice, pImageFormatInfo, pImageFormatProperties
		));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DumpImageFormatProperties(pImageFormatProperties->imageFormatProperties);
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceProperties* FormatProperties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBegin(FString::Printf(TEXT("vkGetPhysicalDeviceProperties(PhysicalDevice=0x%p, Properties=0x%p)[...]"), PhysicalDevice, FormatProperties));
#endif
	}
}

void FWrapLayer::BeginCommandBuffer(VkResult Result, VkCommandBuffer CommandBuffer, const VkCommandBufferBeginInfo* BeginInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkBeginCommandBuffer(CmdBuffer=0x%p, Info=0x%p)[...]"), CommandBuffer, BeginInfo));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::EndCommandBuffer(VkResult Result, VkCommandBuffer CommandBuffer)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBeginResult(CommandBuffer, FString::Printf(TEXT("vkEndCommandBuffer(Cmd=0x%p)")));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::CmdResetQueryPool(VkResult Result, VkCommandBuffer CommandBuffer, VkQueryPool QueryPool, uint32 FirstQuery, uint32 QueryCount)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdResetQueryPool(QueryPool=0x%p, FirstQuery=%d, NumQueries=%d)"), QueryPool, FirstQuery, QueryCount));
#endif
	}
}

void FWrapLayer::CmdWriteTimestamp(VkResult Result, VkCommandBuffer CommandBuffer, VkPipelineStageFlagBits PipelineStage, VkQueryPool QueryPool, uint32 Query)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdWriteTimestamp(PipelineStage=%d, QueryPool=0x%p, Query=%d)"), (int32)PipelineStage, QueryPool, Query));
#endif
	}
}

void FWrapLayer::CmdBindPipeline(VkResult Result, VkCommandBuffer CommandBuffer, VkPipelineBindPoint PipelineBindPoint, VkPipeline Pipeline)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdBindPipeline(BindPoint=%d, Pipeline=0x%p)[...]"), (int32)PipelineBindPoint, Pipeline));
#endif
	}
}

void FWrapLayer::CmdPushConstants(VkResult Result, VkCommandBuffer CommandBuffer, VkPipelineLayout Layout, VkShaderStageFlags StageFlags, uint32_t Offset, uint32_t Size, const void* pValues)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdPushConstants(Layout=0x%p, StageFlags=0x%08X, Offset=%u, Size=%u, pValues=0x%p)"), Layout, StageFlags, Offset, Size, pValues));
#endif
	}
}

void FWrapLayer::CmdPushDescriptorSetKHR(VkResult Result, VkCommandBuffer CommandBuffer, VkPipelineBindPoint PipelineBindPoint, VkPipelineLayout Layout, uint32_t Set, uint32_t DescriptorWriteCount, const VkWriteDescriptorSet* DescriptorWrites)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdPushDescriptorSetKHR()")));
#endif
	}
}

void FWrapLayer::CmdPushDescriptorSet2KHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkPushDescriptorSetInfo* PushDescriptorSetInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdPushDescriptorSet2KHR()")));
#endif
	}
}

void FWrapLayer::CmdBeginRenderPass(VkResult Result, VkCommandBuffer CommandBuffer, const VkRenderPassBeginInfo* RenderPassBegin, VkSubpassContents Contents)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		auto GetSubpassContents = [](VkSubpassContents InContents) -> FString
			{
				switch (InContents)
				{
					case VK_SUBPASS_CONTENTS_INLINE: return TEXT("INLINE");
					case VK_SUBPASS_CONTENTS_SECONDARY_COMMAND_BUFFERS: return TEXT("SECONDARY_CMD_BUFS");
					default: return FString::Printf(TEXT("%d"), (int32)InContents);
				}
			};
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("----- vkCmdBeginRenderPass(BeginInfo=0x%p, Contents=%s)"), RenderPassBegin, *GetSubpassContents(Contents)));
		DebugLog += FString::Printf(TEXT("%sBeginInfo: RenderPass=0x%p, Framebuffer=0x%p, renderArea=(x:%d, y:%d, %s), clearValues=%d\n"),
			Tabs, RenderPassBegin->renderPass, RenderPassBegin->framebuffer,
			RenderPassBegin->renderArea.offset.x, RenderPassBegin->renderArea.offset.y,
			*GetExtentString(RenderPassBegin->renderArea.extent),
			RenderPassBegin->clearValueCount);
		for (uint32 Index = 0; Index < RenderPassBegin->clearValueCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%s\tclearValue[%d]=(%s)\n"), Tabs, Index, *GetClearValueString(RenderPassBegin->pClearValues[Index]));
		}

		FRenderPassInfo* FoundRPInfo = GRenderPassInfo.Find(RenderPassBegin->renderPass);
		ensure(FoundRPInfo);
		if (FoundRPInfo)
		{
			FFBInfo* FoundFBInfo = GFramebufferInfo.Find(RenderPassBegin->framebuffer);
			ensure(FoundFBInfo);
			if (FoundFBInfo)
			{
				for (uint32 Index = 0; Index < FoundFBInfo->Info.attachmentCount; ++Index)
				{
					VkImageView View = FoundFBInfo->Attachments[Index];
					auto* FoundImageInfo = GVulkanTrackingImageViews.Find(View);
					// Can be null for swapchain images!
					if (FoundImageInfo)
					{
						DebugLog += FString::Printf(TEXT("%s\t\tAttachment[%d]: ImageView=0x%p(I:0x%p)\n"), Tabs, Index, View, FoundImageInfo->CreateInfo.image);
					}
				}
			}
		}

		FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::CmdBeginRenderPass2KHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkRenderPassBeginInfo* RenderPassBegin, const VkSubpassBeginInfo* SubpassBeginInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		auto GetSubpassContents = [](VkSubpassContents InContents) -> FString
		{
			switch (InContents)
			{
			case VK_SUBPASS_CONTENTS_INLINE: return TEXT("INLINE");
			case VK_SUBPASS_CONTENTS_SECONDARY_COMMAND_BUFFERS: return TEXT("SECONDARY_CMD_BUFS");
			default: return FString::Printf(TEXT("%d"), (int32)InContents);
			}
		};
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("----- vkCmdBeginRenderPass2KHR(BeginInfo=0x%p, Contents=%s)"), RenderPassBegin, *GetSubpassContents(SubpassBeginInfo->contents)));
		DebugLog += FString::Printf(TEXT("%sBeginInfo: RenderPass=0x%p, Framebuffer=0x%p, renderArea=(x:%d, y:%d, %s), clearValues=%d\n"),
									Tabs, RenderPassBegin->renderPass, RenderPassBegin->framebuffer,
									RenderPassBegin->renderArea.offset.x, RenderPassBegin->renderArea.offset.y,
									*GetExtentString(RenderPassBegin->renderArea.extent),
									RenderPassBegin->clearValueCount);
		for (uint32 Index = 0; Index < RenderPassBegin->clearValueCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%s\tclearValue[%d]=(%s)\n"), Tabs, Index, *GetClearValueString(RenderPassBegin->pClearValues[Index]));
		}

		FRenderPassInfo* FoundRPInfo = GRenderPassInfo.Find(RenderPassBegin->renderPass);
		ensure(FoundRPInfo);
		if (FoundRPInfo)
		{
			FFBInfo* FoundFBInfo = GFramebufferInfo.Find(RenderPassBegin->framebuffer);
			ensure(FoundFBInfo);
			if (FoundFBInfo)
			{
				for (uint32 Index = 0; Index < FoundFBInfo->Info.attachmentCount; ++Index)
				{
					VkImageView View = FoundFBInfo->Attachments[Index];
					auto* FoundImageInfo = GVulkanTrackingImageViews.Find(View);
					// Can be null for swapchain images!
					if (FoundImageInfo)
					{
						DebugLog += FString::Printf(TEXT("%s\t\tAttachment[%d]: ImageView=0x%p(I:0x%p)\n"), Tabs, Index, View, FoundImageInfo->CreateInfo.image);
					}
				}
			}
		}

		FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::CmdEndRenderPass(VkResult Result, VkCommandBuffer CommandBuffer)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, TEXT("----- vkCmdEndRenderPass()"));
#endif
	}
}

void FWrapLayer::CmdEndRenderPass2KHR(VkResult Result, VkCommandBuffer CommandBuffer)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, TEXT("----- vkCmdEndRenderPass2KHR()"));
#endif
	}
}

void FWrapLayer::CmdNextSubpass(VkResult Result, VkCommandBuffer CommandBuffer, VkSubpassContents Contents)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("----- vkNextSubpass(Contents=0x%d)"), (uint32)Contents));
#endif
	}
}

void FWrapLayer::CmdBeginRenderingKHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkRenderingInfo* RenderingInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, TEXT("----- vkCmdBeginRenderingKHR()"));
#endif
	}
}

void FWrapLayer::CmdEndRenderingKHR(VkResult Result, VkCommandBuffer CommandBuffer)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, TEXT("----- vkCmdEndRenderingKHR()"));
#endif
	}
}

void FWrapLayer::CmdEndRendering2KHR(VkResult Result, VkCommandBuffer CommandBuffer)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, TEXT("----- vkCmdEndRendering2KHR()"));
#endif
	}
}

void FWrapLayer::CmdExecuteCommands(VkResult Result, VkCommandBuffer CommandBuffer, uint32 CommandBufferCount, const VkCommandBuffer* pCommandBuffers)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdExecuteCommands(CommandBufferCount=%u, pCommandBuffers=0x%p)"), CommandBufferCount, pCommandBuffers));
#endif
	}
}

void FWrapLayer::CmdBeginQuery(VkResult Result, VkCommandBuffer CommandBuffer, VkQueryPool QueryPool, uint32 Query, VkQueryControlFlags Flags)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdBeginQuery(QueryPool=0x%p, Query=%d Flags=%d)"), QueryPool, Query, Flags));
#endif
	}
}
void FWrapLayer::CmdEndQuery(VkResult Result, VkCommandBuffer CommandBuffer, VkQueryPool QueryPool, uint32 Query)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdEndQuery(QueryPool=0x%p, Query=%d)"), QueryPool, Query));
#endif
	}
}


void FWrapLayer::CmdBindVertexBuffers(VkResult Result, VkCommandBuffer CommandBuffer, uint32 FirstBinding, uint32 BindingCount, const VkBuffer* Buffers, const VkDeviceSize* Offsets)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdBindVertexBuffers(FirstBinding=%d, NumBindings=%d, Buffers=0x%p, Offsets=0x%p)[...]"), FirstBinding, BindingCount, Buffers, Offsets));
		FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::CmdBindIndexBuffer(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer IndexBuffer, VkDeviceSize Offset, VkIndexType IndexType)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdBindIndexBuffer(Buffer=0x%p, Offset=%d, IndexType=%d)"), IndexBuffer, (int32)Offset, (int32)IndexType));
		FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::CmdSetViewport(VkResult Result, VkCommandBuffer CommandBuffer, uint32 FirstViewport, uint32 ViewportCount, const VkViewport* Viewports)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdSetViewport(FirstViewport=%d, ViewportCount=%d, Viewports=0x%p)[...]"), FirstViewport, ViewportCount, Viewports));
#endif
	}
}

void FWrapLayer::CmdSetScissor(VkResult Result, VkCommandBuffer CommandBuffer, uint32 FirstScissor, uint32 ScissorCount, const VkRect2D* Scissors)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdSetScissor(FirstScissor=%d, ScissorCount=%d, Scissors=0x%p)[...]"), FirstScissor, ScissorCount, Scissors));
#endif
	}
}

void FWrapLayer::CmdSetLineWidth(VkResult Result, VkCommandBuffer CommandBuffer, float LineWidth)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdSetLineWidth(Width=%f)"), LineWidth));
#endif
	}
}

void FWrapLayer::CmdDraw(VkResult Result, VkCommandBuffer CommandBuffer, uint32 VertexCount, uint32 InstanceCount, uint32 FirstVertex, uint32 FirstInstance)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdDraw(NumVertices=%d, NumInstances=%d, FirstVertex=%d, FirstInstance=%d)"), VertexCount, InstanceCount, FirstVertex, FirstInstance));
#endif
	}
}

void FWrapLayer::CmdDrawIndexed(VkResult Result, VkCommandBuffer CommandBuffer, uint32 IndexCount, uint32 InstanceCount, uint32 FirstIndex, int32_t VertexOffset, uint32 FirstInstance)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdDrawIndexed(IndexCount=%d, NumInstances=%d, FirstIndex=%d, VertexOffset=%d, FirstInstance=%d)"), IndexCount, InstanceCount, FirstIndex, VertexOffset, FirstInstance));
#endif
	}
}

void FWrapLayer::CmdDrawIndirect(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer Buffer, VkDeviceSize Offset, uint32 DrawCount, uint32 Stride)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdDrawIndirect(Buffer=0x%p, Offset=%d, DrawCount=%d, Stride=%d)"), (void*)Buffer, Offset, DrawCount, Stride));
#endif
	}
}

void FWrapLayer::CmdDrawIndexedIndirect(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer Buffer, VkDeviceSize Offset, uint32 DrawCount, uint32 Stride)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdDrawIndexedIndirect(Buffer=0x%p, Offset=%d, DrawCount=%d, Stride=%d)"), (void*)Buffer, Offset, DrawCount, Stride));
#endif
	}
}

void FWrapLayer::CmdDispatch(VkResult Result, VkCommandBuffer CommandBuffer, uint32 X, uint32 Y, uint32 Z)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdDispatch(X=%d, Y=%d Z=%d)"), X, Y, Z));
#endif
	}
}

void FWrapLayer::CmdDispatchIndirect(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer Buffer, VkDeviceSize Offset)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdDispatchIndirect(Buffer=0x%p, Offset=%d)"), (void*)Buffer, Offset));
#endif
	}
}

void FWrapLayer::CmdCopyImage(VkResult Result, VkCommandBuffer CommandBuffer, VkImage SrcImage, VkImageLayout SrcImageLayout, VkImage DstImage, VkImageLayout DstImageLayout, uint32 RegionCount, const VkImageCopy* Regions)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdCopyImage(SrcImage=0x%p, SrcImageLayout=%d, DstImage=0x%p, DstImageLayout=%d, RegionCount=%d, Regions=0x%p)[...]"), SrcImage, (int32)SrcImageLayout, DstImage, (int32)DstImageLayout, RegionCount, Regions));
#endif
#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER
		BreakOnTrackingImage(SrcImage);
		BreakOnTrackingImage(DstImage);
		FScopeLock ScopeLock(&GTrackingCS);
		FTrackingImage* FoundSrc = GVulkanTrackingImageLayouts.Find(SrcImage);
		FTrackingImage* FoundDest = GVulkanTrackingImageLayouts.Find(DstImage);
		ensure(FoundSrc && FoundDest);
		if (FoundSrc && FoundDest)
		{
			for (uint32 Index = 0; Index < RegionCount; ++Index)
			{
				ensure(Regions[Index].srcSubresource.layerCount == Regions[Index].dstSubresource.layerCount);
				for (uint32 LIndex = 0; LIndex < Regions[Index].srcSubresource.layerCount; ++LIndex)
				{
					ensure(FoundSrc->ArrayLayouts[Regions[Index].srcSubresource.baseArrayLayer + LIndex][Regions[Index].srcSubresource.mipLevel] == SrcImageLayout);
					ensure(FoundDest->ArrayLayouts[Regions[Index].dstSubresource.baseArrayLayer + LIndex][Regions[Index].dstSubresource.mipLevel] == DstImageLayout);
				}
			}
		}
#endif
	}
}



void FWrapLayer::CmdCopyBufferToImage(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer SrcBuffer, VkImage DstImage, VkImageLayout DstImageLayout, uint32 RegionCount, const VkBufferImageCopy* Regions)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdCopyBufferToImage(SrcBuffer=0x%p, DstImage=0x%p, DstImageLayout=%s, NumRegions=%d, Regions=0x%p)"),
			SrcBuffer, DstImage, *GetVkImageLayoutString(DstImageLayout), RegionCount, Regions));
		for (uint32 Index = 0; Index < RegionCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%sRegion[%d]: %s\n"), Tabs, Index, *GetBufferImageCopyString(Regions[Index]));
		}
		FlushDebugWrapperLog();
#endif
#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER
		BreakOnTrackingImage(DstImage);
		FScopeLock ScopeLock(&GTrackingCS);
		FTrackingImage* FoundDest = GVulkanTrackingImageLayouts.Find(DstImage);
		ensure(FoundDest);
		if (FoundDest)
		{
			for (uint32 Index = 0; Index < RegionCount; ++Index)
			{
				for (uint32 LIndex = 0; LIndex < Regions[Index].imageSubresource.layerCount; ++LIndex)
				{
					ensure(FoundDest->ArrayLayouts[Regions[Index].imageSubresource.baseArrayLayer + LIndex][Regions[Index].imageSubresource.mipLevel] == DstImageLayout);
				}
			}
		}
#endif
	}
}

void FWrapLayer::CmdCopyImageToBuffer(VkResult Result, VkCommandBuffer CommandBuffer, VkImage SrcImage, VkImageLayout SrcImageLayout, VkBuffer DstBuffer, uint32 RegionCount, const VkBufferImageCopy* Regions)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdCopyImageToBuffer(SrcImage=0x%p, SrcImageLayout=%s, SrcBuffer=0x%p, NumRegions=%d, Regions=0x%p)"),
			SrcImage, *GetVkImageLayoutString(SrcImageLayout), DstBuffer, RegionCount, Regions));
		for (uint32 Index = 0; Index < RegionCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%sRegion[%d]: %s\n"), Tabs, Index, *GetBufferImageCopyString(Regions[Index]));
		}
		FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::CmdCopyBuffer(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer SrcBuffer, VkBuffer DstBuffer, uint32 RegionCount, const VkBufferCopy* Regions)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdCopyBuffer(SrcBuffer=0x%p, DstBuffer=0x%p, NumRegions=%d, Regions=0x%p)"), SrcBuffer, DstBuffer, RegionCount, Regions));
		for (uint32 Index = 0; Index < RegionCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%sRegion[%d]: SrcOffset=%d DestOffset=%d Size=%d\n"), Tabs, Index,
				(int32)Regions[Index].srcOffset, (int32)Regions[Index].dstOffset, (int32)Regions[Index].size);
		}
		FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::CmdBlitImage(VkResult Result, VkCommandBuffer CommandBuffer, VkImage SrcImage, VkImageLayout SrcImageLayout, VkImage DstImage, VkImageLayout DstImageLayout, uint32 RegionCount, const VkImageBlit* Regions, VkFilter Filter)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdBlitImage(SrcImage=0x%p, SrcImageLayout=%d, DstImage=0x%p, DstImageLayout=%d, RegionCount=%d, Regions=0x%p, Filter=%d)[...]"), SrcImage, (int32)SrcImageLayout, DstImage, (int32)DstImageLayout, RegionCount, Regions, (int32)Filter));

		FlushDebugWrapperLog();
#endif
#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER
		BreakOnTrackingImage(SrcImage);
		BreakOnTrackingImage(DstImage);
		FScopeLock ScopeLock(&GTrackingCS);
		FTrackingImage* FoundSrc = GVulkanTrackingImageLayouts.Find(SrcImage);
		FTrackingImage* FoundDest = GVulkanTrackingImageLayouts.Find(DstImage);
		ensure(FoundSrc && FoundDest);
		if (FoundSrc && FoundDest)
		{
			for (uint32 Index = 0; Index < RegionCount; ++Index)
			{
				ensure(Regions[Index].srcSubresource.layerCount == Regions[Index].dstSubresource.layerCount);
				for (uint32 LIndex = 0; LIndex < Regions[Index].srcSubresource.layerCount; ++LIndex)
				{
					ensure(FoundSrc->ArrayLayouts[Regions[Index].srcSubresource.baseArrayLayer + LIndex][Regions[Index].srcSubresource.mipLevel] == SrcImageLayout);
					ensure(FoundDest->ArrayLayouts[Regions[Index].dstSubresource.baseArrayLayer + LIndex][Regions[Index].dstSubresource.mipLevel] == DstImageLayout);
				}
			}
		}
#endif
	}
}

#if VULKAN_ENABLE_DUMP_LAYER
static void DumpImageSubresourceLayout(VkSubresourceLayout* pLayout)
{
	if (pLayout)
	{
		DebugLog += FString::Printf(TEXT("VkSubresourceLayout: [...]\n"));
	}
	else
	{
		DumpErrorInvalidPointer(TEXT("VkSubresourceLayout"));
	}
	FlushDebugWrapperLog();
}
#endif

void FWrapLayer::GetImageSubresourceLayout(VkResult Result, VkDevice Device, VkImage Image, const VkImageSubresource* Subresource, VkSubresourceLayout* Layout)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkGetImageSubresourceLayout(Image=0x%p, Subresource=0x%p, OutLayout=0x%p)"), Image, Subresource, Layout));
		FlushDebugWrapperLog();
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DumpImageSubresourceLayout(Layout);
#endif
	}
}

void FWrapLayer::GetImageSubresourceLayout2(VkResult Result, VkDevice Device, VkImage Image, const VkImageSubresource2KHR* Subresource, VkSubresourceLayout2KHR* Layout)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkGetImageSubresourceLayout2(Image=0x%p, Subresource=0x%p, OutLayout=0x%p)"), (void*)Image, Subresource, Layout));
		FlushDebugWrapperLog();
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DumpImageSubresourceLayout(Layout->layout);
#endif
	}
}

void FWrapLayer::GetSwapChainImagesKHR(VkResult Result, VkDevice Device, VkSwapchainKHR Swapchain, uint32_t* SwapchainImageCount, VkImage* SwapchainImages)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkGetSwapchainImagesKHR(Swapchain=0x%p, OutSwapchainImageCount=0x%p, OutSwapchainImages=0x%p)\n"), Swapchain, SwapchainImageCount, SwapchainImages));
		FlushDebugWrapperLog();
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
		if (SwapchainImages)
		{
			for (uint32 Index = 0; Index < *SwapchainImageCount; ++Index)
			{
				DebugLog += FString::Printf(TEXT("%sImage[%d]=0x%p\n"), Tabs, Index, SwapchainImages[Index]);
			}
		}
		else
		{
			DebugLog += FString::Printf(TEXT("%sNumImages=%d\n"), Tabs, *SwapchainImageCount);
		}
#endif
#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER
		if (SwapchainImages)
		{
			FScopeLock ScopeLock(&GTrackingCS);
			for (uint32 Index = 0; Index < *SwapchainImageCount; ++Index)
			{
				BreakOnTrackingImage(SwapchainImages[Index]);
				FTrackingImage& TrackingImage = GVulkanTrackingImageLayouts.FindOrAdd(SwapchainImages[Index]);
				TrackingImage.Setup(1, 1, VK_IMAGE_LAYOUT_UNDEFINED, true);
			}
		}
#endif
	}
}

void FWrapLayer::CmdClearAttachments(VkResult Result, VkCommandBuffer CommandBuffer, uint32 AttachmentCount, const VkClearAttachment* Attachments, uint32 RectCount, const VkClearRect* Rects)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdClearAttachments(AttachmentCount=%d, Attachments=0x%p, RectCount=%d, Rects=0x%p)"), AttachmentCount, Attachments, RectCount, Rects));
		for (uint32 Index = 0; Index < AttachmentCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%sAttachment[%d]= aspect=%s ColorAtt=%d ClearValue=%s\n"), Tabs, Index,
				*GetAspectMaskString(Attachments[Index].aspectMask), Attachments[Index].colorAttachment, *GetClearValueString(Attachments[Index].clearValue));
		}

		for (uint32 Index = 0; Index < RectCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%sRects[%d]= Rect=(%s) BaseArrayLayer=%d NumLayers=%d\n"), Tabs, Index, *GetRectString(Rects[Index].rect), Rects[Index].baseArrayLayer, Rects[Index].layerCount);
		}

		FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::CmdClearColorImage(VkResult Result, VkCommandBuffer CommandBuffer, VkImage Image, VkImageLayout ImageLayout, const VkClearColorValue* Color, uint32 RangeCount, const VkImageSubresourceRange* Ranges)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdClearColorImage(Image=0x%p, ImageLayout=%s, Color=%s, RangeCount=%d, Ranges=0x%p)"), Image, *GetVkImageLayoutString(ImageLayout), *GetClearColorValueString(*Color), RangeCount, Ranges));
		for (uint32 Index = 0; Index < RangeCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%sRange[%d]= %s\n"), Tabs, Index, *GetImageSubResourceRangeString(Ranges[Index]));
		}
		FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::CmdClearDepthStencilImage(VkResult Result, VkCommandBuffer CommandBuffer, VkImage Image, VkImageLayout ImageLayout, const VkClearDepthStencilValue* DepthStencil, uint32 RangeCount, const VkImageSubresourceRange* Ranges)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdClearDepthStencilImage(Image=0x%p, ImageLayout=%s, DepthStencil=%s, RangeCount=%d, Ranges=0x%p)"), Image, *GetVkImageLayoutString(ImageLayout), *GetClearDepthStencilValueString(*DepthStencil), RangeCount, Ranges));
		for (uint32 Index = 0; Index < RangeCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%sRange[%d]= %s\n"), Tabs, Index, *GetImageSubResourceRangeString(Ranges[Index]));
		}
		FlushDebugWrapperLog();
#endif
	}
}

void FWrapLayer::QueuePresent(VkResult Result, VkQueue Queue, const VkPresentInfoKHR* PresentInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkQueuePresentKHR(Queue=0x%p, Info=0x%p)[...]"), Queue, PresentInfo));

		DebugLog += FString::Printf(TEXT("\n%sPresentInfo: Results=0x%p"), Tabs, PresentInfo->pResults);
		if (PresentInfo->waitSemaphoreCount > 0)
		{
			DebugLog += FString::Printf(TEXT("\n%s\tWaitSemaphores: "), Tabs);
			for (uint32 SubIndex = 0; SubIndex < PresentInfo->waitSemaphoreCount; ++SubIndex)
			{
				DebugLog += FString::Printf(TEXT("0x%p "), PresentInfo->pWaitSemaphores[SubIndex]);
			}
		}
		if (PresentInfo->swapchainCount > 0)
		{
			DebugLog += FString::Printf(TEXT("\n%s\tSwapchains (ImageIndex): "), Tabs);
			for (uint32 SubIndex = 0; SubIndex < PresentInfo->swapchainCount; ++SubIndex)
			{
				DebugLog += FString::Printf(TEXT("0x%p(%d)"), PresentInfo->pSwapchains[SubIndex], PresentInfo->pImageIndices[SubIndex]);
			}
		}
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::CreateGraphicsPipelines(VkResult Result, VkDevice Device, VkPipelineCache PipelineCache, uint32 CreateInfoCount, const VkGraphicsPipelineCreateInfo* CreateInfos, VkPipeline* Pipelines)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkCreateGraphicsPipelines(PipelineCache=0x%p, CreateInfoCount=%d, CreateInfos=0x%p, OutPipelines=0x%p)[...]"), PipelineCache, CreateInfoCount, CreateInfos, Pipelines));
		for (uint32 Index = 0; Index < CreateInfoCount; ++Index)
		{
			const VkGraphicsPipelineCreateInfo& CreateInfo = CreateInfos[Index];
			DebugLog += FString::Printf(TEXT("%s%d: Flags=%d Stages=%d Layout=0x%p RenderPass=0x%p Subpass=%d\n"), Tabs, Index,
				CreateInfo.flags, CreateInfo.stageCount, (void*)CreateInfo.layout, (void*)CreateInfo.renderPass, CreateInfo.subpass);
			DebugLog += FString::Printf(TEXT("%s\tDepth Test %d Write %d %s Bounds %d (min %f max %f) Stencil %d\n"), Tabs, CreateInfo.pDepthStencilState->depthTestEnable, CreateInfo.pDepthStencilState->depthWriteEnable, *GetVkCompareOpString(CreateInfo.pDepthStencilState->depthCompareOp), CreateInfo.pDepthStencilState->depthBoundsTestEnable, CreateInfo.pDepthStencilState->minDepthBounds, CreateInfo.pDepthStencilState->maxDepthBounds, CreateInfo.pDepthStencilState->stencilTestEnable);

			auto PrintStencilOp = [](const VkStencilOpState& State)
			{
				return FString::Printf(TEXT("Fail %s Pass %s DepthFail %s Compare %s CompareMask 0x%x WriteMask 0x%x Ref 0x%0x"), *GetVkStencilOpString(State.failOp), *GetVkStencilOpString(State.passOp), *GetVkStencilOpString(State.depthFailOp), *GetVkCompareOpString(State.compareOp), State.compareMask, State.writeMask, State.reference);
			};

			DebugLog += FString::Printf(TEXT("%s\t\tFront: %s\n"), Tabs, *PrintStencilOp(CreateInfo.pDepthStencilState->front));
			DebugLog += FString::Printf(TEXT("%s\t\tBack: %s\n"), Tabs, *PrintStencilOp(CreateInfo.pDepthStencilState->back));
/*
			DebugLog += FString::Printf(TEXT(""));
			typedef struct VkGraphicsPipelineCreateInfo {
				const VkPipelineShaderStageCreateInfo*           pStages;
				const VkPipelineVertexInputStateCreateInfo*      pVertexInputState;
				const VkPipelineInputAssemblyStateCreateInfo*    pInputAssemblyState;
				const VkPipelineTessellationStateCreateInfo*     pTessellationState;
				const VkPipelineViewportStateCreateInfo*         pViewportState;
				const VkPipelineRasterizationStateCreateInfo*    pRasterizationState;
				const VkPipelineMultisampleStateCreateInfo*      pMultisampleState;
				const VkPipelineColorBlendStateCreateInfo*       pColorBlendState;
				const VkPipelineDynamicStateCreateInfo*          pDynamicState;
				VkPipeline                                       basePipelineHandle;
				int32_t                                          basePipelineIndex;
			} VkGraphicsPipelineCreateInfo;
*/
		}
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		//#todo-rco: Multiple pipelines!
		PrintResultAndNamedHandle(Result, TEXT("Pipeline"), Pipelines[0]);
#endif
	}
}

void FWrapLayer::GetDeviceQueue(VkResult Result, VkDevice Device, uint32 QueueFamilyIndex, uint32 QueueIndex, VkQueue* Queue)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkGetDeviceQueue(QueueFamilyIndex=%d, QueueIndex=%d, OutQueue=0x%p)\n"), QueueFamilyIndex, QueueIndex, Queue));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(VK_SUCCESS, TEXT("Queue"), *Queue);
#endif
	}
}

void FWrapLayer::DeviceWaitIdle(VkResult Result, VkDevice Device)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkDeviceWaitIdle()")));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::MapMemory(VkResult Result, VkDevice Device, VkDeviceMemory Memory, VkDeviceSize Offset, VkDeviceSize Size, VkMemoryMapFlags Flags, void** Data)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkMapMemory(DevMem=0x%p, Off=%d, Size=%d, Flags=0x%x, OutData=0x%p)\n"), Memory, (uint32)Offset, (uint32)Size, Flags, Data));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndPointer(Result, *Data);
#endif
	}
}

void FWrapLayer::UnmapMemory(VkResult Result, VkDevice Device, VkDeviceMemory Memory)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkUnmapMemory(DevMem=0x%p)\n"), Memory));
#endif
	}
	else
	{
	}
}

#if VULKAN_ENABLE_DUMP_LAYER
static void DumpCommittedMemoryInBytes(VkDeviceSize* pCommittedMemoryInBytes)
{
	if (pCommittedMemoryInBytes)
	{
		DebugLog += FString::Printf(TEXT(" -> Size=%llu\n"), *pCommittedMemoryInBytes);
	}
	else
	{
		DumpErrorInvalidPointer(TEXT("VkDeviceSize"));
	}
	FlushDebugWrapperLog();
}
#endif

void FWrapLayer::GetDeviceMemoryCommitment(VkResult Result, VkDevice Device, VkDeviceMemory Memory, VkDeviceSize* pCommittedMemoryInBytes)
{
#if VULKAN_ENABLE_DUMP_LAYER
	if (Result == VK_RESULT_MAX_ENUM)
	{
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkGetDeviceMemoryCommitment(Memory=0x%p, pCommittedMemoryInBytes=0x%p)"), Memory, pCommittedMemoryInBytes));
	}
	else
	{
		DumpCommittedMemoryInBytes(pCommittedMemoryInBytes);
	}
#endif
}

void FWrapLayer::BindBufferMemory(VkResult Result, VkDevice Device, VkBuffer Buffer, VkDeviceMemory Memory, VkDeviceSize MemoryOffset)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkBindBufferMemory(Buffer=0x%p, Memory=0x%p, MemoryOffset=%llu)\n"), Buffer, Memory, MemoryOffset));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::BindImageMemory(VkResult Result, VkDevice Device, VkImage Image, VkDeviceMemory Memory, VkDeviceSize MemoryOffset)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkBindImageMemory(Image=0x%p, DevMem=0x%p, MemOff=%d)\n"), Image, Memory, (uint32)MemoryOffset));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::GetFenceStatus(VkResult Result, VkDevice Device, VkFence Fence)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkGetFenceStatus(Fence=0x%p)"), Fence));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::GetQueryPoolResults(VkResult Result, VkDevice Device, VkQueryPool QueryPool, uint32 FirstQuery, uint32 QueryCount, size_t DataSize, void* Data, VkDeviceSize Stride, VkQueryResultFlags Flags)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkGetQueryPoolResults(QueryPool=0x%p, FirstQuery=%d, QueryCount=%d, DataSize=%d, Data=0x%p, Stride=%d, Flags=%d)[...]"), QueryPool, FirstQuery, QueryCount, (int32)DataSize, Data, Stride, Flags));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::CreateComputePipelines(VkResult Result, VkDevice Device, VkPipelineCache PipelineCache, uint32 CreateInfoCount, const VkComputePipelineCreateInfo* CreateInfos, VkPipeline* Pipelines)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkCreateComputePipelines(PipelineCache=0x%p, CreateInfoCount=%d, CreateInfos=0x%p, OutPipelines=0x%p)[...]\n"), PipelineCache, CreateInfoCount, CreateInfos, Pipelines));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		//#todo-rco: Multiple pipelines!
		PrintResultAndNamedHandle(Result, TEXT("Pipeline"), Pipelines[0]);
#endif
	}
}

void FWrapLayer::ResetCommandPool(VkResult Result, VkDevice Device, VkCommandPool CommandPool, VkCommandPoolResetFlags Flags)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkResetCommandPool(CommandPool=0x%p, Flags=0x%08X)"), CommandPool, Flags));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::TrimCommandPool(VkResult Result, VkDevice Device, VkCommandPool CommandPool, VkCommandPoolTrimFlags Flags)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkTrimCommandPool(CommandPool=0x%p, Flags=0x%08X)"), CommandPool, Flags));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::AllocateCommandBuffers(VkResult Result, VkDevice Device, const VkCommandBufferAllocateInfo* AllocateInfo, VkCommandBuffer* CommandBuffers)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkAllocateCommandBuffers(AllocateInfo=0x%p, OutCommandBuffers=0x%p)[...]\n"), AllocateInfo, CommandBuffers));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("CommandBuffers"), *CommandBuffers);
#endif
	}
}

void FWrapLayer::CreateSwapchainKHR(VkResult Result, VkDevice Device, const VkSwapchainCreateInfoKHR* CreateInfo, VkSwapchainKHR* Swapchain)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkCreateSwapchainKHR(SwapChainInfo=0x%p, OutSwapChain=0x%p)[...]"), CreateInfo, Swapchain));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("SwapChain"), *Swapchain);
#endif
	}
}

void FWrapLayer::AcquireNextImageKHR(VkResult Result, VkDevice Device, VkSwapchainKHR Swapchain, uint64_t Timeout, VkSemaphore Semaphore, VkFence Fence, uint32_t* ImageIndex)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkAcquireNextImageKHR(Swapchain=0x%p, Timeout=0x%llx, Semaphore=0x%p, Fence=0x%p, OutImageIndex=0x%p)[...]\n"), Swapchain, Timeout, Semaphore, Fence, ImageIndex));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("ImageIndex"), *ImageIndex);
#endif
	}
}

void FWrapLayer::FreeMemory(VkResult Result, VkDevice Device, VkDeviceMemory Memory)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkFreeMemory(DevMem=0x%p)"), Memory));
#endif
	}
}


void FWrapLayer::DestroyFence(VkResult Result, VkDevice Device, VkFence Fence)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyFence(Fence=0x%p)"), Fence));
#endif
	}
}

void FWrapLayer::DestroySemaphore(VkResult Result, VkDevice Device, VkSemaphore Semaphore)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroySemaphore(Semaphore=0x%p)"), Semaphore));
#endif
	}
}

void FWrapLayer::CreateEvent(VkResult Result, VkDevice Device, const VkEventCreateInfo* CreateInfo, VkEvent* Event)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkCreateEvent(CreateInfo=0x%p, OutEvent=0x%p)"), CreateInfo, Event));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndNamedHandle(Result, TEXT("Event"), *Event);
#endif
	}
}

void FWrapLayer::DestroyEvent(VkResult Result, VkDevice Device, VkEvent Event)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyEvent(Event=0x%p)"), Event));
#endif
	}
	else
	{
	}
}

void FWrapLayer::DestroyBuffer(VkResult Result, VkDevice Device, VkBuffer Buffer)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyBuffer(Buffer=0x%p)"), Buffer));
#endif
	}
	else
	{
#if VULKAN_ENABLE_BUFFER_TRACKING_LAYER
		{
			FScopeLock ScopeLock(&GTrackingCS);
			int32 NumRemoved = GVulkanTrackingBuffers.Remove(Buffer);
			ensure(NumRemoved > 0);

			auto* Found = GVulkanTrackingBufferToBufferViews.Find(Buffer);
			if (Found)
			{
				ensure(Found->Num() > 0);
			}
		}
#endif
	}
}

void FWrapLayer::DestroyBufferView(VkResult Result, VkDevice Device, VkBufferView BufferView)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyBufferView(BufferView=0x%p)"), BufferView));
#endif
	}
	else
	{
#if VULKAN_ENABLE_BUFFER_TRACKING_LAYER
		{
			FScopeLock ScopeLock(&GTrackingCS);
			int32 NumRemoved = GVulkanTrackingBufferViews.Remove(BufferView);
			ensure(NumRemoved > 0);
		}
#endif
	}
}

void FWrapLayer::DestroyQueryPool(VkResult Result, VkDevice Device, VkQueryPool QueryPool)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyQueryPool(QueryPool=0x%p)"), QueryPool));
#endif
	}
}

void FWrapLayer::DestroyImageView(VkResult Result, VkDevice Device, VkImageView ImageView)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyImageView(ImageView=0x%p)"), ImageView));
#endif
	}
	else
	{
#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER
		{
			FScopeLock ScopeLock(&GTrackingCS);
			BreakOnTrackingImageView(ImageView);
			int32 NumRemoved = GVulkanTrackingImageViews.Remove(ImageView);
			ensure(NumRemoved > 0);
		}
#endif
	}
}

void FWrapLayer::DestroyShaderModule(VkResult Result, VkDevice Device, VkShaderModule ShaderModule)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
			DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyShaderModule(ShaderModule=0x%p)"), ShaderModule));
#endif
	}
}

void FWrapLayer::DestroyPipelineCache(VkResult Result, VkDevice Device, VkPipelineCache PipelineCache)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyPipelineCache(PipelineCache=0x%p)"), PipelineCache));
#endif
	}
}

void FWrapLayer::GetPipelineCacheData(VkResult Result, VkDevice Device, VkPipelineCache PipelineCache, size_t* DataSize, void* Data)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkGetPipelineCacheData(PipelineCache=0x%p, DataSize=%d, [Data])"), PipelineCache, DataSize));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::MergePipelineCaches(VkResult Result, VkDevice Device, VkPipelineCache DestCache, uint32 SourceCacheCount, const VkPipelineCache* SrcCaches)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkMergePipelineCaches(DestCache=0x%p, SourceCacheCount=%d, [SrcCaches])"), DestCache, SourceCacheCount));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::DestroySampler(VkResult Result, VkDevice Device, VkSampler Sampler)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroySampler(Sampler=0x%p)"), Sampler));
#endif
	}
}

void FWrapLayer::DestroySwapchainKHR(VkResult Result, VkDevice Device, VkSwapchainKHR SwapchainKHR)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroySwapchainKHR(SwapchainKHR=0x%p)"), SwapchainKHR));
#endif
	}
}

void FWrapLayer::FreeCommandBuffers(VkResult Result, VkDevice Device, VkCommandPool CommandPool, uint32 CommandBufferCount, const VkCommandBuffer* CommandBuffers)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkFreeCommandBuffers(CommandPool=0x%p, CommandBufferCount=%d, CommandBuffers=0x%p)[...]"), CommandPool, CommandBufferCount, CommandBuffers));
#endif
	}
}


void FWrapLayer::DestroyInstance(VkResult Result, VkInstance Instance)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBegin(FString::Printf(TEXT("vkDestroyInstance(Instance=0x%p)"), Instance));
#endif
	}
}

void FWrapLayer::ResetDescriptorPool(VkResult Result, VkDevice Device, VkDescriptorPool DescriptorPool, VkDescriptorPoolResetFlags Flags)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkResetDescriptorPool(DescriptorPool=0x%p, Flags=0x%x)"), DescriptorPool, (uint32)Flags));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::DestroyDescriptorPool(VkResult Result, VkDevice Device, VkDescriptorPool DescriptorPool)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
			DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyDescriptorPool(DescriptorPool=0x%p)"), DescriptorPool));
#endif
	}
}

void FWrapLayer::DestroyDescriptorSetLayout(VkResult Result, VkDevice Device, VkDescriptorSetLayout DescriptorSetLayout)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyDescriptorSetLayout(DescriptorSetLayout=0x%p)"), DescriptorSetLayout));
#endif
	}
}

void FWrapLayer::DestroySurfaceKHR(VkResult Result, VkInstance Instance, VkSurfaceKHR SurfaceKHR)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkDestroySurfaceKHR(PhysicalDevice=%llu, Surface=%llu)"), (uint64)Instance, (uint64)SurfaceKHR));
#endif
	}
}
void FWrapLayer::DestroyFramebuffer(VkResult Result, VkDevice Device, VkFramebuffer Framebuffer)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyFramebuffer(Framebuffer=0x%p)"), Framebuffer));
#endif
	}
}

void FWrapLayer::DestroyRenderPass(VkResult Result, VkDevice Device, VkRenderPass RenderPass)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyRenderPass(RenderPass=0x%p)"), RenderPass));
#endif
	}
}

#if VULKAN_ENABLE_DUMP_LAYER
static void DumpExtent2D(const VkExtent2D* pExtent)
{
	if (pExtent)
	{
		DebugLog += FString::Printf(TEXT(" -> Width=%u Height=%u\n"), pExtent->width, pExtent->height);
	}
	else
	{
		DumpErrorInvalidPointer(TEXT("VkExtent2D"));
	}
	FlushDebugWrapperLog();
}
#endif

void FWrapLayer::GetRenderAreaGranularity(VkResult Result, VkDevice Device, VkRenderPass RenderPass, VkExtent2D* pGranularity)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBeginResult(Device, FString::Printf(TEXT("vkGetRenderAreaGranularity(RenderPass=0x%p, pGranularity=0x%p)"), RenderPass, pGranularity));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DumpExtent2D(pGranularity);
#endif
	}
}

void FWrapLayer::DestroyCommandPool(VkResult Result, VkDevice Device, VkCommandPool CommandPool)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyCommandPool(CommandPool=0x%p)"), CommandPool));
#endif
	}
}

void FWrapLayer::DestroyPipelineLayout(VkResult Result, VkDevice Device, VkPipelineLayout PipelineLayout)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyPipelineLayout(PipelineLayout=0x%p)"), PipelineLayout));
#endif
	}
}

void FWrapLayer::DestroyPipeline(VkResult Result, VkDevice Device, VkPipeline Pipeline)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyPipeline(Pipeline=0x%p)"), Pipeline));
#endif
	}
}

void FWrapLayer::DestroyDevice(VkResult Result, VkDevice Device)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkDestroyDevice(Device=0x%p)"), Device));
#endif
	}
}

void FWrapLayer::ResetCommandBuffer(VkResult Result, VkCommandBuffer CommandBuffer, VkCommandBufferResetFlags Flags)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkResetCommandBuffer(Cmd=0x%p, Flags=%d)"), CommandBuffer, Flags));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}


void FWrapLayer::GetPhysicalDeviceQueueFamilyProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, uint32* QueueFamilyPropertyCount, VkQueueFamilyProperties* QueueFamilyProperties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkGetPhysicalDeviceQueueFamilyProperties(PhysicalDevice=0x%p, QueueFamilyPropertyCount=0x%p, QueueFamilyProperties=0x%p)[...]"), PhysicalDevice, QueueFamilyPropertyCount, QueueFamilyProperties));
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceQueueFamilyProperties2(VkResult Result, VkPhysicalDevice PhysicalDevice, uint32* QueueFamilyPropertyCount, VkQueueFamilyProperties2* QueueFamilyProperties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkGetPhysicalDeviceQueueFamilyProperties2(PhysicalDevice=0x%p, QueueFamilyPropertyCount=0x%p, QueueFamilyProperties2=0x%p)[...]"), PhysicalDevice, QueueFamilyPropertyCount, QueueFamilyProperties));
#endif
	}
}

void FWrapLayer::GetImageMemoryRequirements2(VkResult Result, VkDevice Device, const VkImageMemoryRequirementsInfo2* Info, VkMemoryRequirements2* MemoryRequirements)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkGetImageMemoryRequirements2(Info=0x%p, MemReqs=0x%p)[...]"), Info, MemoryRequirements));
#endif
	}
}

void FWrapLayer::GetBufferMemoryRequirements2(VkResult Result, VkDevice Device, const VkBufferMemoryRequirementsInfo2* Info, VkMemoryRequirements2* MemoryRequirements)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkGetBufferemoryRequirements2(Info=0x%p, MemReqs=0x%p)[...]"), Info, MemoryRequirements));
#endif
	}
}

void FWrapLayer::GetImageSparseMemoryRequirements(VkResult Result, VkDevice Device, VkImage Image, uint32_t* pSparseMemoryRequirementCount, VkSparseImageMemoryRequirements* pSparseMemoryRequirements)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkGetImageSparseMemoryRequirements(Device=0x%p, Image=0x%p, pSparseMemoryRequirementCount=0x%p, pSparseMemoryRequirements=0x%p)[...]"), Device, Image, pSparseMemoryRequirementCount, pSparseMemoryRequirements));
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceSparseImageFormatProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, VkFormat Format, VkImageType Type, VkSampleCountFlagBits Samples, VkImageUsageFlags Usage, VkImageTiling Tiling, uint32_t* pPropertyCount, VkSparseImageFormatProperties* pProperties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBegin(FString::Printf(
			TEXT("vkGetPhysicalDeviceSparseImageFormatProperties(PhysicalDevice=0x%p, Format=%s, Type=%d, Samples=0x%08X, Usage=%d, Tiling=%d, pPropertyCount=%u, pProperties=0x%p)[...]"),
			PhysicalDevice, *GetVkFormatString(Format), (int32)Type, (int32)Samples, (int32)Usage, *GetVkImageTilingString(Tiling), (pPropertyCount != nullptr ? *pPropertyCount : 0u), pProperties
		));
#endif
	}
}

void FWrapLayer::QueueBindSparse(VkResult Result, VkQueue Queue, uint32_t BindInfoCount, const VkBindSparseInfo* pBindInfo, VkFence Fence)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkQueueBindSparse(Queue=0x%p, BindInfoCount=%u, pBindInfo=0x%p, Fence=0x%p)"), Queue, BindInfoCount, pBindInfo, Fence));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceMemoryProperties2(VkResult Result, VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceMemoryProperties2* MemoryProperties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBegin(FString::Printf(TEXT("vkGetPhysicalDeviceMemoryProperties2(PhysicalDevice=0x%p, Properties=0x%p)[...]"), PhysicalDevice, MemoryProperties));
#endif
	}
}


void FWrapLayer::GetPhysicalDeviceProperties2(VkResult Result, VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceProperties2* Properties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBegin(FString::Printf(TEXT("vkGetPhysicalDeviceProperties2(PhysicalDevice=0x%p, Properties=0x%p)[...]"), PhysicalDevice, Properties));
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceFeatures2(VkResult Result, VkPhysicalDevice PhysicalDevice, VkPhysicalDeviceFeatures2* Features)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBegin(FString::Printf(TEXT("vkGetPhysicalDeviceFeatures2(PhysicalDevice=0x%p, Features=0x%p)[...]"), PhysicalDevice, Features));
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceFragmentShadingRatesKHR(VkResult Result, VkPhysicalDevice PhysicalDevice, uint32* FragmentShadingRateCount, VkPhysicalDeviceFragmentShadingRateKHR* FragmentShadingRates)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBegin(FString::Printf(TEXT("GetPhysicalDeviceFragmentShadingRatesKHR(PhysicalDevice=0x%p, FragmentShadingRateCount=0x%p, FragmentShadingRates=0x%p)[...]"), PhysicalDevice, FragmentShadingRateCount, FragmentShadingRates));
#endif
	}
}

void FWrapLayer::CmdSetDepthBias(VkResult Result, VkCommandBuffer CommandBuffer, float DepthBiasConstantFactor, float DepthBiasClamp, float DepthBiasSlopeFactor)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdSetDepthBias(ConstFactor=%f, Clamp=%f, SlopeFactor=%f)"), DepthBiasConstantFactor, DepthBiasClamp, DepthBiasSlopeFactor));
#endif
	}
}

void FWrapLayer::CmdSetBlendConstants(VkResult Result, VkCommandBuffer CommandBuffer, const float BlendConstants[4])
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdSetBlendConstants(BlendConstants=[%f, %f, %f, %f])"), BlendConstants[0], BlendConstants[1], BlendConstants[2], BlendConstants[3]));
#endif
	}
}

void FWrapLayer::CmdSetDepthBounds(VkResult Result, VkCommandBuffer CommandBuffer, float MinDepthBounds, float MaxDepthBounds)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdSetDepthBounds(Min=%f Max=%f])"), MinDepthBounds, MaxDepthBounds));
#endif
	}
}

void FWrapLayer::CmdSetStencilCompareMask(VkResult Result, VkCommandBuffer CommandBuffer, VkStencilFaceFlags FaceMask, uint32 CompareMask)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdSetStencilCompareMask(FaceMask=%d, CompareMask=%d)"), (int32)FaceMask, (int32)CompareMask));
#endif
	}
}

void FWrapLayer::CmdSetStencilWriteMask(VkResult Result, VkCommandBuffer CommandBuffer, VkStencilFaceFlags FaceMask, uint32 WriteMask)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdSetStencilWriteMask(FaceMask=%d, CompareMask=%d)"), (int32)FaceMask, (int32)WriteMask));
#endif
	}
}

void FWrapLayer::CmdSetStencilReference(VkResult Result, VkCommandBuffer CommandBuffer, VkStencilFaceFlags FaceMask, uint32 Reference)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdSetStencilReference(FaceMask=%d, Ref=%d)"), (int32)FaceMask, (int32)Reference));
#endif
	}
}

void FWrapLayer::CmdUpdateBuffer(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer DstBuffer, VkDeviceSize DstOffset, VkDeviceSize DataSize, const void* pData)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdUpdateBuffer(DstBuffer=0x%p, DstOffset=%d, Size=%d, Data=0x%p)"), DstBuffer, (uint32)DstOffset, (uint32)DataSize, pData));
#endif
	}
}

void FWrapLayer::CmdFillBuffer(VkResult Result, VkCommandBuffer CommandBuffer, VkBuffer DstBuffer, VkDeviceSize DstOffset, VkDeviceSize Size, uint32 Data)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdFillBuffer(DstBuffer=0x%p, DstOffset=%d, Size=%d, Data=0x%x)"), DstBuffer, (uint32)DstOffset, (uint32)Size, Data));
#endif
	}
}

void FWrapLayer::CmdSetEvent(VkResult Result, VkCommandBuffer CommandBuffer, VkEvent Event, VkPipelineStageFlags StageMask)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdSetEvent(Event=0x%p, StageMask=0x%x)"), Event, StageMask));
#endif
	}
}

void FWrapLayer::CmdResetEvent(VkResult Result, VkCommandBuffer CommandBuffer, VkEvent Event, VkPipelineStageFlags StageMask)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdResetEvent(Event=0x%p, StageMask=0x%x)"), Event, StageMask));
#endif
	}
}

void FWrapLayer::SetEvent(VkResult Result, VkDevice Device, VkEvent Event)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkCmdSetEvent(Event=0x%p)"), Event));
#endif
	}
}

void FWrapLayer::ResetEvent(VkResult Result, VkDevice Device, VkEvent Event)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("vkCmdResetEvent(Event=0x%p)"), Event));
#endif
	}
}

void FWrapLayer::GetEventStatus(VkResult Result, VkDevice Device, VkEvent Event)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		DevicePrintfBegin(Device, FString::Printf(TEXT("GetEventStatus(Event=0x%p)"), Event));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::CmdCopyQueryPoolResults(VkResult Result, VkCommandBuffer CommandBuffer, VkQueryPool QueryPool, uint32 FirstQuery, uint32 QueryCount, VkBuffer DstBuffer, VkDeviceSize DstOffset, VkDeviceSize Stride, VkQueryResultFlags Flags)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdCopyQueryPoolResults(QueryPool=0x%p, FirstQuery=%d, QueryCount=%d, DstBuffer=0x%p, DstOffset=%d, Stride=%d, Flags=0x%x)"),
				(void*)QueryPool, FirstQuery, QueryCount, (void*)DstBuffer, (uint32)DstOffset, (uint32)Stride, (uint32)Flags));
#endif
	}
}

void FWrapLayer::GetInstanceProcAddr(VkResult Result, VkInstance Instance, const char* Name, PFN_vkVoidFunction VoidFunction)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
			PrintfBeginResult(FString::Printf(TEXT("vkGetInstanceProcAddr(Instance=0x%p, Name=%s)[...]"), Instance, ANSI_TO_TCHAR(Name)));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndPointer(VK_SUCCESS, (void*)VoidFunction);
#endif
	}
}

void FWrapLayer::GetDeviceProcAddr(VkResult Result, VkDevice Device, const char* Name, PFN_vkVoidFunction VoidFunction)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkGetDeviceProcAddr(Device=0x%p, Name=%s)[...]"), Device, ANSI_TO_TCHAR(Name)));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndPointer(VK_SUCCESS, (void*)VoidFunction);
#endif
	}
}

void FWrapLayer::EnumerateInstanceExtensionProperties(VkResult Result, const char* LayerName, uint32* PropertyCount, VkExtensionProperties* Properties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkEnumerateInstanceExtensionProperties(LayerName=%s, PropertyCount=0x%p, Properties=0x%p)[...]"), ANSI_TO_TCHAR(LayerName), PropertyCount, Properties));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndPointer(Result, (void*)(uint64)PropertyCount);
		PrintResultAndPointer(Result, (void*)Properties);
#endif
	}
}

void FWrapLayer::EnumerateDeviceExtensionProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, const char* LayerName, uint32* PropertyCount, VkExtensionProperties* Properties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkEnumerateDeviceExtensionProperties(Device=0x%p, LayerName=%s, PropertyCount=0x%p, Properties=0x%p)[...]"), PhysicalDevice, ANSI_TO_TCHAR(LayerName), PropertyCount, Properties));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndPointer(Result, (void*)(uint64)PropertyCount);
		PrintResultAndPointer(Result, (void*)Properties);
#endif
	}
}

void FWrapLayer::EnumerateInstanceLayerProperties(VkResult Result, uint32* PropertyCount, VkLayerProperties* Properties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkEnumerateInstanceLayerProperties(PropertyCount=0x%p, Properties=0x%p)[...]"), PropertyCount, Properties));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndPointer(Result, (void*)(uint64)PropertyCount);
		PrintResultAndPointer(Result, (void*)Properties);
#endif
	}
}

void FWrapLayer::EnumerateDeviceLayerProperties(VkResult Result, VkPhysicalDevice PhysicalDevice, uint32* PropertyCount, VkLayerProperties* Properties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkEnumerateDeviceLayerProperties(Device=0x%p, PropertyCount=0x%p, Properties=0x%p)[...]"), PhysicalDevice, PropertyCount, Properties));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResultAndPointer(Result, (void*)(uint64)PropertyCount);
		PrintResultAndPointer(Result, (void*)Properties);
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceSurfaceCapabilitiesKHR(VkResult Result, VkPhysicalDevice PhysicalDevice, VkSurfaceKHR Surface, VkSurfaceCapabilitiesKHR* SurfaceCapabilities)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkGetPhysicalDeviceSurfaceCapabilitiesKHR(PhysicalDevice=0x%p, Surface=0x%p)[...]"), PhysicalDevice, Surface));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceSurfaceFormatsKHR(VkResult Result, VkPhysicalDevice PhysicalDevice, VkSurfaceKHR Surface, uint32_t* SurfaceFormatCountPtr, VkSurfaceFormatKHR* SurfaceFormats)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkGetPhysicalDeviceSurfaceFormatsKHR(PhysicalDevice=0x%p, Surface=0x%p)[...]"), PhysicalDevice, Surface));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceSurfaceSupportKHR(VkResult Result, VkPhysicalDevice PhysicalDevice, uint32_t QueueFamilyIndex, VkSurfaceKHR Surface, VkBool32* SupportedPtr)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkGetPhysicalDeviceSurfaceSupportKHR(PhysicalDevice=0x%p, QueueFamilyIndex=%d, Surface=0x%p)[...]"), PhysicalDevice, QueueFamilyIndex, (void*)Surface));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceSurfacePresentModesKHR(VkResult Result, VkPhysicalDevice PhysicalDevice, VkSurfaceKHR Surface, uint32_t* PresentModeCountPtr, VkPresentModeKHR* PresentModesPtr)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkGetPhysicalDeviceSurfacePresentModesKHR(PhysicalDevice=0x%p, Surface=0x%p, PresentModeCountPtr=%d, PresentModesPtr=0x%p)"), PhysicalDevice, Surface, PresentModeCountPtr ? *PresentModeCountPtr:0, PresentModesPtr));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

#if VULKAN_USE_CREATE_ANDROID_SURFACE
void FWrapLayer::CreateAndroidSurfaceKHR(VkResult Result, VkInstance Instance, const VkAndroidSurfaceCreateInfoKHR* CreateInfo, VkSurfaceKHR* Surface)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkCreateAndroidSurfaceKHR(Instance=0x%p, CreateInfo=0x%p, Surface=0x%p)[...]"), Instance, CreateInfo, Surface));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}
#endif

#if VULKAN_USE_CREATE_WIN32_SURFACE
void FWrapLayer::CreateWin32SurfaceKHR(VkResult Result, VkInstance Instance, const VkWin32SurfaceCreateInfoKHR* CreateInfo, VkSurfaceKHR* Surface)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkCreateWin32SurfaceKHR(Instance=0x%p, CreateInfo=0x%p, Surface=0x%p)[...]"), Instance, CreateInfo, Surface));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}
#endif

void FWrapLayer::CreateAccelerationStructureKHR(VkResult Result, VkDevice Device, const VkAccelerationStructureCreateInfoKHR* CreateInfo, const VkAllocationCallbacks* Allocator, VkAccelerationStructureKHR* AccelerationStructure)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
	PrintfBeginResult(FString::Printf(TEXT("vkCreateAccelerationStructureKHR(Device=0x%p, CreateInfo=0x%p, AccelerationStructure=0x%p)[...]"), Device, CreateInfo, AccelerationStructure));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
	PrintResult(Result);
#endif
	}
}

void FWrapLayer::DestroyAccelerationStructureKHR(VkResult Result, VkDevice Device, VkAccelerationStructureKHR AccelerationStructure, const VkAllocationCallbacks* Allocator)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkDestroyAccelerationStructureKHR(Device=0x%p, AccelerationStructure=0x%p)[...]"), Device, AccelerationStructure));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::CmdBuildAccelerationStructuresKHR(VkResult Result, VkCommandBuffer CommandBuffer, uint32 InfoCount, const VkAccelerationStructureBuildGeometryInfoKHR* Infos, const VkAccelerationStructureBuildRangeInfoKHR* const* BuildRangeInfos)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkCmdBuildAccelerationStructuresKHR")));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::GetAccelerationStructureBuildSizesKHR(VkResult Result, VkDevice Device, VkAccelerationStructureBuildTypeKHR BuildType, const VkAccelerationStructureBuildGeometryInfoKHR* BuildInfo, const uint32* MaxPrimitiveCounts, VkAccelerationStructureBuildSizesInfoKHR* SizeInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkGetAccelerationStructureBuildSizesKHR")));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::GetAccelerationStructureDeviceAddressKHR(VkResult Result, VkDevice Device, const VkAccelerationStructureDeviceAddressInfoKHR* Info)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkGetAccelerationStructureDeviceAddressKHR")));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::CmdTraceRaysKHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkStridedDeviceAddressRegionKHR* RaygenShaderBindingTable, const VkStridedDeviceAddressRegionKHR* MissShaderBindingTable, const VkStridedDeviceAddressRegionKHR* HitShaderBindingTable, const VkStridedDeviceAddressRegionKHR* CallableShaderBindingTable, uint32 width, uint32 height, uint32 depth)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkCmdTraceRaysKHR")));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::CmdTraceRaysIndirectKHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkStridedDeviceAddressRegionKHR* RaygenShaderBindingTable, const VkStridedDeviceAddressRegionKHR* MissShaderBindingTable, const VkStridedDeviceAddressRegionKHR* HitShaderBindingTable, const VkStridedDeviceAddressRegionKHR* CallableShaderBindingTable, VkDeviceAddress IndirectDeviceAddress)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkCmdTraceRaysIndirectKHR")));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::CmdTraceRaysIndirect2KHR(VkResult Result, VkCommandBuffer CommandBuffer, VkDeviceAddress IndirectDeviceAddress)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkCmdTraceRaysIndirect2KHR")));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::CreateRayTracingPipelinesKHR(VkResult Result, VkDevice Device, VkDeferredOperationKHR DeferredOperation, VkPipelineCache PipelineCache, uint32_t CreateInfoCount, const VkRayTracingPipelineCreateInfoKHR* CreateInfos, const VkAllocationCallbacks* Allocator, VkPipeline* Pipelines)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkCreateRayTracingPipelinesKHR")));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::GetRayTracingShaderGroupHandlesKHR(VkResult Result, VkDevice Device, VkPipeline Pipeline, uint32_t FirstGroup, uint32_t GroupCount, size_t DataSize, void* Data)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkGetRayTracingShaderGroupHandlesKHR")));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::CmdWriteAccelerationStructuresPropertiesKHR(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t AccelerationStructureCount, const VkAccelerationStructureKHR* AccelerationStructures, VkQueryType QueryType, VkQueryPool QueryPool, uint32_t FirstQuery)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkCmdWriteAccelerationStructuresPropertiesKHR")));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::CmdCopyAccelerationStructureKHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkCopyAccelerationStructureInfoKHR* Info)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkCmdCopyAccelerationStructureKHR")));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}


void FWrapLayer::GetBufferDeviceAddressKHR(VkResult Result, VkDevice Device, const VkBufferDeviceAddressInfo* Info)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkGetBufferDeviceAddressKHR(Device=0x%p, Buffer=0x%p)"), Device, Info->buffer));
#endif
	}
	else
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintResult(Result);
#endif
	}
}

void FWrapLayer::GetDeviceImageMemoryRequirementsKHR(VkResult Result, VkDevice Device, const VkDeviceImageMemoryRequirements* Info, VkMemoryRequirements2* MemoryRequirements)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkGetDeviceImageMemoryRequirementsKHR")));
#endif
	}
}

void FWrapLayer::GetDeviceBufferMemoryRequirementsKHR(VkResult Result, VkDevice Device, const VkDeviceBufferMemoryRequirements* Info, VkMemoryRequirements2* MemoryRequirements)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("vkGetDeviceBufferMemoryRequirementsKHR")));
#endif
	}
}

void FWrapLayer::ResetQueryPoolEXT(VkResult Result, VkDevice Device, VkQueryPool QueryPool, uint32_t FirstQuery, uint32_t QueryCount)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("ResetQueryPoolEXT(Device=0x%p, QueryPool=0x%p, FirstQuery=%u, QueryCount=%u)"), Device, QueryPool, FirstQuery, QueryCount));
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceCalibrateableTimeDomainsEXT(VkResult Result, VkPhysicalDevice PhysicalDevice, uint32_t* TimeDomainCount, VkTimeDomainEXT* TimeDomains)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("PhysicalDeviceCalibrateableTimeDomainsEXT(PhysicalDevice=0x%p, TimeDomains=0x%p)"), PhysicalDevice, TimeDomains));
#endif
	}
}

void FWrapLayer::GetCalibratedTimestampsEXT(VkResult Result, VkDevice Device, uint32_t TimestampCount, const VkCalibratedTimestampInfoEXT* TimestampInfos, uint64_t* Timestamps, uint64_t* MaxDeviation)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetCalibratedTimestampsEXT(Device=0x%p, TimestampCount=%u)"), Device, TimestampCount));
#endif
	}
}

void FWrapLayer::BindBufferMemory2(VkResult Result, VkDevice Device, uint32_t BindInfoCount, const VkBindBufferMemoryInfo* BindInfos)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("BindBufferMemory2(Device=0x%p, BindInfoCount=%u)"), Device, BindInfoCount));
#endif
	}
}

void FWrapLayer::BindImageMemory2(VkResult Result, VkDevice Device, uint32_t BindInfoCount, const VkBindImageMemoryInfo* BindInfos)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("BindImageMemory2(Device=0x%p, BindInfoCount=%u)"), Device, BindInfoCount));
#endif
	}
}

void FWrapLayer::GetDescriptorSetLayoutSizeEXT(VkResult Result, VkDevice Device, VkDescriptorSetLayout Layout, VkDeviceSize* OutLayoutSizeInBytes)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetDescriptorSetLayoutSizeEXT(Device=0x%p, Layout=0x%p)"), Device, Layout));
#endif
	}
}

void FWrapLayer::GetDescriptorSetLayoutBindingOffsetEXT(VkResult Result, VkDevice Device, VkDescriptorSetLayout Layout, uint32_t Binding, VkDeviceSize* Offset)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetDescriptorSetLayoutBindingOffsetEXT(Device=0x%p, Layout=0x%p)"), Device, Layout));
#endif
	}
}

void FWrapLayer::CmdBindDescriptorBuffersEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t BufferCount, const VkDescriptorBufferBindingInfoEXT* BindingInfos)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdBindDescriptorBuffersEXT(CommandBuffer=0x%p, BufferCount=%u)"), CommandBuffer, BufferCount));
		for (uint32_t Index = 0; Index  < BufferCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%s%d - Address=0x%p Usage=0x%X\n"), Tabs, Index, BindingInfos[Index].address, BindingInfos[Index].usage);
		}
#endif
	}
}

void FWrapLayer::CmdSetDescriptorBufferOffsetsEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkPipelineBindPoint PipelineBindPoint, VkPipelineLayout Layout, uint32_t FirstSet, uint32_t SetCount, const uint32_t* BufferIndices, const VkDeviceSize* Offsets)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetDescriptorBufferOffsetsEXT(CommandBuffer=0x%p, BindPoint=%s, PipelineLayout=0x%p, FirstSet=%u, SetCount=%u)"), 
			CommandBuffer, VK_TYPE_TO_STRING(VkPipelineBindPoint, PipelineBindPoint), Layout, FirstSet, SetCount));
		for (uint32_t Index = 0; Index < SetCount; ++Index)
		{
			DebugLog += FString::Printf(TEXT("%sSet %u - BufferIndex=%u Offset=%llu\n"), Tabs, FirstSet + Index, BufferIndices[Index], Offsets[Index]);
		}
#endif
	}
}

void FWrapLayer::CmdSetDescriptorBufferOffsets2EXT(VkResult Result, VkCommandBuffer CommandBuffer, const VkSetDescriptorBufferOffsetsInfoEXT* SetDescriptorBufferOffsetsInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetDescriptorBufferOffsets2EXT(CommandBuffer=0x%p, SetDescriptorBufferOffsetsInfo=0x%p)"), 
			CommandBuffer, SetDescriptorBufferOffsetsInfo));
#endif
	}
}

void FWrapLayer::GetDescriptorEXT(VkResult Result, VkDevice Device, const VkDescriptorGetInfoEXT* DescriptorInfo, size_t DataSize, void* Descriptor)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetDescriptorEXT(Device=0x%p, DataSize=%llu DescriptorType=%s)"), Device, DataSize, VK_TYPE_TO_STRING(VkDescriptorType, DescriptorInfo->type)));
#endif
	}
}

void FWrapLayer::CreateDeferredOperationKHR(VkResult Result, VkDevice Device, const VkAllocationCallbacks* Allocator, VkDeferredOperationKHR* DeferredOperation)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CreateDeferredOperationKHR(Device=0x%p, DeferredOperation=0x%p)"), Device, DeferredOperation));
#endif
	}
}

void FWrapLayer::DestroyDeferredOperationKHR(VkResult Result, VkDevice Device, VkDeferredOperationKHR DeferredOperation, const VkAllocationCallbacks* Allocator)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("DestroyDeferredOperationKHR(Device=0x%p, DeferredOperation=0x%p)"), Device, DeferredOperation));
#endif
	}
}

void FWrapLayer::DeferredOperationJoinKHR(VkResult Result, VkDevice Device, VkDeferredOperationKHR DeferredOperation)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("DeferredOperationJoinKHR(Device=0x%p, DeferredOperation=0x%p)"), Device, DeferredOperation));
#endif
	}
}

void FWrapLayer::GetDeferredOperationMaxConcurrencyKHR(VkResult Result, VkDevice Device, VkDeferredOperationKHR DeferredOperation)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetDeferredOperationMaxConcurrencyKHR(Device=0x%p, DeferredOperation=0x%p)"), Device, DeferredOperation));
#endif
	}
}

void FWrapLayer::GetDeferredOperationResultKHR(VkResult Result, VkDevice Device, VkDeferredOperationKHR DeferredOperation)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetDeferredOperationResultKHR(Device=0x%p, DeferredOperation=0x%p)"), Device, DeferredOperation));
#endif
	}
}

void FWrapLayer::GetDeviceFaultInfoEXT(VkResult Result, VkDevice Device, VkDeviceFaultCountsEXT* FaultCounts, VkDeviceFaultInfoEXT* FaultInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetDeviceFaultInfoEXT(Device=0x%p)"), Device));
#endif
	}
}

void FWrapLayer::CmdPipelineBarrier2KHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkDependencyInfo* DependencyInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdPipelineBarrier2(CommandBuffer=0x%p, Flags=%d, NumMemB=%d, MemB=0x%p,"), CommandBuffer, (uint32)DependencyInfo->dependencyFlags, DependencyInfo->memoryBarrierCount, DependencyInfo->pMemoryBarriers));
		DebugLog += FString::Printf(TEXT("%s\tNumBufferB=%d, BufferB=0x%p, NumImageB=%d, ImageB=0x%p)[...]\n"), Tabs, DependencyInfo->bufferMemoryBarrierCount, DependencyInfo->pBufferMemoryBarriers, DependencyInfo->imageMemoryBarrierCount, DependencyInfo->pImageMemoryBarriers);
		DumpImageMemoryBarriers(DependencyInfo->imageMemoryBarrierCount, DependencyInfo->pImageMemoryBarriers);
		FlushDebugWrapperLog();
#endif

#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER
		{
			FScopeLock ScopeLock(&GTrackingCS);
			for (uint32 Index = 0; Index < DependencyInfo->imageMemoryBarrierCount; ++Index)
			{
				const VkImageMemoryBarrier2& ImageBarrier = DependencyInfo->pImageMemoryBarriers[Index];

				BreakOnTrackingImage(ImageBarrier.image);
				if (DumpTrackImage(ImageBarrier.image))
				{
					CmdPrintfBegin(CommandBuffer, FString::Printf(TEXT("vkCmdPipelineBarrier2(Flags=%d, NumMemB=%d, MemB=0x%p,"), (uint32)DependencyInfo->dependencyFlags, DependencyInfo->memoryBarrierCount, DependencyInfo->pMemoryBarriers));
					DebugLog += FString::Printf(TEXT("%s\tNumBufferB=%d, BufferB=0x%p, NumImageB=%d, ImageB=0x%p)[...]\n"), Tabs, DependencyInfo->bufferMemoryBarrierCount, DependencyInfo->pBufferMemoryBarriers, DependencyInfo->imageMemoryBarrierCount, DependencyInfo->pImageMemoryBarriers);
					DumpImageMemoryBarriers(DependencyInfo->imageMemoryBarrierCount, DependencyInfo->pImageMemoryBarriers);
					FlushDebugWrapperLog();
				}
				FTrackingImage* TrackingImage = GVulkanTrackingImageLayouts.Find(ImageBarrier.image);
				check(TrackingImage);
#if VULKAN_ENABLE_TRACKING_CALLSTACK
				FTrackingImage::FHistoryEntry* HistoryEntry = new (TrackingImage->History) FTrackingImage::FHistoryEntry;
				HistoryEntry->ArrayLayouts = TrackingImage->ArrayLayouts;
#endif
				const VkImageSubresourceRange& Range = ImageBarrier.subresourceRange;
				uint32 NumLayers = (Range.layerCount == VK_REMAINING_ARRAY_LAYERS) ? (TrackingImage->Info.CreateInfo.arrayLayers - Range.baseArrayLayer) : Range.layerCount;
				for (uint32 LIndex = Range.baseArrayLayer; LIndex < Range.baseArrayLayer + NumLayers; ++LIndex)
				{
					TArray<VkImageLayout>& MipLayouts = TrackingImage->ArrayLayouts[LIndex];
					uint32 NumLevels = (Range.levelCount == VK_REMAINING_MIP_LEVELS) ? (TrackingImage->Info.CreateInfo.mipLevels - Range.baseMipLevel) : Range.levelCount;
					for (uint32 MIndex = Range.baseMipLevel; MIndex < Range.baseMipLevel + NumLevels; ++MIndex)
					{
						if (ImageBarrier.oldLayout != VK_IMAGE_LAYOUT_UNDEFINED && MipLayouts[MIndex] != ImageBarrier.oldLayout)
						{
							ensure(0);
						}
						MipLayouts[MIndex] = ImageBarrier.newLayout;
					}
				}
#if VULKAN_ENABLE_TRACKING_CALLSTACK
				CaptureCallStack(HistoryEntry->Callstack, 2);
#endif
			}
		}
#endif
	}
}

void FWrapLayer::CmdResetEvent2KHR(VkResult Result, VkCommandBuffer CommandBuffer, VkEvent Event, VkPipelineStageFlags2 StageMask)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdResetEvent2KHR(CommandBuffer=0x%p, Event=0x%p, StageMask=0x%llx)"), CommandBuffer, Event, StageMask));
#endif
	}
}

void FWrapLayer::CmdSetEvent2KHR(VkResult Result, VkCommandBuffer CommandBuffer, VkEvent Event, const VkDependencyInfo* DependencyInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetEvent2KHR(CommandBuffer=0x%p, Event=0x%p)"), CommandBuffer, Event));
#endif
	}
}

void FWrapLayer::CmdWaitEvents2KHR(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t EventCount, const VkEvent* Events, const VkDependencyInfo* DependencyInfos)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdWaitEvents2KHR(CommandBuffer=0x%p, EventCount=%u, Events=0x%p)"), CommandBuffer, EventCount, Events));
#endif
	}
}

void FWrapLayer::QueueSubmit2KHR(VkResult Result, VkQueue Queue, uint32_t SubmitCount, const VkSubmitInfo2* Submits, VkFence Fence)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("QueueSubmit2KHR(Queue=0x%p, SubmitCount=%u, Submits=0x%p, Fence=0x%p)"), Queue, SubmitCount, Submits, Fence));
#endif
	}
}

void FWrapLayer::CreateSamplerYcbcrConversion(VkResult Result, VkDevice Device, const VkSamplerYcbcrConversionCreateInfo* CreateInfo, const VkAllocationCallbacks* Allocator, VkSamplerYcbcrConversion* YcbcrConversion)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CreateSamplerYcbcrConversion(Device=0x%p, CreateInfo=0x%p, Allocator=0x%p, YcbcrConversion=0x%p))"), Device, CreateInfo, Allocator, YcbcrConversion));
#endif
	}
}

void FWrapLayer::DestroySamplerYcbcrConversion(VkResult Result, VkDevice Device, VkSamplerYcbcrConversion YcbcrConversion, const VkAllocationCallbacks* Allocator)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("DestroySamplerYcbcrConversion(Device=0x%p, YcbcrConversion=0x%p, Allocator=0x%p))"), Device, YcbcrConversion, Allocator));
#endif
	}
}

#if PLATFORM_ANDROID
void FWrapLayer::GetAndroidHardwareBufferPropertiesANDROID(VkResult Result, VkDevice Device, const struct AHardwareBuffer* Buffer, VkAndroidHardwareBufferPropertiesANDROID* Properties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetAndroidHardwareBufferPropertiesANDROID(Device=0x%p, Buffer=0x%p, Properties=0x%p))"), Device, Buffer, Properties));
#endif
	}
}
#endif

void FWrapLayer::GetSemaphoreCounterValueKHR(VkResult Result, VkDevice Device, VkSemaphore Semaphore, uint64_t* Value)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetSemaphoreCounterValueKHR(Device=0x%p, Semaphore=0x%p, Value=0x%p))"), Device, Semaphore, Value));
#endif
	}
}

void FWrapLayer::WaitSemaphoresKHR(VkResult Result, VkDevice Device, const VkSemaphoreWaitInfo* WaitInfo, uint64_t Timeout)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("WaitSemaphoresKHR(Device=0x%p, WaitInfo=0x%p, Timeout=0x%llx))"), Device, WaitInfo, Timeout));
#endif
	}
}

void FWrapLayer::SignalSemaphoreKHR(VkResult Result, VkDevice Device, const VkSemaphoreSignalInfo* SignalInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("SignalSemaphoreKHR(Device=0x%p, SignalInfo=0x%p))"), Device, SignalInfo));
#endif
	}
}

void FWrapLayer::CreateShadersEXT(VkResult Result, VkDevice Device, uint32_t CreateInfoCount, const VkShaderCreateInfoEXT* CreateInfos, const VkAllocationCallbacks* Allocator, VkShaderEXT* Shaders)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CreateShadersEXT()")));
#endif
	}
}

void FWrapLayer::DestroyShaderEXT(VkResult Result, VkDevice Device, VkShaderEXT Shader, const VkAllocationCallbacks* Allocator)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("DestroyShaderEXT()")));
#endif
	}
}

void FWrapLayer::GetShaderBinaryDataEXT(VkResult Result, VkDevice Device, VkShaderEXT Shader, size_t* DataSize, void* Data)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetShaderBinaryDataEXT()")));
#endif
	}
}

void FWrapLayer::CmdBindShadersEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t StageCount, const VkShaderStageFlagBits* Stages, const VkShaderEXT* Shaders)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdBindShadersEXT()")));
#endif
	}
}

void FWrapLayer::CmdBindVertexBuffers2EXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t FirstBinding, uint32_t BindingCount, const VkBuffer* Buffers, const VkDeviceSize* Offsets, const VkDeviceSize* Sizes, const VkDeviceSize* Strides)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdBindVertexBuffers2EXT()")));
#endif
	}
}

void FWrapLayer::CmdSetCullModeEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkCullModeFlags CullMode)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetCullModeEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetColorBlendEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t FirstAttachment, uint32_t AttachmentCount, const VkBool32* ColorBlendEnables)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetColorBlendEnableEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetColorBlendEquationEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t FirstAttachment, uint32_t AttachmentCount, const VkColorBlendEquationEXT* ColorBlendEquations)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetColorBlendEquationEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetColorWriteMaskEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t FirstAttachment, uint32_t AttachmentCount, const VkColorComponentFlags* ColorWriteMasks)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetColorWriteMaskEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetFrontFaceEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkFrontFace FrontFace)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetFrontFaceEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetPolygonModeEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkPolygonMode PolygonMode)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetPolygonModeEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetPrimitiveTopologyEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkPrimitiveTopology PrimitiveTopology)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetPrimitiveTopologyEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetRasterizationSamplesEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkSampleCountFlagBits RasterizationSamples)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetRasterizationSamplesEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetScissorWithCountEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t ScissorCount, const VkRect2D* Scissors)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetScissorWithCountEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetStencilOpEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkStencilFaceFlags FaceMask, VkStencilOp FailOp, VkStencilOp passOp, VkStencilOp depthFailOp, VkCompareOp compareOp)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetStencilOpEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetStencilTestEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 StencilTestEnable)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetStencilTestEnableEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetVertexInputEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t VertexBindingDescriptionCount, const VkVertexInputBindingDescription2EXT* pVertexBindingDescriptions, uint32_t vertexAttributeDescriptionCount, const VkVertexInputAttributeDescription2EXT* pVertexAttributeDescriptions)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetVertexInputEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetViewportWithCountEXT(VkResult Result, VkCommandBuffer CommandBuffer, uint32_t ViewportCount, const VkViewport* pViewports)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetViewportWithCountEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetDepthBiasEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 DepthBiasEnable)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetDepthBiasEnableEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetDepthBoundsTestEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 DepthBoundsTestEnable)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetDepthBoundsTestEnableEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetDepthClampEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 DepthClampEnable)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetDepthClampEnableEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetDepthCompareOpEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkCompareOp DepthCompareOp)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetDepthCompareOpEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetDepthTestEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 DepthTestEnable)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetDepthTestEnableEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetDepthWriteEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 DepthWriteEnable)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetDepthWriteEnableEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetRasterizerDiscardEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 RasterizerDiscardEnable)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetRasterizerDiscardEnableEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetPrimitiveRestartEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 PrimitiveRestartEnable)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetPrimitiveRestartEnableEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetLogicOpEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 LogicOpEnable)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetLogicOpEnableEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetAlphaToCoverageEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 AlphaToCoverageEnable)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetAlphaToCoverageEnableEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetAlphaToOneEnableEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 AlphaToOneEnable)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetAlphaToOneEnableEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetSampleMaskEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkSampleCountFlagBits Samples, const VkSampleMask* SampleMask)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetSampleMaskEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetFragmentShadingRateKHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkExtent2D* FragmentSize, const VkFragmentShadingRateCombinerOpKHR combinerOps[2])
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetFragmentShadingRateKHR()")));
#endif
	}
}

void FWrapLayer::CmdExecuteGeneratedCommandsEXT(VkResult Result, VkCommandBuffer CommandBuffer, VkBool32 IsPreprocessed, const VkGeneratedCommandsInfoEXT* GeneratedCommandsInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdExecuteGeneratedCommandsEXT()")));
#endif
	}
}

void FWrapLayer::CmdPreprocessGeneratedCommandsEXT(VkResult Result, VkCommandBuffer CommandBuffer, const VkGeneratedCommandsInfoEXT* GeneratedCommandsInfo, VkCommandBuffer StateCommandBuffer)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdPreprocessGeneratedCommandsEXT()")));
#endif
	}
}

void FWrapLayer::CreateIndirectCommandsLayoutEXT(VkResult Result, VkDevice Device, const VkIndirectCommandsLayoutCreateInfoEXT* CreateInfo, const VkAllocationCallbacks* Allocator, VkIndirectCommandsLayoutEXT* IndirectCommandsLayout)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CreateIndirectCommandsLayoutEXT()")));
#endif
	}
}

void FWrapLayer::CreateIndirectExecutionSetEXT(VkResult Result, VkDevice Device, const VkIndirectExecutionSetCreateInfoEXT* CreateInfo, const VkAllocationCallbacks* Allocator, VkIndirectExecutionSetEXT* IndirectExecutionSet)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CreateIndirectExecutionSetEXT()")));
#endif
	}
}

void FWrapLayer::DestroyIndirectCommandsLayoutEXT(VkResult Result, VkDevice Device, VkIndirectCommandsLayoutEXT IndirectCommandsLayout, const VkAllocationCallbacks* Allocator)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("DestroyIndirectCommandsLayoutEXT()")));
#endif
	}
}

void FWrapLayer::DestroyIndirectExecutionSetEXT(VkResult Result, VkDevice Device, VkIndirectExecutionSetEXT IndirectExecutionSet, const VkAllocationCallbacks* Allocator)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("DestroyIndirectExecutionSetEXT()")));
#endif
	}
}

void FWrapLayer::GetGeneratedCommandsMemoryRequirementsEXT(VkResult Result, VkDevice Device, const VkGeneratedCommandsMemoryRequirementsInfoEXT* Info, VkMemoryRequirements2* MemoryRequirements)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetGeneratedCommandsMemoryRequirementsEXT()")));
#endif
	}
}

void FWrapLayer::UpdateIndirectExecutionSetPipelineEXT(VkResult Result, VkDevice Device, VkIndirectExecutionSetEXT IndirectExecutionSet, uint32_t ExecutionSetWriteCount, const VkWriteIndirectExecutionSetPipelineEXT* ExecutionSetWrites)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("UpdateIndirectExecutionSetPipelineEXT()")));
#endif
	}
}

void FWrapLayer::UpdateIndirectExecutionSetShaderEXT(VkResult Result, VkDevice Device, VkIndirectExecutionSetEXT IndirectExecutionSet, uint32_t ExecutionSetWriteCount, const VkWriteIndirectExecutionSetShaderEXT* ExecutionSetWrites)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("UpdateIndirectExecutionSetShaderEXT()")));
#endif
	}
}



void FWrapLayer::CopyMemoryToImageEXT(VkResult Result, VkDevice Device, const VkCopyMemoryToImageInfo* CopyMemoryToImageInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CopyMemoryToImageEXT()")));
#endif
	}
}

void FWrapLayer::TransitionImageLayoutEXT(VkResult Result, VkDevice Device, uint32_t TransitionCount, const VkHostImageLayoutTransitionInfo* Transitions)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("TransitionImageLayoutEXT()")));
#endif
	}
}

void FWrapLayer::GetPastPresentationTimingEXT(VkResult Result, VkDevice Device, const VkPastPresentationTimingInfoEXT* PastPresentationTimingInfo, VkPastPresentationTimingPropertiesEXT* PastPresentationTimingProperties)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetPastPresentationTimingEXT()")));
#endif
	}
}

void FWrapLayer::GetSwapchainTimeDomainPropertiesEXT(VkResult Result, VkDevice Device, VkSwapchainKHR Swapchain, VkSwapchainTimeDomainPropertiesEXT* SwapchainTimeDomainProperties, uint64_t* TimeDomainsCounter)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetSwapchainTimeDomainPropertiesEXT()")));
#endif
	}
}

void FWrapLayer::GetSwapchainTimingPropertiesEXT(VkResult Result, VkDevice Device, VkSwapchainKHR Swapchain, VkSwapchainTimingPropertiesEXT* SwapchainTimingProperties, uint64_t* SwapchainTimingPropertiesCounter)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetSwapchainTimingPropertiesEXT()")));
#endif
	}
}

void FWrapLayer::SetSwapchainPresentTimingQueueSizeEXT(VkResult Result, VkDevice Device, VkSwapchainKHR Swapchain, uint32_t Size)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("SetSwapchainPresentTimingQueueSizeEXT()")));
#endif
	}
}

void FWrapLayer::CmdBindResourceHeapEXT(VkResult Result, VkCommandBuffer CommandBuffer, const VkBindHeapInfoEXT* BindInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdBindResourceHeapEXT()")));
#endif
	}
}

void FWrapLayer::CmdBindSamplerHeapEXT(VkResult Result, VkCommandBuffer CommandBuffer, const VkBindHeapInfoEXT* BindInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdBindSamplerHeapEXT()")));
#endif
	}
}

void FWrapLayer::CmdPushDataEXT(VkResult Result, VkCommandBuffer CommandBuffer, const VkPushDataInfoEXT* PushDataInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdPushDataEXT()")));
#endif
	}
}

void FWrapLayer::GetPhysicalDeviceDescriptorSizeEXT(vkDeviceSize OutSize, VkPhysicalDevice PhysicalDevice, VkDescriptorType DescriptorType)
{
	if (OutSize > 0)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("GetPhysicalDeviceDescriptorSizeEXT()")));
#endif
	}
}

void FWrapLayer::WriteResourceDescriptorsEXT(VkResult Result, VkDevice Device, uint32_t ResourceCount, const VkResourceDescriptorInfoEXT* Resources, const VkHostAddressRangeEXT* Descriptors)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("WriteResourceDescriptorsEXT()")));
#endif
	}
}

void FWrapLayer::WriteSamplerDescriptorsEXT(VkResult Result, VkDevice Device, uint32_t SamplerCount, const VkSamplerCreateInfo* Samplers, const VkHostAddressRangeEXT* Descriptors)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("WriteSamplerDescriptorsEXT()")));
#endif
	}
}

void FWrapLayer::CmdSetRenderingAttachmentLocationsKHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkRenderingAttachmentLocationInfo* LocationInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetRenderingAttachmentLocationsKHR()")));
#endif
	}
}

void FWrapLayer::CmdSetRenderingInputAttachmentIndicesKHR(VkResult Result, VkCommandBuffer CommandBuffer, const VkRenderingInputAttachmentIndexInfo* InputAttachmentIndexInfo)
{
	if (Result == VK_RESULT_MAX_ENUM)
	{
#if VULKAN_ENABLE_DUMP_LAYER
		PrintfBeginResult(FString::Printf(TEXT("CmdSetRenderingInputAttachmentIndicesKHR()")));
#endif
	}
}






#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER
namespace VulkanRHI
{
	void BindDebugLabelName(VkImage Image, const TCHAR* Name)
	{
		if(Image == VK_NULL_HANDLE)
		{
			return;
		}
		FScopeLock ScopeLock(&GTrackingCS);
		FTrackingImage* Found = GVulkanTrackingImageLayouts.Find(Image);
		if (Found)
		{
			Found->Info.DebugName = Name ? Name : TEXT("null");
		}
		else
		{
			ensure(0);
		}
	}
}
#endif

#endif	// VULKAN_ENABLE_WRAP_LAYER

#if VULKAN_ENABLE_DUMP_LAYER
namespace VulkanRHI
{
	static struct FGlobalDumpLog
	{
		~FGlobalDumpLog()
		{
			FlushDebugWrapperLog();
		}
	} GGlobalDumpLogInstance;

	static TArray<FString> GMarkers;
	void DumpLayerPushMarker(const TCHAR* InName)
	{
		FString String = "***** vkCmdDbgMarkerBeginEXT: ";
		for (auto& Name : GMarkers)
		{
			String += Name;
			String += TEXT("/");
		}
		GMarkers.Push(InName);
		String += InName;
		String += TEXT("\n");

		DebugLog += String;
		FlushDebugWrapperLog();
	}

	void DumpLayerPopMarker()
	{
		FString String = "***** vkCmdDbgMarkerEndEXT: ";
		GMarkers.Pop();
		for (auto& Name : GMarkers)
		{
			String += Name;
			String += TEXT("/");
		}

		String += TEXT("\n");

		DebugLog += String;
		FlushDebugWrapperLog();
	}
}
#endif	// VULKAN_ENABLE_DUMP_LAYER

#undef VULKAN_REPORT_LOG
#endif // VULKAN_HAS_DEBUGGING_ENABLED
