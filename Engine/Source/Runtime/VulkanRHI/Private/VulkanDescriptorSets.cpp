// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanDescriptorSets.h"
#include "VulkanDevice.h"
#include "VulkanBindlessDescriptorManager.h"

TAutoConsoleVariable<int32> GDynamicGlobalUBs(
	TEXT("r.Vulkan.DynamicGlobalUBs"),
	2,
	TEXT("2 to treat ALL uniform buffers as dynamic [default]\n")\
	TEXT("1 to treat global/packed uniform buffers as dynamic\n")\
	TEXT("0 to treat them as regular"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

template<bool bIsCompute>
void FVulkanDescriptorSetsLayoutInfo::FinalizeBindings(const FVulkanDevice& Device, const FUniformBufferGatherInfo& UBGatherInfo, bool bUsesBindless)
{
	// We'll be reusing this struct
	VkDescriptorSetLayoutBinding Binding;
	FMemory::Memzero(Binding);
	Binding.descriptorCount = 1;

	const bool bConvertAllUBsToDynamic = !bUsesBindless && (GDynamicGlobalUBs.GetValueOnAnyThread() > 1);
	const bool bConvertPackedUBsToDynamic = !bUsesBindless && (bConvertAllUBsToDynamic || (GDynamicGlobalUBs.GetValueOnAnyThread() == 1));
	const uint32 MaxDescriptorSetUniformBuffersDynamic = Device.GetLimits().maxDescriptorSetUniformBuffersDynamic;

	for (int32 Stage = 0; Stage < (bIsCompute ? ShaderStage::NumComputeStages : ShaderStage::NumGraphicsStages); ++Stage)
	{
		checkSlow(StageInfos[Stage].IsEmpty());

		if (const FVulkanShaderHeader* ShaderHeader = UBGatherInfo.CodeHeaders[Stage])
		{
			FStageInfo& StageInfo = StageInfos[Stage];

			Binding.stageFlags = UEFrequencyToVKStageBit(bIsCompute ? SF_Compute : ShaderStage::GetFrequencyForGfxStage((ShaderStage::EStage)Stage));

			StageInfo.PackedGlobalsSize = ShaderHeader->PackedGlobalsSize;
			StageInfo.NumBoundUniformBuffers = ShaderHeader->NumBoundUniformBuffers;

			for (int32 BindingIndex = 0; BindingIndex < ShaderHeader->Bindings.Num(); ++BindingIndex)
			{
				const VkDescriptorType DescriptorType = (VkDescriptorType)ShaderHeader->Bindings[BindingIndex].DescriptorType;

				const bool bIsUniformBuffer = (DescriptorType == VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER);
				const bool bIsGlobalPackedConstants = bIsUniformBuffer && ShaderHeader->PackedGlobalsSize && (BindingIndex == 0);

				if (bIsGlobalPackedConstants)
				{
					const VkDescriptorType UBType = bConvertPackedUBsToDynamic ? VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC : VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER;

					const uint32 NewBindingIndex = StageInfo.Types.Add(UBType);
					checkf(NewBindingIndex == 0, TEXT("Packed globals should always be the first binding!"));

					Binding.binding = NewBindingIndex;
					Binding.descriptorType = UBType;
					AddDescriptor(Stage, Binding);
				}
				else if (bIsUniformBuffer)
				{
					VkDescriptorType UBType = VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER;
					if (bConvertAllUBsToDynamic && LayoutTypes[VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC] < MaxDescriptorSetUniformBuffersDynamic)
					{
						UBType = VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC;
					}

					// Here we might mess up with the stageFlags, so reset them every loop
					Binding.descriptorType = UBType;
					const FVulkanShaderHeader::FUniformBufferInfo& UBInfo = ShaderHeader->UniformBufferInfos[BindingIndex];
					const bool bUBHasConstantData = (BindingIndex < (int32)ShaderHeader->NumBoundUniformBuffers);
					if (bUBHasConstantData)
					{
						const uint32 NewBindingIndex = StageInfo.Types.Add(UBType);
						check(NewBindingIndex == BindingIndex);
						Binding.binding = NewBindingIndex;
						AddDescriptor(Stage, Binding);
					}
				}
				else
				{
					const uint32 NewTypeIndex = StageInfo.Types.Add(DescriptorType);
					check(NewTypeIndex == BindingIndex);
					Binding.binding = BindingIndex;
					Binding.descriptorType = DescriptorType;
					AddDescriptor(Stage, Binding);
				}
			}
		}
	}

	CompileTypesUsageID();
	GenerateHash(bIsCompute ? VK_PIPELINE_BIND_POINT_COMPUTE : VK_PIPELINE_BIND_POINT_GRAPHICS);
}

template void FVulkanDescriptorSetsLayoutInfo::FinalizeBindings<true>(const FVulkanDevice& Device, const FUniformBufferGatherInfo& UBGatherInfo, bool bUsesBindless);
template void FVulkanDescriptorSetsLayoutInfo::FinalizeBindings<false>(const FVulkanDevice& Device, const FUniformBufferGatherInfo& UBGatherInfo, bool bUsesBindless);


// Increments a value and asserts on overflow.
// FSetInfo uses narrow integer types for descriptor counts,
// which may feasibly overflow one day (for example if we add bindless resources).
template <typename T>
static void IncrementChecked(T& Value)
{
	check(Value < TNumericLimits<T>::Max());
	++Value;
}

void FVulkanDescriptorSetsLayoutInfo::AddDescriptor(int32 DescriptorSetIndex, const VkDescriptorSetLayoutBinding& Descriptor)
{
	// Increment type usage
	if (LayoutTypes.Contains(Descriptor.descriptorType))
	{
		LayoutTypes[Descriptor.descriptorType]++;
	}
	else
	{
		LayoutTypes.Add(Descriptor.descriptorType, 1);
	}

	if (DescriptorSetIndex >= SetLayouts.Num())
	{
		SetLayouts.SetNum(DescriptorSetIndex + 1, EAllowShrinking::No);
	}

	FSetLayout& DescSetLayout = SetLayouts[DescriptorSetIndex];

	VkDescriptorSetLayoutBinding* Binding = new(DescSetLayout.LayoutBindings) VkDescriptorSetLayoutBinding;
	*Binding = Descriptor;

	const FStageInfo& SetInfo = StageInfos[DescriptorSetIndex];
	check(SetInfo.Types[Descriptor.binding] == Descriptor.descriptorType);
	switch (Descriptor.descriptorType)
	{
	case VK_DESCRIPTOR_TYPE_SAMPLER:
	case VK_DESCRIPTOR_TYPE_COMBINED_IMAGE_SAMPLER:
	case VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE:
	case VK_DESCRIPTOR_TYPE_STORAGE_IMAGE:
	case VK_DESCRIPTOR_TYPE_INPUT_ATTACHMENT:
		IncrementChecked(StageInfos[DescriptorSetIndex].NumImageInfos);
		break;
	case VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER:
	case VK_DESCRIPTOR_TYPE_STORAGE_BUFFER:
	case VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC:
		IncrementChecked(StageInfos[DescriptorSetIndex].NumBufferInfos);
		break;
	case VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR:
		IncrementChecked(StageInfos[DescriptorSetIndex].NumAccelerationStructures);
		break;
	case VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER:
	case VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER:
		break;
	default:
		checkf(0, TEXT("Unsupported descriptor type %d"), (int32)Descriptor.descriptorType);
		break;
	}
}

void FVulkanDescriptorSetsLayoutInfo::GenerateHash(VkPipelineBindPoint InBindPoint)
{
	const int32 LayoutCount = SetLayouts.Num();
	Hash = FCrc::MemCrc32(&TypesUsageID, sizeof(uint32), LayoutCount);

	for (int32 layoutIndex = 0; layoutIndex < LayoutCount; ++layoutIndex)
	{
		SetLayouts[layoutIndex].GenerateHash();
		Hash = FCrc::MemCrc32(&SetLayouts[layoutIndex].Hash, sizeof(uint32), Hash);
	}

	const uint32 NumStages = GetNumStagesForBindPoint(InBindPoint);
	for (uint32 RemapingIndex = 0; RemapingIndex < NumStages; ++RemapingIndex)
	{
		const FStageInfo& StageInfo = StageInfos[RemapingIndex];

		Hash = FCrc::TypeCrc32(StageInfo.PackedGlobalsSize, Hash);
		Hash = FCrc::TypeCrc32(StageInfo.NumBoundUniformBuffers, Hash);
		Hash = FCrc::TypeCrc32(StageInfo.NumImageInfos, Hash);
		Hash = FCrc::TypeCrc32(StageInfo.NumBufferInfos, Hash);
		Hash = FCrc::TypeCrc32(StageInfo.NumAccelerationStructures, Hash);

		const TArray<VkDescriptorType>& Types = StageInfo.Types;
		Hash = FCrc::MemCrc32(Types.GetData(), sizeof(VkDescriptorType) * Types.Num(), Hash);
	}

	// It would be better to store this when the object is created, but it's not available at that time, so we'll do it here.
	BindPoint = InBindPoint;

	// Include the bind point in the hash, because we can have graphics and compute PSOs with the same descriptor info, and we don't want them to collide.
	Hash = FCrc::MemCrc32(&BindPoint, sizeof(BindPoint), Hash);
}

static FCriticalSection GTypesUsageCS;
void FVulkanDescriptorSetsLayoutInfo::CompileTypesUsageID()
{
	FScopeLock ScopeLock(&GTypesUsageCS);

	static TMap<uint32, uint32> GTypesUsageHashMap;
	static uint32 GUniqueID = 1;

	LayoutTypes.KeySort([](VkDescriptorType A, VkDescriptorType B)
		{
			return static_cast<uint32>(A) < static_cast<uint32>(B);
		});

	uint32 TypesUsageHash = 0;
	for (const auto& Elem : LayoutTypes)
	{
		TypesUsageHash = FCrc::MemCrc32(&Elem.Value, sizeof(uint32), TypesUsageHash);
	}

	uint32* UniqueID = GTypesUsageHashMap.Find(TypesUsageHash);
	if (UniqueID == nullptr)
	{
		TypesUsageID = GTypesUsageHashMap.Add(TypesUsageHash, GUniqueID++);
	}
	else
	{
		TypesUsageID = *UniqueID;
	}
}

// 
// FVulkanDescriptorSetsLayout

FVulkanDescriptorSetsLayout::FVulkanDescriptorSetsLayout(FVulkanDevice* InDevice) :
	Device(InDevice)
{
}

FVulkanDescriptorSetsLayout::~FVulkanDescriptorSetsLayout()
{
	// Handles are owned by FVulkanPipelineStateCacheManager
	LayoutHandles.Reset(0);
}

void FVulkanDescriptorSetsLayout::Compile(FVulkanDescriptorSetLayoutMap& DSetLayoutMap)
{
	check(LayoutHandles.Num() == 0);

	// Check if we obey limits
	const VkPhysicalDeviceLimits& Limits = Device->GetLimits();

	// Check for maxDescriptorSetSamplers
	check(LayoutTypes[VK_DESCRIPTOR_TYPE_SAMPLER]
		+ LayoutTypes[VK_DESCRIPTOR_TYPE_COMBINED_IMAGE_SAMPLER]
			<= Limits.maxDescriptorSetSamplers);

	// Check for maxDescriptorSetUniformBuffers
	check(LayoutTypes[VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER]
		+ LayoutTypes[VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC]
			<= Limits.maxDescriptorSetUniformBuffers);

	// Check for maxDescriptorSetUniformBuffersDynamic
	check(Device->GetVendorId() == EGpuVendorId::Amd ||
		LayoutTypes[VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC]
		<= Limits.maxDescriptorSetUniformBuffersDynamic);

	// Check for maxDescriptorSetStorageBuffers
	check(LayoutTypes[VK_DESCRIPTOR_TYPE_STORAGE_BUFFER]
		+ LayoutTypes[VK_DESCRIPTOR_TYPE_STORAGE_BUFFER_DYNAMIC]
			<= Limits.maxDescriptorSetStorageBuffers);

	// Check for maxDescriptorSetStorageBuffersDynamic
	check(LayoutTypes[VK_DESCRIPTOR_TYPE_STORAGE_BUFFER_DYNAMIC]
		<= Limits.maxDescriptorSetStorageBuffersDynamic);

	// Check for maxDescriptorSetSampledImages
	check(LayoutTypes[VK_DESCRIPTOR_TYPE_COMBINED_IMAGE_SAMPLER]
		+ LayoutTypes[VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE]
			+ LayoutTypes[VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER]
				<= Limits.maxDescriptorSetSampledImages);

	// Check for maxDescriptorSetStorageImages
	check(LayoutTypes[VK_DESCRIPTOR_TYPE_STORAGE_IMAGE]
		+ LayoutTypes[VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER]
			<= Limits.maxDescriptorSetStorageImages);

	check(LayoutTypes[VK_DESCRIPTOR_TYPE_INPUT_ATTACHMENT] <= Limits.maxDescriptorSetInputAttachments);

	if (GRHISupportsRayTracing)
	{
		check(LayoutTypes[VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR] < Device->GetOptionalExtensionProperties().AccelerationStructureProps.maxDescriptorSetAccelerationStructures);
	}

	LayoutHandles.Empty(SetLayouts.Num());

	if (UseVulkanDescriptorCache())
	{
		LayoutHandleIds.Empty(SetLayouts.Num());
	}

	for (FSetLayout& Layout : SetLayouts)
	{
		VkDescriptorSetLayout* LayoutHandle = new(LayoutHandles) VkDescriptorSetLayout;

		uint32* LayoutHandleId = nullptr;
		if (UseVulkanDescriptorCache())
		{
			LayoutHandleId = new(LayoutHandleIds) uint32;
		}

		if (FVulkanDescriptorSetLayoutEntry* Found = DSetLayoutMap.Find(Layout))
		{
			*LayoutHandle = Found->Handle;
			if (LayoutHandleId)
			{
				*LayoutHandleId = Found->HandleId;
			}
			continue;
		}

		VkDescriptorSetLayoutCreateInfo DescriptorLayoutInfo;
		ZeroVulkanStruct(DescriptorLayoutInfo, VK_STRUCTURE_TYPE_DESCRIPTOR_SET_LAYOUT_CREATE_INFO);
		DescriptorLayoutInfo.bindingCount = Layout.LayoutBindings.Num();
		DescriptorLayoutInfo.pBindings = Layout.LayoutBindings.GetData();

		VERIFYVULKANRESULT(VulkanRHI::vkCreateDescriptorSetLayout(Device->GetHandle(), &DescriptorLayoutInfo, VULKAN_CPU_ALLOCATOR, LayoutHandle));

		if (LayoutHandleId)
		{
			*LayoutHandleId = ++GVulkanDSetLayoutHandleIdCounter;
		}

		FVulkanDescriptorSetLayoutEntry DescriptorSetLayoutEntry;
		DescriptorSetLayoutEntry.Handle = *LayoutHandle;
		DescriptorSetLayoutEntry.HandleId = LayoutHandleId ? *LayoutHandleId : 0;

		DSetLayoutMap.Add(Layout, DescriptorSetLayoutEntry);
	}

	if (TypesUsageID == ~0)
	{
		CompileTypesUsageID();
	}

	ZeroVulkanStruct(DescriptorSetAllocateInfo, VK_STRUCTURE_TYPE_DESCRIPTOR_SET_ALLOCATE_INFO);
	DescriptorSetAllocateInfo.descriptorSetCount = LayoutHandles.Num();
	DescriptorSetAllocateInfo.pSetLayouts = LayoutHandles.GetData();
}

//
// FVulkanDescriptorSetWriter

uint32 FVulkanDescriptorSetWriter::SetupDescriptorWrites(
	const TArray<VkDescriptorType>& Types, FVulkanHashableDescriptorInfo* InHashableDescriptorInfos,
	VkWriteDescriptorSet* InWriteDescriptors, VkDescriptorImageInfo* InImageInfo, VkDescriptorBufferInfo* InBufferInfo, uint8* InBindingToDynamicOffsetMap,
	VkWriteDescriptorSetAccelerationStructureKHR* InAccelerationStructuresWriteDescriptors,
	VkAccelerationStructureKHR* InAccelerationStructures,
	const FVulkanSamplerState& DefaultSampler, const FVulkanView::FTextureView& DefaultImageView)
{
	HashableDescriptorInfos = InHashableDescriptorInfos;
	WriteDescriptors = InWriteDescriptors;
	NumWrites = Types.Num();

	BindingToDynamicOffsetMap = InBindingToDynamicOffsetMap;

	InitWrittenMasks(NumWrites);

	uint32 DynamicOffsetIndex = 0;

	for (int32 Index = 0; Index < Types.Num(); ++Index)
	{
		InWriteDescriptors->sType = VK_STRUCTURE_TYPE_WRITE_DESCRIPTOR_SET;
		InWriteDescriptors->dstBinding = Index;
		InWriteDescriptors->descriptorCount = 1;
		InWriteDescriptors->descriptorType = Types[Index];

		switch (Types[Index])
		{
		case VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC:
			BindingToDynamicOffsetMap[Index] = DynamicOffsetIndex;
			++DynamicOffsetIndex;
			InWriteDescriptors->pBufferInfo = InBufferInfo++;
			break;
		case VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER:
		case VK_DESCRIPTOR_TYPE_STORAGE_BUFFER:
			InWriteDescriptors->pBufferInfo = InBufferInfo++;
			break;
		case VK_DESCRIPTOR_TYPE_SAMPLER:
			SetWrittenBase(Index); //samplers have a default setting, don't assert on those yet.
		case VK_DESCRIPTOR_TYPE_COMBINED_IMAGE_SAMPLER:
		case VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE:
		case VK_DESCRIPTOR_TYPE_STORAGE_IMAGE:
		case VK_DESCRIPTOR_TYPE_INPUT_ATTACHMENT:
			// Texture.Load() still requires a default sampler...
			if (InHashableDescriptorInfos) // UseVulkanDescriptorCache()
			{
				InHashableDescriptorInfos[Index].Image.SamplerId = DefaultSampler.SamplerId;
				InHashableDescriptorInfos[Index].Image.ImageViewId = DefaultImageView.ViewId;
				InHashableDescriptorInfos[Index].Image.ImageLayout = static_cast<uint32>(VK_IMAGE_LAYOUT_GENERAL);
			}
			InImageInfo->sampler = DefaultSampler.Sampler;
			InImageInfo->imageView = DefaultImageView.View;
			InImageInfo->imageLayout = VK_IMAGE_LAYOUT_GENERAL;
			InWriteDescriptors->pImageInfo = InImageInfo++;
			break;
		case VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER:
		case VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER:
			break;
		case VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR:
			InAccelerationStructuresWriteDescriptors->sType = VK_STRUCTURE_TYPE_WRITE_DESCRIPTOR_SET_ACCELERATION_STRUCTURE_KHR;
			InAccelerationStructuresWriteDescriptors->pNext = nullptr;
			InAccelerationStructuresWriteDescriptors->accelerationStructureCount = 1;
			InAccelerationStructuresWriteDescriptors->pAccelerationStructures = InAccelerationStructures++;
			InWriteDescriptors->pNext = InAccelerationStructuresWriteDescriptors++;
			break;
		default:
			checkf(0, TEXT("Unsupported descriptor type %d"), (int32)Types[Index]);
			break;
		}
		++InWriteDescriptors;
	}

	return DynamicOffsetIndex;
}

void FVulkanDescriptorSetWriter::CheckAllWritten()
{
#if VULKAN_VALIDATE_DESCRIPTORS_WRITTEN
	auto GetVkDescriptorTypeString = [](VkDescriptorType Type)
		{
			switch (Type)
			{
				// + 19 to skip "VK_DESCRIPTOR_TYPE_"
#define VKSWITCHCASE(x)	case x: return FString(&TEXT(#x)[19]);
				VKSWITCHCASE(VK_DESCRIPTOR_TYPE_SAMPLER)
					VKSWITCHCASE(VK_DESCRIPTOR_TYPE_COMBINED_IMAGE_SAMPLER)
					VKSWITCHCASE(VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE)
					VKSWITCHCASE(VK_DESCRIPTOR_TYPE_STORAGE_IMAGE)
					VKSWITCHCASE(VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER)
					VKSWITCHCASE(VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER)
					VKSWITCHCASE(VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER)
					VKSWITCHCASE(VK_DESCRIPTOR_TYPE_STORAGE_BUFFER)
					VKSWITCHCASE(VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC)
					VKSWITCHCASE(VK_DESCRIPTOR_TYPE_STORAGE_BUFFER_DYNAMIC)
					VKSWITCHCASE(VK_DESCRIPTOR_TYPE_INPUT_ATTACHMENT)
#undef VKSWITCHCASE
			default:
				break;
			}

			return FString::Printf(TEXT("Unknown VkDescriptorType %d"), (int32)Type);
		};

	const uint32 Writes = NumWrites;
	if (Writes == 0)
		return;

	auto GetWrittenMask = [WrittenMask=WrittenMask, ExcludedMask=ExcludedMask](int32 Index)
	{
			return WrittenMask[Index] | ExcludedMask[Index];
	};

	bool bFail = false;
	if (Writes <= 32) //early out for the most common case.
	{
		bFail = GetWrittenMask(0) != ((1llu << Writes) - 1);
	}
	else
	{
		const int32 Last = int32(WrittenMask.Num() - 1);
		for (int32 i = 0; !bFail && i < Last; ++i)
		{
			const uint64 Mask = GetWrittenMask(i);
			bFail = bFail || Mask != 0xffffffff;
		}

		const uint32 TailCount = Writes - (Last * 32);
		check(TailCount != 0);
		const uint32 TailMask = (1llu << TailCount) - 1;
		bFail = bFail || TailMask != GetWrittenMask(Last);
	}

	if (bFail)
	{
		FString Descriptors;
		for (uint32 i = 0; i < Writes; ++i)
		{
			uint32 Index = i / 32;
			uint32 Mask = i % 32;
			if (0 == (WrittenMask[Index] & (1llu << Mask)))
			{
				FString TypeString = GetVkDescriptorTypeString(WriteDescriptors[i].descriptorType);
				Descriptors += FString::Printf(TEXT("\t\tDescriptorWrite %d/%d Was not written(Type %s)\n"), i, NumWrites, *TypeString);
			}
		}
		UE_LOGF(LogVulkanRHI, Warning, "Not All descriptors where filled out. this can/will cause a driver crash\n%ls\n", *Descriptors);
		ensureMsgf(false, TEXT("Not All descriptors where filled out. this can/will cause a driver crash\n%s\n"), *Descriptors);
	}
#endif
}

void FVulkanDescriptorSetWriter::Reset()
{
	bHasVolatileResources = false;

#if VULKAN_VALIDATE_DESCRIPTORS_WRITTEN
	WrittenMask = BaseWrittenMask;
#endif
}
void FVulkanDescriptorSetWriter::SetWritten(uint32 DescriptorIndex)
{
#if VULKAN_VALIDATE_DESCRIPTORS_WRITTEN
	uint32 Index = DescriptorIndex / 32;
	uint32 Mask = DescriptorIndex % 32;
	WrittenMask[Index] |= (1 << Mask);
#endif
}
void FVulkanDescriptorSetWriter::SetWrittenBase(uint32 DescriptorIndex)
{
#if VULKAN_VALIDATE_DESCRIPTORS_WRITTEN	
	uint32 Index = DescriptorIndex / 32;
	uint32 Mask = DescriptorIndex % 32;
	BaseWrittenMask[Index] |= (1 << Mask);
#endif
}

void FVulkanDescriptorSetWriter::InitWrittenMasks(uint32 NumDescriptorWrites)
{
#if VULKAN_VALIDATE_DESCRIPTORS_WRITTEN	
	uint32 Size = (NumDescriptorWrites + 31) / 32;
	WrittenMask.Empty(Size);
	WrittenMask.SetNumZeroed(Size);
	BaseWrittenMask.Empty(Size);
	BaseWrittenMask.SetNumZeroed(Size);
	ExcludedMask.Empty(Size);
	ExcludedMask.SetNumZeroed(Size);
#endif
}

void FVulkanDescriptorSetWriter::SetExcludedResources(const TArrayView<uint32> ExcludedResourceCount)
{
#if VULKAN_VALIDATE_DESCRIPTORS_WRITTEN	
	for (uint32 DescriptorIndex : ExcludedResourceCount)
	{
		const uint32 Index = DescriptorIndex / 32;
		const uint32 Mask = DescriptorIndex % 32;
		ExcludedMask[Index] |= (1 << Mask);
	}
#endif
}

bool FVulkanDescriptorSetWriter::WriteBindlessUniformBuffer(FVulkanDevice& Device, uint32 DescriptorIndex, const FVulkanUniformBuffer* UniformBuffer)
{
	if (Device.GetBindlessDescriptorManager()->UseDescriptorHeaps())
	{
		BindlessUBIndices[DescriptorIndex] = UniformBuffer->GetBindlessHandle().GetIndex();
		return true;
	}
	else
	{
		const VulkanRHI::FVulkanAllocation& Allocation = UniformBuffer->GetAllocation();
		const VkDeviceSize Range = UniformBuffer->IsUniformView() ? PLATFORM_MAX_UNIFORM_BUFFER_RANGE : UniformBuffer->GetSize();
		return WriteUniformBuffer(DescriptorIndex, Allocation.GetBufferHandle(), Allocation.HandleId, UniformBuffer->GetOffset(), Range);
	}
}
