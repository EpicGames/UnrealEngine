// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "VulkanConfiguration.h"
#include "VulkanDevice.h"
#include "VulkanMemory.h"
#include "VulkanRHIPrivate.h"
#include "VulkanShaderResources.h"

#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#define VULKAN_VALIDATE_DESCRIPTORS_WRITTEN 0
#else
#define VULKAN_VALIDATE_DESCRIPTORS_WRITTEN RHI_INCLUDE_SHADER_DEBUG_DATA
#endif


class FVulkanCommandBufferPool;

// (AlwaysCompareData == true) because of CRC-32 hash collisions
class FVulkanDSetKey : public TDataKey<FVulkanDSetKey, true> {};
class FVulkanDSetsKey : public TDataKey<FVulkanDSetsKey, true> {};

struct FUniformBufferGatherInfo
{
	const FVulkanShaderHeader* CodeHeaders[ShaderStage::MaxNumStages]{};
};


// Information for the layout of descriptor sets; does not hold runtime objects
class FVulkanDescriptorSetsLayoutInfo
{
public:
	FVulkanDescriptorSetsLayoutInfo()
	{
		// Add expected descriptor types
		for (uint32 i = VK_DESCRIPTOR_TYPE_BEGIN_RANGE; i <= VK_DESCRIPTOR_TYPE_END_RANGE; ++i)
		{
			LayoutTypes.Add(static_cast<VkDescriptorType>(i), 0);
		}

		LayoutTypes.Add(VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR, 0);
	}

	inline uint32 GetTypesUsed(VkDescriptorType Type) const
	{
		if (LayoutTypes.Contains(Type))
		{
			return LayoutTypes[Type];
		}
		else
		{
			return 0;
		}
	}

	struct FSetLayout
	{
		TArray<VkDescriptorSetLayoutBinding> LayoutBindings;
		uint32 Hash;

		inline void GenerateHash()
		{
			Hash = FCrc::MemCrc32(LayoutBindings.GetData(), sizeof(VkDescriptorSetLayoutBinding) * LayoutBindings.Num());
		}

		friend uint32 GetTypeHash(const FSetLayout& In)
		{
			return In.Hash;
		}

		inline bool operator == (const FSetLayout& In) const
		{
			if (In.Hash != Hash)
			{
				return false;
			}

			const int32 NumBindings = LayoutBindings.Num();
			if (In.LayoutBindings.Num() != NumBindings)
			{
				return false;
			}

			if (NumBindings != 0 && FMemory::Memcmp(In.LayoutBindings.GetData(), LayoutBindings.GetData(), NumBindings * sizeof(VkDescriptorSetLayoutBinding)) != 0)
			{
				return false;
			}

			return true;
		}

		inline bool operator != (const FSetLayout& In) const
		{
			return !(*this == In);
		}
	};

	const TArray<FSetLayout>& GetLayouts() const
	{
		return SetLayouts;
	}

	template<bool bIsCompute>
	void FinalizeBindings(const FVulkanDevice& Device, const FUniformBufferGatherInfo& UBGatherInfo, bool bUsesBindless);

	void GenerateHash(VkPipelineBindPoint InBindPoint);

	friend uint32 GetTypeHash(const FVulkanDescriptorSetsLayoutInfo& In)
	{
		return In.Hash;
	}

	inline bool operator == (const FVulkanDescriptorSetsLayoutInfo& In) const
	{
		if (In.Hash != Hash)
		{
			return false;
		}

		if (In.BindPoint != BindPoint)
		{
			return false;
		}

		if (In.SetLayouts.Num() != SetLayouts.Num())
		{
			return false;
		}

		if (In.TypesUsageID != TypesUsageID)
		{
			return false;
		}

		for (int32 Index = 0; Index < In.SetLayouts.Num(); ++Index)
		{
			if (In.SetLayouts[Index] != SetLayouts[Index])
			{
				return false;
			}
		}

		if (StageInfos != In.StageInfos)
		{
			return false;
		}

		return true;
	}

	void CopyFrom(const FVulkanDescriptorSetsLayoutInfo& Info)
	{
		LayoutTypes = Info.LayoutTypes;
		Hash = Info.Hash;
		TypesUsageID = Info.TypesUsageID;
		SetLayouts = Info.SetLayouts;
		StageInfos = Info.StageInfos;
	}

	inline const TMap<VkDescriptorType, uint32>& GetLayoutTypes() const
	{
		return LayoutTypes;
	}

	inline uint32 GetTypesUsageID() const
	{
		return TypesUsageID;
	}

	inline bool HasInputAttachments() const
	{
		return GetTypesUsed(VK_DESCRIPTOR_TYPE_INPUT_ATTACHMENT) > 0;
	}

	struct FStageInfo
	{
		TArray<VkDescriptorType>	Types;
		uint32						PackedGlobalsSize = 0;
		uint32						NumBoundUniformBuffers = 0;
		uint16						NumImageInfos = 0;
		uint16						NumBufferInfos = 0;
		uint16						NumAccelerationStructures = 0;

		inline bool IsEmpty() const
		{
			if ((Types.Num() != 0) || (PackedGlobalsSize != 0) || (NumBoundUniformBuffers != 0))
			{
				return false;
			}

			return true;
		}

		inline bool operator==(const FStageInfo& In) const
		{
			if (PackedGlobalsSize != In.PackedGlobalsSize ||
				NumBoundUniformBuffers != In.NumBoundUniformBuffers ||
				NumBufferInfos != In.NumBufferInfos ||
				NumImageInfos != In.NumImageInfos ||
				NumAccelerationStructures != In.NumAccelerationStructures ||
				Types.Num() != In.Types.Num() ||
				FMemory::Memcmp(Types.GetData(), In.Types.GetData(), Types.NumBytes()))
			{
				return false;
			}

			return true;
		}
	};
	TStaticArray<FStageInfo, ShaderStage::MaxNumStages> StageInfos;

protected:
	TMap<VkDescriptorType, uint32> LayoutTypes;
	TArray<FSetLayout> SetLayouts;

	uint32 Hash = 0;

	uint32 TypesUsageID = ~0;

	VkPipelineBindPoint BindPoint = VK_PIPELINE_BIND_POINT_MAX_ENUM;

	void CompileTypesUsageID();

	void AddDescriptor(int32 DescriptorSetIndex, const VkDescriptorSetLayoutBinding& Descriptor);

	friend class FVulkanPipelineStateCacheManager;
	friend class FVulkanCommonPipelineDescriptorState;
	friend class FVulkanLayout;
};

struct FVulkanDescriptorSetLayoutEntry
{
	VkDescriptorSetLayout Handle = 0;
	uint32 HandleId = 0;
};

using FVulkanDescriptorSetLayoutMap = TMap<FVulkanDescriptorSetsLayoutInfo::FSetLayout, FVulkanDescriptorSetLayoutEntry>;

// The actual run-time descriptor set layouts
class FVulkanDescriptorSetsLayout : public FVulkanDescriptorSetsLayoutInfo
{
public:
	FVulkanDescriptorSetsLayout(FVulkanDevice* InDevice);
	~FVulkanDescriptorSetsLayout();

	// Can be called only once, the idea is that the Layout remains fixed.
	void Compile(FVulkanDescriptorSetLayoutMap& DSetLayoutMap);

	inline const TArray<VkDescriptorSetLayout>& GetHandles() const
	{
		return LayoutHandles;
	}

	inline const TArray<uint32>& GetHandleIds() const
	{
		return LayoutHandleIds;
	}

	inline const VkDescriptorSetAllocateInfo& GetAllocateInfo() const
	{
		return DescriptorSetAllocateInfo;
	}

	inline uint32 GetHash() const
	{
		return Hash;
	}

private:
	FVulkanDevice* Device;
	//uint32 Hash = 0;
	TArray<VkDescriptorSetLayout> LayoutHandles;
	TArray<uint32> LayoutHandleIds;
	VkDescriptorSetAllocateInfo DescriptorSetAllocateInfo;
};

class FVulkanDescriptorPool
{
public:
	FVulkanDescriptorPool(FVulkanDevice* InDevice, const FVulkanDescriptorSetsLayout& Layout, uint32 MaxSetsAllocations);
	~FVulkanDescriptorPool();

	inline VkDescriptorPool GetHandle() const
	{
		return DescriptorPool;
	}

	inline bool CanAllocate(const FVulkanDescriptorSetsLayout& InLayout) const
	{
		return MaxDescriptorSets > NumAllocatedDescriptorSets + InLayout.GetLayouts().Num();
	}

	void TrackAddUsage(const FVulkanDescriptorSetsLayout& InLayout);
	void TrackRemoveUsage(const FVulkanDescriptorSetsLayout& InLayout);

	inline bool IsEmpty() const
	{
		return NumAllocatedDescriptorSets == 0;
	}

	void Reset();
	bool AllocateDescriptorSets(const VkDescriptorSetAllocateInfo& InDescriptorSetAllocateInfo, VkDescriptorSet* OutSets);
	inline uint32 GetNumAllocatedDescriptorSets() const
	{
		return NumAllocatedDescriptorSets;
	}

private:
	FVulkanDevice* Device;

	uint32 MaxDescriptorSets;
	uint32 NumAllocatedDescriptorSets;
	uint32 PeakAllocatedDescriptorSets;

	// Tracks number of allocated types, to ensure that we are not exceeding our allocated limit
	const FVulkanDescriptorSetsLayout& Layout;
	VkDescriptorPool DescriptorPool;

	friend class FVulkanCommandListContext;
};


class FVulkanTypedDescriptorPoolSet
{
	typedef TList<FVulkanDescriptorPool*> FPoolList;

	FVulkanDescriptorPool* GetFreePool(bool bForceNewPool = false);
	FVulkanDescriptorPool* PushNewPool();

protected:
	friend class FVulkanDescriptorPoolSetContainer;
	friend class FVulkanCommandBuffer;

	FVulkanTypedDescriptorPoolSet(FVulkanDevice* InDevice, const FVulkanDescriptorSetsLayout& InLayout)
		: Device(InDevice)
		, Layout(InLayout)
		, PoolsCount(0)
	{
		PushNewPool();
	};

	~FVulkanTypedDescriptorPoolSet();

	void Reset();

public:
	bool AllocateDescriptorSets(const FVulkanDescriptorSetsLayout& Layout, VkDescriptorSet* OutSets);

private:
	FVulkanDevice* Device;
	const FVulkanDescriptorSetsLayout& Layout;
	uint32 PoolsCount;

	FPoolList* PoolListHead = nullptr;
	FPoolList* PoolListCurrent = nullptr;
};

class FVulkanDescriptorPoolSetContainer
{
public:
	FVulkanDescriptorPoolSetContainer(FVulkanDevice* InDevice)
		: Device(InDevice)
		, LastFrameUsed(GFrameNumberRenderThread)
		, bUsed(true)
	{
	}

	~FVulkanDescriptorPoolSetContainer();

	FVulkanTypedDescriptorPoolSet* AcquireTypedPoolSet(const FVulkanDescriptorSetsLayout& Layout);

	void Reset();

	inline void SetUsed(bool bInUsed)
	{
		bUsed = bInUsed;
		LastFrameUsed = bUsed ? GFrameNumberRenderThread : LastFrameUsed;
	}

	inline bool IsUnused() const
	{
		return !bUsed;
	}

	inline uint32 GetLastFrameUsed() const
	{
		return LastFrameUsed;
	}

private:
	FVulkanDevice* Device;

	TMap<uint32, FVulkanTypedDescriptorPoolSet*> TypedDescriptorPools;

	uint32 LastFrameUsed;
	bool bUsed;
};

class FVulkanDescriptorPoolsManager
{
	class FVulkanAsyncPoolSetDeletionWorker : public FNonAbandonableTask
	{
		FVulkanDescriptorPoolSetContainer* PoolSet;

	public:
		FVulkanAsyncPoolSetDeletionWorker(FVulkanDescriptorPoolSetContainer* InPoolSet)
			: PoolSet(InPoolSet)
		{};

		void DoWork()
		{
			check(PoolSet != nullptr);

			delete PoolSet;

			PoolSet = nullptr;
		}

		void SetPoolSet(FVulkanDescriptorPoolSetContainer* InPoolSet)
		{
			check(PoolSet == nullptr);
			PoolSet = InPoolSet;
		}

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FVulkanAsyncPoolSetDeletionWorker, STATGROUP_ThreadPoolAsyncTasks);
		}
	};

public:
	~FVulkanDescriptorPoolsManager();

	void Init(FVulkanDevice* InDevice)
	{
		Device = InDevice;
	}

	FVulkanDescriptorPoolSetContainer& AcquirePoolSetContainer();
	void ReleasePoolSet(FVulkanDescriptorPoolSetContainer& PoolSet);
	void GC();

private:
	FVulkanDevice* Device = nullptr;
	FAsyncTask<FVulkanAsyncPoolSetDeletionWorker>* AsyncDeletionTask = nullptr;

	FCriticalSection CS;
	TArray<FVulkanDescriptorPoolSetContainer*> PoolSets;
};

union FVulkanHashableDescriptorInfo
{
	struct
	{
		uint32 Max0;
		uint32 Max1;
		uint32 LayoutId;
	} Layout;
	struct
	{
		uint32 Id;
		uint32 Offset;
		uint32 Range;
	} Buffer;
	struct
	{
		uint32 SamplerId;
		uint32 ImageViewId;
		uint32 ImageLayout;
	} Image;
	struct
	{
		uint32 Id;
		uint32 Zero1;
		uint32 Zero2;
	} BufferView;
};

// This container holds the actual VkWriteDescriptorSet structures; a Compute pipeline uses the arrays 'as-is', whereas a 
// Gfx PSO will have one big array and chunk it depending on the stage (eg Vertex, Pixel).
struct FVulkanDescriptorSetWriteContainer
{
	TArray<FVulkanHashableDescriptorInfo> HashableDescriptorInfo;
	TArray<VkDescriptorImageInfo> DescriptorImageInfo;
	TArray<VkDescriptorBufferInfo> DescriptorBufferInfo;
	TArray<VkWriteDescriptorSet> DescriptorWrites;
	TArray<VkAccelerationStructureKHR> AccelerationStructures;
	TArray<VkWriteDescriptorSetAccelerationStructureKHR> AccelerationStructureWrites;

	TArray<uint8> BindingToDynamicOffsetMap;
};


// This class encapsulates updating VkWriteDescriptorSet structures (but doesn't own them), and their flags for dirty ranges; it is intended
// to be used to access a sub-region of a long array of VkWriteDescriptorSet (ie FVulkanDescriptorSetWriteContainer)
class FVulkanDescriptorSetWriter
{
public:
	FVulkanDescriptorSetWriter()
		: WriteDescriptors(nullptr)
		, BindingToDynamicOffsetMap(nullptr)
		, DynamicOffsets(nullptr)
		, NumWrites(0)
		, HashableDescriptorInfos(nullptr)
		, bIsKeyDirty(true)
	{
	}

	const FVulkanDSetKey& GetKey() const
	{
		check(UseVulkanDescriptorCache());
		if (bIsKeyDirty)
		{
			Key.GenerateFromData(HashableDescriptorInfos, sizeof(FVulkanHashableDescriptorInfo) * (NumWrites + 1)); // Add 1 for the Layout
			bIsKeyDirty = false;
		}
		return Key;
	}

	const VkWriteDescriptorSet* GetWriteDescriptors() const
	{
		return WriteDescriptors;
	}

	const uint32 GetNumWrites() const
	{
		return NumWrites;
	}

	bool WriteBindlessUniformBuffer(FVulkanDevice& Device, uint32 DescriptorIndex, const FVulkanUniformBuffer* UniformBuffer);

	bool WriteUniformBuffer(uint32 DescriptorIndex, VkBuffer BufferHandle, uint32 HandleId, VkDeviceSize Offset, VkDeviceSize Range)
	{
		return WriteBuffer<VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER>(DescriptorIndex, BufferHandle, HandleId, Offset, Range);
	}

	bool WriteDynamicUniformBuffer(uint32 DescriptorIndex, VkBuffer BufferHandle, uint32 HandleId, VkDeviceSize Offset, VkDeviceSize Range, uint32 DynamicOffset)
	{
		return WriteBuffer<VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC>(DescriptorIndex, BufferHandle, HandleId, Offset, Range, DynamicOffset);
	}

	bool WriteSampler(uint32 DescriptorIndex, const FVulkanSamplerState& Sampler)
	{
		check(DescriptorIndex < NumWrites);
		check(WriteDescriptors[DescriptorIndex].descriptorType == VK_DESCRIPTOR_TYPE_SAMPLER || WriteDescriptors[DescriptorIndex].descriptorType == VK_DESCRIPTOR_TYPE_COMBINED_IMAGE_SAMPLER);
		VkDescriptorImageInfo* ImageInfo = const_cast<VkDescriptorImageInfo*>(WriteDescriptors[DescriptorIndex].pImageInfo);
		check(ImageInfo);

		bool bChanged = false;
		if (UseVulkanDescriptorCache())
		{
			FVulkanHashableDescriptorInfo& HashableInfo = HashableDescriptorInfos[DescriptorIndex];
			check(Sampler.SamplerId > 0);
			if (HashableInfo.Image.SamplerId != Sampler.SamplerId)
			{
				HashableInfo.Image.SamplerId = Sampler.SamplerId;
				ImageInfo->sampler = Sampler.Sampler;
				bChanged = true;
			}
			bIsKeyDirty |= bChanged;
		}
		else
		{
			bChanged = CopyAndReturnNotEqual(ImageInfo->sampler, Sampler.Sampler);
		}

		return bChanged;
	}

	bool WriteImage(uint32 DescriptorIndex, const FVulkanView::FTextureView& TextureView, VkImageLayout Layout)
	{
		return WriteTextureView<VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE>(DescriptorIndex, TextureView, Layout);
	}

	bool WriteInputAttachment(uint32 DescriptorIndex, const FVulkanView::FTextureView& TextureView, VkImageLayout Layout)
	{
		return WriteTextureView<VK_DESCRIPTOR_TYPE_INPUT_ATTACHMENT>(DescriptorIndex, TextureView, Layout);
	}

	bool WriteStorageImage(uint32 DescriptorIndex, const FVulkanView::FTextureView& TextureView, VkImageLayout Layout)
	{
		return WriteTextureView<VK_DESCRIPTOR_TYPE_STORAGE_IMAGE>(DescriptorIndex, TextureView, Layout);
	}

	bool WriteStorageTexelBuffer(uint32 DescriptorIndex, const FVulkanView::FTypedBufferView& View)
	{
		return WriteBufferView<VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER>(DescriptorIndex, View);
	}

	bool WriteStorageBuffer(uint32 DescriptorIndex, const FVulkanView::FStructuredBufferView& View)
	{
		return WriteBuffer<VK_DESCRIPTOR_TYPE_STORAGE_BUFFER>(DescriptorIndex, View.Buffer, View.HandleId, View.Offset, View.Size);
	}

	bool WriteUniformTexelBuffer(uint32 DescriptorIndex, const FVulkanView::FTypedBufferView& View)
	{
		return WriteBufferView<VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER>(DescriptorIndex, View);
	}

	bool WriteAccelerationStructure(uint32 DescriptorIndex, VkAccelerationStructureKHR InAccelerationStructure)
	{
		checkf(!UseVulkanDescriptorCache(), TEXT("Descriptor cache path for WriteAccelerationStructure() is not implemented"));

		check(DescriptorIndex < NumWrites);
		SetWritten(DescriptorIndex);

		check(WriteDescriptors[DescriptorIndex].descriptorType == VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR);

		// Find the acceleration structure extension in the generic VkWriteDescriptorSet.
		const VkWriteDescriptorSetAccelerationStructureKHR* FoundWrite = nullptr;
		const VkBaseInStructure* Cursor = reinterpret_cast<const VkBaseInStructure*>(WriteDescriptors[DescriptorIndex].pNext);
		while (Cursor)
		{
			if (Cursor->sType == VK_STRUCTURE_TYPE_WRITE_DESCRIPTOR_SET_ACCELERATION_STRUCTURE_KHR)
			{
				FoundWrite = reinterpret_cast<const VkWriteDescriptorSetAccelerationStructureKHR*>(Cursor);
				break;
			}
			Cursor = Cursor->pNext;
		}

		checkf(FoundWrite,
			TEXT("Expected to find a VkWriteDescriptorSetAccelerationStructureKHR that's needed to bind an acceleration structure descriptor. ")
			TEXT("Possibly something went wrong in SetupDescriptorWrites()."));

		checkf(FoundWrite->accelerationStructureCount == 1, TEXT("Acceleration structure write operation is expected to contain exactly one descriptor"));

		VkAccelerationStructureKHR& AccelerationStructure = *const_cast<VkAccelerationStructureKHR*>(FoundWrite->pAccelerationStructures);

		bool bChanged = CopyAndReturnNotEqual(AccelerationStructure, InAccelerationStructure);

		return bChanged;
	}

	void SetDescriptorSet(VkDescriptorSet DescriptorSet)
	{
		for (uint32 Index = 0; Index < NumWrites; ++Index)
		{
			WriteDescriptors[Index].dstSet = DescriptorSet;
		}
	}

protected:
	template <VkDescriptorType DescriptorType>
	bool WriteBuffer(uint32 DescriptorIndex, VkBuffer BufferHandle, uint32 HandleId, VkDeviceSize Offset, VkDeviceSize Range, uint32 DynamicOffset = 0)
	{
		check(DescriptorIndex < NumWrites);
		SetWritten(DescriptorIndex);		
		if (DescriptorType == VK_DESCRIPTOR_TYPE_STORAGE_BUFFER)
		{
			checkf(WriteDescriptors[DescriptorIndex].descriptorType == VK_DESCRIPTOR_TYPE_STORAGE_BUFFER || WriteDescriptors[DescriptorIndex].descriptorType == VK_DESCRIPTOR_TYPE_STORAGE_BUFFER_DYNAMIC,
				TEXT("DescriptorType mismatch at index %d: called WriteBuffer<%s> and was expecting %s."), 
				DescriptorIndex, VK_TYPE_TO_STRING(VkDescriptorType, DescriptorType), VK_TYPE_TO_STRING(VkDescriptorType, WriteDescriptors[DescriptorIndex].descriptorType));
		}
		else
		{
			checkf(WriteDescriptors[DescriptorIndex].descriptorType == DescriptorType,
				TEXT("DescriptorType mismatch at index %d: called WriteBuffer<%s> and was expecting %s."),
				DescriptorIndex, VK_TYPE_TO_STRING(VkDescriptorType, DescriptorType), VK_TYPE_TO_STRING(VkDescriptorType, WriteDescriptors[DescriptorIndex].descriptorType));
		}
		VkDescriptorBufferInfo* BufferInfo = const_cast<VkDescriptorBufferInfo*>(WriteDescriptors[DescriptorIndex].pBufferInfo);
		check(BufferInfo);

		bool bChanged = false;
		if (UseVulkanDescriptorCache())
		{
			FVulkanHashableDescriptorInfo& HashableInfo = HashableDescriptorInfos[DescriptorIndex];
			check(HandleId > 0);
			if (HashableInfo.Buffer.Id != HandleId)
			{
				HashableInfo.Buffer.Id = HandleId;
				BufferInfo->buffer = BufferHandle;
				bChanged = true;
			}
			if (HashableInfo.Buffer.Offset != static_cast<uint32>(Offset))
			{
				HashableInfo.Buffer.Offset = static_cast<uint32>(Offset);
				BufferInfo->offset = Offset;
				bChanged = true;
			}
			if (HashableInfo.Buffer.Range != static_cast<uint32>(Range))
			{
				HashableInfo.Buffer.Range = static_cast<uint32>(Range);
				BufferInfo->range = Range;
				bChanged = true;
			}
			bIsKeyDirty |= bChanged;
		}
		else
		{
			bChanged = CopyAndReturnNotEqual(BufferInfo->buffer, BufferHandle);
			bChanged |= CopyAndReturnNotEqual(BufferInfo->offset, Offset);
			bChanged |= CopyAndReturnNotEqual(BufferInfo->range, Range);
		}

		if (DescriptorType == VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC)
		{
			const uint8 DynamicOffsetIndex = BindingToDynamicOffsetMap[DescriptorIndex];
			DynamicOffsets[DynamicOffsetIndex] = DynamicOffset;
		}
		return bChanged;
	}

	template <VkDescriptorType DescriptorType>
	bool WriteTextureView(uint32 DescriptorIndex, const FVulkanView::FTextureView& TextureView, VkImageLayout Layout)
	{
		check(DescriptorIndex < NumWrites);
		SetWritten(DescriptorIndex);
		if (DescriptorType == VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE)
		{
			checkf(WriteDescriptors[DescriptorIndex].descriptorType == VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE || WriteDescriptors[DescriptorIndex].descriptorType == VK_DESCRIPTOR_TYPE_COMBINED_IMAGE_SAMPLER,
				TEXT("DescriptorType mismatch at index %d: called WriteTextureView<%d> and was expecting %d."),
				DescriptorIndex, (uint32)DescriptorType, (uint32)WriteDescriptors[DescriptorIndex].descriptorType);
			ensureMsgf(Layout == VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL ||
				  Layout == VK_IMAGE_LAYOUT_SHARED_PRESENT_KHR ||
				  Layout == VK_IMAGE_LAYOUT_DEPTH_READ_ONLY_STENCIL_ATTACHMENT_OPTIMAL ||
				  Layout == VK_IMAGE_LAYOUT_DEPTH_ATTACHMENT_STENCIL_READ_ONLY_OPTIMAL ||
				  Layout == VK_IMAGE_LAYOUT_DEPTH_STENCIL_READ_ONLY_OPTIMAL || 
				  Layout == VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL ||
				  Layout == VK_IMAGE_LAYOUT_GENERAL, TEXT("Invalid Layout %s, Index %d, Type %s\n"), 
				VK_TYPE_TO_STRING(VkImageLayout, Layout), DescriptorIndex, VK_TYPE_TO_STRING(VkDescriptorType, WriteDescriptors[DescriptorIndex].descriptorType));
		}
		else
		{
			checkf(WriteDescriptors[DescriptorIndex].descriptorType == DescriptorType,
				TEXT("DescriptorType mismatch at index %d: called WriteTextureView<%s> and was expecting %s."),
				DescriptorIndex, VK_TYPE_TO_STRING(VkDescriptorType, DescriptorType), VK_TYPE_TO_STRING(VkDescriptorType, WriteDescriptors[DescriptorIndex].descriptorType));
		}
		VkDescriptorImageInfo* ImageInfo = const_cast<VkDescriptorImageInfo*>(WriteDescriptors[DescriptorIndex].pImageInfo);
		check(ImageInfo);

		bool bChanged = false;
		if (UseVulkanDescriptorCache())
		{
			FVulkanHashableDescriptorInfo& HashableInfo = HashableDescriptorInfos[DescriptorIndex];
			check(TextureView.ViewId > 0);
			if (HashableInfo.Image.ImageViewId != TextureView.ViewId)
			{
				HashableInfo.Image.ImageViewId = TextureView.ViewId;
				ImageInfo->imageView = TextureView.View;
				bChanged = true;
			}
			if (HashableInfo.Image.ImageLayout != static_cast<uint32>(Layout))
			{
				HashableInfo.Image.ImageLayout = static_cast<uint32>(Layout);
				ImageInfo->imageLayout = Layout;
				bChanged = true;
			}
			bIsKeyDirty |= bChanged;
		}
		else
		{
			bChanged = CopyAndReturnNotEqual(ImageInfo->imageView, TextureView.View);
			bChanged |= CopyAndReturnNotEqual(ImageInfo->imageLayout, Layout);
		}

		return bChanged;
	}

	template <VkDescriptorType DescriptorType>
	bool WriteBufferView(uint32 DescriptorIndex, const FVulkanView::FTypedBufferView& View)
	{
		check(DescriptorIndex < NumWrites);
		checkf(WriteDescriptors[DescriptorIndex].descriptorType == DescriptorType, 
			TEXT("DescriptorType mismatch at index %d: called WriteBufferView<%s> and was expecting %s."), 
			DescriptorIndex, VK_TYPE_TO_STRING(VkDescriptorType, DescriptorType), VK_TYPE_TO_STRING(VkDescriptorType, WriteDescriptors[DescriptorIndex].descriptorType));
		SetWritten(DescriptorIndex);
		WriteDescriptors[DescriptorIndex].pTexelBufferView = &View.View;

		const bool bVolatile = View.bVolatile;

		bHasVolatileResources|= bVolatile;
				
		if (!bVolatile && UseVulkanDescriptorCache())
		{
			bool bChanged = false;
			FVulkanHashableDescriptorInfo& HashableInfo = HashableDescriptorInfos[DescriptorIndex];
			check(View.ViewId > 0);
			if (HashableInfo.BufferView.Id != View.ViewId)
			{
				HashableInfo.BufferView.Id = View.ViewId;
				bChanged = true;
			}
			bIsKeyDirty |= bChanged;
			return bChanged;
		}
		else
		{
			return true;
		}
	}

protected:
	// A view into someone else's descriptors
	VkWriteDescriptorSet* WriteDescriptors;

	// A view into the mapping from binding index to dynamic uniform buffer offsets
	uint8* BindingToDynamicOffsetMap;

	// A view into someone else's dynamic uniform buffer offsets
	uint32* DynamicOffsets;

	uint32 NumWrites;

	// UB handles when running with bindless descriptor heaps
	TArray<uint32> BindlessUBIndices;

	FVulkanHashableDescriptorInfo* HashableDescriptorInfos;
	mutable FVulkanDSetKey Key;
	mutable bool bIsKeyDirty;
	bool bHasVolatileResources = false;

	uint32 SetupDescriptorWrites(const TArray<VkDescriptorType>& Types,
		FVulkanHashableDescriptorInfo* InHashableDescriptorInfos,
		VkWriteDescriptorSet* InWriteDescriptors, VkDescriptorImageInfo* InImageInfo,
		VkDescriptorBufferInfo* InBufferInfo, uint8* InBindingToDynamicOffsetMap,
		VkWriteDescriptorSetAccelerationStructureKHR* InAccelerationStructuresWriteDescriptors,
		VkAccelerationStructureKHR* InAccelerationStructures,
		const FVulkanSamplerState& DefaultSampler, const FVulkanView::FTextureView& DefaultImageView);

	friend class FVulkanCommonPipelineDescriptorState;
	friend class FVulkanComputePipelineDescriptorState;
	friend class FVulkanGraphicsPipelineDescriptorState;
	friend class FVulkanDescriptorSetCache;

#if VULKAN_VALIDATE_DESCRIPTORS_WRITTEN
	TArray<uint32, TInlineAllocator<2> > WrittenMask;
	TArray<uint32, TInlineAllocator<2> > BaseWrittenMask;
	TArray<uint32, TInlineAllocator<2> > ExcludedMask;
#endif
	void CheckAllWritten();
	void Reset();
	void SetWritten(uint32 DescriptorIndex);
	void SetWrittenBase(uint32 DescriptorIndex);
	void InitWrittenMasks(uint32 NumDescriptorWrites);
	void SetExcludedResources(const TArrayView<uint32> ExcludedResourceCount);
};


class FVulkanGenericDescriptorPool : FNoncopyable
{
public:
	FVulkanGenericDescriptorPool(FVulkanDevice* InDevice, uint32 InMaxDescriptorSets, const float PoolSizes[VK_DESCRIPTOR_TYPE_RANGE_SIZE]);
	~FVulkanGenericDescriptorPool();

	FVulkanDevice* GetDevice() const
	{
		return Device;
	}

	uint32 GetMaxDescriptorSets() const
	{
		return MaxDescriptorSets;
	}

	void Reset();
	bool AllocateDescriptorSet(VkDescriptorSetLayout Layout, VkDescriptorSet& OutSet);

private:
	FVulkanDevice* const Device;
	const uint32 MaxDescriptorSets;
	VkDescriptorPool DescriptorPool;
	// information for debugging
	uint32 PoolSizes[VK_DESCRIPTOR_TYPE_RANGE_SIZE];
};

class FVulkanDescriptorSetCache : FNoncopyable
{
public:
	FVulkanDescriptorSetCache(FVulkanDevice* InDevice);
	~FVulkanDescriptorSetCache();

	void GetDescriptorSets(const FVulkanDSetsKey& DSetsKey, const FVulkanDescriptorSetsLayout& SetsLayout,
		TArray<FVulkanDescriptorSetWriter>& DSWriters, VkDescriptorSet* OutSets);
	void GC();

private:
	void UpdateAllocRatio();
	void AddCachedPool();

private:
	struct FSetsEntry
	{
		TStaticArray<VkDescriptorSet, ShaderStage::MaxNumStages> Sets;
		int32 NumSets;
	};

	class FCachedPool : FNoncopyable
	{
	public:
		FCachedPool(FVulkanDevice* InDevice, uint32 InMaxDescriptorSets, const float PoolSizesRatio[VK_DESCRIPTOR_TYPE_RANGE_SIZE]);

		inline uint32 GetMaxDescriptorSets() const
		{
			return Pool.GetMaxDescriptorSets();
		}
			
		void Reset();
		bool CanGC() const;
		float CalcAllocRatio() const;

		bool FindDescriptorSets(const FVulkanDSetsKey& DSetsKey, VkDescriptorSet* OutSets);
		bool CreateDescriptorSets(const FVulkanDSetsKey& DSetsKey, const FVulkanDescriptorSetsLayout& SetsLayout,
			TArray<FVulkanDescriptorSetWriter>& DSWriters, VkDescriptorSet* OutSets);

		void CalcPoolSizesRatio(float PoolSizesRatio[VK_DESCRIPTOR_TYPE_RANGE_SIZE]);

	private:
		static const float MinAllocRatio;
		static const float MaxAllocRatio;

	private:
		const uint32 SetCapacity;
		FVulkanGenericDescriptorPool Pool;
		TMap<FVulkanDSetsKey, FSetsEntry> SetsCache;
		TMap<FVulkanDSetKey, VkDescriptorSet> SetCache;
		uint32 RecentFrame;
	
	public:
		uint32 PoolSizesStatistic[VK_DESCRIPTOR_TYPE_RANGE_SIZE];
	};

private:
	FVulkanDevice* const Device;
	TArray<TUniquePtr<FCachedPool>> CachedPools;
	TUniquePtr<FCachedPool> FreePool;
	float PoolAllocRatio;
};
