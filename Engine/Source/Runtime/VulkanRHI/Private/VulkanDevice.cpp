// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanDevice.cpp: Vulkan device RHI implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanDevice.h"
#include "VulkanPendingState.h"
#include "VulkanContext.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "VulkanPlatform.h"
#include "VulkanLLM.h"
#include "VulkanTransientResourceAllocator.h"
#include "VulkanExtensions.h"
#include "VulkanRenderpass.h"
#include "VulkanRayTracing.h"
#include "VulkanDescriptorSets.h"
#include "VulkanChunkedPipelineCache.h"
#include "VulkanBindlessDescriptorManager.h"
#include "VulkanShaderObjectManager.h"

static TAutoConsoleVariable<int32> GRHIAllowAsyncComputeCvar(
	TEXT("r.Vulkan.AllowAsyncCompute"),
	1,
	TEXT("0 to disable async compute queue (default)\n")
	TEXT("1 to allow async compute queue if available"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> GRHIAllowTransferQueueCvar(
	TEXT("r.Vulkan.AllowTransferQueue"),
	0,
	TEXT("0 to disable transfer queue (default)\n")
	TEXT("1 to allow transfer queue if available"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GAllowPresentOnComputeQueue(
	TEXT("r.Vulkan.AllowPresentOnComputeQueue"),
	0,
	TEXT("0 to present on the graphics queue")
	TEXT("1 to allow presenting on the compute queue if available")
);

TAutoConsoleVariable<int32> GCVarRobustBufferAccess(
	TEXT("r.Vulkan.RobustBufferAccess"),
	1,
	TEXT("0 to disable robust buffer access")
	TEXT("1 to enable (default)"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> CVarVulkanUseD24(
	TEXT("r.Vulkan.Depth24Bit"),
	0,
	TEXT("0: Use 32-bit float depth buffer (default)\n1: Use 24-bit fixed point depth buffer\n"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> GCVarVulkanTempBlockSizeKB(
	TEXT("r.Vulkan.TempBlockSizeKB"),
	4096,
	TEXT("Size in KB of the temporary blocks allocate by contexts, used for single use ub allocs and copies (default: 4096KB)."),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> GCVarVulkanUseMinimalSubmits(
	TEXT("r.Vulkan.UseMinimalSubmits"),
	0,
	TEXT("Can be specified in device profiles to keep submit counts low on devices with a single queue and single RHI thread."),
	ECVF_RenderThreadSafe
);

// Mirror GPixelFormats with format information for buffers
VkFormat GVulkanBufferFormat[PF_MAX];

// Mirror GPixelFormats with format information for buffers
VkFormat GVulkanSRGBFormat[PF_MAX];

EDelayAcquireImageType GVulkanDelayAcquireImage = EDelayAcquireImageType::DelayAcquire;

TAutoConsoleVariable<int32> CVarDelayAcquireBackBuffer(
	TEXT("r.Vulkan.DelayAcquireBackBuffer"),
	1,
	TEXT("Whether to delay acquiring the back buffer \n")
	TEXT(" 0: acquire next image on frame start \n")
	TEXT(" 1: acquire next image just before presenting, rendering is done to intermediate image which is then copied to a real backbuffer (default) \n"),
	ECVF_ReadOnly
);

static EDelayAcquireImageType DelayAcquireBackBuffer()
{
	const int32 DelayType = CVarDelayAcquireBackBuffer.GetValueOnAnyThread();
	return (DelayType == 0) ? EDelayAcquireImageType::None : EDelayAcquireImageType::DelayAcquire;
}

#if VULKAN_SUPPORTS_VALIDATION_CACHE
static void LoadValidationCache(VkDevice Device, VkValidationCacheEXT& OutValidationCache)
{
	VkValidationCacheCreateInfoEXT ValidationCreateInfo;
	ZeroVulkanStruct(ValidationCreateInfo, VK_STRUCTURE_TYPE_VALIDATION_CACHE_CREATE_INFO_EXT);
	TArray<uint8> InData;

	const FString& CacheFilename = VulkanRHI::GetValidationCacheFilename();
	UE_LOGF(LogVulkanRHI, Display, "Trying validation cache file %ls", *CacheFilename);
	if (FFileHelper::LoadFileToArray(InData, *CacheFilename, FILEREAD_Silent) && InData.Num() > 0)
	{
		// The code below supports SDK 1.0.65 Vulkan spec, which contains the following table:
		//
		// Offset	 Size            Meaning
		// ------    ------------    ------------------------------------------------------------------
		//      0               4    length in bytes of the entire validation cache header written as a
		//                           stream of bytes, with the least significant byte first
		//      4               4    a VkValidationCacheHeaderVersionEXT value written as a stream of
		//                           bytes, with the least significant byte first
		//      8    VK_UUID_SIZE    a layer commit ID expressed as a UUID, which uniquely identifies
		//                           the version of the validation layers used to generate these
		//                           validation results
		int32* DataPtr = (int32*)InData.GetData();
		if (*DataPtr > 0)
		{
			++DataPtr;
			int32 Version = *DataPtr++;
			if (Version == VK_PIPELINE_CACHE_HEADER_VERSION_ONE)
			{
				DataPtr += VK_UUID_SIZE / sizeof(int32);
			}
			else
			{
				UE_LOGF(LogVulkanRHI, Warning, "Bad validation cache file %ls, version=%d, expected %d", *CacheFilename, Version, VK_PIPELINE_CACHE_HEADER_VERSION_ONE);
				InData.Reset(0);
			}
		}
		else
		{
			UE_LOGF(LogVulkanRHI, Warning, "Bad validation cache file %ls, header size=%d", *CacheFilename, *DataPtr);
			InData.Reset(0);
		}
	}

	ValidationCreateInfo.initialDataSize = InData.Num();
	ValidationCreateInfo.pInitialData = InData.Num() > 0 ? InData.GetData() : nullptr;
	//ValidationCreateInfo.flags = 0;
	PFN_vkCreateValidationCacheEXT vkCreateValidationCache = (PFN_vkCreateValidationCacheEXT)(void*)VulkanRHI::vkGetDeviceProcAddr(Device, "vkCreateValidationCacheEXT");
	if (vkCreateValidationCache)
	{
		VkResult Result = vkCreateValidationCache(Device, &ValidationCreateInfo, VULKAN_CPU_ALLOCATOR, &OutValidationCache);
		if (Result != VK_SUCCESS)
		{
			UE_LOGF(LogVulkanRHI, Warning, "Failed to create Vulkan validation cache, VkResult=%d", Result);
		}
	}
}
#endif

static VkExtent2D GetBestMatchedShadingRateExtents(uint32 ShadingRate, const TArray<VkPhysicalDeviceFragmentShadingRateKHR>& FragmentShadingRates)
{
	// Given that for Vulkan we need to query available device shading rates, we're not guaranteed to have everything that's in our enum;
	// This function walks the list of supported fragment rates returned by the device, and returns the closest match to the rate requested.
	const VkExtent2D DirectMappedExtent = { 
		1u << (ShadingRate >> 2), 
		1u << (ShadingRate & 0x03) 
	};
	VkExtent2D BestMatchedExtent = { 1, 1 };

	if (BestMatchedExtent.width != DirectMappedExtent.width || 
		BestMatchedExtent.height != DirectMappedExtent.height)
	{
		for (auto const& Rate : FragmentShadingRates)
		{
			if (Rate.fragmentSize.width == DirectMappedExtent.width && 
				Rate.fragmentSize.height == DirectMappedExtent.height)
			{
				BestMatchedExtent = DirectMappedExtent;
				break;
			}

			if ((Rate.fragmentSize.width >= BestMatchedExtent.width && Rate.fragmentSize.width <= DirectMappedExtent.width && Rate.fragmentSize.height <= DirectMappedExtent.height && Rate.fragmentSize.height >= BestMatchedExtent.height) ||
				(Rate.fragmentSize.height >= BestMatchedExtent.height && Rate.fragmentSize.height <= DirectMappedExtent.height && Rate.fragmentSize.width <= DirectMappedExtent.width && Rate.fragmentSize.width >= BestMatchedExtent.width))
			{
				BestMatchedExtent = Rate.fragmentSize;
			}
		}
	}

	return BestMatchedExtent;
}


void FVulkanPhysicalDeviceFeatures::Query(VkPhysicalDevice PhysicalDevice, uint32 APIVersion)
{
	VkPhysicalDeviceFeatures2 PhysicalDeviceFeatures2;
	ZeroVulkanStruct(PhysicalDeviceFeatures2, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_FEATURES_2);

	PhysicalDeviceFeatures2.pNext = &Core_1_1;
	Core_1_1.sType = VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_VULKAN_1_1_FEATURES;

	if (APIVersion >= VK_API_VERSION_1_2)
	{
		Core_1_1.pNext = &Core_1_2;
		Core_1_2.sType = VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_VULKAN_1_2_FEATURES;
	}

	if (APIVersion >= VK_API_VERSION_1_3)
	{
		Core_1_2.pNext = &Core_1_3;
		Core_1_3.sType = VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_VULKAN_1_3_FEATURES;
	}

	if (APIVersion >= VK_API_VERSION_1_4)
	{
		Core_1_3.pNext = &Core_1_4;
		Core_1_4.sType = VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_VULKAN_1_4_FEATURES;
	}

	VulkanRHI::vkGetPhysicalDeviceFeatures2(PhysicalDevice, &PhysicalDeviceFeatures2);

	// Copy features into old struct for convenience
	Core_1_0 = PhysicalDeviceFeatures2.features;

	// Apply config modifications
	Core_1_0.robustBufferAccess = GCVarRobustBufferAccess.GetValueOnAnyThread() > 0 ? VK_TRUE : VK_FALSE;

	// Apply platform restrictions
	FVulkanPlatform::RestrictEnabledPhysicalDeviceFeatures(this);
}



FVulkanDevice::FVulkanDevice(FVulkanDynamicRHI* InRHI, VkPhysicalDevice InGpu)
	: Device(VK_NULL_HANDLE)
	, MemoryManager(*this)
	, DeferredDeletionQueue(*this)
	, FenceManager(*this)
	, Gpu(InGpu)
	, ImmediateContext(nullptr)
	, PipelineStateCache(nullptr)
{
	RHI = InRHI;
	FMemory::Memzero(Queues);
	FMemory::Memzero(GpuProps);
	FMemory::Memzero(FormatProperties);
	FMemory::Memzero(PixelFormatComponentMapping);

	ZeroVulkanStruct(GpuIdProps, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_ID_PROPERTIES_KHR);
	ZeroVulkanStruct(GpuSubgroupProps, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_SUBGROUP_PROPERTIES);

	{
		VkPhysicalDeviceProperties2KHR PhysicalDeviceProperties2;
		ZeroVulkanStruct(PhysicalDeviceProperties2, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_PROPERTIES_2_KHR);
		PhysicalDeviceProperties2.pNext = &GpuIdProps;
		GpuIdProps.pNext = &GpuSubgroupProps;
		VulkanRHI::vkGetPhysicalDeviceProperties2(Gpu, &PhysicalDeviceProperties2);
		GpuProps = PhysicalDeviceProperties2.properties;
	}

	// First get the VendorId. We'll have to get properties again after finding out which extensions we want to use
	VendorId = RHIConvertToGpuVendorId(GpuProps.vendorID);

	UE_LOGF(LogVulkanRHI, Display, "- DeviceName: %ls", ANSI_TO_TCHAR(GpuProps.deviceName));
	UE_LOGF(LogVulkanRHI, Display, "- API=%d.%d.%d (0x%x) Driver=0x%x VendorId=0x%x", VK_VERSION_MAJOR(GpuProps.apiVersion), VK_VERSION_MINOR(GpuProps.apiVersion), VK_VERSION_PATCH(GpuProps.apiVersion), GpuProps.apiVersion, GpuProps.driverVersion, GpuProps.vendorID);
	UE_LOGF(LogVulkanRHI, Display, "- DeviceID=0x%x Type=%ls", GpuProps.deviceID, VK_TYPE_TO_STRING(VkPhysicalDeviceType, GpuProps.deviceType));
	UE_LOGF(LogVulkanRHI, Display, "- Max Descriptor Sets Bound %d", GpuProps.limits.maxBoundDescriptorSets);
	UE_LOGF(LogVulkanRHI, Display, "- Timestamps:  ComputeAndGraphics=%d  Domain=%ls  Period=%f", 
		GpuProps.limits.timestampComputeAndGraphics, VK_TYPE_TO_STRING(VkTimeDomainKHR, FVulkanPlatform::GetTimeDomain()), GpuProps.limits.timestampPeriod);

	ensureMsgf(VendorId != EGpuVendorId::Unknown, TEXT("Unknown vendor ID 0x%x"), GpuProps.vendorID);
}

FVulkanDevice::~FVulkanDevice()
{
	if (Device != VK_NULL_HANDLE)
	{
		Destroy();
		Device = VK_NULL_HANDLE;
	}
}

static inline FString GetQueueInfoString(const VkQueueFamilyProperties& Props)
{
	FString Info;
	if ((Props.queueFlags & VK_QUEUE_GRAPHICS_BIT) == VK_QUEUE_GRAPHICS_BIT)
	{
		Info += TEXT(" Gfx");
	}
	if ((Props.queueFlags & VK_QUEUE_COMPUTE_BIT) == VK_QUEUE_COMPUTE_BIT)
	{
		Info += TEXT(" Compute");
	}
	if ((Props.queueFlags & VK_QUEUE_TRANSFER_BIT) == VK_QUEUE_TRANSFER_BIT)
	{
		Info += TEXT(" Xfer");
	}
	if ((Props.queueFlags & VK_QUEUE_SPARSE_BINDING_BIT) == VK_QUEUE_SPARSE_BINDING_BIT)
	{
		Info += TEXT(" Sparse");
	}

	return Info;
};

void FVulkanDevice::CreateDevice(TArray<const ANSICHAR*>& DeviceLayers, FVulkanDeviceExtensionArray& UEExtensions)
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanMisc);
	check(Device == VK_NULL_HANDLE);

	// Setup extension and layer info
	VkDeviceCreateInfo DeviceInfo;
	ZeroVulkanStruct(DeviceInfo, VK_STRUCTURE_TYPE_DEVICE_CREATE_INFO);

	DeviceInfo.pEnabledFeatures = &PhysicalDeviceFeatures.Core_1_0;

	for (TUniquePtr<FVulkanDeviceExtension>& UEExtension : UEExtensions)
	{
		if (UEExtension->InUse())
		{
			DeviceExtensions.Add(UEExtension->GetExtensionName());
			UEExtension->PreCreateDevice(DeviceInfo);
		}
	}

	DeviceInfo.enabledExtensionCount = DeviceExtensions.Num();
	DeviceInfo.ppEnabledExtensionNames = DeviceExtensions.GetData();

	DeviceInfo.enabledLayerCount = DeviceLayers.Num();
	DeviceInfo.ppEnabledLayerNames = (DeviceInfo.enabledLayerCount > 0) ? DeviceLayers.GetData() : nullptr;

	// Setup Queue info
	TArray<VkDeviceQueueCreateInfo> QueueFamilyInfos;
	int32 GfxQueueFamilyIndex = -1;
	int32 ComputeQueueFamilyIndex = -1;
	int32 TransferQueueFamilyIndex = -1;
	UE_LOGF(LogVulkanRHI, Display, "Found %d Queue Families", QueueFamilyProps.Num());
	uint32 NumPriorities = 0;
	for (int32 FamilyIndex = 0; FamilyIndex < QueueFamilyProps.Num(); ++FamilyIndex)
	{
		const VkQueueFamilyProperties& CurrProps = QueueFamilyProps[FamilyIndex];

		bool bIsValidQueue = false;
		if (VKHasAllFlags(CurrProps.queueFlags, VK_QUEUE_GRAPHICS_BIT))
		{
			if (GfxQueueFamilyIndex == -1)
			{
				GfxQueueFamilyIndex = FamilyIndex;
				bIsValidQueue = true;
			}
			else
			{
				//#todo-rco: Support for multi-queue/choose the best queue!
			}
		}

		if (VKHasAllFlags(CurrProps.queueFlags, VK_QUEUE_COMPUTE_BIT))
		{
			// Allocate a queue for async compute if:
 			// - async compute queue hasn't been found already
			// - cvars allow for a dedicated async compute queue
			// - a new family index is available
			// - Sync2 is available
			// - maintenance9 is available
			if ((ComputeQueueFamilyIndex == -1) &&
				(GRHIAllowAsyncComputeCvar.GetValueOnAnyThread() != 0 || GAllowPresentOnComputeQueue.GetValueOnAnyThread() != 0 || FParse::Param(FCommandLine::Get(), TEXT("ForceAsyncCompute"))) &&
				!FParse::Param(FCommandLine::Get(), TEXT("DisableAsyncCompute")) &&
				(GfxQueueFamilyIndex != FamilyIndex) && 
				SupportsParallelRendering() &&
				OptionalDeviceExtensions.HasKHRMaintenance9)
			{
				ComputeQueueFamilyIndex = FamilyIndex;
				bIsValidQueue = true;
			}
		}
		
		if (VKHasAllFlags(CurrProps.queueFlags, VK_QUEUE_TRANSFER_BIT))
		{
			// Allocate a transfer queue if:
			// - a transfer queue hasn't been found already
			// - there is a dedicated non graphics/compute queue
			// - cvars allow for a dedicated transfer queue
			// - a new family index is available
			// - Sync2 is available
			// - maintenance9 is available
			if ((TransferQueueFamilyIndex == -1) &&
				!VKHasAnyFlags(CurrProps.queueFlags, VK_QUEUE_GRAPHICS_BIT | VK_QUEUE_COMPUTE_BIT) &&
				(GRHIAllowTransferQueueCvar.GetValueOnAnyThread() != 0) &&
				(GfxQueueFamilyIndex != FamilyIndex) && (ComputeQueueFamilyIndex != FamilyIndex) &&
				SupportsParallelRendering() &&
				OptionalDeviceExtensions.HasKHRMaintenance9)
			{
				TransferQueueFamilyIndex = FamilyIndex;
				bIsValidQueue = true;
			}
		}

		if (!bIsValidQueue)
		{
			UE_LOGF(LogVulkanRHI, Display, "Skipping unnecessary Queue Family %d: %d queues%ls", FamilyIndex, CurrProps.queueCount, *GetQueueInfoString(CurrProps));
			continue;
		}

		VkDeviceQueueCreateInfo& CurrQueue = QueueFamilyInfos.AddZeroed_GetRef();
		CurrQueue.sType = VK_STRUCTURE_TYPE_DEVICE_QUEUE_CREATE_INFO;
		CurrQueue.queueFamilyIndex = FamilyIndex;
		CurrQueue.queueCount = CurrProps.queueCount;
		NumPriorities += CurrProps.queueCount;
		UE_LOGF(LogVulkanRHI, Display, "Initializing Queue Family %d: %d queues%ls", FamilyIndex, CurrProps.queueCount, *GetQueueInfoString(CurrProps));
	}

	TArray<float> QueuePriorities;
	QueuePriorities.AddUninitialized(NumPriorities);
	float* CurrentPriority = QueuePriorities.GetData();
	for (int32 Index = 0; Index < QueueFamilyInfos.Num(); ++Index)
	{
		VkDeviceQueueCreateInfo& CurrQueue = QueueFamilyInfos[Index];
		CurrQueue.pQueuePriorities = CurrentPriority;

		const VkQueueFamilyProperties& CurrProps = QueueFamilyProps[CurrQueue.queueFamilyIndex];
		for (int32 QueueIndex = 0; QueueIndex < (int32)CurrProps.queueCount; ++QueueIndex)
		{
			*CurrentPriority++ = 1.0f;
		}
	}

	DeviceInfo.queueCreateInfoCount = QueueFamilyInfos.Num();
	DeviceInfo.pQueueCreateInfos = QueueFamilyInfos.GetData();

	// Create the device
	VkResult Result = VulkanRHI::vkCreateDevice(Gpu, &DeviceInfo, VULKAN_CPU_ALLOCATOR, &Device);
	if (Result == VK_ERROR_INITIALIZATION_FAILED)
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Cannot create a Vulkan device. Try updating your video driver to a more recent version.\n"), TEXT("Vulkan device creation failed"));
		FPlatformMisc::RequestExitWithStatus(true, 1);
	}
	VERIFYVULKANRESULT_EXPANDED(Result);

	FVulkanPlatform::NotifyFoundDeviceLayersAndExtensions(Gpu, DeviceLayers, DeviceExtensions);

	ActiveQueueFamilies.Reserve((int32)EVulkanQueueType::Count);

	const uint32 NumBits = QueueFamilyProps[GfxQueueFamilyIndex].timestampValidBits;
	UE_LOGF(LogVulkanRHI, Display, "TimestampValidBits=%u", NumBits);

	// Create Graphics Queue, here we submit command buffers for execution
	Queues[(int32)EVulkanQueueType::Graphics] = new FVulkanQueue(*this, GfxQueueFamilyIndex, EVulkanQueueType::Graphics);
	ActiveQueueFamilies.Add(GfxQueueFamilyIndex);
	if (ComputeQueueFamilyIndex == -1)
	{
		// If we didn't find a dedicated Queue, leave it null
		Queues[(int32)EVulkanQueueType::AsyncCompute] = nullptr;
	}
	else
	{
		ensure(NumBits == QueueFamilyProps[ComputeQueueFamilyIndex].timestampValidBits);
		Queues[(int32)EVulkanQueueType::AsyncCompute] = new FVulkanQueue(*this, ComputeQueueFamilyIndex, EVulkanQueueType::AsyncCompute);
		ActiveQueueFamilies.Add(ComputeQueueFamilyIndex);
	}

	if (TransferQueueFamilyIndex == -1)
	{
		// If we didn't find a dedicated Queue, leave it null
		Queues[(int32)EVulkanQueueType::Transfer] = nullptr;
	}
	else
	{
		Queues[(int32)EVulkanQueueType::Transfer] = new FVulkanQueue(*this, TransferQueueFamilyIndex, EVulkanQueueType::Transfer);
		ActiveQueueFamilies.Add(TransferQueueFamilyIndex);
	}

	if (HasAsyncComputeQueue() && OptionalDeviceExtensions.HasKHRMaintenance9)
	{
		TArray<VkQueueFamilyProperties2> QueueFamilyProperties2;
		QueueFamilyProperties2.SetNumZeroed(QueueFamilyProps.Num());
		TArray<VkQueueFamilyOwnershipTransferPropertiesKHR> OwnershipTransferProperties;
		OwnershipTransferProperties.SetNumUninitialized(QueueFamilyProperties2.Num());

		for (int32 QueueIndex = 0; QueueIndex < QueueFamilyProperties2.Num(); ++QueueIndex)
		{
			OwnershipTransferProperties[QueueIndex] = {
				.sType = VK_STRUCTURE_TYPE_QUEUE_FAMILY_OWNERSHIP_TRANSFER_PROPERTIES_KHR,
				.pNext = nullptr,
				.optimalImageTransferToQueueFamilies = 0
			};

			QueueFamilyProperties2[QueueIndex] = {
				.sType = VK_STRUCTURE_TYPE_QUEUE_FAMILY_PROPERTIES_2,
				.pNext = &OwnershipTransferProperties[QueueIndex],
			};
		}

		uint32 QueueCount = QueueFamilyProperties2.Num();
		VulkanRHI::vkGetPhysicalDeviceQueueFamilyProperties2(Gpu, &QueueCount, QueueFamilyProperties2.GetData());

		const bool bGraphicsOwnership = OwnershipTransferProperties[GfxQueueFamilyIndex].optimalImageTransferToQueueFamilies & (1u << ComputeQueueFamilyIndex);
		const bool bAsyncOwnership = OwnershipTransferProperties[ComputeQueueFamilyIndex].optimalImageTransferToQueueFamilies & (1u << GfxQueueFamilyIndex);
		const bool bTransferOwnership = (TransferQueueFamilyIndex == -1) || (
			(OwnershipTransferProperties[TransferQueueFamilyIndex].optimalImageTransferToQueueFamilies & (1u << GfxQueueFamilyIndex)) &&
			(OwnershipTransferProperties[TransferQueueFamilyIndex].optimalImageTransferToQueueFamilies & (1u << ComputeQueueFamilyIndex)));
		if (bGraphicsOwnership && bAsyncOwnership && bTransferOwnership)
		{
			// Queue ownership transfers aren't necessary between the used queue families
			bNeedsQueueFamilyOwnershipTransfers = false;
		}
		else
		{
			// Do not merge pipelines together when queue family ownership transfers are necessary
			GRHIMultiPipelineMergeableAccessMask = ERHIAccess::Unknown;
		}
	}

	// Enumerate the available shading rates
	if (OptionalDeviceExtensions.HasKHRFragmentShadingRate)
	{
		uint32 FragmentShadingRateCount = 0;
		VulkanRHI::vkGetPhysicalDeviceFragmentShadingRatesKHR(Gpu, &FragmentShadingRateCount, nullptr);
		if (FragmentShadingRateCount != 0)
		{
			FragmentShadingRates.SetNum(FragmentShadingRateCount);
			for (uint32 i = 0; i < FragmentShadingRateCount; ++i)
			{
				ZeroVulkanStruct(FragmentShadingRates[i], VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_FRAGMENT_SHADING_RATE_KHR);
			}
			VulkanRHI::vkGetPhysicalDeviceFragmentShadingRatesKHR(Gpu, &FragmentShadingRateCount, FragmentShadingRates.GetData());

			// Build a map from EVRSShadingRate to fragment size
			for (uint32 ShadingRate = 0u; ShadingRate < (uint32)FragmentSizeMap.Num(); ++ShadingRate)
			{
				FragmentSizeMap[ShadingRate] = GetBestMatchedShadingRateExtents(ShadingRate, FragmentShadingRates);
			}
		}
	}

	UE_LOGF(LogVulkanRHI, Display, "Using %d device layers%ls", DeviceLayers.Num(), DeviceLayers.Num() ? TEXT(":") : TEXT("."));
	for (const ANSICHAR* Layer : DeviceLayers)
	{
		UE_LOGF(LogVulkanRHI, Display, "* %ls", ANSI_TO_TCHAR(Layer));
	}

	UE_LOGF(LogVulkanRHI, Display, "Using %d device extensions:", DeviceExtensions.Num());
	for (const ANSICHAR* Extension : DeviceExtensions)
	{
		UE_LOGF(LogVulkanRHI, Display, "* %ls", ANSI_TO_TCHAR(Extension));
	}

	GVulkanDelayAcquireImage = DelayAcquireBackBuffer();

	SetupDrawMarkers();
}

bool FVulkanDevice::UseMinimalSubmits() const
{
	return (ActiveQueueFamilies.Num() == 1) && 
		!GRHIGlobals.SupportsParallelRHIExecute &&
		!FVulkanDynamicRHI::Get().HasInterruptThread() &&
		!FVulkanDynamicRHI::Get().HasSubmissionThread() &&
		GCVarVulkanUseMinimalSubmits.GetValueOnRenderThread();
}

bool FVulkanDevice::SupportsDeviceLocalHostVisible() const
{
	if (HasUnifiedMemory())
	{
		return true;
	}

	if (DeviceMemoryManager.SupportsMemoryType(VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT | VK_MEMORY_PROPERTY_HOST_COHERENT_BIT | VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT))
	{
		return (VendorId == EGpuVendorId::Amd) || (GpuProps.deviceType == VK_PHYSICAL_DEVICE_TYPE_INTEGRATED_GPU);
	}

	return false;
}

#if VULKAN_ENABLE_DRAW_MARKERS
TAutoConsoleVariable<int32> CVarVulkanDebugMarkers(
	TEXT("r.Vulkan.DebugMarkers"),
	4,
	TEXT("0 to disable all debug markers\n")
	TEXT("1 to enable debug names for resources\n")
	TEXT("2 to enable debug labels for commands\n")
	TEXT("3 to enable debug resource names command labels\n")
	TEXT("4 to automatically enable markers depending on tool detection (default)\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);
#endif // VULKAN_ENABLE_DRAW_MARKERS

void FVulkanDevice::SetupDrawMarkers()
{
#if VULKAN_ENABLE_DRAW_MARKERS
	const bool bGPUCrashDebugging = UE::RHI::UseGPUCrashDebugging();
	bool bTraceToolFound = bGPUCrashDebugging;

#if VULKAN_HAS_DEBUGGING_ENABLED
	bTraceToolFound |= GRenderDocFound;
#endif // VULKAN_HAS_DEBUGGING_ENABLED

	if (RHI->SupportsDebugUtilsExt() || bUseLegacyDebugMarkerExt)
	{
		bool bUseLabel = bGPUCrashDebugging;
		bool bUseName = false;

		if (OptionalDeviceExtensions.HasEXTToolingInfo)
		{
			uint32_t ToolCount = 0;
			VulkanRHI::vkGetPhysicalDeviceToolPropertiesEXT(Gpu, &ToolCount, nullptr);
			TArray<VkPhysicalDeviceToolPropertiesEXT> ToolProperties;
			ToolProperties.SetNumUninitialized(ToolCount);
			for (VkPhysicalDeviceToolPropertiesEXT& Tool : ToolProperties)
			{
				ZeroVulkanStruct(Tool, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_TOOL_PROPERTIES);
			}

			VulkanRHI::vkGetPhysicalDeviceToolPropertiesEXT(Gpu, &ToolCount, ToolProperties.GetData());
			for (VkPhysicalDeviceToolPropertiesEXT const& Tool : ToolProperties)
			{
				if (Tool.purposes & VK_TOOL_PURPOSE_DEBUG_MARKERS_BIT_EXT)
				{
					bUseName = true;
					if (Tool.purposes & VK_TOOL_PURPOSE_TRACING_BIT_EXT)
					{
						bUseLabel = true;
					}
				}
				if (Tool.purposes & VK_TOOL_PURPOSE_TRACING_BIT_EXT)
				{
					bTraceToolFound = true;
				}
				UE_LOGF(LogVulkanRHI, Display, "Tool \"%ls\" version %ls PurposeFlags=0x%x", ANSI_TO_TCHAR(Tool.name), ANSI_TO_TCHAR(Tool.version), Tool.purposes);
			}
		}

		const int VulkanDebugMarkers = FParse::Param(FCommandLine::Get(), TEXT("forcevulkandrawmarkers")) ? 3 : CVarVulkanDebugMarkers.GetValueOnRenderThread();
		if (VulkanDebugMarkers != 4)
		{
			bUseLabel = bUseLabel || (VulkanDebugMarkers & 2);
			bUseName = bUseName || (VulkanDebugMarkers & 1);
		}

		if (bUseLegacyDebugMarkerExt)
		{
			if (bUseLabel)
			{
				static PFN_vkCmdDebugMarkerBeginEXT LegacyCmdBeginDebugLabel;
				static PFN_vkCmdDebugMarkerEndEXT LegacyCmdEndDebugLabel;
				LegacyCmdBeginDebugLabel = (PFN_vkCmdDebugMarkerBeginEXT)(void*)VulkanRHI::vkGetDeviceProcAddr(Device, "vkCmdDebugMarkerBeginEXT");
				LegacyCmdEndDebugLabel = (PFN_vkCmdDebugMarkerEndEXT)(void*)VulkanRHI::vkGetDeviceProcAddr(Device, "vkCmdDebugMarkerEndEXT");
				if (LegacyCmdBeginDebugLabel && LegacyCmdEndDebugLabel)
				{
					struct Wrap
					{
						static VKAPI_ATTR void VKAPI_CALL CmdBeginDebugLabel(VkCommandBuffer CommandBuffer, const VkDebugUtilsLabelEXT* LabelInfo)
						{
							VkDebugMarkerMarkerInfoEXT MarkerInfo{ VK_STRUCTURE_TYPE_DEBUG_MARKER_MARKER_INFO_EXT };
							MarkerInfo.pMarkerName = LabelInfo->pLabelName;
							memcpy(MarkerInfo.color, LabelInfo->color, sizeof(LabelInfo->color));
							LegacyCmdBeginDebugLabel(CommandBuffer, &MarkerInfo);
						}
						static VKAPI_ATTR void VKAPI_CALL CmdEndDebugLabel(VkCommandBuffer CommandBuffer)
						{
							LegacyCmdEndDebugLabel(CommandBuffer);
						}
					};
					DebugMarkers.CmdBeginDebugLabel = &Wrap::CmdBeginDebugLabel;
					DebugMarkers.CmdEndDebugLabel = &Wrap::CmdEndDebugLabel;
				}
			}
			if (bUseName)
			{
				static PFN_vkDebugMarkerSetObjectNameEXT LegacySetObjectName;
				LegacySetObjectName = (PFN_vkDebugMarkerSetObjectNameEXT)(void*)VulkanRHI::vkGetDeviceProcAddr(Device, "vkDebugMarkerSetObjectNameEXT");
				if (LegacySetObjectName)
				{
					struct Wrap
					{
						static VKAPI_ATTR VkResult VKAPI_CALL SetObjectName(VkDevice Device, const VkDebugUtilsObjectNameInfoEXT* NameInfo)
						{
							if (NameInfo->objectType <= 25)
							{
								VkDebugMarkerObjectNameInfoEXT MarkerInfo{ VK_STRUCTURE_TYPE_DEBUG_MARKER_OBJECT_NAME_INFO_EXT };
								MarkerInfo.pObjectName = NameInfo->pObjectName;
								MarkerInfo.object = NameInfo->objectHandle;
								MarkerInfo.objectType = static_cast<VkDebugReportObjectTypeEXT>(NameInfo->objectType);
								LegacySetObjectName(Device, &MarkerInfo);
							}
							return VK_SUCCESS;
						}
					};

					DebugMarkers.SetDebugName = &Wrap::SetObjectName;
				}
			}
		}
		else
		{
			if (bUseLabel)
			{
				DebugMarkers.CmdBeginDebugLabel = (PFN_vkCmdBeginDebugUtilsLabelEXT)(void*)VulkanRHI::vkGetInstanceProcAddr(RHI->GetInstance(), "vkCmdBeginDebugUtilsLabelEXT");
				DebugMarkers.CmdEndDebugLabel = (PFN_vkCmdEndDebugUtilsLabelEXT)(void*)VulkanRHI::vkGetInstanceProcAddr(RHI->GetInstance(), "vkCmdEndDebugUtilsLabelEXT");
				DebugMarkers.CmdInsertLabel = (PFN_vkCmdInsertDebugUtilsLabelEXT)(void*)VulkanRHI::vkGetInstanceProcAddr(RHI->GetInstance(), "vkCmdInsertDebugUtilsLabelEXT");
			}
			if (bUseName)
			{
				DebugMarkers.SetDebugName = (PFN_vkSetDebugUtilsObjectNameEXT)(void*)VulkanRHI::vkGetInstanceProcAddr(RHI->GetInstance(), "vkSetDebugUtilsObjectNameEXT");
			}
		}

		UE_LOGF(LogVulkanRHI, Display, "Vulkan debug markers support: resource names %ls, begin/end labels %ls.", bUseName ? TEXT("enabled") : TEXT("disabled"), bUseLabel ? TEXT("enabled") : TEXT("disabled"));
	}

#if VULKAN_HAS_DEBUGGING_ENABLED
	if (DebugMarkers.CmdBeginDebugLabel && DebugMarkers.CmdEndDebugLabel && bTraceToolFound)
	{
		// We're running under RenderDoc or other trace tool, so enable capturing mode
		FDynamicRHI::EnableIdealGPUCaptureOptions(true);
	}
	else if(bTraceToolFound)
	{
		UE_LOGF(LogVulkanRHI, Warning, "Vulkan API trace tool detected but not running in ideal GPU capture mode.");
	}
#endif // VULKAN_HAS_DEBUGGING_ENABLED
#endif // VULKAN_ENABLE_DRAW_MARKERS

#if VULKAN_ENABLE_DUMP_LAYER
	FDynamicRHI::EnableIdealGPUCaptureOptions(true);
#endif
}

void FVulkanDevice::SetupFormats()
{
	for (uint32 Index = 0; Index < VK_FORMAT_RANGE_SIZE; ++Index)
	{
		const VkFormat Format = (VkFormat)Index;
		FMemory::Memzero(FormatProperties[Index]);
		VulkanRHI::vkGetPhysicalDeviceFormatProperties(Gpu, Format, &FormatProperties[Index]);
	}

	static_assert(sizeof(VkFormat) <= sizeof(GPixelFormats[0].PlatformFormat), "PlatformFormat must be increased!");

	// Create shortcuts for the possible component mappings
	const VkComponentMapping ComponentMappingRGBA = { VK_COMPONENT_SWIZZLE_R, VK_COMPONENT_SWIZZLE_G, VK_COMPONENT_SWIZZLE_B, VK_COMPONENT_SWIZZLE_A };
	const VkComponentMapping ComponentMappingRGB1 = { VK_COMPONENT_SWIZZLE_R, VK_COMPONENT_SWIZZLE_G, VK_COMPONENT_SWIZZLE_B, VK_COMPONENT_SWIZZLE_ONE };
	const VkComponentMapping ComponentMappingRG01 = { VK_COMPONENT_SWIZZLE_R, VK_COMPONENT_SWIZZLE_G, VK_COMPONENT_SWIZZLE_ZERO, VK_COMPONENT_SWIZZLE_ONE };
	const VkComponentMapping ComponentMappingR001 = { VK_COMPONENT_SWIZZLE_R, VK_COMPONENT_SWIZZLE_ZERO, VK_COMPONENT_SWIZZLE_ZERO, VK_COMPONENT_SWIZZLE_ONE };
	const VkComponentMapping ComponentMappingRIII = { VK_COMPONENT_SWIZZLE_R, VK_COMPONENT_SWIZZLE_IDENTITY, VK_COMPONENT_SWIZZLE_IDENTITY, VK_COMPONENT_SWIZZLE_IDENTITY };
	const VkComponentMapping ComponentMapping000R = { VK_COMPONENT_SWIZZLE_ZERO, VK_COMPONENT_SWIZZLE_ZERO, VK_COMPONENT_SWIZZLE_ZERO, VK_COMPONENT_SWIZZLE_R };
	const VkComponentMapping ComponentMappingR000 = { VK_COMPONENT_SWIZZLE_R, VK_COMPONENT_SWIZZLE_ZERO, VK_COMPONENT_SWIZZLE_ZERO, VK_COMPONENT_SWIZZLE_ZERO };
	const VkComponentMapping ComponentMappingRR01 = { VK_COMPONENT_SWIZZLE_R, VK_COMPONENT_SWIZZLE_R, VK_COMPONENT_SWIZZLE_ZERO, VK_COMPONENT_SWIZZLE_ONE };


	// Initialize the platform pixel format map.
	for (int32 Index = 0; Index < PF_MAX; ++Index)
	{
		GPixelFormats[Index].PlatformFormat = VK_FORMAT_UNDEFINED;
		GPixelFormats[Index].Supported = false;
		GVulkanBufferFormat[Index] = VK_FORMAT_UNDEFINED;
		
		// Set default component mapping
		PixelFormatComponentMapping[Index] = ComponentMappingRGBA;
	}

	const EPixelFormatCapabilities ColorRenderTargetRequiredCapabilities = (EPixelFormatCapabilities::TextureSample | EPixelFormatCapabilities::RenderTarget);

	// Default formats
	MapFormatSupport(PF_B8G8R8A8, { VK_FORMAT_B8G8R8A8_UNORM }, ComponentMappingRGBA);
	MapFormatSupport(PF_G8, { VK_FORMAT_R8_UNORM }, ComponentMappingR001);
	MapFormatSupport(PF_FloatRGB, { VK_FORMAT_B10G11R11_UFLOAT_PACK32, VK_FORMAT_R16G16B16_SFLOAT, VK_FORMAT_R16G16B16A16_SFLOAT }, ComponentMappingRGB1, ColorRenderTargetRequiredCapabilities);
	MapFormatSupport(PF_FloatRGBA, { VK_FORMAT_R16G16B16A16_SFLOAT }, ComponentMappingRGBA, 8);
	MapFormatSupport(PF_ShadowDepth, { VK_FORMAT_D16_UNORM }, ComponentMappingRIII);
	MapFormatSupport(PF_G32R32F, { VK_FORMAT_R32G32_SFLOAT }, ComponentMappingRG01, 8);  // Requirement for GPU particles
	MapFormatSupport(PF_A32B32G32R32F, { VK_FORMAT_R32G32B32A32_SFLOAT }, ComponentMappingRGBA, 16);
	MapFormatSupport(PF_G16R16, { VK_FORMAT_R16G16_UNORM, VK_FORMAT_R16G16_SFLOAT }, ComponentMappingRG01);
	MapFormatSupport(PF_G16R16F, { VK_FORMAT_R16G16_SFLOAT }, ComponentMappingRG01);
	MapFormatSupport(PF_G16R16F_FILTER, { VK_FORMAT_R16G16_SFLOAT }, ComponentMappingRG01);
	MapFormatSupport(PF_R16_UINT, { VK_FORMAT_R16_UINT }, ComponentMappingR001);
	MapFormatSupport(PF_R16_SINT, { VK_FORMAT_R16_SINT }, ComponentMappingR001);
	MapFormatSupport(PF_R32_UINT, { VK_FORMAT_R32_UINT }, ComponentMappingR001);
	MapFormatSupport(PF_R32_SINT, { VK_FORMAT_R32_SINT }, ComponentMappingR001);
	MapFormatSupport(PF_R8_UINT, { VK_FORMAT_R8_UINT }, ComponentMappingR001);
	MapFormatSupport(PF_D24, { VK_FORMAT_X8_D24_UNORM_PACK32, VK_FORMAT_D24_UNORM_S8_UINT, VK_FORMAT_D16_UNORM_S8_UINT, VK_FORMAT_D32_SFLOAT, VK_FORMAT_D32_SFLOAT_S8_UINT }, ComponentMappingR000);
	MapFormatSupport(PF_R16F, { VK_FORMAT_R16_SFLOAT }, ComponentMappingR001);
	MapFormatSupport(PF_R16F_FILTER, { VK_FORMAT_R16_SFLOAT }, ComponentMappingR001);
	MapFormatSupport(PF_FloatR11G11B10, { VK_FORMAT_B10G11R11_UFLOAT_PACK32, VK_FORMAT_R16G16B16_SFLOAT, VK_FORMAT_R16G16B16A16_SFLOAT }, ComponentMappingRGB1, ColorRenderTargetRequiredCapabilities);
	MapFormatSupport(PF_A2B10G10R10, { VK_FORMAT_A2B10G10R10_UNORM_PACK32 }, ComponentMappingRGBA, 4);
	MapFormatSupport(PF_A16B16G16R16, { VK_FORMAT_R16G16B16A16_UNORM, VK_FORMAT_R16G16B16A16_SFLOAT }, ComponentMappingRGBA, 8);
	MapFormatSupport(PF_A8, { VK_FORMAT_R8_UNORM }, ComponentMapping000R);
	MapFormatSupport(PF_R5G6B5_UNORM, { VK_FORMAT_R5G6B5_UNORM_PACK16 }, ComponentMappingRGBA);
	MapFormatSupport(PF_B5G5R5A1_UNORM, { VK_FORMAT_A1R5G5B5_UNORM_PACK16, VK_FORMAT_R5G5B5A1_UNORM_PACK16, VK_FORMAT_B8G8R8A8_UNORM }, ComponentMappingRGBA);
	MapFormatSupport(PF_R8G8B8A8, { VK_FORMAT_R8G8B8A8_UNORM }, ComponentMappingRGBA);
	MapFormatSupport(PF_R8G8B8A8_UINT, { VK_FORMAT_R8G8B8A8_UINT }, ComponentMappingRGBA);
	MapFormatSupport(PF_R8G8B8A8_SNORM, { VK_FORMAT_R8G8B8A8_SNORM }, ComponentMappingRGBA);
	MapFormatSupport(PF_R16G16_UINT, { VK_FORMAT_R16G16_UINT }, ComponentMappingRG01);
	MapFormatSupport(PF_R16G16_SINT, { VK_FORMAT_R16G16_SINT }, ComponentMappingRG01);
	MapFormatSupport(PF_R16G16B16A16_UINT, { VK_FORMAT_R16G16B16A16_UINT }, ComponentMappingRGBA);
	MapFormatSupport(PF_R16G16B16A16_SINT, { VK_FORMAT_R16G16B16A16_SINT }, ComponentMappingRGBA);
	MapFormatSupport(PF_R32G32_UINT, { VK_FORMAT_R32G32_UINT }, ComponentMappingRG01);
	MapFormatSupport(PF_R32G32B32A32_UINT, { VK_FORMAT_R32G32B32A32_UINT }, ComponentMappingRGBA);
	MapFormatSupport(PF_R16G16B16A16_SNORM, { VK_FORMAT_R16G16B16A16_SNORM, VK_FORMAT_R16G16B16A16_SFLOAT }, ComponentMappingRGBA);
	MapFormatSupport(PF_R16G16B16A16_UNORM, { VK_FORMAT_R16G16B16A16_UNORM, VK_FORMAT_R16G16B16A16_SFLOAT }, ComponentMappingRGBA);
	MapFormatSupport(PF_R8G8, { VK_FORMAT_R8G8_UNORM }, ComponentMappingRG01);
	MapFormatSupport(PF_V8U8, { VK_FORMAT_R8G8_UNORM }, ComponentMappingRG01);
	MapFormatSupport(PF_R32_FLOAT, { VK_FORMAT_R32_SFLOAT }, ComponentMappingR001);
	MapFormatSupport(PF_R8, { VK_FORMAT_R8_UNORM }, ComponentMappingR001);
	MapFormatSupport(PF_G16R16_SNORM, { VK_FORMAT_R16G16_SNORM }, ComponentMappingRG01);
	MapFormatSupport(PF_R8G8_UINT, { VK_FORMAT_R8G8_UINT }, ComponentMappingRG01);
	MapFormatSupport(PF_R32G32B32_UINT, { VK_FORMAT_R32G32B32_UINT }, ComponentMappingRGB1);
	MapFormatSupport(PF_R32G32B32_SINT, { VK_FORMAT_R32G32B32_SINT }, ComponentMappingRGB1);
	MapFormatSupport(PF_R32G32B32F, { VK_FORMAT_R32G32B32_SFLOAT }, ComponentMappingRGB1);
	MapFormatSupport(PF_R8_SINT, { VK_FORMAT_R8_SINT }, ComponentMappingR001);
	MapFormatSupport(PF_R8G8B8, { VK_FORMAT_R8G8B8_UNORM }, ComponentMappingRGB1, ColorRenderTargetRequiredCapabilities);

	// This will be the format used for 64bit image atomics
	// This format is SM5 only, skip it for mobile to not confuse QA with a logged error about missing pixel format
	if (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5)
	{
#if VULKAN_HAS_DEBUGGING_ENABLED
		const EPixelFormatCapabilities RequiredCaps64U = GRenderDocFound ? EPixelFormatCapabilities::UAV : (EPixelFormatCapabilities::UAV | EPixelFormatCapabilities::TextureAtomics);
#else
		const EPixelFormatCapabilities RequiredCaps64U = (EPixelFormatCapabilities::UAV | EPixelFormatCapabilities::TextureAtomics);
#endif
		MapFormatSupport(PF_R64_UINT, { VK_FORMAT_R64_UINT, VK_FORMAT_R32G32_UINT }, ComponentMappingR001, RequiredCaps64U);
		// Shaders were patched to use UAV, make sure we don't expose texture sampling
		GPixelFormats[PF_R64_UINT].Capabilities &= ~(EPixelFormatCapabilities::AnyTexture | EPixelFormatCapabilities::TextureSample);
		if (GRHISupportsAtomicUInt64 && !EnumHasAnyFlags(GPixelFormats[PF_R64_UINT].Capabilities, EPixelFormatCapabilities::UAV))
		{
			UE_LOGF(LogVulkanRHI, Warning, "64bit image atomics were enabled, but the R64 format does not have UAV capabilities.  Disabling support.");
			GRHISupportsAtomicUInt64 = false;
		}
	}

	if (CVarVulkanUseD24.GetValueOnAnyThread() != 0)
	{
		// prefer VK_FORMAT_D24_UNORM_S8_UINT
		MapFormatSupport(PF_DepthStencil, { VK_FORMAT_D24_UNORM_S8_UINT, VK_FORMAT_D32_SFLOAT_S8_UINT, VK_FORMAT_D16_UNORM_S8_UINT }, ComponentMappingRIII);
		MapFormatSupport(PF_X24_G8, { VK_FORMAT_D24_UNORM_S8_UINT, VK_FORMAT_D32_SFLOAT_S8_UINT, VK_FORMAT_D16_UNORM_S8_UINT }, ComponentMappingRR01);
		GPixelFormats[PF_DepthStencil].bIs24BitUnormDepthStencil = true;
	}
	else
	{
		// prefer VK_FORMAT_D32_SFLOAT_S8_UINT
		MapFormatSupport(PF_DepthStencil, { VK_FORMAT_D32_SFLOAT_S8_UINT, VK_FORMAT_D24_UNORM_S8_UINT, VK_FORMAT_D16_UNORM_S8_UINT }, ComponentMappingRIII);
		MapFormatSupport(PF_X24_G8, { VK_FORMAT_D32_SFLOAT_S8_UINT, VK_FORMAT_D24_UNORM_S8_UINT, VK_FORMAT_D16_UNORM_S8_UINT }, ComponentMappingRR01);
		GPixelFormats[PF_DepthStencil].bIs24BitUnormDepthStencil = false;
	}

	if (FVulkanPlatform::SupportsBCTextureFormats())
	{
		MapFormatSupport(PF_DXT1, { VK_FORMAT_BC1_RGB_UNORM_BLOCK }, ComponentMappingRGB1);	// Also what OpenGL expects (RGBA instead RGB, but not SRGB)
		MapFormatSupport(PF_DXT3, { VK_FORMAT_BC2_UNORM_BLOCK },     ComponentMappingRGBA);
		MapFormatSupport(PF_DXT5, { VK_FORMAT_BC3_UNORM_BLOCK },     ComponentMappingRGBA);
		MapFormatSupport(PF_BC4,  { VK_FORMAT_BC4_UNORM_BLOCK },     ComponentMappingRGBA);
		MapFormatSupport(PF_BC5,  { VK_FORMAT_BC5_UNORM_BLOCK },     ComponentMappingRGBA);
		MapFormatSupport(PF_BC6H, { VK_FORMAT_BC6H_UFLOAT_BLOCK },   ComponentMappingRGBA);
		MapFormatSupport(PF_BC7,  { VK_FORMAT_BC7_UNORM_BLOCK },     ComponentMappingRGBA);
	}

	if (FVulkanPlatform::SupportsASTCTextureFormats())
	{
		MapFormatSupport(PF_ASTC_4x4,   { VK_FORMAT_ASTC_4x4_UNORM_BLOCK },   ComponentMappingRGBA);
		MapFormatSupport(PF_ASTC_6x6,   { VK_FORMAT_ASTC_6x6_UNORM_BLOCK },   ComponentMappingRGBA);
		MapFormatSupport(PF_ASTC_8x8,   { VK_FORMAT_ASTC_8x8_UNORM_BLOCK },   ComponentMappingRGBA);
		MapFormatSupport(PF_ASTC_10x10, { VK_FORMAT_ASTC_10x10_UNORM_BLOCK }, ComponentMappingRGBA);
		MapFormatSupport(PF_ASTC_12x12, { VK_FORMAT_ASTC_12x12_UNORM_BLOCK }, ComponentMappingRGBA);
	}

	if (FVulkanPlatform::SupportsETC2TextureFormats())
	{
		MapFormatSupport(PF_ETC2_RGB,      { VK_FORMAT_ETC2_R8G8B8_UNORM_BLOCK },   ComponentMappingRGB1);
		MapFormatSupport(PF_ETC2_RGBA,     { VK_FORMAT_ETC2_R8G8B8A8_UNORM_BLOCK }, ComponentMappingRGBA);
		MapFormatSupport(PF_ETC2_R11_EAC,  { VK_FORMAT_EAC_R11_UNORM_BLOCK },       ComponentMappingR001);
		MapFormatSupport(PF_ETC2_RG11_EAC, { VK_FORMAT_EAC_R11G11_UNORM_BLOCK },    ComponentMappingRG01);
	}
	if (FVulkanPlatform::SupportsR16UnormTextureFormat())
	{
		MapFormatSupport(PF_G16, { VK_FORMAT_R16_UNORM, VK_FORMAT_R16_SFLOAT }, ComponentMappingR001);
	}
	else
	{
		MapFormatSupport(PF_G16, { VK_FORMAT_R16_SFLOAT, VK_FORMAT_R16_UNORM }, ComponentMappingR001);
	}

	if (GetOptionalExtensions().HasEXTTextureCompressionASTCHDR)
	{
		MapFormatSupport(PF_ASTC_4x4_HDR,   { VK_FORMAT_ASTC_4x4_SFLOAT_BLOCK_EXT },   ComponentMappingRGBA);
		MapFormatSupport(PF_ASTC_6x6_HDR,   { VK_FORMAT_ASTC_6x6_SFLOAT_BLOCK_EXT },   ComponentMappingRGBA);
		MapFormatSupport(PF_ASTC_8x8_HDR,   { VK_FORMAT_ASTC_8x8_SFLOAT_BLOCK_EXT },   ComponentMappingRGBA);
		MapFormatSupport(PF_ASTC_10x10_HDR, { VK_FORMAT_ASTC_10x10_SFLOAT_BLOCK_EXT }, ComponentMappingRGBA);
		MapFormatSupport(PF_ASTC_12x12_HDR, { VK_FORMAT_ASTC_12x12_SFLOAT_BLOCK_EXT }, ComponentMappingRGBA);
	}

	// When this extension is available, PF_Unknown texture can have an external buffer attached which has an
	// internal format, that can be sampled. If it really can be sampled depends on the VK_IMAGE_USAGE_SAMPLED_BIT
	if (GetOptionalExtensions().HasANDROIDExternalMemoryHardwareBuffer)
	{
		GPixelFormats[PF_Unknown].Capabilities |= EPixelFormatCapabilities::TextureSample;
	}

	// Verify available Vertex Formats
	{
		static_assert(VET_None == 0, "Change loop below to skip VET_None");
		for (int32 VETIndex = (int32)VET_None + 1; VETIndex < VET_MAX; ++VETIndex)
		{
			const EVertexElementType UEType = (EVertexElementType)VETIndex;
			const VkFormat VulkanFormat = UEToVkBufferFormat(UEType);
			const VkFormatProperties& VertexFormatProperties = GetFormatProperties(VulkanFormat);
			if (VertexFormatProperties.bufferFeatures == 0)
			{
				UE_LOGF(LogVulkanRHI, Warning, "EVertexElementType(%d) is not supported with VkFormat %d", (int32)UEType, (int32)VulkanFormat);
			}
		}
	}

	// Verify the potential SRGB formats and fill GVulkanSRGBFormat
	{
		auto GetSRGBMapping = [this](const VkFormat InFormat)
		{
			VkFormat SRGBFormat = InFormat;
			switch (InFormat)
			{
			case VK_FORMAT_B8G8R8A8_UNORM:				SRGBFormat = VK_FORMAT_B8G8R8A8_SRGB; break;
			case VK_FORMAT_A8B8G8R8_UNORM_PACK32:		SRGBFormat = VK_FORMAT_A8B8G8R8_SRGB_PACK32; break;
			case VK_FORMAT_R8_UNORM:					SRGBFormat = ((GMaxRHIFeatureLevel <= ERHIFeatureLevel::ES3_1) ? VK_FORMAT_R8_UNORM : VK_FORMAT_R8_SRGB); break;
			case VK_FORMAT_R8G8_UNORM:					SRGBFormat = VK_FORMAT_R8G8_SRGB; break;
			case VK_FORMAT_R8G8B8_UNORM:				SRGBFormat = VK_FORMAT_R8G8B8_SRGB; break;
			case VK_FORMAT_R8G8B8A8_UNORM:				SRGBFormat = VK_FORMAT_R8G8B8A8_SRGB; break;
			case VK_FORMAT_BC1_RGB_UNORM_BLOCK:			SRGBFormat = VK_FORMAT_BC1_RGB_SRGB_BLOCK; break;
			case VK_FORMAT_BC1_RGBA_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_BC1_RGBA_SRGB_BLOCK; break;
			case VK_FORMAT_BC2_UNORM_BLOCK:				SRGBFormat = VK_FORMAT_BC2_SRGB_BLOCK; break;
			case VK_FORMAT_BC3_UNORM_BLOCK:				SRGBFormat = VK_FORMAT_BC3_SRGB_BLOCK; break;
			case VK_FORMAT_BC7_UNORM_BLOCK:				SRGBFormat = VK_FORMAT_BC7_SRGB_BLOCK; break;
			case VK_FORMAT_ETC2_R8G8B8_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ETC2_R8G8B8_SRGB_BLOCK; break;
			case VK_FORMAT_ETC2_R8G8B8A1_UNORM_BLOCK:	SRGBFormat = VK_FORMAT_ETC2_R8G8B8A1_SRGB_BLOCK; break;
			case VK_FORMAT_ETC2_R8G8B8A8_UNORM_BLOCK:	SRGBFormat = VK_FORMAT_ETC2_R8G8B8A8_SRGB_BLOCK; break;
			case VK_FORMAT_ASTC_4x4_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ASTC_4x4_SRGB_BLOCK; break;
			case VK_FORMAT_ASTC_5x4_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ASTC_5x4_SRGB_BLOCK; break;
			case VK_FORMAT_ASTC_5x5_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ASTC_5x5_SRGB_BLOCK; break;
			case VK_FORMAT_ASTC_6x5_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ASTC_6x5_SRGB_BLOCK; break;
			case VK_FORMAT_ASTC_6x6_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ASTC_6x6_SRGB_BLOCK; break;
			case VK_FORMAT_ASTC_8x5_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ASTC_8x5_SRGB_BLOCK; break;
			case VK_FORMAT_ASTC_8x6_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ASTC_8x6_SRGB_BLOCK; break;
			case VK_FORMAT_ASTC_8x8_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ASTC_8x8_SRGB_BLOCK; break;
			case VK_FORMAT_ASTC_10x5_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ASTC_10x5_SRGB_BLOCK; break;
			case VK_FORMAT_ASTC_10x6_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ASTC_10x6_SRGB_BLOCK; break;
			case VK_FORMAT_ASTC_10x8_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ASTC_10x8_SRGB_BLOCK; break;
			case VK_FORMAT_ASTC_10x10_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ASTC_10x10_SRGB_BLOCK; break;
			case VK_FORMAT_ASTC_12x10_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ASTC_12x10_SRGB_BLOCK; break;
			case VK_FORMAT_ASTC_12x12_UNORM_BLOCK:		SRGBFormat = VK_FORMAT_ASTC_12x12_SRGB_BLOCK; break;
				//		case VK_FORMAT_PVRTC1_2BPP_UNORM_BLOCK_IMG:	Format = VK_FORMAT_PVRTC1_2BPP_SRGB_BLOCK_IMG; break;
				//		case VK_FORMAT_PVRTC1_4BPP_UNORM_BLOCK_IMG:	Format = VK_FORMAT_PVRTC1_4BPP_SRGB_BLOCK_IMG; break;
				//		case VK_FORMAT_PVRTC2_2BPP_UNORM_BLOCK_IMG:	Format = VK_FORMAT_PVRTC2_2BPP_SRGB_BLOCK_IMG; break;
				//		case VK_FORMAT_PVRTC2_4BPP_UNORM_BLOCK_IMG:	Format = VK_FORMAT_PVRTC2_4BPP_SRGB_BLOCK_IMG; break;
			default:	break;
			}

			// If we're introducing a new format, make sure it's supported
			if (InFormat != SRGBFormat)
			{
				const VkFormatProperties& SRGBFormatProperties = GetFormatProperties(SRGBFormat);
				if (!VKHasAnyFlags(SRGBFormatProperties.optimalTilingFeatures, VK_FORMAT_FEATURE_SAMPLED_IMAGE_BIT))
				{
					// If we can't even sample from it, then reject the suggested SRGB format
					SRGBFormat = InFormat;
				}
			}

			return SRGBFormat;
		};

		for (int32 PixelFormatIndex = 0; PixelFormatIndex < PF_MAX; ++PixelFormatIndex)
		{
			const FPixelFormatInfo& PixelFormatInfo = GPixelFormats[PixelFormatIndex];
			if (PixelFormatInfo.Supported)
			{
				const VkFormat OriginalFormat = (VkFormat)PixelFormatInfo.PlatformFormat;
				GVulkanSRGBFormat[PixelFormatIndex] = GetSRGBMapping(OriginalFormat);
			}
			else
			{
				GVulkanSRGBFormat[PixelFormatIndex] = VK_FORMAT_UNDEFINED;
			}
		}
	}

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT

	// Print the resulting pixel format support
	if (FParse::Param(FCommandLine::Get(), TEXT("PrintVulkanPixelFormatMappings")))
	{
		auto GetFormatCapabilities = [](EPixelFormatCapabilities FormatCapabilities)
		{
#define VULKAN_CHECK_FORMAT_CAPABILITY(PF_Name) if (EnumHasAllFlags(FormatCapabilities, EPixelFormatCapabilities::PF_Name)) { CapabilitiesString += TEXT(#PF_Name) TEXT(", ");}
			FString CapabilitiesString;

			VULKAN_CHECK_FORMAT_CAPABILITY(TextureSample);
			VULKAN_CHECK_FORMAT_CAPABILITY(TextureCube);
			VULKAN_CHECK_FORMAT_CAPABILITY(RenderTarget);
			VULKAN_CHECK_FORMAT_CAPABILITY(DepthStencil);
			VULKAN_CHECK_FORMAT_CAPABILITY(TextureBlendable);
			VULKAN_CHECK_FORMAT_CAPABILITY(TextureAtomics);

			VULKAN_CHECK_FORMAT_CAPABILITY(Buffer);
			VULKAN_CHECK_FORMAT_CAPABILITY(VertexBuffer);
			VULKAN_CHECK_FORMAT_CAPABILITY(IndexBuffer);
			VULKAN_CHECK_FORMAT_CAPABILITY(BufferAtomics);

			VULKAN_CHECK_FORMAT_CAPABILITY(UAV);

			return CapabilitiesString;
#undef VULKAN_CHECK_FORMAT_CAPABILITY
		};

		UE_LOGF(LogVulkanRHI, Warning, "Pixel Format Mappings for Vulkan:");
		UE_LOGF(LogVulkanRHI, Warning, "%24ls | %24ls | BlockBytes | Components | ComponentMapping | BufferFormat | Capabilities | SRGBFormat", 
			TEXT("PixelFormatName"), TEXT("VulkanFormat"));
		for (int32 PixelFormatIndex = 0; PixelFormatIndex < PF_MAX; ++PixelFormatIndex)
		{
			if (GPixelFormats[PixelFormatIndex].Supported)
			{
				const VkComponentMapping& ComponentMapping = PixelFormatComponentMapping[PixelFormatIndex];

				const VkFormat VulkanFormat = (VkFormat)GPixelFormats[PixelFormatIndex].PlatformFormat;
				FString VulkanFormatStr(VK_TYPE_TO_STRING(VkFormat, VulkanFormat));
				VulkanFormatStr.RightChopInline(10);  // Chop the VK_FORMAT_

				FString SRGBFormat;
				if (VulkanFormat != GVulkanSRGBFormat[PixelFormatIndex])
				{
					SRGBFormat = VK_TYPE_TO_STRING(VkFormat, GVulkanSRGBFormat[PixelFormatIndex]);
					SRGBFormat.RightChopInline(10);  // Chop the VK_FORMAT_
				}

				UE_LOGF(LogVulkanRHI, Warning, "%24ls | %24ls | %10d | %10d | %10d,%d,%d,%d | %12d |  0x%08X  | %ls",
					GPixelFormats[PixelFormatIndex].Name,
					*VulkanFormatStr,
					GPixelFormats[PixelFormatIndex].BlockBytes,
					GPixelFormats[PixelFormatIndex].NumComponents,
					ComponentMapping.r, ComponentMapping.g, ComponentMapping.b, ComponentMapping.a,
					(int32)GVulkanBufferFormat[PixelFormatIndex],
					(uint32)GPixelFormats[PixelFormatIndex].Capabilities,
					*SRGBFormat
					);
			}
		}

		UE_LOGF(LogVulkanRHI, Warning, "Pixel Format Capabilities for Vulkan:");
		for (int32 PixelFormatIndex = 0; PixelFormatIndex < PF_MAX; ++PixelFormatIndex)
		{
			if (GPixelFormats[PixelFormatIndex].Supported)
			{
				const FString CapabilitiesString = GetFormatCapabilities(GPixelFormats[PixelFormatIndex].Capabilities);
				UE_LOGF(LogVulkanRHI, Warning, "%24ls : %ls", GPixelFormats[PixelFormatIndex].Name, *CapabilitiesString);
			}
		}
	}
#endif  // UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
}

const VkFormatProperties& FVulkanDevice::GetFormatProperties(VkFormat InFormat) const
{
	if (InFormat >= 0 && InFormat < VK_FORMAT_RANGE_SIZE)
	{
		return FormatProperties[InFormat];
	}

	// Check for extension formats
	const VkFormatProperties* FoundProperties = ExtensionFormatProperties.Find(InFormat);
	if (FoundProperties)
	{
		return *FoundProperties;
	}

	// Add it for faster caching next time
	VkFormatProperties& NewProperties = ExtensionFormatProperties.Add(InFormat);
	FMemory::Memzero(NewProperties);
	VulkanRHI::vkGetPhysicalDeviceFormatProperties(Gpu, InFormat, &NewProperties);
	return NewProperties;
}

void FVulkanDevice::MapBufferFormatSupport(FPixelFormatInfo& PixelFormatInfo, EPixelFormat UEFormat, VkFormat VulkanFormat)
{
	check(GVulkanBufferFormat[UEFormat] == VK_FORMAT_UNDEFINED);

	const VkFormatProperties& LocalFormatProperties = GetFormatProperties(VulkanFormat);
	EPixelFormatCapabilities Capabilities = EPixelFormatCapabilities::None;

	auto ConvertBufferCap = [&Capabilities, &LocalFormatProperties](EPixelFormatCapabilities UnrealCap, VkFormatFeatureFlags InFlag)
	{
		const bool HasBufferFeature = VKHasAllFlags(LocalFormatProperties.bufferFeatures, InFlag);
		if (HasBufferFeature)
		{
			EnumAddFlags(Capabilities, UnrealCap);
		}

		// Make sure we aren't looking in the wrong place for a bit
		check(!VKHasAnyFlags(LocalFormatProperties.linearTilingFeatures, InFlag));
		check(!VKHasAnyFlags(LocalFormatProperties.optimalTilingFeatures, InFlag));
	};

	// Check for buffer caps, use the first one with any caps
	if (LocalFormatProperties.bufferFeatures != 0)
	{
		EnumAddFlags(Capabilities, EPixelFormatCapabilities::Buffer);

		ConvertBufferCap(EPixelFormatCapabilities::VertexBuffer, VK_FORMAT_FEATURE_VERTEX_BUFFER_BIT);
		ConvertBufferCap(EPixelFormatCapabilities::BufferLoad, VK_FORMAT_FEATURE_UNIFORM_TEXEL_BUFFER_BIT);
		ConvertBufferCap(EPixelFormatCapabilities::BufferStore, VK_FORMAT_FEATURE_STORAGE_TEXEL_BUFFER_BIT);
		ConvertBufferCap(EPixelFormatCapabilities::BufferAtomics, VK_FORMAT_FEATURE_STORAGE_TEXEL_BUFFER_ATOMIC_BIT);

		// Vulkan index buffers aren't tied to formats, so any 16 or 32 bit UINT format with a single component will do...
		// But because we can't check for uint vs float, hardcode supported formats for now
		if (EnumHasAllFlags(Capabilities, (EPixelFormatCapabilities::BufferLoad | EPixelFormatCapabilities::BufferStore)) &&
			((VulkanFormat == VK_FORMAT_R16_UINT) || (VulkanFormat == VK_FORMAT_R32_UINT)))
		{
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::IndexBuffer);
		}

		GVulkanBufferFormat[UEFormat] = VulkanFormat;
		PixelFormatInfo.Capabilities |= Capabilities;
	}
}

void FVulkanDevice::MapImageFormatSupport(FPixelFormatInfo& PixelFormatInfo, const TArrayView<const VkFormat>& PrioritizedFormats, EPixelFormatCapabilities RequiredCapabilities)
{
	// Query for MipMap support with typical parameters
	auto SupportsMipMap = [this](VkFormat InFormat)
	{
		VkImageFormatProperties ImageFormatProperties;
		VkResult RetVal = VulkanRHI::vkGetPhysicalDeviceImageFormatProperties(Gpu, InFormat, VK_IMAGE_TYPE_2D, VK_IMAGE_TILING_OPTIMAL, VK_IMAGE_USAGE_SAMPLED_BIT, 0, &ImageFormatProperties);
		return (RetVal == VK_SUCCESS) && (ImageFormatProperties.maxMipLevels > 1);
	};

	EPixelFormatCapabilities Capabilities = EPixelFormatCapabilities::None;
	auto ConvertImageCap = [&Capabilities](const VkFormatProperties& InFormatProperties, EPixelFormatCapabilities UnrealCap, VkFormatFeatureFlags InFlag, bool bOnlyOptimalTiling)
	{
		// Do not distinguish between Linear and Optimal for now.
		bool HasImageFeature = VKHasAllFlags(InFormatProperties.optimalTilingFeatures, InFlag);
		if (!bOnlyOptimalTiling)
		{
			HasImageFeature |= VKHasAllFlags(InFormatProperties.linearTilingFeatures, InFlag);
		}

		if (HasImageFeature)
		{
			EnumAddFlags(Capabilities, UnrealCap);
		}

		// Make sure we aren't looking in the wrong place for a bit
		check(!VKHasAnyFlags(InFormatProperties.bufferFeatures, InFlag));
	};

	// Go through the PrioritizedFormats and use the first one that meets RequiredCapabilities
	for (int32 FormatIndex = 0; FormatIndex < PrioritizedFormats.Num(); ++FormatIndex)
	{
		Capabilities = EPixelFormatCapabilities::None;

		const VkFormat VulkanFormat = PrioritizedFormats[FormatIndex];
		const VkFormatProperties& LocalFormatProperties = GetFormatProperties(VulkanFormat);

		// Check for individual texture caps
		ConvertImageCap(LocalFormatProperties, EPixelFormatCapabilities::AnyTexture | EPixelFormatCapabilities::TextureSample | EPixelFormatCapabilities::TextureLoad, VK_FORMAT_FEATURE_SAMPLED_IMAGE_BIT, false);
		ConvertImageCap(LocalFormatProperties, EPixelFormatCapabilities::DepthStencil, VK_FORMAT_FEATURE_DEPTH_STENCIL_ATTACHMENT_BIT, true);
		ConvertImageCap(LocalFormatProperties, EPixelFormatCapabilities::RenderTarget, VK_FORMAT_FEATURE_COLOR_ATTACHMENT_BIT, true);
		ConvertImageCap(LocalFormatProperties, EPixelFormatCapabilities::TextureBlendable, VK_FORMAT_FEATURE_COLOR_ATTACHMENT_BLEND_BIT, true);
		ConvertImageCap(LocalFormatProperties, EPixelFormatCapabilities::AllUAVFlags | EPixelFormatCapabilities::TextureStore, VK_FORMAT_FEATURE_STORAGE_IMAGE_BIT, false);

		ConvertImageCap(LocalFormatProperties, EPixelFormatCapabilities::TextureAtomics, VK_FORMAT_FEATURE_STORAGE_IMAGE_ATOMIC_BIT, false);
		ConvertImageCap(LocalFormatProperties, EPixelFormatCapabilities::TextureFilterable, VK_FORMAT_FEATURE_SAMPLED_IMAGE_FILTER_LINEAR_BIT, false);

		if (EnumHasAllFlags(Capabilities, EPixelFormatCapabilities::AnyTexture))
		{
			// We support gather, but some of our shaders assume offsets so check against features
			if (GetPhysicalDeviceFeatures().Core_1_0.shaderImageGatherExtended)
			{
				EnumAddFlags(Capabilities, EPixelFormatCapabilities::TextureGather);
			}

			if (SupportsMipMap(VulkanFormat))
			{
				EnumAddFlags(Capabilities, EPixelFormatCapabilities::TextureMipmaps);
			}

			if (OptionalDeviceExtensions.HasEXTImageCompressionControl)
			{
				VkImageCompressionPropertiesEXT ImageCompressionProperties{ VK_STRUCTURE_TYPE_IMAGE_COMPRESSION_PROPERTIES_EXT };
				VkImageFormatProperties2 ImageFormatProperties{ VK_STRUCTURE_TYPE_IMAGE_FORMAT_PROPERTIES_2 };
				ImageFormatProperties.pNext = &ImageCompressionProperties;
				VkImageCompressionControlEXT CompressionControl{ VK_STRUCTURE_TYPE_IMAGE_COMPRESSION_CONTROL_EXT };
				CompressionControl.flags = VK_IMAGE_COMPRESSION_FIXED_RATE_DEFAULT_EXT;

				VkPhysicalDeviceImageFormatInfo2 ImageFormatInfo{ VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_IMAGE_FORMAT_INFO_2 };
				ImageFormatInfo.pNext = &CompressionControl;
				ImageFormatInfo.format = VulkanFormat;
				ImageFormatInfo.type = VK_IMAGE_TYPE_2D;
				ImageFormatInfo.tiling = VK_IMAGE_TILING_OPTIMAL;
				ImageFormatInfo.usage = VK_IMAGE_USAGE_SAMPLED_BIT;

				if (VulkanRHI::vkGetPhysicalDeviceImageFormatProperties2(Gpu, &ImageFormatInfo, &ImageFormatProperties) == VK_SUCCESS &&
					(ImageCompressionProperties.imageCompressionFlags & VK_IMAGE_COMPRESSION_FIXED_RATE_EXPLICIT_EXT) != 0)
				{
					EnumAddFlags(Capabilities, EPixelFormatCapabilities::LossyCompressible);
				};
			}
		}

		if (EnumHasAllFlags(Capabilities, RequiredCapabilities))
		{
			PixelFormatInfo.PlatformFormat = VulkanFormat;
			PixelFormatInfo.Capabilities |= Capabilities;

			if (FormatIndex > 0)
			{
				UE_LOGF(LogVulkanRHI, Display, "MapImageFormatSupport: %ls is not supported with VkFormat %d, falling back to VkFormat %d", PixelFormatInfo.Name, (int32)PrioritizedFormats[0], (int32)PrioritizedFormats[FormatIndex]);
			}

			break;
		}
	}
}

// Minimum capabilities required for a Vulkan format to be considered as supported
static constexpr EPixelFormatCapabilities kDefaultTextureCapabilities = EPixelFormatCapabilities::TextureSample;
// Passthrough to specify we want to keep the initial BlockBytes value set in the PixelFormat
static constexpr int32 kDefaultBlockBytes = -1;

void FVulkanDevice::MapFormatSupport(EPixelFormat UEFormat, std::initializer_list<VkFormat> InPrioritizedFormats, const VkComponentMapping& ComponentMapping, EPixelFormatCapabilities RequiredCapabilities, int32 BlockBytes)
{
	TArrayView<const VkFormat> PrioritizedFormats = MakeArrayView(InPrioritizedFormats);
	FPixelFormatInfo& PixelFormatInfo = GPixelFormats[UEFormat];

	check(PrioritizedFormats.Num() > 0);
	check(!PixelFormatInfo.Supported);
	check(PixelFormatInfo.Capabilities == EPixelFormatCapabilities::None);

	MapBufferFormatSupport(PixelFormatInfo, UEFormat, PrioritizedFormats[0]);
	MapImageFormatSupport(PixelFormatInfo, PrioritizedFormats, RequiredCapabilities);

	// Flag the pixel format as supported if we can do anything with it
	PixelFormatInfo.Supported = EnumHasAllFlags(PixelFormatInfo.Capabilities, RequiredCapabilities) || EnumHasAnyFlags(PixelFormatInfo.Capabilities, EPixelFormatCapabilities::Buffer);
	if (PixelFormatInfo.Supported)
	{
		PixelFormatComponentMapping[UEFormat] = ComponentMapping;
		if (BlockBytes > 0)
		{
			PixelFormatInfo.BlockBytes = BlockBytes;
		}
	}
	else
	{
		UE_LOGF(LogVulkanRHI, Error, "MapFormatSupport: %ls is not supported with VkFormat %d", PixelFormatInfo.Name, (int32)PrioritizedFormats[0]);
	}
}

void FVulkanDevice::MapFormatSupport(EPixelFormat UEFormat, std::initializer_list<VkFormat> PrioritizedFormats, const VkComponentMapping& ComponentMapping)
{
	MapFormatSupport(UEFormat, PrioritizedFormats, ComponentMapping, kDefaultTextureCapabilities, kDefaultBlockBytes);
}
void FVulkanDevice::MapFormatSupport(EPixelFormat UEFormat, std::initializer_list<VkFormat> PrioritizedFormats, const VkComponentMapping& ComponentMapping, int32 BlockBytes)
{
	MapFormatSupport(UEFormat, PrioritizedFormats, ComponentMapping, kDefaultTextureCapabilities, BlockBytes);
}
void FVulkanDevice::MapFormatSupport(EPixelFormat UEFormat, std::initializer_list<VkFormat> PrioritizedFormats, const VkComponentMapping& ComponentMapping, EPixelFormatCapabilities RequiredCapabilities)
{
	MapFormatSupport(UEFormat, PrioritizedFormats, ComponentMapping, RequiredCapabilities, kDefaultBlockBytes);
}

bool FVulkanDevice::SupportsBindless() const
{
	checkSlow(BindlessDescriptorManager != nullptr);
	return BindlessDescriptorManager->IsSupported();
}

void FVulkanDevice::ChooseVariableRateShadingMethod()
{
	auto IsFragmentShadingRateAvailable = [](VkPhysicalDeviceFragmentShadingRateFeaturesKHR& FragmentShadingRateFeatures)
	{
		return FragmentShadingRateFeatures.attachmentFragmentShadingRate == VK_TRUE;
	};

	auto IsFragmentDensityMapAvailable = [](FOptionalVulkanDeviceExtensions& ExtensionFlags)
	{
		return ExtensionFlags.HasEXTFragmentDensityMap == 1;
	};

	auto TurnOffFragmentShadingRate = [](VkPhysicalDeviceFragmentShadingRateFeaturesKHR& FragmentShadingRateFeatures)
	{
		FragmentShadingRateFeatures.primitiveFragmentShadingRate = VK_FALSE;
		FragmentShadingRateFeatures.attachmentFragmentShadingRate = VK_FALSE;
		FragmentShadingRateFeatures.pipelineFragmentShadingRate = VK_FALSE;
		GRHISupportsPipelineVariableRateShading = false;
		GRHISupportsLargerVariableRateShadingSizes = false;
	};

	auto TurnOffFragmentDensityMap = [](FOptionalVulkanDeviceExtensions& ExtensionFlags, VkPhysicalDeviceFragmentDensityMapFeaturesEXT& FragmentDensityMapFeatures, VkPhysicalDeviceFragmentDensityMap2FeaturesEXT& FragmentDensityMap2Features)
	{
		ExtensionFlags.HasEXTFragmentDensityMap = 0;
		FragmentDensityMapFeatures.fragmentDensityMap = VK_FALSE;
		FragmentDensityMapFeatures.fragmentDensityMapDynamic = VK_FALSE;
		FragmentDensityMapFeatures.fragmentDensityMapNonSubsampledImages = VK_FALSE;
		ExtensionFlags.HasEXTFragmentDensityMap2 = 0;
		FragmentDensityMap2Features.fragmentDensityMapDeferred = VK_FALSE;
	};

	int32 VRSFormatPreference = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Vulkan.VRSFormat"))->GetValueOnAnyThread();
	UE_LOGF(LogVulkanRHI, Display, "Vulkan Variable Rate Shading choice: %d.", VRSFormatPreference);

	const bool bFSRAvailable = IsFragmentShadingRateAvailable(OptionalDeviceExtensionProperties.FragmentShadingRateFeatures);
	const bool bFDMAvailable = IsFragmentDensityMapAvailable(OptionalDeviceExtensions);

	// Whether we prefer Fragment Shading Rate, or Fragment Density Map.
	const bool bPreferFSR = (VRSFormatPreference <= (uint8)EVulkanVariableRateShadingPreference::RequireFSR);
	
	// Whether we allow falling back to the other extension if our preferred extension is unavailable.
	const bool bAllowFallback = (VRSFormatPreference == (uint8)EVulkanVariableRateShadingPreference::PreferFSR || VRSFormatPreference == (uint8)EVulkanVariableRateShadingPreference::PreferFDM);

	// Whether we should print an error if our preferred extension is unavailable.
	const bool bErrorIfUnavailable = !bAllowFallback && (VRSFormatPreference == (uint8)EVulkanVariableRateShadingPreference::RequireFSR || VRSFormatPreference == (uint8)EVulkanVariableRateShadingPreference::RequireFDM);

	const bool bPreferredAvailable = bPreferFSR ? bFSRAvailable : bFDMAvailable;
	const bool bFallbackAvailable = bAllowFallback && (bPreferFSR ? bFDMAvailable : bFSRAvailable);

	if (bPreferredAvailable)
	{
		UE_LOGF(LogVulkanRHI, Display, "Fragment %ls was %ls and is available.", bPreferFSR ? TEXT("Shading Rate") : TEXT("Density Map"), bAllowFallback ? TEXT("requested") : TEXT("required"));

		// Turn off our non-preferred method in case both are available in order to prevent Vulkan validation layers warnings.
		if (bPreferFSR)
		{
			TurnOffFragmentDensityMap(OptionalDeviceExtensions, OptionalDeviceExtensionProperties.FragmentDensityMapFeatures, OptionalDeviceExtensionProperties.FragmentDensityMap2Features);
		}
		else
		{
			TurnOffFragmentShadingRate(OptionalDeviceExtensionProperties.FragmentShadingRateFeatures);
		}
	}
	else
	{
		if (bErrorIfUnavailable)
		{
			UE_LOGF(LogVulkanRHI, Error, "Fragment %ls was required but is not available.", bPreferFSR ? TEXT("Shading Rate") : TEXT("Density Map"));
		}
		else
		{
			UE_LOGF(LogVulkanRHI, Display, "Fragment %ls was %ls but is not available.", bPreferFSR ? TEXT("Shading Rate") : TEXT("Density Map"), bAllowFallback ? TEXT("requested") : TEXT("required"));
		}

		if (bAllowFallback)
		{
			if (bFallbackAvailable)
			{
				UE_LOGF(LogVulkanRHI, Display, "Falling back to Fragment %ls.", bPreferFSR ? TEXT("Density Map") : TEXT("Shading Rate"));
			}
			else
			{
				UE_LOGF(LogVulkanRHI, Display, "Fragment %ls is not available as a fallback.", bPreferFSR ? TEXT("Density Map") : TEXT("Shading Rate"));
			}
		}
	}
}

void FVulkanDevice::InitGPU()
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanMisc);

	uint32 QueueCount = 0;
	VulkanRHI::vkGetPhysicalDeviceQueueFamilyProperties(Gpu, &QueueCount, nullptr);
	check(QueueCount >= 1);

	QueueFamilyProps.AddUninitialized(QueueCount);
	VulkanRHI::vkGetPhysicalDeviceQueueFamilyProperties(Gpu, &QueueCount, QueueFamilyProps.GetData());

	// Query base features
	PhysicalDeviceFeatures.Query(Gpu, RHI->GetApiVersion());

	// Setup layers and extensions
	FVulkanDeviceExtensionArray UEExtensions = FVulkanDeviceExtension::GetUESupportedDeviceExtensions(this, RHI->GetApiVersion());
	TArray<const ANSICHAR*> DeviceLayers = SetupDeviceLayers(UEExtensions);

	// Query advanced features
	{
		VkPhysicalDeviceFeatures2 PhysicalDeviceFeatures2;
		ZeroVulkanStruct(PhysicalDeviceFeatures2, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_FEATURES_2);

		for (TUniquePtr<FVulkanDeviceExtension>& UEExtension : UEExtensions)
		{
			if (UEExtension->InUse())
			{
				UEExtension->PrePhysicalDeviceFeatures(PhysicalDeviceFeatures2);
			}
		}

		VulkanRHI::vkGetPhysicalDeviceFeatures2(Gpu, &PhysicalDeviceFeatures2);

		for (TUniquePtr<FVulkanDeviceExtension>& UEExtension : UEExtensions)
		{
			if (UEExtension->InUse())
			{
				UEExtension->PostPhysicalDeviceFeatures(OptionalDeviceExtensions);
			}
		}
	}

	// Query advances properties
	{
		VkPhysicalDeviceProperties2 PhysicalDeviceProperties2;
		ZeroVulkanStruct(PhysicalDeviceProperties2, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_PROPERTIES_2);
		PhysicalDeviceProperties2.pNext = &GpuIdProps;
		ZeroVulkanStruct(GpuIdProps, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_ID_PROPERTIES);

		for (TUniquePtr<FVulkanDeviceExtension>& UEExtension : UEExtensions)
		{
			if (UEExtension->InUse())
			{
				UEExtension->PrePhysicalDeviceProperties(PhysicalDeviceProperties2);
			}
		}

		VulkanRHI::vkGetPhysicalDeviceProperties2(Gpu, &PhysicalDeviceProperties2);

		for (TUniquePtr<FVulkanDeviceExtension>& UEExtension : UEExtensions)
		{
			if (UEExtension->InUse())
			{
				UEExtension->PostPhysicalDeviceProperties(OptionalDeviceExtensions);
			}
		}
	}

	ChooseVariableRateShadingMethod();

	UE_LOGF(LogVulkanRHI, Display, "Device properties: Geometry %d BufferAtomic64 %lld ImageAtomic64 %lld", 
		PhysicalDeviceFeatures.Core_1_0.geometryShader, OptionalDeviceExtensions.HasKHRShaderAtomicInt64, OptionalDeviceExtensions.HasImageAtomicInt64);

	CreateDevice(DeviceLayers, UEExtensions);

	FVulkanPlatform::InitDevice(this);

	SetupFormats();

	DeviceMemoryManager.Init(this);

	MemoryManager.Init();

	StagingManager.Init(this);

	// TempBlockAllocator is currently used for UB uploads and copies
	{
		uint32 BlockAlignment = FMath::Max<uint32>(GetLimits().minUniformBufferOffsetAlignment, 16u);
		VkBufferUsageFlags BufferUsageFlags =
			(GetOptionalExtensions().HasBufferDeviceAddress ? VK_BUFFER_USAGE_SHADER_DEVICE_ADDRESS_BIT : 0) |
			VK_BUFFER_USAGE_TRANSFER_SRC_BIT |
			VK_BUFFER_USAGE_UNIFORM_BUFFER_BIT;
		if (GetOptionalExtensions().HasRayTracingPipeline)
		{
			BufferUsageFlags |= VK_BUFFER_USAGE_SHADER_BINDING_TABLE_BIT_KHR;
			BlockAlignment = FMath::Max<uint32>(GetOptionalExtensionProperties().RayTracingPipelineProps.shaderGroupBaseAlignment, BlockAlignment);
		}

		const int32 VulkanTempBlockSize = GCVarVulkanTempBlockSizeKB.GetValueOnAnyThread() * 1024;
		TempBlockAllocator = new VulkanRHI::FTempBlockAllocator(*this, VulkanTempBlockSize, BlockAlignment, BufferUsageFlags);
	}

	RenderPassManager = new FVulkanRenderPassManager(*this);

	if (UseVulkanDescriptorCache())
	{
		DescriptorSetCache = new FVulkanDescriptorSetCache(this);
	}
	
	DescriptorPoolsManager = new FVulkanDescriptorPoolsManager();
	DescriptorPoolsManager->Init(this);

	BindlessDescriptorManager = new FVulkanBindlessDescriptorManager(*this);
	BindlessDescriptorManager->Init();

	ShaderObjectManager = new FVulkanShaderObjectManager(*this);

	InitGlobalSamplers();

	if (UE::RHI::UseGPUCrashBreadcrumbs())
	{
		ForEachQueue([](FVulkanQueue& Queue)
			{
				Queue.InitDiagnosticBuffer();
			});
	}

	PipelineStateCache = new FVulkanPipelineStateCacheManager(this);

	TArray<FString> CacheFilenames = FVulkanPlatform::GetPSOCacheFilenames();

	// always look in the saved directory (for the cache from previous run that wasn't moved over to stage directory)
	CacheFilenames.Add(VulkanRHI::GetPipelineCacheFilename());

	ImmediateContext = new FVulkanCommandListContext(*this, ERHIPipeline::Graphics, true);

#if VULKAN_SUPPORTS_VALIDATION_CACHE
	if (OptionalDeviceExtensions.HasEXTValidationCache)
	{
		LoadValidationCache(Device, ValidationCache);
	}
#endif

	FVulkanChunkedPipelineCacheManager::Init();

	PipelineStateCache->InitAndLoad(CacheFilenames);

	if (RHISupportsRayTracing(GMaxRHIShaderPlatform) && GetOptionalExtensions().HasRaytracingExtensions())
	{
		check(RayTracingCompactionRequestHandler == nullptr);
		RayTracingCompactionRequestHandler = new FVulkanRayTracingCompactionRequestHandler(*this);
	}

	FVulkanPlatform::PostInitGPU(*this);
}

void FVulkanDevice::PrepareForDestroy()
{
	WaitUntilIdle();
}

void FVulkanDevice::Destroy()
{
#if VULKAN_SUPPORTS_VALIDATION_CACHE
	if (ValidationCache != VK_NULL_HANDLE)
	{
		PFN_vkDestroyValidationCacheEXT vkDestroyValidationCache = (PFN_vkDestroyValidationCacheEXT)(void*)VulkanRHI::vkGetDeviceProcAddr(Device, "vkDestroyValidationCacheEXT");
		if (vkDestroyValidationCache)
		{
			vkDestroyValidationCache(Device, ValidationCache, VULKAN_CPU_ALLOCATOR);
			ValidationCache = VK_NULL_HANDLE;
		}
	}
#endif

	// Release pending state that might hold references to RHI resources before we do final FlushPendingDeletes
	ImmediateContext->ReleasePendingState();

	if (TransientHeapCache)
	{
		delete TransientHeapCache;
		TransientHeapCache = nullptr;
	}

	// Flush all pending deletes before destroying the device and any Vulkan context objects.
	FRHICommandListExecutor::GetImmediateCommandList().ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

	delete DescriptorSetCache;
	DescriptorSetCache = nullptr;
	
	delete DescriptorPoolsManager;
	DescriptorPoolsManager = nullptr;

	delete ImmediateContext;
	ImmediateContext = nullptr;

	delete RenderPassManager;
	RenderPassManager = nullptr;

	for (TArray<FVulkanQueryPool*>& PoolArray : FreeQueryPools)
	{
		for (FVulkanQueryPool* Pool : PoolArray)
		{
			delete Pool;
		}
		PoolArray.Empty();
	}

	delete PipelineStateCache;
	PipelineStateCache = nullptr;
	StagingManager.Deinit();

	DeferredDeletionQueue.Clear();

	if (ShaderObjectManager)
	{
		delete ShaderObjectManager;
		ShaderObjectManager = nullptr;
	}

	BindlessDescriptorManager->Deinit();
	delete BindlessDescriptorManager;
	BindlessDescriptorManager = nullptr;

	FVulkanUploadContext::DestroyPool();

	for (VkEvent Event : BarrierEvents)
	{
		VulkanRHI::vkDestroyEvent(Device, Event, VULKAN_CPU_ALLOCATOR);
	}
	BarrierEvents.Empty();

	delete TempBlockAllocator;
	TempBlockAllocator = nullptr;

	for (int32 QueueIndex = 0; QueueIndex < (int32)EVulkanQueueType::Count; ++QueueIndex)
	{
		delete Queues[QueueIndex];
		Queues[QueueIndex] = nullptr;
	}

	MemoryManager.Deinit();
	FenceManager.Deinit();
	DeviceMemoryManager.Deinit();
	FVulkanChunkedPipelineCacheManager::Shutdown();

	VulkanRHI::vkDestroyDevice(Device, VULKAN_CPU_ALLOCATOR);
	Device = VK_NULL_HANDLE;
}

void FVulkanDevice::WaitUntilIdle()
{
	RHI->RHIBlockUntilGPUIdle();

	VERIFYVULKANRESULT(VulkanRHI::vkDeviceWaitIdle(Device));
}

const VkComponentMapping& FVulkanDevice::GetFormatComponentMapping(EPixelFormat UEFormat) const
{
	check(GPixelFormats[UEFormat].Supported);
	return PixelFormatComponentMapping[UEFormat];
}

void FVulkanDevice::NotifyDeletedImage(VkImage Image, bool bRenderTarget)
{
	if (bRenderTarget)
	{
		// Contexts first, as it may clear the current framebuffer
		GetImmediateContext().NotifyDeletedRenderTarget(Image);
		// Delete framebuffers using this image
		GetRenderPassManager().NotifyDeletedRenderTarget(Image);
	}
}

void FVulkanDevice::NotifyDeletedGfxPipeline(class FVulkanGraphicsPipelineState* Pipeline)
{
	//#todo-rco: Loop through all contexts!
	if (ImmediateContext && ImmediateContext->PendingGfxState)
	{
		ImmediateContext->PendingGfxState->NotifyDeletedPipeline(Pipeline);
	}
}

void FVulkanDevice::NotifyDeletedComputePipeline(class FVulkanComputePipeline* Pipeline)
{
	//#todo-rco: Loop through all contexts!
	if (ImmediateContext && ImmediateContext->PendingComputeState)
	{
		ImmediateContext->PendingComputeState->NotifyDeletedPipeline(Pipeline);
	}

	if (PipelineStateCache)
	{
		PipelineStateCache->NotifyDeletedComputePipeline(Pipeline);
	}
}

void FVulkanDevice::VulkanSetObjectName(VkObjectType Type, uint64_t Handle, const TCHAR* Name)
{
#if VULKAN_ENABLE_DRAW_MARKERS
	if(DebugMarkers.SetDebugName)
	{
		FTCHARToUTF8 Converter(Name);
		VkDebugUtilsObjectNameInfoEXT Info;
		ZeroVulkanStruct(Info, VK_STRUCTURE_TYPE_DEBUG_UTILS_OBJECT_NAME_INFO_EXT);
		Info.objectType = Type;
		Info.objectHandle = Handle;
		Info.pObjectName = Converter.Get();
		DebugMarkers.SetDebugName(Device, &Info);
	}
#endif // VULKAN_ENABLE_DRAW_MARKERS
}

FVulkanTransientHeapCache& FVulkanDevice::GetOrCreateTransientHeapCache()
{
	if (!TransientHeapCache)
	{
		TransientHeapCache = FVulkanTransientHeapCache::Create(*this);
	}
	return *TransientHeapCache;
}

void FVulkanDevice::GetCalibrationTimestamp(FVulkanTiming& InOutTiming)
{
	// TimestampPeriod is the number of nanoseconds required for a timestamp query to be incremented by 1
	InOutTiming.GPUFrequency = (uint64)((1000.0 * 1000.0 * 1000.0) / (double)GetDeviceProperties().limits.timestampPeriod);
	InOutTiming.CPUFrequency = uint64(1.0 / FPlatformTime::GetSecondsPerCycle64());

	if (GetOptionalExtensions().HasEXTCalibratedTimestamps)
	{
		uint64_t Timestamps[2] = { 0, 0 };
		uint64_t MaxDeviations[2] = { 0, 0 };
		VkCalibratedTimestampInfoKHR CalibratedTimestampInfo[2];
		ZeroVulkanStruct(CalibratedTimestampInfo[0], VK_STRUCTURE_TYPE_CALIBRATED_TIMESTAMP_INFO_KHR);
		ZeroVulkanStruct(CalibratedTimestampInfo[1], VK_STRUCTURE_TYPE_CALIBRATED_TIMESTAMP_INFO_KHR);
		CalibratedTimestampInfo[0].timeDomain = VK_TIME_DOMAIN_DEVICE_KHR;
		CalibratedTimestampInfo[1].timeDomain = FVulkanPlatform::GetTimeDomain();

		const bool bHasHostTimeDomain = (CalibratedTimestampInfo[1].timeDomain != VK_TIME_DOMAIN_DEVICE_KHR);
		const uint32 TimestampCount = bHasHostTimeDomain ? 2 : 1;

		VERIFYVULKANRESULT(VulkanDynamicAPI::vkGetCalibratedTimestampsEXT(GetHandle(), TimestampCount, CalibratedTimestampInfo, Timestamps, MaxDeviations));

		InOutTiming.GPUTimestamp = Timestamps[0];
		switch (CalibratedTimestampInfo[1].timeDomain)
		{
		case VK_TIME_DOMAIN_CLOCK_MONOTONIC_KHR:
			// Divide to match values returned by Cycles64() (to be moved to FVulkanPlatform)
#if PLATFORM_ANDROID
			InOutTiming.CPUTimestamp = Timestamps[1] / 1000ULL;  // Android Cycle64 divides by 1000
#else
			InOutTiming.CPUTimestamp = Timestamps[1] / 100ULL;  // Linux Cycle64 divides by 100
#endif
			break;

		case VK_TIME_DOMAIN_QUERY_PERFORMANCE_COUNTER_KHR:
			InOutTiming.CPUTimestamp = Timestamps[1];
			break;

		default:
			InOutTiming.CPUTimestamp = FPlatformTime::Cycles64();
			break;

		}

		checkf(InOutTiming.CPUTimestamp <= FPlatformTime::Cycles64(), TEXT("New calibration timestamp (%llu) ahead of current time (%llu)."), InOutTiming.CPUTimestamp, FPlatformTime::Cycles64());
	}
	else
	{
		InOutTiming.GPUTimestamp = 0;
		InOutTiming.CPUTimestamp = FPlatformTime::Cycles64();
	}
}

void FVulkanDevice::ForEachQueue(TFunctionRef<void(FVulkanQueue&)> Callback)
{
	for (int32 QueueIndex = 0; QueueIndex < Queues.Num(); ++QueueIndex)
	{
		if (Queues[QueueIndex])
		{
			Callback(*Queues[QueueIndex]);
		}
	}
}

void FVulkanDevice::InitGlobalSamplers()
{
	checkf(SamplerMap.Num() == 0, TEXT("Global Samplers should be the first samplers created."));

	GlobalSamplers[(uint32)FVulkanShaderHeader::EGlobalSamplerType::PointClampedSampler] =
		ResourceCast(RHICreateSamplerState(FSamplerStateInitializerRHI(SF_Point, AM_Clamp, AM_Clamp, AM_Clamp)).GetReference());

	GlobalSamplers[(uint32)FVulkanShaderHeader::EGlobalSamplerType::PointWrappedSampler] = 
		ResourceCast(RHICreateSamplerState(FSamplerStateInitializerRHI(SF_Point, AM_Wrap, AM_Wrap, AM_Wrap)).GetReference());

	GlobalSamplers[(uint32)FVulkanShaderHeader::EGlobalSamplerType::BilinearClampedSampler] =
		ResourceCast(RHICreateSamplerState(FSamplerStateInitializerRHI(SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp)).GetReference());

	GlobalSamplers[(uint32)FVulkanShaderHeader::EGlobalSamplerType::BilinearWrappedSampler] =
		ResourceCast(RHICreateSamplerState(FSamplerStateInitializerRHI(SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap)).GetReference());

	GlobalSamplers[(uint32)FVulkanShaderHeader::EGlobalSamplerType::TrilinearClampedSampler] =
		ResourceCast(RHICreateSamplerState(FSamplerStateInitializerRHI(SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp)).GetReference());

	GlobalSamplers[(uint32)FVulkanShaderHeader::EGlobalSamplerType::TrilinearWrappedSampler] =
		ResourceCast(RHICreateSamplerState(FSamplerStateInitializerRHI(SF_Trilinear, AM_Wrap, AM_Wrap, AM_Wrap)).GetReference());

	static_assert((uint32)FVulkanShaderHeader::EGlobalSamplerType::Count == 6, "Need to add new global sampler decl!");

	// Make sure if bindless handles are being generated that these samplers are the very first
	check(!GlobalSamplers[0]->GetBindlessHandle().IsValid() || GlobalSamplers[0]->GetBindlessHandle().GetIndex() == 0);
}

VkBuffer FVulkanDevice::CreateBuffer(VkDeviceSize BufferSize, VkBufferUsageFlags BufferUsageFlags, VkBufferCreateFlags BufferCreateFlags) const
{
	VkBufferCreateInfo BufferCreateInfo = 
	{
		.sType = VK_STRUCTURE_TYPE_BUFFER_CREATE_INFO,
		.pNext = nullptr,
		.flags = BufferCreateFlags,
		.size = BufferSize,
		.usage = BufferUsageFlags,
		.sharingMode = VK_SHARING_MODE_EXCLUSIVE,
		.queueFamilyIndexCount = 0,
		.pQueueFamilyIndices = nullptr
	};

	// For bindless
	if (GetOptionalExtensions().HasBufferDeviceAddress)
	{
		BufferCreateInfo.usage |= VK_BUFFER_USAGE_SHADER_DEVICE_ADDRESS_BIT;
	}

	VkBuffer BufferHandle = VK_NULL_HANDLE;
	VERIFYVULKANRESULT(VulkanRHI::vkCreateBuffer(Device, &BufferCreateInfo, VULKAN_CPU_ALLOCATOR, &BufferHandle));
	return BufferHandle;
}

VkEvent FVulkanDevice::GetBarrierEvent()
{
	check(SupportsParallelRendering());

	VkEvent Handle = VK_NULL_HANDLE;

	// Check if we already have one
	{
		FScopeLock Lock(&BarrierEventLock);
		if (BarrierEvents.Num())
		{
			Handle = BarrierEvents.Pop(EAllowShrinking::No);
		}
	}

	// Create a new handle if none were available
	if (!Handle)
	{
		VkEventCreateInfo Info;
		ZeroVulkanStruct(Info, VK_STRUCTURE_TYPE_EVENT_CREATE_INFO);
		Info.flags = VK_EVENT_CREATE_DEVICE_ONLY_BIT;
		VERIFYVULKANRESULT(VulkanRHI::vkCreateEvent(Device, &Info, VULKAN_CPU_ALLOCATOR, &Handle));
		INC_DWORD_STAT(STAT_VulkanNumPooledBarrierEvents);
	}

	return Handle;
}

void FVulkanDevice::ReleaseBarrierEvent(VkEvent Handle)
{
	FScopeLock Lock(&BarrierEventLock);
	BarrierEvents.Add(Handle);
}
