// Copyright Epic Games, Inc. All Rights Reserved..

/*=============================================================================
	VulkanDevice.h: Private Vulkan RHI definitions.
=============================================================================*/

#pragma once

#include "VulkanMemory.h"
#include "VulkanSynchronization.h"
#include "VulkanResources.h"
#include "VulkanQueue.h"
#include "VulkanQuery.h"
#include "GPUProfiler.h"

class FVulkanSamplerState;
class FVulkanDynamicRHI;
class FVulkanDescriptorSetCache;
class FVulkanDescriptorPool;
class FVulkanDescriptorPoolsManager;
class FVulkanBindlessDescriptorManager;
class FVulkanTransientHeapCache;
class FVulkanDeviceExtension;
class FVulkanRenderPassManager;
class FVulkanRayTracingCompactionRequestHandler;
class FVulkanRayTracingPipelineLibraryCache;
struct FVulkanParallelRenderPassInfo;
class FVulkanFenceManager;
class FVulkanShaderObjectManager;

#if VULKAN_USE_DEBUG_NAMES
#define VULKAN_SET_DEBUG_NAME(Device, Type, Handle, Format, ...) (Device).VulkanSetObjectName(Type, (uint64)Handle, *FString::Printf(Format, __VA_ARGS__))
#else
#define VULKAN_SET_DEBUG_NAME(Device, Type, Handle, Format, ...) do{}while(0)
#endif

struct FOptionalVulkanDeviceExtensions
{
	union
	{
		struct
		{
			// Optional Extensions
			uint64 HasEXTValidationCache : 1;
			uint64 HasMemoryPriority : 1;
			uint64 HasMemoryBudget : 1;
			uint64 HasEXTASTCDecodeMode : 1;
			uint64 HasEXTFragmentDensityMap : 1;
			uint64 HasEXTFragmentDensityMap2 : 1;
			uint64 HasEXTFragmentDensityMapOffset : 1;
			uint64 HasKHRFragmentShadingRate : 1;
			uint64 HasKHRFragmentShaderBarycentric : 1;
			uint64 HasEXTFullscreenExclusive : 1;
			uint64 HasImageAtomicInt64 : 1;
			uint64 HasAccelerationStructure : 1;
			uint64 HasRayTracingPipeline : 1;
			uint64 HasRayQuery : 1;
			uint64 HasKHRPipelineLibrary : 1;
			uint64 HasDeferredHostOperations : 1;
			uint64 HasEXTCalibratedTimestamps : 1;
			uint64 HasEXTDescriptorBuffer : 1;
			uint64 HasEXTDescriptorHeap : 1;
			uint64 HasEXTDeviceFault : 1;
			uint64 HasEXTMeshShader : 1;
			uint64 HasEXTToolingInfo : 1;
			uint64 HasEXTImageCompressionControl : 1;
			uint64 HasEXTMutableDescriptorType : 1;
			uint64 HasKHRMaintenance7 : 1;
			uint64 HasKHRMaintenance9 : 1;
			uint64 HasEXTShaderObject : 1;
			uint64 HasEXTGraphicsPipelineLibrary : 1;
			uint64 HasEXTDeviceGeneratedCommands : 1;
			uint64 HasEXTDeviceAddressBindingReport : 1;

			// Vendor specific
			uint64 HasAMDBufferMarker : 1;
			uint64 HasNVDiagnosticCheckpoints : 1;
			uint64 HasNVDeviceDiagnosticConfig : 1;
			uint64 HasANDROIDExternalMemoryHardwareBuffer : 1;

			// Promoted to 1.1
			uint64 HasKHRMultiview : 1;
			uint64 HasKHR16bitStorage : 1;
			uint64 HasKHRSamplerYcbcrConversion : 1;

			// Promoted to 1.2
			uint64 HasKHRRenderPass2 : 1;
			uint64 HasKHRImageFormatList : 1;
			uint64 HasKHRShaderAtomicInt64 : 1;
			uint64 HasEXTScalarBlockLayout : 1;
			uint64 HasBufferDeviceAddress : 1;
			uint64 HasSPIRV_14 : 1;
			uint64 HasShaderFloatControls : 1;
			uint64 HasKHRShaderFloat16 : 1;
			uint64 HasEXTDescriptorIndexing : 1;
			uint64 HasSeparateDepthStencilLayouts : 1;
			uint64 HasEXTHostQueryReset : 1;
			uint64 HasQcomRenderPassShaderResolve : 1;
			uint64 HasKHRDepthStencilResolve : 1;
			uint64 HasKHRTimelineSemaphore : 1;

			// Promoted to 1.3
			uint64 HasEXTTextureCompressionASTCHDR : 1;
			uint64 HasKHRMaintenance4 : 1;
			uint64 HasKHRSynchronization2 : 1;
			uint64 HasKHRDynamicRendering : 1;
			uint64 HasEXTSubgroupSizeControl : 1;
			uint64 HasEXTPipelineCreationCacheControl : 1;
			uint64 HasEXTExtendedDynamicState1 : 1;
			uint64 HasEXTExtendedDynamicState2 : 1;
			uint64 HasEXTExtendedDynamicState3 : 1;
			uint64 HasEXTVertexInputDynamicState : 1;
			uint64 HasEXTImageRobustness : 1;

			// Promoted to 1.4
			uint64 HasKHRMaintenance5 : 1;
			uint64 HasKHRMaintenance6 : 1;
			uint64 HasKHRPushDescriptors : 1;
			uint64 HasEXTLoadStoreOpNone : 1;
			uint64 HasEXTHostImageCopy : 1;
		};
		uint64 Packed[2];
	};

	FOptionalVulkanDeviceExtensions()
	{
		static_assert(sizeof(Packed) == sizeof(FOptionalVulkanDeviceExtensions), "More bits needed for Packed!");
		FMemory::Memzero(Packed);
	}

	bool HasGPUCrashDumpExtensions() const
	{
		return HasAMDBufferMarker || HasNVDiagnosticCheckpoints;
	}

	bool HasRaytracingExtensions() const
	{
		return 
			HasAccelerationStructure && 
			((HasRayTracingPipeline && HasKHRPipelineLibrary) || HasRayQuery) &&
			HasEXTDescriptorIndexing &&
			HasBufferDeviceAddress && 
			HasDeferredHostOperations && 
			HasSPIRV_14 && 
			HasShaderFloatControls;
	}

	bool HasAnyExtendedDynamicState() const
	{
		return HasEXTExtendedDynamicState1 || 
			HasEXTExtendedDynamicState2 || 
			HasEXTExtendedDynamicState3 || 
			HasEXTVertexInputDynamicState;
	}
};

// All the features and properties we need to keep around from extension initialization
struct FOptionalVulkanDeviceExtensionProperties
{
	FOptionalVulkanDeviceExtensionProperties()
	{
		FMemory::Memzero(*this);
	}

	VkPhysicalDeviceDriverPropertiesKHR PhysicalDeviceDriverProperties;
	VkPhysicalDeviceMaintenance7PropertiesKHR PhysicalDeviceMaintenance7Properties;

	VkPhysicalDeviceDescriptorBufferPropertiesEXT DescriptorBufferProps;
	VkPhysicalDeviceDescriptorHeapPropertiesEXT DescriptorHeapProps;
	VkPhysicalDeviceSubgroupSizeControlPropertiesEXT SubgroupSizeControlProperties;

	VkPhysicalDeviceAccelerationStructurePropertiesKHR AccelerationStructureProps;
	VkPhysicalDeviceRayTracingPipelinePropertiesKHR RayTracingPipelineProps;

	VkPhysicalDeviceFragmentShadingRateFeaturesKHR FragmentShadingRateFeatures;
	VkPhysicalDeviceFragmentDensityMapFeaturesEXT FragmentDensityMapFeatures;
	VkPhysicalDeviceFragmentDensityMap2FeaturesEXT FragmentDensityMap2Features;

	VkPhysicalDeviceFragmentShaderBarycentricPropertiesKHR FragmentShaderBarycentricProps;
	VkPhysicalDeviceComputeShaderDerivativesFeaturesKHR ComputeShaderDerivativesFeatures;
	VkPhysicalDeviceGraphicsPipelineLibraryPropertiesEXT GraphicsPipelineLibraryProperties;

	VkPhysicalDeviceMeshShaderPropertiesEXT MeshShaderProperties;

	VkPhysicalDeviceDeviceGeneratedCommandsPropertiesEXT DeviceGeneratedCommandsProperties;
};

class FVulkanPhysicalDeviceFeatures
{
public:
	FVulkanPhysicalDeviceFeatures()
	{
		FMemory::Memzero(*this);
	}

	void Query(VkPhysicalDevice PhysicalDevice, uint32 APIVersion);

	VkPhysicalDeviceFeatures	     Core_1_0;
	VkPhysicalDeviceVulkan11Features Core_1_1;
private:
	// Anything above Core 1.1 cannot be assumed, they should only be used by the device at init time
	VkPhysicalDeviceVulkan12Features Core_1_2;
	VkPhysicalDeviceVulkan13Features Core_1_3;
	VkPhysicalDeviceVulkan14Features Core_1_4;

	friend class FVulkanDevice;
	friend FVulkanDynamicRHI;
};


namespace VulkanRHI
{
	class FDeferredDeletionQueue2
	{

	public:
		FDeferredDeletionQueue2(FVulkanDevice& InDevice);
		~FDeferredDeletionQueue2();

		enum class EType
		{
			RenderPass,
			Buffer,
			BufferView,
			Image,
			ImageView,
			Pipeline,
			PipelineLayout,
			Framebuffer,
			DescriptorSetLayout,
			Sampler,
			Semaphore,
			ShaderModule,
			Event,
			ResourceAllocation,
			DeviceMemoryAllocation,
			BufferSuballocation,
			AccelerationStructure,
			BindlessHandle,
			Func
		};

		template <typename T>
		inline void EnqueueResource(EType Type, T Handle)
		{
			static_assert(sizeof(T) <= sizeof(uint64), "Vulkan resource handle type size too large.");
			EnqueueGenericResource(Type, (uint64)Handle);
		}

		inline void EnqueueBindlessHandle(FRHIDescriptorHandle DescriptorHandle)
		{
			if (DescriptorHandle.IsValid())
			{
				static_assert(sizeof(FRHIDescriptorHandle) <= sizeof(uint64), "FRHIDescriptorHandle is too large");
				EnqueueResource(EType::BindlessHandle, *reinterpret_cast<uint64*>(&DescriptorHandle));
			}
		}

		void EnqueueFunc(TUniqueFunction<void(class FVulkanDevice&)>&& Func);
		void EnqueueResourceAllocation(FVulkanAllocation& Allocation);
		void EnqueueDeviceAllocation(FDeviceMemoryAllocation* DeviceMemoryAllocation);

		void ReleaseResources(bool bDeleteImmediately = false);

		inline void Clear()
		{
			ReleaseResources(true);
		}

	private:
		void EnqueueGenericResource(EType Type, uint64 Handle);

		struct FEntry
		{
			EType StructureType;
			uint32 FrameNumber;
			uint64 Handle = 0;
			FVulkanAllocation Allocation;
			FDeviceMemoryAllocation* DeviceMemoryAllocation = nullptr;
			TUniqueFunction<void(class FVulkanDevice&)>* Func = nullptr;
		};

		void ReleaseResourcesImmediately(const TArray<FEntry>& InEntries);

		FVulkanDevice& Device;
		FCriticalSection CS;
		TArray<FEntry> Entries;
	};
}


class FVulkanDevice
{
public:
	FVulkanDevice(FVulkanDynamicRHI* InRHI, VkPhysicalDevice Gpu);

	~FVulkanDevice();

	void InitGPU();

	void CreateDevice(TArray<const ANSICHAR*>& DeviceLayers, FVulkanDeviceExtensionArray& UEExtensions);
	void ChooseVariableRateShadingMethod();

	void PrepareForDestroy();
	void Destroy();

	void WaitUntilIdle();

	EGpuVendorId GetVendorId() const
	{
		return VendorId;
	}

	bool HasAsyncComputeQueue() const
	{
		return Queues[(int32)EVulkanQueueType::AsyncCompute] != nullptr;
	}

	bool HasTransferQueue() const
	{
		return Queues[(int32)EVulkanQueueType::Transfer] != nullptr;
	}

	bool HasMultipleQueues() const
	{
		return HasAsyncComputeQueue() || HasTransferQueue();
	}

	bool CanPresentOnComputeQueue() const
	{
		return bPresentOnComputeQueue;
	}

	FVulkanQueue* GetQueue(ERHIPipeline Pipeline)
	{
		if (Pipeline == ERHIPipeline::Graphics) 
		{
			return GetQueue(EVulkanQueueType::Graphics); 
		}
		else if (Pipeline == ERHIPipeline::AsyncCompute) 
		{ 
			return GetQueue(EVulkanQueueType::AsyncCompute); 
		}
		checkNoEntry();
		return nullptr;
	}

	FVulkanQueue* GetQueue(EVulkanQueueType QueueType)
	{
		return Queues[(int32)QueueType];
	}

	FVulkanQueue* GetGraphicsQueue()
	{
		return GetQueue(EVulkanQueueType::Graphics);
	}

	FVulkanQueue* GetComputeQueue()
	{
		return GetQueue(EVulkanQueueType::AsyncCompute);
	}

	FVulkanQueue* GetTransferQueue()
	{
		return GetQueue(EVulkanQueueType::Transfer);
	}

	FVulkanQueue* GetPresentQueue()
	{
		return PresentQueue;
	}

	void ForEachQueue(TFunctionRef<void(FVulkanQueue&)> Callback);

	VkPhysicalDevice GetPhysicalHandle() const
	{
		return Gpu;
	}

	const VkPhysicalDeviceProperties& GetDeviceProperties() const
	{
		return GpuProps;
	}

	VkExtent2D GetBestMatchedFragmentSize(EVRSShadingRate Rate) const
	{
		return FragmentSizeMap[Rate];
	}

	const VkPhysicalDeviceLimits& GetLimits() const
	{
		return GpuProps.limits;
	}

	const VkPhysicalDeviceIDPropertiesKHR& GetDeviceIdProperties() const
	{
		return GpuIdProps;
	}

	const VkPhysicalDeviceSubgroupProperties& GetDeviceSubgroupProperties() const
	{
		return GpuSubgroupProps;
	}

	FVulkanRayTracingCompactionRequestHandler* GetRayTracingCompactionRequestHandler()
	{
		return RayTracingCompactionRequestHandler; 
	}

	FVulkanRayTracingPipelineLibraryCache* GetRayTracingPipelineLibraryCache()
	{
		return RayTracingPipelineLibraryCache;
	}

	void InitializeRayTracing();
	void CleanUpRayTracing();

#if VULKAN_SUPPORTS_VALIDATION_CACHE
	VkValidationCacheEXT GetValidationCache() const
	{
		return ValidationCache;
	}
#endif

	const FVulkanPhysicalDeviceFeatures& GetPhysicalDeviceFeatures() const
	{
		return PhysicalDeviceFeatures;
	}

	bool HasUnifiedMemory() const
	{
		return DeviceMemoryManager.HasUnifiedMemory();
	}

	bool SupportsBindless() const;

	const VkComponentMapping& GetFormatComponentMapping(EPixelFormat UEFormat) const;

	VkDevice GetHandle() const
	{
		return Device;
	}

	const FVulkanSamplerState& GetDefaultSampler() const
	{
		return GetGlobalSamplers(FVulkanShaderHeader::EGlobalSamplerType::PointWrappedSampler);
	}

	const VkFormatProperties& GetFormatProperties(VkFormat InFormat) const;

	VulkanRHI::FDeviceMemoryManager& GetDeviceMemoryManager()
	{
		return DeviceMemoryManager;
	}

	const VkPhysicalDeviceMemoryProperties& GetDeviceMemoryProperties() const
	{
		return DeviceMemoryManager.GetMemoryProperties();
	}

	VulkanRHI::FMemoryManager& GetMemoryManager()
	{
		return MemoryManager;
	}

	VulkanRHI::FDeferredDeletionQueue2& GetDeferredDeletionQueue()
	{
		return DeferredDeletionQueue;
	}

	VulkanRHI::FStagingManager& GetStagingManager()
	{
		return StagingManager;
	}

	FVulkanFenceManager& GetFenceManager()
	{
		return FenceManager;
	}

	VulkanRHI::FTempBlockAllocator& GetTempBlockAllocator()
	{
		return *TempBlockAllocator;
	}

	FVulkanRenderPassManager& GetRenderPassManager()
	{
		return *RenderPassManager;
	}

	FVulkanDescriptorSetCache& GetDescriptorSetCache()
	{
		return *DescriptorSetCache;
	}

	FVulkanDescriptorPoolsManager& GetDescriptorPoolsManager()
	{
		return *DescriptorPoolsManager;
	}

	FVulkanBindlessDescriptorManager* GetBindlessDescriptorManager()
	{
		return BindlessDescriptorManager;
	}

	TMap<uint32, TRefCountPtr<FVulkanSamplerState>>& GetSamplerMap()
	{
		return SamplerMap;
	}

	FVulkanShaderFactory& GetShaderFactory()
	{
		return ShaderFactory;
	}

	FVulkanCommandListContext& GetImmediateContext()
	{
		return *ImmediateContext;
	}

	void NotifyDeletedImage(VkImage Image, bool bRenderTarget);

#if VULKAN_ENABLE_DRAW_MARKERS
	inline PFN_vkCmdInsertDebugUtilsLabelEXT GetCmdInsertDebugLabel() const
	{
		return DebugMarkers.CmdInsertLabel;
	}

	PFN_vkCmdBeginDebugUtilsLabelEXT GetCmdBeginDebugLabel() const
	{
		return DebugMarkers.CmdBeginDebugLabel;
	}

	PFN_vkCmdEndDebugUtilsLabelEXT GetCmdEndDebugLabel() const
	{
		return DebugMarkers.CmdEndDebugLabel;
	}

	PFN_vkSetDebugUtilsObjectNameEXT GetSetDebugName() const
	{
		return DebugMarkers.SetDebugName;
	}
#endif

	FVulkanQueryPool* AcquireOcclusionQueryPool(uint32 NumQueries);
	FVulkanQueryPool* AcquireTimingQueryPool();
	void ReleaseQueryPool(FVulkanQueryPool* Pool);
	void RemoveStaleQueryPools();

	class FVulkanPipelineStateCacheManager* GetPipelineStateCache()
	{
		return PipelineStateCache;
	}

	void NotifyDeletedGfxPipeline(class FVulkanGraphicsPipelineState* Pipeline);
	void NotifyDeletedComputePipeline(class FVulkanComputePipeline* Pipeline);

	void VulkanSetObjectName(VkObjectType Type, uint64_t Handle, const TCHAR* Name);
	const FOptionalVulkanDeviceExtensions& GetOptionalExtensions() const
	{
		return OptionalDeviceExtensions;
	}

	const FOptionalVulkanDeviceExtensionProperties& GetOptionalExtensionProperties() const
	{
		return OptionalDeviceExtensionProperties;
	}

	bool NeedsAllPlanes() const
	{
		return !SupportsParallelRendering();
	}

	bool SupportsParallelRendering() const
	{
		return OptionalDeviceExtensions.HasSeparateDepthStencilLayouts && OptionalDeviceExtensions.HasKHRSynchronization2 && OptionalDeviceExtensions.HasKHRRenderPass2;
	}

	bool SupportsGraphicPipelineLibraries() const
	{
		return OptionalDeviceExtensions.HasEXTGraphicsPipelineLibrary &&
			// Extended dynamic states make it possible to create the pipeline library at shader creation,
			// and they allow for a single library for vertex_input and fragment_output_state
			OptionalDeviceExtensions.HasEXTExtendedDynamicState1 &&
			OptionalDeviceExtensions.HasEXTExtendedDynamicState2 &&
			OptionalDeviceExtensions.HasEXTExtendedDynamicState3 &&
			// use a single vertex input stage for all pipelines
			OptionalDeviceExtensions.HasEXTVertexInputDynamicState &&
			// dynamic rendering removes need for renderpass at pipeline creation
			OptionalDeviceExtensions.HasKHRDynamicRendering &&  
			// in bindless a single pipeline layout is needed
			SupportsBindless();
	}

	FVulkanShaderObjectManager* GetShaderObjectManager()
	{
		return ShaderObjectManager;
	}

	bool SupportsShaderObjects() const
	{
		// NOTE: No need to check individual dynamic state extension, they all get enabled by turning on shader objects
		return OptionalDeviceExtensions.HasEXTShaderObject &&
			// dynamic rendering removes need for renderpass at pipeline creation
			OptionalDeviceExtensions.HasKHRDynamicRendering &&  
			// in bindless a single pipeline layout is needed
			SupportsBindless();
	}

	void SetupPresentQueue(VkSurfaceKHR Surface);

	const TArray<VkQueueFamilyProperties>& GetQueueFamilyProps()
	{
		return QueueFamilyProps;
	}

	bool NeedsQueueFamilyOwnershipTransfers() const
	{
		return bNeedsQueueFamilyOwnershipTransfers;
	}

	FVulkanTransientHeapCache& GetOrCreateTransientHeapCache();

	const TArray<const ANSICHAR*>& GetDeviceExtensions() { return DeviceExtensions; }

	void GetCalibrationTimestamp(FVulkanTiming& InOutTiming);

	const FVulkanSamplerState& GetGlobalSamplers(FVulkanShaderHeader::EGlobalSamplerType Type) const
	{
		return *GlobalSamplers[(uint32)Type];
	}

	VkEvent GetBarrierEvent();
	void ReleaseBarrierEvent(VkEvent Handle);

	VkBuffer CreateBuffer(VkDeviceSize BufferSize, VkBufferUsageFlags BufferUsageFlags, VkBufferCreateFlags BufferCreateFlags=0) const;

	const TArray<uint32>& GetActiveQueueFamilies() const
	{
		return ActiveQueueFamilies;
	}

	bool UseMinimalSubmits() const;

	bool SupportsDeviceLocalHostVisible() const;

private:
	void MapBufferFormatSupport(FPixelFormatInfo& PixelFormatInfo, EPixelFormat UEFormat, VkFormat VulkanFormat);
	void MapImageFormatSupport(FPixelFormatInfo& PixelFormatInfo, const TArrayView<const VkFormat>& PrioritizedFormats, EPixelFormatCapabilities RequiredCapabilities);
	void MapFormatSupport(EPixelFormat UEFormat, std::initializer_list<VkFormat> PrioritizedFormats, const VkComponentMapping& ComponentMapping, EPixelFormatCapabilities RequiredCapabilities, int32 BlockBytes);
	void MapFormatSupport(EPixelFormat UEFormat, std::initializer_list<VkFormat> PrioritizedFormats, const VkComponentMapping& ComponentMapping);
	void MapFormatSupport(EPixelFormat UEFormat, std::initializer_list<VkFormat> PrioritizedFormats, const VkComponentMapping& ComponentMapping, int32 BlockBytes);
	void MapFormatSupport(EPixelFormat UEFormat, std::initializer_list<VkFormat> PrioritizedFormats, const VkComponentMapping& ComponentMapping, EPixelFormatCapabilities RequiredCapabilities);

	void InitGlobalSamplers();
	TStaticArray<FVulkanSamplerState*, (uint32)FVulkanShaderHeader::EGlobalSamplerType::Count> GlobalSamplers;

	VkDevice Device;

	VulkanRHI::FDeviceMemoryManager DeviceMemoryManager;

	VulkanRHI::FMemoryManager MemoryManager;

	VulkanRHI::FDeferredDeletionQueue2 DeferredDeletionQueue;

	VulkanRHI::FStagingManager StagingManager;

	FVulkanFenceManager FenceManager;

	VulkanRHI::FTempBlockAllocator* TempBlockAllocator = nullptr;

	FVulkanRenderPassManager* RenderPassManager;

	FVulkanTransientHeapCache* TransientHeapCache = nullptr;

	// Active on ES3.1
	FVulkanDescriptorSetCache* DescriptorSetCache = nullptr;
	// Active on >= SM4
	FVulkanDescriptorPoolsManager* DescriptorPoolsManager = nullptr;

	FVulkanBindlessDescriptorManager* BindlessDescriptorManager = nullptr;

	FVulkanShaderFactory ShaderFactory;
	FVulkanShaderObjectManager* ShaderObjectManager = nullptr;

	VkPhysicalDevice Gpu;
	VkPhysicalDeviceProperties GpuProps;

	TArray<VkPhysicalDeviceFragmentShadingRateKHR> FragmentShadingRates;
	TStaticArray<VkExtent2D, (EVRSShadingRate::VRSSR_Last+1)> FragmentSizeMap;

	// Extension specific properties
	VkPhysicalDeviceIDPropertiesKHR GpuIdProps;
	VkPhysicalDeviceSubgroupProperties GpuSubgroupProps;

	FVulkanRayTracingCompactionRequestHandler* RayTracingCompactionRequestHandler = nullptr;
	FVulkanRayTracingPipelineLibraryCache* RayTracingPipelineLibraryCache = nullptr;

	FVulkanPhysicalDeviceFeatures PhysicalDeviceFeatures;

	TArray<VkQueueFamilyProperties> QueueFamilyProps;
	bool bNeedsQueueFamilyOwnershipTransfers = true;

	VkFormatProperties FormatProperties[VK_FORMAT_RANGE_SIZE];
	// Info for formats that are not in the core Vulkan spec (i.e. extensions)
	mutable TMap<VkFormat, VkFormatProperties> ExtensionFormatProperties;

	// Reusable query pools
	FCriticalSection QueryPoolLock;
	uint32 OcclusionQueryPoolSize = 256;
	TStaticArray<TArray<FVulkanQueryPool*>, (int32)EVulkanQueryPoolType::Count> FreeQueryPools;

	// Reusable gpu-only barrier events
	FCriticalSection BarrierEventLock;
	TArray<VkEvent> BarrierEvents;

	TStaticArray<FVulkanQueue*, (int32)EVulkanQueueType::Count> Queues;
	FVulkanQueue* PresentQueue = nullptr;  // points to an existing queue
	bool bPresentOnComputeQueue = false;
	TArray<uint32> ActiveQueueFamilies;

	EGpuVendorId VendorId = EGpuVendorId::NotQueried;

	VkComponentMapping PixelFormatComponentMapping[PF_MAX];

	TMap<uint32, TRefCountPtr<FVulkanSamplerState>> SamplerMap;

	FVulkanCommandListContext* ImmediateContext;

	FVulkanDynamicRHI* RHI = nullptr;
	
	TArray<const ANSICHAR*> SetupDeviceLayers(FVulkanDeviceExtensionArray& UEExtensions);

	FOptionalVulkanDeviceExtensions	OptionalDeviceExtensions;
	FOptionalVulkanDeviceExtensionProperties OptionalDeviceExtensionProperties;
	TArray<const ANSICHAR*>			DeviceExtensions;

	void SetupFormats();

#if VULKAN_SUPPORTS_VALIDATION_CACHE
	VkValidationCacheEXT ValidationCache = VK_NULL_HANDLE;
#endif

#if VULKAN_ENABLE_DRAW_MARKERS
	bool bUseLegacyDebugMarkerExt = false;
	struct
	{
		PFN_vkSetDebugUtilsObjectNameEXT	SetDebugName = nullptr;
		PFN_vkCmdInsertDebugUtilsLabelEXT	CmdInsertLabel = nullptr;
		PFN_vkCmdBeginDebugUtilsLabelEXT	CmdBeginDebugLabel = nullptr;
		PFN_vkCmdEndDebugUtilsLabelEXT		CmdEndDebugLabel = nullptr;
	} DebugMarkers;
	friend class FVulkanCommandListContext;
#endif
	void SetupDrawMarkers();

	class FVulkanPipelineStateCacheManager* PipelineStateCache;
	friend class FVulkanDynamicRHI;
	friend class FVulkanGraphicsPipelineState;
};
