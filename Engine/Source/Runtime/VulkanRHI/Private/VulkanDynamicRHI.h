// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanDynamicRHI.h: Vulkan RHI definitions.
=============================================================================*/

#pragma once

#include "IVulkanDynamicRHI.h"
#include "VulkanConfiguration.h"
#include "VulkanQueue.h"

class FVulkanFramebuffer;
class FVulkanDevice;
class FVulkanUniformBuffer;
class FVulkanViewport;
class FVulkanPayload;
class FVulkanSemaphore;
class FVulkanTexture;
class IHeadMountedDisplayVulkanExtensions;
class FVulkanThread;
struct FRHITransientHeapAllocation;
struct FVulkanPlatformCommandList;


struct FOptionalVulkanInstanceExtensions
{
	union
	{
		struct
		{
			UE_DEPRECATED(5.3, "Vulkan 1.1 is now a requirement so there is no need to check these capabilities.")
			uint32 HasKHRExternalMemoryCapabilities : 1;

			UE_DEPRECATED(5.3, "Vulkan 1.1 is now a requirement so there is no need to check these capabilities.")
			uint32 HasKHRGetPhysicalDeviceProperties2 : 1;
		};
		uint32 Packed;
	};

	FOptionalVulkanInstanceExtensions()
	{
		static_assert(sizeof(Packed) == sizeof(FOptionalVulkanInstanceExtensions), "More bits needed!");
		Packed = 0;
	}
};

// Encapsulates the state required for tracking GPU queue performance across a frame.
class FVulkanTiming
{
public:
	FVulkanTiming(FVulkanQueue& InQueue);

	FVulkanQueue& Queue;

	// Timer calibration data
	uint64 GPUFrequency = 0, GPUTimestamp = 0;
	uint64 CPUFrequency = 0, CPUTimestamp = 0;

	UE::RHI::GPUProfiler::FEventStream EventStream;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS

/** The interface which is implemented by the dynamically bound RHI. */
class FVulkanDynamicRHI : public IVulkanDynamicRHI
{
public:

	static FVulkanDynamicRHI& Get() { return *GetDynamicRHI<FVulkanDynamicRHI>(); }

	/** Initialization constructor. */
	FVulkanDynamicRHI();

	/** Destructor */
	~FVulkanDynamicRHI();

	// IVulkanDynamicRHI interface
	virtual uint32 RHIGetVulkanVersion() const final override;
	virtual VkInstance RHIGetVkInstance() const final override;
	virtual VkDevice RHIGetVkDevice() const final override;
	virtual const uint8* RHIGetVulkanDeviceUUID() const final override;
	virtual VkPhysicalDevice RHIGetVkPhysicalDevice() const final override;
	virtual const VkAllocationCallbacks* RHIGetVkAllocationCallbacks() final override;
	virtual IVulkanMemoryAllocator* RHIGetVulkanMemoryAllocator() final override;
	virtual VkQueue RHIGetGraphicsVkQueue() const final override;
	virtual uint32 RHIGetGraphicsQueueIndex() const final override;
	virtual uint32 RHIGetGraphicsQueueFamilyIndex() const final override;
	virtual VkCommandBuffer  RHIGetVkCommandBuffer(FRHICommandListBase& ExecutingCmdList) const final override;
	virtual uint64 RHIGetGraphicsAdapterLUID(VkPhysicalDevice InPhysicalDevice) const final override;
	virtual bool RHIDoesAdapterMatchDevice(const void* InAdapterId) const final override;
	virtual void* RHIGetVkDeviceProcAddr(const char* InName) const final override;
	virtual void* RHIGetVkInstanceProcAddr(const char* InName) const final override;
	virtual void* RHIGetVkInstanceGlobalProcAddr(const char* InName) const final override;
	virtual VkFormat RHIGetSwapChainVkFormat(EPixelFormat InFormat) const final override;
	virtual bool RHISupportsEXTFragmentDensityMap2() const final override;
	virtual TArray<VkExtensionProperties> RHIGetAllInstanceExtensions() const final override;
	virtual TArray<VkExtensionProperties> RHIGetAllDeviceExtensions(VkPhysicalDevice InPhysicalDevice) const final override;
	virtual TArray<FAnsiString> RHIGetLoadedDeviceExtensions() const final override;
	virtual FTextureRHIRef RHICreateTexture2DFromResource(EPixelFormat Format, uint32 SizeX, uint32 SizeY, uint32 NumMips, uint32 NumSamples, VkImage Resource, ETextureCreateFlags Flags, const FClearValueBinding& ClearValueBinding = FClearValueBinding::Transparent, const FVulkanRHIExternalImageDeleteCallbackInfo& ExternalImageDeleteCallbackInfo = {}) final override;
#if PLATFORM_ANDROID
	virtual FTextureRHIRef RHICreateTexture2DFromAndroidHardwareBuffer(AHardwareBuffer* HardwareBuffer) final override;
#endif
	virtual FTextureRHIRef RHICreateTexture2DArrayFromResource(EPixelFormat Format, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, VkImage Resource, ETextureCreateFlags Flags, const FClearValueBinding& ClearValueBinding = FClearValueBinding::Transparent) final override;
	virtual FTextureRHIRef RHICreateTextureCubeFromResource(EPixelFormat Format, uint32 Size, bool bArray, uint32 ArraySize, uint32 NumMips, VkImage Resource, ETextureCreateFlags Flags, const FClearValueBinding& ClearValueBinding = FClearValueBinding::Transparent) final override;
	virtual VkImage RHIGetVkImage(FRHITexture* InTexture) const final override;
	virtual VkFormat RHIGetViewVkFormat(FRHITexture* InTexture) const final override;
	virtual FVulkanRHIAllocationInfo RHIGetAllocationInfo(FRHITexture* InTexture) const final override;
	virtual FVulkanRHIImageViewInfo RHIGetImageViewInfo(FRHITexture* InTexture) const final override;
	virtual FVulkanRHIAllocationInfo RHIGetAllocationInfo(FRHIBuffer* InBuffer) const final override;
	virtual void RHIFinishExternalComputeWork(VkCommandBuffer InCommandBuffer) final override;
	virtual void RHIVerifyResult(VkResult Result, const ANSICHAR* VkFuntion, const ANSICHAR* Filename, uint32 Line) final override;

	// FDynamicRHI interface.
	virtual void Init() final override;
	virtual void PostInit() final override;
	virtual void Shutdown() final override;
	virtual const TCHAR* GetName() final override { return TEXT("Vulkan"); }

	void InitInstance();

	virtual void RHIEndFrame_RenderThread(FRHICommandListImmediate& RHICmdList) final override;
	virtual void RHIEndFrame(const FRHIEndFrameArgs& Args) final override;

	virtual FSamplerStateRHIRef RHICreateSamplerState(const FSamplerStateInitializerRHI& Initializer) final override;
	virtual FRasterizerStateRHIRef RHICreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer) final override;
	virtual FDepthStencilStateRHIRef RHICreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer) final override;
	virtual FBlendStateRHIRef RHICreateBlendState(const FBlendStateInitializerRHI& Initializer) final override;
	virtual FVertexDeclarationRHIRef RHICreateVertexDeclaration(const FVertexDeclarationElementList& Elements) final override;
	virtual FPixelShaderRHIRef RHICreatePixelShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	virtual FVertexShaderRHIRef RHICreateVertexShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	virtual FMeshShaderRHIRef RHICreateMeshShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	virtual FAmplificationShaderRHIRef RHICreateAmplificationShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	virtual FGeometryShaderRHIRef RHICreateGeometryShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	virtual FComputeShaderRHIRef RHICreateComputeShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	virtual FGPUFenceRHIRef RHICreateGPUFence(const FName &Name) final override;
	virtual void RHIWriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIGPUFence* FenceRHI) final override;
	virtual void RHICreateTransition(FRHITransition* Transition, const FRHITransitionCreateInfo& CreateInfo) final override;
	virtual void RHIReleaseTransition(FRHITransition* Transition) final override;
	virtual FStagingBufferRHIRef RHICreateStagingBuffer() final override;
	virtual FBoundShaderStateRHIRef RHICreateBoundShaderState(FRHIVertexDeclaration* VertexDeclaration, FRHIVertexShader* VertexShader, FRHIPixelShader* PixelShader, FRHIGeometryShader* GeometryShader) final override;
	virtual FGraphicsPipelineStateRHIRef RHICreateGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer) final override;
	virtual FComputePipelineStateRHIRef RHICreateComputePipelineState(const FComputePipelineStateInitializer& Initializer) final override;
	virtual FUniformBufferRHIRef RHICreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation) final override;
	virtual void RHIUpdateUniformBuffer(FRHICommandListBase& RHICmdList, FRHIUniformBuffer* UniformBufferRHI, const void* Contents) final override;

	[[nodiscard]] virtual FRHIBufferInitializer RHICreateBufferInitializer(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& CreateDesc) final override;

	virtual void RHIReplaceResources(FRHICommandListBase& RHICmdList, TArray<FRHIResourceReplaceInfo>&& ReplaceInfos) final override;
	virtual void RHIWriteResourceGPUAddresses(FRHICommandListBase& RHICmdList, uint32 GPUIndex, TConstArrayView<FRHIResource*> Resources, uint64* AddressesPtr) final override;

	virtual void* LockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 Offset, uint32 Size, EResourceLockMode LockMode) final override;
	virtual void UnlockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) final override;
	virtual void* RHILockBuffer(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI, uint32 Offset, uint32 Size, EResourceLockMode LockMode) final override;
	virtual void RHIUnlockBuffer(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) final override;
	
#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
	virtual void RHIUpdateAllocationTags(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) final override;
#endif

	virtual FTextureReferenceRHIRef RHICreateTextureReference(FRHICommandListBase& RHICmdList, FRHITexture* InReferencedTexture) final override;
	virtual void RHIUpdateTextureReference(FRHICommandListBase& RHICmdList, FRHITextureReference* TextureRef, FRHITexture* NewTexture) final override;
	virtual FRHICalcTextureSizeResult RHICalcTexturePlatformSize(FRHITextureDesc const& Desc, uint32 FirstMipIndex) final override;
	virtual void RHIGetTextureMemoryStats(FTextureMemoryStats& OutStats) final override;
	virtual bool RHIGetTextureMemoryVisualizeData(FColor* TextureData, int32 SizeX, int32 SizeY, int32 Pitch, int32 PixelSize) final override;

	struct FCreateTextureResult
	{
		FVulkanTexture* Texture;
		VkImageLayout DefaultLayout;
		bool bTransientResource;
		bool bClear;
	};

	FCreateTextureResult BeginCreateTextureInternal(const FRHITextureCreateDesc& CreateDesc, const FRHITransientHeapAllocation* InTransientHeapAllocation);
	FVulkanTexture* FinalizeCreateTextureInternal(FRHICommandListBase& RHICmdList, FCreateTextureResult CreateResult);

	FVulkanTexture* CreateTextureInternal(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc);
	FVulkanTexture* CreateTextureInternal(const FRHITextureCreateDesc& CreateDesc, const FRHITransientHeapAllocation& InTransientHeapAllocation);

	[[nodiscard]] virtual FRHITextureInitializer RHICreateTextureInitializer(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc);

	virtual FTextureRHIRef RHIAsyncCreateTexture2D(uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ERHIAccess InResourceState, void** InitialMipData, uint32 NumInitialMips, const TCHAR* DebugName, FGraphEventRef& OutCompletionEvent) final override;
	virtual uint32 RHIComputeMemorySize(FRHITexture* TextureRHI) final override;

	virtual void RHIGetMemoryStats(FRHIMemoryStats& OutStats) final override;

	virtual FRHILockTextureResult RHILockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) final override;
	virtual void RHIUnlockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) final override;

	virtual void RHIUpdateTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData) final override
	{
		InternalUpdateTexture2D(RHICmdList, Texture, MipIndex, UpdateRegion, SourcePitch, SourceData);
	}
	virtual void RHIUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData) final override
	{
		InternalUpdateTexture3D(RHICmdList, Texture, MipIndex, UpdateRegion, SourceRowPitch, SourceDepthPitch, SourceData);
	}
	virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHITexture* Texture, const TCHAR* Name) final override;
	virtual void RHIReadSurfaceData(FRHITexture* Texture, FIntRect Rect, TArray<FColor>& OutData, FReadSurfaceDataFlags InFlags) final override;
	virtual void RHIReadSurfaceData(FRHITexture* Texture, FIntRect Rect, TArray<FLinearColor>& OutData, FReadSurfaceDataFlags InFlags) final override;
	virtual void RHIMapStagingSurface(FRHITexture* Texture, FRHIGPUFence* Fence, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex = 0) final override;
	virtual void RHIUnmapStagingSurface(FRHITexture* Texture, uint32 GPUIndex = 0) final override;

	virtual void RHIReadSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, TArray<FFloat16Color>& OutData, ECubeFace CubeFace, int32 ArrayIndex, int32 MipIndex) final override;
	virtual void RHIRead3DSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, FIntPoint ZMinMax, TArray<FFloat16Color>& OutData) final override;
	virtual FRenderQueryRHIRef RHICreateRenderQuery(ERenderQueryType QueryType) final override;
	virtual void RHIEndRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery) final override;
	virtual void RHIBeginRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType) final override;
	virtual void RHIEndRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType) final override;
	virtual bool RHIGetRenderQueryResult(FRHIRenderQuery* RenderQuery, uint64& OutResult, bool bWait, uint32 GPUIndex = INDEX_NONE) final override;
	virtual FTextureRHIRef RHIGetViewportBackBuffer(FRHIViewport* Viewport) final override;
	virtual void RHIAliasTextureResources(FTextureRHIRef& DestTexture, FTextureRHIRef& SrcTexture) final override;
	virtual FTextureRHIRef RHICreateAliasedTexture(FTextureRHIRef& SourceTexture) final override;
	virtual FViewportRHIRef RHICreateViewport(void* WindowHandle, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) final override;
	virtual void RHIResizeViewport(FRHIViewport* Viewport, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) final override;
	virtual void RHIEndDrawingViewport(FRHICommandListImmediate& RHICmdList, FRHIViewport* Viewport, const FRHIPresentArgs& PresentArgs) final override;
	virtual void RHITick(float DeltaTime) final override;
	virtual void RHIBlockUntilGPUIdle() final override;
	virtual void RHISuspendRendering() final override;
	virtual void RHIResumeRendering() final override;
	virtual bool RHIIsRenderingSuspended() final override;
	virtual bool RHIGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate) final override;
	virtual void RHIGetSupportedResolution(uint32& Width, uint32& Height) final override;
	virtual void* RHIGetNativeDevice() final override;
	virtual void* RHIGetNativePhysicalDevice() final override;
	virtual void* RHIGetNativeGraphicsQueue() final override;
	virtual void* RHIGetNativeComputeQueue() final override;
	virtual class IRHICommandContext* RHIGetDefaultContext() final override;
	virtual IRHIComputeContext* RHIGetCommandContext(ERHIPipeline Pipeline) final override;
	virtual IRHIUploadContext* RHIGetUploadContext() final override;
	virtual FRHIFinalizeContextsResult RHIFinalizeContexts(FRHIFinalizeContextsArgs&& Args) final override;
	virtual void RHISubmitCommandLists(FRHISubmitCommandListsArgs&& Args) final override;
	virtual uint64 RHIGetMinimumAlignmentForBufferBackedSRV(EPixelFormat Format) final override;

	virtual IRHICommandContext* RHIGetParallelCommandContext(FRHIParallelRenderPassInfo const& ParallelRenderPass) final override;

	// Transient Resource Functions and Helpers
	virtual IRHITransientResourceAllocator* RHICreateTransientResourceAllocator() final override;

	// version of the RHIComputePrecachePSOHash
	static uint32 GetPrecachePSOHashVersion();
	virtual uint64 RHIComputePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer) final override;
	virtual uint64 RHIComputeStatePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer) final override;
	virtual bool RHIMatchPrecachePSOInitializers(const FGraphicsPipelineStateInitializer& LHS, const FGraphicsPipelineStateInitializer& RHS) final override;

	virtual FUpdateTexture3DData RHIBeginUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion) override final;
	virtual void RHIEndUpdateTexture3D(FRHICommandListBase& RHICmdList, FUpdateTexture3DData& UpdateData) override final;

	// SRV / UAV creation functions
	virtual FShaderResourceViewRHIRef  RHICreateShaderResourceView (class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc) final override;
	virtual FUnorderedAccessViewRHIRef RHICreateUnorderedAccessView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc) final override;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	virtual FRHIResourceCollectionRef RHICreateResourceCollection(FRHICommandListBase& RHICmdList, TConstArrayView<FRHIResourceCollectionMember> InMembers) final override;
	virtual void RHIUpdateResourceCollection(FRHICommandListBase& RHICmdList, FRHIResourceCollection* InResourceCollection, uint32 InStartIndex, TConstArrayView<FRHIResourceCollectionMember> InMemberUpdates) final override;
#endif

	virtual FRayTracingAccelerationStructureSize RHICalcRayTracingSceneSize(const FRayTracingSceneInitializer& Initializer) final override;
	virtual FRayTracingAccelerationStructureSize RHICalcRayTracingGeometrySize(const FRayTracingGeometryInitializer& Initializer) final override;
	virtual FRayTracingGeometryRHIRef RHICreateRayTracingGeometry(FRHICommandListBase& RHICmdList, const FRayTracingGeometryInitializer& Initializer) final override;
	virtual FRayTracingSceneRHIRef RHICreateRayTracingScene(FRayTracingSceneInitializer Initializer) final override;
	virtual FRayTracingShaderRHIRef RHICreateRayTracingShader(const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency ShaderFrequency) final override;
	virtual FRayTracingPipelineStateRHIRef RHICreateRayTracingPipelineState(const FRayTracingPipelineStateInitializer& Initializer) final override;

	virtual FShaderBindingTableRHIRef RHICreateShaderBindingTable(FRHICommandListBase& RHICmdList, const FRayTracingShaderBindingTableInitializer& Initializer) final override;

	// Historical number of times we've presented any and all viewports
	uint32 TotalPresentCount = 0;

	const TArray<const ANSICHAR*>& GetInstanceExtensions() const
	{
		return InstanceExtensions;
	}

	const TArray<const ANSICHAR*>& GetInstanceLayers() const
	{
		return InstanceLayers;
	}

	VkInstance GetInstance() const
	{
		return Instance;
	}
	
	FVulkanDevice* GetDevice() const
	{
		return Device;
	}

	bool SupportsDebugUtilsExt() const
	{
		return (ActiveDebugLayerExtension == EActiveDebugLayerExtension::DebugUtilsExtension);
	}

	const FOptionalVulkanInstanceExtensions& GetOptionalExtensions() const
	{
		return OptionalInstanceExtensions;
	}

	void VulkanSetImageLayout( VkCommandBuffer CmdBuffer, VkImage Image, VkImageLayout OldLayout, VkImageLayout NewLayout, const VkImageSubresourceRange& SubresourceRange );

	virtual void* RHILockStagingBuffer(FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI) final override;
	virtual void RHIUnlockStagingBuffer(FRHIStagingBuffer* StagingBuffer) final override;

	TArray<FVulkanViewport*>& GetViewports()
	{
		return Viewports;
	}

	uint32 GetApiVersion() const
	{
		return ApiVersion;
	}

	// Runs code on SubmissionThread with access to VkQueue.  Useful for plugins. 
	virtual void RHIRunOnQueue(EVulkanRHIRunOnQueueType QueueType, TFunction<void(VkQueue)>&& CodeToRun, bool bWaitForSubmission) final override;

	virtual void RHISetFDMOffsets(const FVector2f GazePoint[2], FIntPoint FramebufferSize);

	void DumpCrashMarkers();

	static void SavePipelineCache();

	void ProcessInterruptQueueUntil(FVulkanSyncPointRef SyncPoint);
	void ProcessInterruptQueueUntil(FGraphEvent* GraphEvent);
	void EnqueueEndOfPipeTask(TUniqueFunction<void()> TaskFunc, TUniqueFunction<void(FVulkanPayload&)> ModifyPayloadCallback = {});
	void CompletePayload(FVulkanPayload* Payload);

	TOptional<uint64> ExternalGPUTime;

	// Returns true if the calling thread is the interrupt thread
	// (or acting as the interrupt thread when USE_INTERRUPT_THREAD is zero).
	bool IsInInterruptThread() const;

	// Called when the GPU has crashed. This function will not return.
	void ProcessInterruptQueueOnGPUCrash();
	void TerminateOnGPUCrash(const TCHAR* Message);

	bool HasInterruptThread() const
	{
		return (InterruptThread != nullptr);
	}

	bool HasSubmissionThread() const
	{
		return (SubmissionThread != nullptr);
	}

	void KickSubmissionThread(bool bForce = false);

protected:
	void InitializeSubmissionPipe();
	void ShutdownSubmissionPipe();
	bool ProcessSubmissionQueue();
	bool ProcessInterruptQueue();
	bool WaitAndProcessInterruptQueue();
	void KickInterruptThread();

	struct FVulkanTimingArray : public TArray<TUniquePtr<FVulkanTiming>, TInlineAllocator<(int32)EVulkanQueueType::Count>>
	{
		FVulkanTiming* CreateNew(FVulkanQueue& Queue)
		{
			return Emplace_GetRef(MakeUnique<FVulkanTiming>(Queue)).Get();
		}
	};
	FVulkanTimingArray CurrentTimingPerQueue;

	FCriticalSection SubmissionCS;
	FVulkanThread* SubmissionThread = nullptr;
	FCriticalSection InterruptCS;
	FVulkanThread* InterruptThread = nullptr;
	FVulkanSemaphore* CPUTimelineSemaphore = nullptr;  // used to wake up interrupt thread from CPU
	std::atomic<uint64> CPUTimelineSemaphoreValue = 1;
	TQueue<FVulkanPlatformCommandList*, EQueueMode::Mpsc> PendingPayloadsForSubmission;
	TMap<VkSemaphore, FBinarySemaphoreSignalInfo> SignaledSemaphores;
	FGraphEventRef EopTask;

	// Used when a dedicated interrupt thread is disabled (D3D12_USE_INTERRUPT_THREAD)
	// Tracks the thread ID of the thread currently acting as the interrupt thread.
	uint32 InterruptThreadID = 0;

	uint32 ApiVersion = 0;
	VkInstance Instance;
	TArray<const ANSICHAR*> InstanceExtensions;
	TArray<const ANSICHAR*> InstanceLayers;

	FVulkanDevice* Device;

	/** A list of all viewport RHIs that have been created. */
	TArray<FVulkanViewport*> Viewports;

	void CreateInstance();
	void SelectDevice();

	friend class FVulkanCommandListContext;

	friend class FVulkanViewport;

	/*
	static void WriteEndFrameTimestamp(void*);
	*/

	TArray<const ANSICHAR*> SetupInstanceLayers(FVulkanInstanceExtensionArray& UEExtensions);

	IConsoleObject* SavePipelineCacheCmd = nullptr;
	IConsoleObject* RebuildPipelineCacheCmd = nullptr;
#if VULKAN_SUPPORTS_VALIDATION_CACHE
	IConsoleObject* SaveValidationCacheCmd = nullptr;
#endif

	static void RebuildPipelineCache();
#if VULKAN_SUPPORTS_VALIDATION_CACHE
	static void SaveValidationCache();
#endif

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
	IConsoleObject* DumpMemoryCmd = nullptr;
	IConsoleObject* DumpMemoryFullCmd = nullptr;
	IConsoleObject* DumpStagingMemoryCmd = nullptr;
	IConsoleObject* DumpLRUCmd = nullptr;
	IConsoleObject* TrimLRUCmd = nullptr;
public:
	static void DumpMemory();
	static void DumpMemoryFull();
	static void DumpStagingMemory();
	static void DumpLRU();
	static void TrimLRU();
#endif

public:
	enum class EActiveDebugLayerExtension
	{
		None,
		GfxReconstructLayer,
		VkTraceLayer,
		DebugUtilsExtension
	};

protected:
	bool bIsStandaloneStereoDevice = false;

	EActiveDebugLayerExtension ActiveDebugLayerExtension = EActiveDebugLayerExtension::None;

#if VULKAN_HAS_DEBUGGING_ENABLED
	VkDebugUtilsMessengerEXT Messenger = VK_NULL_HANDLE;
	VkDebugUtilsMessengerEXT AddressBindingMessenger = VK_NULL_HANDLE;

	void SetupDebugCallbacks();
	void RemoveDebugCallbacks();
#endif

	FCriticalSection LockBufferCS;

	void InternalUpdateTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData);
	void InternalUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData);

	static void SetupValidationRequests();

	FOptionalVulkanInstanceExtensions OptionalInstanceExtensions;

public:
	static TSharedPtr< IHeadMountedDisplayVulkanExtensions, ESPMode::ThreadSafe > HMDVulkanExtensions;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** Implements the Vulkan module as a dynamic RHI providing module. */
class FVulkanDynamicRHIModule : public IDynamicRHIModule
{
public:
	// IModuleInterface
	virtual void StartupModule() override;

	// IDynamicRHIModule
	virtual bool IsSupported() override;
	virtual bool IsSupported(ERHIFeatureLevel::Type RequestedFeatureLevel) override;

	virtual FDynamicRHI* CreateRHI(ERHIFeatureLevel::Type RequestedFeatureLevel = ERHIFeatureLevel::Num) override;
};
