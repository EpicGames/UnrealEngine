// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanExtensions.h"
#include "VulkanDevice.h"
#include "VulkanRHIPrivate.h"

#include "IHeadMountedDisplayModule.h"
#include "IHeadMountedDisplayVulkanExtensions.h"
#include "Misc/CommandLine.h"

#include "RHICore.h"
#include "RHICoreNvidiaAftermath.h"

// ADDING A NEW EXTENSION:
// 
// A - If the extension simply needs to be queried for driver support and added at device creation (and set a flag):
//     Add an FVulkanDeviceExtension/FVulkanInstanceExtension directly in the array returned by GetUESupportedDeviceExtensions/GetUESupportedInstanceExtensions.
// 
// B - The extension requires the use of specialized Feature and/or Property structures, or other extended processing:
//     * Add a [instance/device] class for your extension, try to call it FVulkan[ExtensionNameInCamelCaps]Extension so we have consistency
//     * Feed it the extension name string, use the SDK's define if possible : VK_[BLA_BLA_BLA]_EXTENSION_NAME
//     * If there is a VULKAN_SUPPORTS_* define that enables/disables your extension in code, provide it so that the extension knows if it should be used or not.
//       We don't IFDEF the extensions code so that we still know of its existence and we can report warnings if someone tries to use a disabled extension.
//       If there is no define to enable/disable the extension in code, then simply use VULKAN_EXTENSION_ENABLED.
// 
// Tips:
// - Feature and Property structures specific to extensions that won't be needed beyond init should be included in the extension's class instead of the device.
// - To add engine support for a complex extension, but require some kind of external activation (eg plugin) you can use its EExtensionActivation state (see header definition).
// - If an extension is supported on multiple platforms, it may be cleaner to include it here and simply disable its VULKAN_SUPPORTS_* value in the Vulkan platform header where it's not supported.

TAutoConsoleVariable<int32> GRHIAllow64bitShaderAtomicsCvar(
	TEXT("r.Vulkan.Allow64bitShaderAtomics"),
	1,
	TEXT("Whether to enable 64bit buffer/image atomics required by Nanite\n")
	TEXT("0 to disable 64bit atomics\n")
	TEXT("1 to enable (default)"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GRHIAllow16bitOps(
	TEXT("r.Vulkan.Allow16bitOps"),
	1,
	TEXT("Whether to enable 16bit ops to speeds up TSR\n")
	TEXT("0 to disable\n")
	TEXT("1 to enable (default)"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GVulkanRayTracingCVar(
	TEXT("r.Vulkan.RayTracing"),
	1,
	TEXT("0: Do not enable Vulkan ray tracing extensions\n")
	TEXT("1: Enable experimental ray tracing support (default)"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GVulkanAllowHostQueryResetCVar(
	TEXT("r.Vulkan.AllowHostQueryReset"),
	1,
	TEXT("0: Do not enable support for Host Query Reset extension\n")
	TEXT("1: Enable Host Query Reset (default)"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GVulkanAllowSync2BarriersCVar(
	TEXT("r.Vulkan.AllowSynchronization2"),
	1,
	TEXT("Enables the use of advanced barriers that combine the use of the VK_KHR_separate_depth_stencil_layouts \n")
	TEXT("and VK_KHR_synchronization2 to reduce the reliance on layout tracking (except for defragging).\n")
	TEXT("This is necessary in order to support parallel command buffer generation.\n")
	TEXT("0: Do not enable support for sync2 barriers.\n")
	TEXT("1: Enable sync2 barriers (default)"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GVulkanAllowDynamicRenderingCVar(
	TEXT("r.Vulkan.AllowDynamicRendering"),
	1,
	TEXT("Enables the use of VK_KHR_dynamic_rendering to remove the need for render passes. \n")
	TEXT("0: Do not enable support for dynamic rendering\n")
	TEXT("1: Enable dynamic rendering (default)"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GVulkanAllowDynamicStatesCVar(
	TEXT("r.Vulkan.AllowDynamicStates"),
	0,
	TEXT("Enables the use of VK_EXT_extended_dynamic_state1, 2, 3 and \n")
	TEXT("VK_EXT_vertex_input_dynamic_state to reduce amount of unique pipelines. \n")
	TEXT("0: Do not enable support for a dynamic state extensions (default)\n")
	TEXT("1: Enable only VK_EXT_extended_dynamic_state\n")
	TEXT("2: Enable only VK_EXT_extended_dynamic_state2\n")
	TEXT("4: Enable only VK_EXT_extended_dynamic_state3\n")
	TEXT("8: Enable only VK_EXT_vertex_input_dynamic_state\n")
	TEXT("7: Enable VK_EXT_extended_dynamic_state1,2,3.\n")
	TEXT("15: Enable all dynamic state extensions.\n"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GVulkanAllowShaderObjectCVar(
	TEXT("r.Vulkan.AllowShaderObject"),
	0,
	TEXT("Enables the use of VK_EXT_shader_object to replace graphic pipelines. \n")
	TEXT("0: Do not enable support for shader objects (default)\n")
	TEXT("1: Enable shader objects"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GVulkanAllowGraphicPipelineLibraryCVar(
	TEXT("r.Vulkan.AllowGraphicPipelineLibrary"),
	0,
	TEXT("Enables the use of VK_EXT_graphics_pipeline_library to replace graphic pipelines. \n")
	TEXT("0: Do not enable support for graphic pipeline library (default)\n")
	TEXT("1: Enable graphic pipeline library"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GVulkanAllowHostImageCopyCVar(
	TEXT("r.Vulkan.AllowHostImageCopy"),
	0,
	TEXT("Enables the use of VK_EXT_host_image_copy to avoid an extra copy on unified memory devices. \n")
	TEXT("0: Do not enable support for Host Image Copy (default)\n")
	TEXT("1: Enable Host Image Copy"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GVulkanAllowRobustness2CVar(
	TEXT("r.Vulkan.AllowRobustness2"),
	0,
	TEXT("Enables the use of VK_EXT_Robustness2. \n")
	TEXT("0: Do not enable support for Robustness2 (default)\n")
	TEXT("1: Enable Robustness2 if available"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GVulkanVariableRateShadingFormatCVar(
	TEXT("r.Vulkan.VRSFormat"),
	0,
	TEXT("Allows to choose the preferred Variable Rate Shading option. \n")
	TEXT("0: Prefer Fragment Shading Rate if both Fragment Shading Rate and Fragment Density Map are available.\n")
	TEXT("1: Use Fragment Shading Rate if available. A message will be reported if not available. \n")
	TEXT("2: Require Fragment Shading Rate. Will generate an error if the extension is not available. \n")
	TEXT("3: Prefer Fragment Density Map if both Fragment Shading Rate and Fragment Density Map are available.\n")
	TEXT("4: Use Fragment Density Map if available. A message will be reported if not available.\n")
	TEXT("5: Require Fragment Density Map. Will generate an error if the extension is not available."),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GVulkanAllowFDMOffsetCVar(
	TEXT("r.Vulkan.AllowFDMOffset"),
	0,
	TEXT("Enables the use of VK_EXT_fragment_density_map_offset\n")
	TEXT("0: Do not enable support for fragment_density_map_offset (default)\n")
	TEXT("1: Enable fragment_density_map_offset if available"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GVulkanAllowImageRobustness(
	TEXT("r.Vulkan.AllowImageRobustness"),
	1,
	TEXT("Whether to allow VK_EXT_image_robustness\n")
	TEXT("0 to disable\n")
	TEXT("1 to enable (default)"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> GVulkanAllowAddressBindingReport(
	TEXT("r.Vulkan.AllowAddressBindingReport"),
	0,
	TEXT("Whether to allow VK_EXT_device_address_binding_report\n")
	TEXT("0 to disable (default)\n")
	TEXT("1 to enable"),
	ECVF_ReadOnly
);

#if VULKAN_HAS_DEBUGGING_ENABLED
extern TAutoConsoleVariable<int32> GGPUValidationCvar;
#endif

TSharedPtr<IHeadMountedDisplayVulkanExtensions, ESPMode::ThreadSafe> FVulkanDynamicRHI::HMDVulkanExtensions;
TArray<TSharedRef<FVulkanRHIExternalDeviceExtensionBase>> FVulkanDeviceExtension::ExternalExtensions;
TArray<const ANSICHAR*> FVulkanInstanceExtension::ExternalExtensions;


struct FOptionalVulkanDeviceExtensionProperties& FVulkanDeviceExtension::GetDeviceExtensionProperties()
{
	const FOptionalVulkanDeviceExtensionProperties& ExtensionProperties = Device->GetOptionalExtensionProperties();
	return const_cast<FOptionalVulkanDeviceExtensionProperties&>(ExtensionProperties);
}


#define VERIFYVULKANRESULT_INIT(VkFunction)	{ const VkResult ScopedResult = VkFunction; \
												if (ScopedResult == VK_ERROR_INITIALIZATION_FAILED) { \
													UE_LOG(LogVulkanRHI, Error, \
													TEXT("%s failed\n at %s:%u\nThis typically means Vulkan is not properly set up in your system; try running vulkaninfo from the Vulkan SDK."), \
													ANSI_TO_TCHAR(#VkFunction), ANSI_TO_TCHAR(__FILE__), __LINE__); } \
												else if (ScopedResult < VK_SUCCESS) { \
													VulkanRHI::VerifyVulkanResult(ScopedResult, #VkFunction, __FILE__, __LINE__); }}


TArray<VkExtensionProperties> FVulkanDeviceExtension::GetDriverSupportedDeviceExtensions(VkPhysicalDevice Gpu, const ANSICHAR* LayerName)
{
	TArray<VkExtensionProperties> OutDeviceExtensions;
	uint32 Count = 0;
	VERIFYVULKANRESULT_INIT(VulkanRHI::vkEnumerateDeviceExtensionProperties(Gpu, LayerName, &Count, nullptr));
	if (Count > 0)
	{
		OutDeviceExtensions.AddZeroed(Count);
		VERIFYVULKANRESULT_INIT(VulkanRHI::vkEnumerateDeviceExtensionProperties(Gpu, LayerName, &Count, OutDeviceExtensions.GetData()));
	}
	OutDeviceExtensions.Sort([](const VkExtensionProperties& A, const VkExtensionProperties& B) { return FCStringAnsi::Strcmp(A.extensionName, B.extensionName) < 0; });
	return OutDeviceExtensions;
}

TArray<VkExtensionProperties> FVulkanInstanceExtension::GetDriverSupportedInstanceExtensions(const ANSICHAR* LayerName)
{
	TArray<VkExtensionProperties> OutInstanceExtensions;
	uint32 Count = 0;
	VERIFYVULKANRESULT_INIT(VulkanRHI::vkEnumerateInstanceExtensionProperties(LayerName, &Count, nullptr));
	if (Count > 0)
	{
		OutInstanceExtensions.AddZeroed(Count);
		VERIFYVULKANRESULT_INIT(VulkanRHI::vkEnumerateInstanceExtensionProperties(LayerName, &Count, OutInstanceExtensions.GetData()));
	}
	OutInstanceExtensions.Sort([](const VkExtensionProperties& A, const VkExtensionProperties& B) { return FCStringAnsi::Strcmp(A.extensionName, B.extensionName) < 0; });
	return OutInstanceExtensions;
}

#undef VERIFYVULKANRESULT_INIT




// *** Vulkan Device Extension support ***
// Typical flow:
// 1- For the selected rendering device, the engine will query the supported extensions (FVulkanDeviceExtension are constructed and support is queried)
// 2- Followed by a query of Physical Device Features (PrePhysicalDeviceFeatures, PostPhysicalDeviceFeatures)
// 3- Followed by a query of Physical Device Properties (PrePhysicalDeviceProperties, PostPhysicalDeviceProperties)
// 4- Finally, the device is created (PreCreateDevice)



// ***** VK_KHR_maintenance4
class FVulkanKHRMaintenance4Extension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRMaintenance4Extension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_MAINTENANCE_4_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_3)
	{}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(Maintenance4Features, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_MAINTENANCE_4_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, Maintenance4Features);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (Maintenance4Features.maintenance4 == VK_TRUE);
		ExtensionFlags.HasKHRMaintenance4 = bRequirementsPassed;
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		if (bRequirementsPassed)
		{
			ZeroVulkanStruct(Maintenance4Properties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_MAINTENANCE_4_PROPERTIES_KHR);
			AddToPNext(PhysicalDeviceProperties2, Maintenance4Properties);
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, Maintenance4Features);

			GRHIGlobals.MaxViewSizeBytesForNonTypedBuffer = FMath::Min<uint64>(Maintenance4Properties.maxBufferSize, GRHIGlobals.MaxViewSizeBytesForNonTypedBuffer);
		}
	}

private:
	VkPhysicalDeviceMaintenance4Features Maintenance4Features;
	VkPhysicalDeviceMaintenance4Properties Maintenance4Properties;
};


// ***** VK_KHR_maintenance5
class FVulkanKHRMaintenance5Extension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRMaintenance5Extension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_MAINTENANCE_5_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_4)
	{}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(Maintenance5Features, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_MAINTENANCE_5_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, Maintenance5Features);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (Maintenance5Features.maintenance5 == VK_TRUE);
		if (bRequirementsPassed)
		{
			ExtensionFlags.HasKHRMaintenance5 = true;
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, Maintenance5Features);
		}
	}

private:
	VkPhysicalDeviceMaintenance5FeaturesKHR Maintenance5Features;
};

// ***** VK_KHR_maintenance6
class FVulkanKHRMaintenance6Extension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRMaintenance6Extension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_MAINTENANCE_6_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_4)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(Maintenance6Features, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_MAINTENANCE_6_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, Maintenance6Features);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (Maintenance6Features.maintenance6 == VK_TRUE);
		if (bRequirementsPassed)
		{
			ExtensionFlags.HasKHRMaintenance6 = true;
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, Maintenance6Features);
		}
	}

private:
	VkPhysicalDeviceMaintenance6FeaturesKHR Maintenance6Features;
};

// ***** VK_KHR_maintenance7
class FVulkanKHRMaintenance7Extension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRMaintenance7Extension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_MAINTENANCE_7_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(Maintenance7Features, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_MAINTENANCE_7_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, Maintenance7Features);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (Maintenance7Features.maintenance7 == VK_TRUE);
		if (bRequirementsPassed)
		{
			ExtensionFlags.HasKHRMaintenance7 = true;
		}
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		if (bRequirementsPassed)
		{
			VkPhysicalDeviceMaintenance7PropertiesKHR& PhysicalDeviceMaintenance7Properties = GetDeviceExtensionProperties().PhysicalDeviceMaintenance7Properties;
			ZeroVulkanStruct(PhysicalDeviceMaintenance7Properties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_MAINTENANCE_7_PROPERTIES_KHR);
			AddToPNext(PhysicalDeviceProperties2, PhysicalDeviceMaintenance7Properties);
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, Maintenance7Features);
		}
	}

private:
	VkPhysicalDeviceMaintenance7FeaturesKHR Maintenance7Features;
};

// ***** VK_KHR_maintenance9
class FVulkanKHRMaintenance9Extension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRMaintenance9Extension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_MAINTENANCE_9_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(Maintenance9Features, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_MAINTENANCE_9_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, Maintenance9Features);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (Maintenance9Features.maintenance9 == VK_TRUE);
		if (bRequirementsPassed)
		{
			ExtensionFlags.HasKHRMaintenance9 = true;
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, Maintenance9Features);
		}
	}

private:
	VkPhysicalDeviceMaintenance9FeaturesKHR Maintenance9Features;
};

// ***** VK_KHR_driver_properties
class FVulkanKHRDriverPropertiesExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRDriverPropertiesExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_DRIVER_PROPERTIES_EXTENSION_NAME, VULKAN_SUPPORTS_DRIVER_PROPERTIES, VK_API_VERSION_1_2)
	{}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		VkPhysicalDeviceDriverPropertiesKHR& PhysicalDeviceDriverProperties = GetDeviceExtensionProperties().PhysicalDeviceDriverProperties;
		ZeroVulkanStruct(PhysicalDeviceDriverProperties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_DRIVER_PROPERTIES_KHR);
		AddToPNext(PhysicalDeviceProperties2, PhysicalDeviceDriverProperties);
	}

	virtual void PostPhysicalDeviceProperties(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		const VkPhysicalDeviceDriverPropertiesKHR& PhysicalDeviceDriverProperties = GetDeviceExtensionProperties().PhysicalDeviceDriverProperties;
		UE_LOGF(LogVulkanRHI, Display, "Vulkan Physical Device Driver Properties:");
		UE_LOGF(LogVulkanRHI, Display, "- driverName: %s", PhysicalDeviceDriverProperties.driverName);
		UE_LOGF(LogVulkanRHI, Display, "- driverInfo: %s", PhysicalDeviceDriverProperties.driverInfo);
	}
};



// ***** VK_KHR_shader_atomic_int64
class FVulkanShaderAtomicInt64Extension : public FVulkanDeviceExtension
{
public:

	FVulkanShaderAtomicInt64Extension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_SHADER_ATOMIC_INT64_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_2)
	{
		bEnabledInCode = bEnabledInCode && (GRHIAllow64bitShaderAtomicsCvar.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(BufferAtomicFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_SHADER_ATOMIC_INT64_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, BufferAtomicFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (BufferAtomicFeatures.shaderBufferInt64Atomics == VK_TRUE);
		ExtensionFlags.HasKHRShaderAtomicInt64 = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, BufferAtomicFeatures);
		}
	}

	VkPhysicalDeviceShaderAtomicInt64Features BufferAtomicFeatures;
};



// ***** VK_EXT_shader_image_atomic_int64
class FVulkanShaderImageAtomicInt64Extension : public FVulkanDeviceExtension
{
public:

	FVulkanShaderImageAtomicInt64Extension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_SHADER_IMAGE_ATOMIC_INT64_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && (GRHIAllow64bitShaderAtomicsCvar.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(ImageAtomicFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_SHADER_IMAGE_ATOMIC_INT64_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, ImageAtomicFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (ImageAtomicFeatures.shaderImageInt64Atomics == VK_TRUE);
		ExtensionFlags.HasImageAtomicInt64 = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		// The PreCreateDevice() call is after all extension have gone through PostPhysicalDeviceFeatures(), so ExtensionFlags will be filled for both.
		const FOptionalVulkanDeviceExtensions& ExtensionFlags = Device->GetOptionalExtensions();
		GRHISupportsAtomicUInt64 = ExtensionFlags.HasKHRShaderAtomicInt64 && ExtensionFlags.HasImageAtomicInt64;

		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, ImageAtomicFeatures);
		}
	}

	VkPhysicalDeviceShaderImageAtomicInt64FeaturesEXT ImageAtomicFeatures;
};



// ***** VK_EXT_shader_viewport_index_layer
class FVulkanEXTShaderViewportIndexLayerExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTShaderViewportIndexLayerExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_SHADER_VIEWPORT_INDEX_LAYER_EXTENSION_NAME, VULKAN_SUPPORTS_SHADER_VIEWPORT_INDEX_LAYER, VK_API_VERSION_1_2)
	{}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		GRHISupportsArrayIndexFromAnyShader = true;
	}
};



// ***** VK_KHR_separate_depth_stencil_layouts
class FVulkanKHRSeparateDepthStencilLayoutsExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRSeparateDepthStencilLayoutsExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_SEPARATE_DEPTH_STENCIL_LAYOUTS_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_2)
	{
		bEnabledInCode = bEnabledInCode && (GVulkanAllowSync2BarriersCVar.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(SeparateDepthStencilLayoutsFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_SEPARATE_DEPTH_STENCIL_LAYOUTS_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, SeparateDepthStencilLayoutsFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (SeparateDepthStencilLayoutsFeatures.separateDepthStencilLayouts == VK_TRUE);
		ExtensionFlags.HasSeparateDepthStencilLayouts = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, SeparateDepthStencilLayoutsFeatures);
		}
	}

private:
	VkPhysicalDeviceSeparateDepthStencilLayoutsFeaturesKHR SeparateDepthStencilLayoutsFeatures;
};



// ***** VK_KHR_synchronization2
class FVulkanKHRSynchronization2 : public FVulkanDeviceExtension
{
public:

	FVulkanKHRSynchronization2(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_SYNCHRONIZATION_2_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_3)
	{
		bEnabledInCode = bEnabledInCode && (GVulkanAllowSync2BarriersCVar.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(Synchronization2Features, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_SYNCHRONIZATION_2_FEATURES);
		AddToPNext(PhysicalDeviceFeatures2, Synchronization2Features);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (Synchronization2Features.synchronization2 == VK_TRUE);
		ExtensionFlags.HasKHRSynchronization2 = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, Synchronization2Features);
		}
	}

private:
	VkPhysicalDeviceSynchronization2FeaturesKHR Synchronization2Features;
};



// ***** VK_KHR_multiview
class FVulkanKHRMultiviewExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRMultiviewExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_MULTIVIEW_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_1)
	{}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(DeviceMultiviewFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_MULTIVIEW_FEATURES);
		AddToPNext(PhysicalDeviceFeatures2, DeviceMultiviewFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final 
	{
		bRequirementsPassed = (DeviceMultiviewFeatures.multiview == VK_TRUE);
		ExtensionFlags.HasKHRMultiview = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, DeviceMultiviewFeatures);
		}
	}

private:
	VkPhysicalDeviceMultiviewFeatures DeviceMultiviewFeatures;
};



// ***** VK_EXT_scalar_block_layout
class FVulkanEXTScalarBlockLayoutExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTScalarBlockLayoutExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_SCALAR_BLOCK_LAYOUT_EXTENSION_NAME, VULKAN_SUPPORTS_SCALAR_BLOCK_LAYOUT, VK_API_VERSION_1_2)
	{}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(ScalarBlockLayoutFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_SCALAR_BLOCK_LAYOUT_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, ScalarBlockLayoutFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final 
	{
		bRequirementsPassed = (ScalarBlockLayoutFeatures.scalarBlockLayout == VK_TRUE);
		ExtensionFlags.HasEXTScalarBlockLayout = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, ScalarBlockLayoutFeatures);
		}
	}

private:
	VkPhysicalDeviceScalarBlockLayoutFeaturesEXT ScalarBlockLayoutFeatures;
};



// ***** VK_EXT_descriptor_indexing
class FVulkanEXTDescriptorIndexingExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTDescriptorIndexingExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_DESCRIPTOR_INDEXING_EXTENSION_NAME, VULKAN_SUPPORTS_DESCRIPTOR_INDEXING, VK_API_VERSION_1_2)
	{}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(DescriptorIndexingFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_DESCRIPTOR_INDEXING_FEATURES);
		AddToPNext(PhysicalDeviceFeatures2, DescriptorIndexingFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (DescriptorIndexingFeatures.runtimeDescriptorArray == VK_TRUE) &&
			(DescriptorIndexingFeatures.descriptorBindingPartiallyBound == VK_TRUE) &&
			(DescriptorIndexingFeatures.descriptorBindingUpdateUnusedWhilePending == VK_TRUE) &&
			(DescriptorIndexingFeatures.descriptorBindingVariableDescriptorCount == VK_TRUE);

		ExtensionFlags.HasEXTDescriptorIndexing = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, DescriptorIndexingFeatures);
		}
	}

private:
	VkPhysicalDeviceDescriptorIndexingFeaturesEXT DescriptorIndexingFeatures;
};



// ***** VK_KHR_fragment_shading_rate
class FVulkanKHRFragmentShadingRateExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRFragmentShadingRateExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_FRAGMENT_SHADING_RATE_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode &= HardwareVariableRateShadingSupportedByPlatform(GMaxRHIShaderPlatform);

		// FSR should be enabled even if FDM is preferred because it could be not available.
		int32 VRSFormatPreference = GVulkanVariableRateShadingFormatCVar->GetInt();
		bEnabledInCode &= (VRSFormatPreference <= (uint8) EVulkanVariableRateShadingPreference::PreferFDM);
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		ZeroVulkanStruct(FragmentShadingRateProperties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_FRAGMENT_SHADING_RATE_PROPERTIES_KHR);
		AddToPNext(PhysicalDeviceProperties2, FragmentShadingRateProperties);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		ExtensionFlags.HasKHRFragmentShadingRate = 1;

		VkPhysicalDeviceFragmentShadingRateFeaturesKHR& FragmentShadingRateFeatures = GetDeviceExtensionProperties().FragmentShadingRateFeatures;
		GRHISupportsAttachmentVariableRateShading = (FragmentShadingRateFeatures.attachmentFragmentShadingRate == VK_TRUE);
		GRHISupportsPipelineVariableRateShading = (FragmentShadingRateFeatures.pipelineFragmentShadingRate == VK_TRUE);

		if (FragmentShadingRateFeatures.attachmentFragmentShadingRate == VK_TRUE)
		{
			GRHIVariableRateShadingImageDataType = VRSImage_Palette;
			GRHIVariableRateShadingImageFormat = PF_R8_UINT;
		}
		else
		{
			GRHIVariableRateShadingImageDataType = VRSImage_NotSupported;
			GRHIVariableRateShadingImageFormat = PF_Unknown;
		}
	}

	virtual void PostPhysicalDeviceProperties(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		VkPhysicalDeviceFragmentShadingRateFeaturesKHR& FragmentShadingRateFeatures = GetDeviceExtensionProperties().FragmentShadingRateFeatures;
		if (FragmentShadingRateFeatures.attachmentFragmentShadingRate == VK_TRUE)
		{
			GRHIVariableRateShadingImageTileMinWidth = FragmentShadingRateProperties.minFragmentShadingRateAttachmentTexelSize.width;
			GRHIVariableRateShadingImageTileMinHeight = FragmentShadingRateProperties.minFragmentShadingRateAttachmentTexelSize.height;
			GRHIVariableRateShadingImageTileMaxWidth = FragmentShadingRateProperties.maxFragmentShadingRateAttachmentTexelSize.width;
			GRHIVariableRateShadingImageTileMaxHeight = FragmentShadingRateProperties.maxFragmentShadingRateAttachmentTexelSize.height;

			if (FragmentShadingRateProperties.maxFragmentSize.width >= 4 && FragmentShadingRateProperties.maxFragmentSize.height >= 4)
			{
				// FYI FVulkanDevice::GetBestMatchedShadingRateExtents does extent filtering
				GRHISupportsLargerVariableRateShadingSizes = GRHISupportsPipelineVariableRateShading;
			}

			// todo: We don't currently care much about the other properties here, but at some point in the future we probably will.

			UE_LOGF(LogVulkanRHI, Verbose, "Image-based Variable Rate Shading supported via KHRFragmentShadingRate extension. Selected VRS tile size %u by %u pixels per VRS image texel.", GRHIVariableRateShadingImageTileMinWidth, GRHIVariableRateShadingImageTileMinHeight);
		}
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		VkPhysicalDeviceFragmentShadingRateFeaturesKHR& FragmentShadingRateFeatures = GetDeviceExtensionProperties().FragmentShadingRateFeatures;
		ZeroVulkanStruct(FragmentShadingRateFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_FRAGMENT_SHADING_RATE_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, FragmentShadingRateFeatures);
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		VkPhysicalDeviceFragmentShadingRateFeaturesKHR& FragmentShadingRateFeatures = GetDeviceExtensionProperties().FragmentShadingRateFeatures;
		if (FragmentShadingRateFeatures.attachmentFragmentShadingRate == VK_TRUE || FragmentShadingRateFeatures.pipelineFragmentShadingRate == VK_TRUE)
		{
			AddToPNext(DeviceCreateInfo, FragmentShadingRateFeatures);
		}
	}

private:
	VkPhysicalDeviceFragmentShadingRatePropertiesKHR FragmentShadingRateProperties;
};



// ***** VK_EXT_fragment_density_map
class FVulkanEXTFragmentDensityMapExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTFragmentDensityMapExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_FRAGMENT_DENSITY_MAP_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode &= HardwareVariableRateShadingSupportedByPlatform(GMaxRHIShaderPlatform);

		// FDM should be enabled even if the preferred choice is FSR because that might not be available.
		int32 VRSFormatPreference = GVulkanVariableRateShadingFormatCVar->GetInt();
		bEnabledInCode &= (VRSFormatPreference >= (uint8) EVulkanVariableRateShadingPreference::PreferFDM || VRSFormatPreference == (uint8) EVulkanVariableRateShadingPreference::PreferFSR);
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		ZeroVulkanStruct(FragmentDensityMapProperties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_FRAGMENT_DENSITY_MAP_PROPERTIES_EXT);
		AddToPNext(PhysicalDeviceProperties2, FragmentDensityMapProperties);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		VkPhysicalDeviceFragmentDensityMapFeaturesEXT& FragmentDensityMapFeatures = GetDeviceExtensionProperties().FragmentDensityMapFeatures;
		bRequirementsPassed = (FragmentDensityMapFeatures.fragmentDensityMap == VK_TRUE);
		ExtensionFlags.HasEXTFragmentDensityMap = bRequirementsPassed;
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		VkPhysicalDeviceFragmentDensityMapFeaturesEXT& FragmentDensityMapFeatures = GetDeviceExtensionProperties().FragmentDensityMapFeatures;
		ZeroVulkanStruct(FragmentDensityMapFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_FRAGMENT_DENSITY_MAP_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, FragmentDensityMapFeatures);
	}

	virtual void PostPhysicalDeviceProperties(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		// Use the Fragment Density Map extension if the Fragment Shading Rate extension is not available or if both are available but 
		// Fragment Density Map is the preferred user choice.
		// NOTE: FVulkanKHRFragmentShadingRateExtension must be placed before FVulkanEXTFragmentDensityMapExtension for this to work!
		if ((!GRHISupportsAttachmentVariableRateShading || (GVulkanVariableRateShadingFormatCVar->GetInt() >= (uint8) EVulkanVariableRateShadingPreference::PreferFDM)) && bRequirementsPassed)
		{
			GRHISupportsAttachmentVariableRateShading = true;

			// Go with the smallest tile size for now, and also force to square, since this seems to be standard.
			// TODO: Eventually we may want to surface the range of possible tile sizes depending on end use cases, but for now this is being used for foveated rendering and smallest tile size
			// is preferred.

			GRHIVariableRateShadingImageTileMinWidth = FragmentDensityMapProperties.minFragmentDensityTexelSize.width;
			GRHIVariableRateShadingImageTileMinHeight = FragmentDensityMapProperties.minFragmentDensityTexelSize.height;
			GRHIVariableRateShadingImageTileMaxWidth = FragmentDensityMapProperties.maxFragmentDensityTexelSize.width;
			GRHIVariableRateShadingImageTileMaxHeight = FragmentDensityMapProperties.maxFragmentDensityTexelSize.height;

			GRHIVariableRateShadingImageDataType = VRSImage_Fractional;
			GRHIVariableRateShadingImageFormat = PF_R8G8;

			UE_LOGF(LogVulkanRHI, Display, "Image-based Variable Rate Shading supported via EXTFragmentDensityMap extension. Selected VRS tile size %u by %u pixels per VRS image texel.", GRHIVariableRateShadingImageTileMinWidth, GRHIVariableRateShadingImageTileMinHeight);
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			VkPhysicalDeviceFragmentDensityMapFeaturesEXT& FragmentDensityMapFeatures = GetDeviceExtensionProperties().FragmentDensityMapFeatures;
			AddToPNext(DeviceCreateInfo, FragmentDensityMapFeatures);
		}
	}

private:
	VkPhysicalDeviceFragmentDensityMapPropertiesEXT FragmentDensityMapProperties;
};



// ***** VK_EXT_fragment_density_map2
class FVulkanEXTFragmentDensityMap2Extension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTFragmentDensityMap2Extension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_FRAGMENT_DENSITY_MAP_2_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode &= HardwareVariableRateShadingSupportedByPlatform(GMaxRHIShaderPlatform);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		VkPhysicalDeviceFragmentDensityMap2FeaturesEXT& FragmentDensityMap2Features = GetDeviceExtensionProperties().FragmentDensityMap2Features;
		ZeroVulkanStruct(FragmentDensityMap2Features, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_FRAGMENT_DENSITY_MAP_2_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, FragmentDensityMap2Features);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		VkPhysicalDeviceFragmentDensityMap2FeaturesEXT& FragmentDensityMap2Features = GetDeviceExtensionProperties().FragmentDensityMap2Features;
		bRequirementsPassed = (FragmentDensityMap2Features.fragmentDensityMapDeferred == VK_TRUE);
		ExtensionFlags.HasEXTFragmentDensityMap2 = bRequirementsPassed;

		GRHISupportsLateVariableRateShadingUpdate = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			VkPhysicalDeviceFragmentDensityMap2FeaturesEXT& FragmentDensityMap2Features = GetDeviceExtensionProperties().FragmentDensityMap2Features;
			AddToPNext(DeviceCreateInfo, FragmentDensityMap2Features);
		}
	}
};

class FVulkanEXTFragmentDensityMapOffsetExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTFragmentDensityMapOffsetExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_FRAGMENT_DENSITY_MAP_OFFSET_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && 
			HardwareVariableRateShadingSupportedByPlatform(GMaxRHIShaderPlatform) && GVulkanAllowFDMOffsetCVar.GetValueOnAnyThread();
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(FragmentDensityMapOffsetFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_FRAGMENT_DENSITY_MAP_OFFSET_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, FragmentDensityMapOffsetFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (FragmentDensityMapOffsetFeatures.fragmentDensityMapOffset == VK_TRUE) && (ExtensionFlags.HasEXTFragmentDensityMap || ExtensionFlags.HasEXTFragmentDensityMap2);
		ExtensionFlags.HasEXTFragmentDensityMapOffset = bRequirementsPassed;
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		ZeroVulkanStruct(FragmentDensityMapOffsetProperties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_FRAGMENT_DENSITY_MAP_OFFSET_PROPERTIES_EXT);
		AddToPNext(PhysicalDeviceProperties2, FragmentDensityMapOffsetProperties);
	}

	virtual void PostPhysicalDeviceProperties(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		if (bRequirementsPassed)
		{
			GRHISupportsVariableRateShadingImageOffsets = true;

			GRHIVariableRateShadingImageOffsetGranularity.X = FragmentDensityMapOffsetProperties.fragmentDensityOffsetGranularity.width;
			GRHIVariableRateShadingImageOffsetGranularity.Y = FragmentDensityMapOffsetProperties.fragmentDensityOffsetGranularity.height;
		}
	}

private:
	VkPhysicalDeviceFragmentDensityMapOffsetFeaturesEXT FragmentDensityMapOffsetFeatures;
	VkPhysicalDeviceFragmentDensityMapOffsetPropertiesEXT FragmentDensityMapOffsetProperties;
};


// ***** VK_KHR_fragment_shader_barycentric
class FVulkanKHRFragmentShaderBarycentricExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRFragmentShaderBarycentricExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_FRAGMENT_SHADER_BARYCENTRIC_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		VkPhysicalDeviceFragmentShaderBarycentricPropertiesKHR& FragmentShaderBarycentricProps = GetDeviceExtensionProperties().FragmentShaderBarycentricProps;
		ZeroVulkanStruct(FragmentShaderBarycentricProps, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_FRAGMENT_SHADER_BARYCENTRIC_PROPERTIES_KHR);
		AddToPNext(PhysicalDeviceProperties2, FragmentShaderBarycentricProps);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(FragmentShaderBarycentricFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_FRAGMENT_SHADER_BARYCENTRIC_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, FragmentShaderBarycentricFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (FragmentShaderBarycentricFeatures.fragmentShaderBarycentric == VK_TRUE);

		// Should be runtime guaranteed through SM6 profile
		ExtensionFlags.HasKHRFragmentShaderBarycentric = bRequirementsPassed;
		GRHIGlobals.SupportsBarycentricsSemantic = bRequirementsPassed;
	}

	virtual void PostPhysicalDeviceProperties(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		VkPhysicalDeviceFragmentShaderBarycentricPropertiesKHR& FragmentShaderBarycentricProps = GetDeviceExtensionProperties().FragmentShaderBarycentricProps;
		//UE_LOGF(LogVulkanRHI, Verbose, "triStripVertexOrderIndependentOfProvokingVertex is %ls", FragmentShaderBarycentricProps.triStripVertexOrderIndependentOfProvokingVertex ? TEXT("true") : TEXT("false"));
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		// fragmentShaderBarycentric indicates that the implementation supports the BaryCoordKHR and BaryCoordNoPerspKHR SPIR - V fragment shader built - ins 
		// and supports the PerVertexKHR SPIR - V decoration on fragment shader input variables.
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, FragmentShaderBarycentricFeatures);
		}
	}

	VkPhysicalDeviceFragmentShaderBarycentricFeaturesKHR FragmentShaderBarycentricFeatures;
};



// ***** VK_KHR_compute_shader_derivatives
class FVulkanKHRComputeShaderDerivatives : public FVulkanDeviceExtension
{
public:

	FVulkanKHRComputeShaderDerivatives(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_COMPUTE_SHADER_DERIVATIVES_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		VkPhysicalDeviceComputeShaderDerivativesFeaturesKHR& ComputeShaderDerivativesFeatures = GetDeviceExtensionProperties().ComputeShaderDerivativesFeatures;
		ZeroVulkanStruct(ComputeShaderDerivativesFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_COMPUTE_SHADER_DERIVATIVES_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, ComputeShaderDerivativesFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		VkPhysicalDeviceComputeShaderDerivativesFeaturesKHR& ComputeShaderDerivativesFeatures = GetDeviceExtensionProperties().ComputeShaderDerivativesFeatures;
		bRequirementsPassed = (ComputeShaderDerivativesFeatures.computeDerivativeGroupLinear == VK_TRUE);
		// Should be runtime guaranteed through SM6 profile
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			VkPhysicalDeviceComputeShaderDerivativesFeaturesKHR& ComputeShaderDerivativesFeatures = GetDeviceExtensionProperties().ComputeShaderDerivativesFeatures;
			AddToPNext(DeviceCreateInfo, ComputeShaderDerivativesFeatures);
		}
	}
};



// ***** VK_KHR_get_memory_requirements2
class FVulkanKHRGetMemoryRequirements2Extension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRGetMemoryRequirements2Extension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_GET_MEMORY_REQUIREMENTS_2_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_1)
	{}
};



// ***** VK_KHR_buffer_device_address
class FVulkanKHRBufferDeviceAddressExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRBufferDeviceAddressExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_BUFFER_DEVICE_ADDRESS_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_2) {}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(BufferDeviceAddressFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_BUFFER_DEVICE_ADDRESS_FEATURES);
		AddToPNext(PhysicalDeviceFeatures2, BufferDeviceAddressFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final 
	{
		bRequirementsPassed = (BufferDeviceAddressFeatures.bufferDeviceAddress == VK_TRUE);
		ExtensionFlags.HasBufferDeviceAddress = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, BufferDeviceAddressFeatures);
		}
	}

private:
	VkPhysicalDeviceBufferDeviceAddressFeaturesKHR BufferDeviceAddressFeatures;
};



// ***** VK_KHR_acceleration_structure
class FVulkanKHRAccelerationStructureExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRAccelerationStructureExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_ACCELERATION_STRUCTURE_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && GVulkanRayTracingCVar.GetValueOnAnyThread() && !FParse::Param(FCommandLine::Get(), TEXT("noraytracing"));
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(AccelerationStructureFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_ACCELERATION_STRUCTURE_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, AccelerationStructureFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (AccelerationStructureFeatures.accelerationStructure == VK_TRUE) && 
			(AccelerationStructureFeatures.descriptorBindingAccelerationStructureUpdateAfterBind == VK_TRUE);

		ExtensionFlags.HasAccelerationStructure = bRequirementsPassed;
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		VkPhysicalDeviceAccelerationStructurePropertiesKHR& AccelerationStructure = GetDeviceExtensionProperties().AccelerationStructureProps;
		ZeroVulkanStruct(AccelerationStructure, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_ACCELERATION_STRUCTURE_PROPERTIES_KHR);
		AddToPNext(PhysicalDeviceProperties2, AccelerationStructure);
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, AccelerationStructureFeatures);
		}
	}

private:
	VkPhysicalDeviceAccelerationStructureFeaturesKHR AccelerationStructureFeatures;
};



// ***** VK_KHR_ray_tracing_pipeline
class FVulkanKHRRayTracingPipelineExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRRayTracingPipelineExtension(FVulkanDevice* InDevice) 
		: FVulkanDeviceExtension(InDevice, VK_KHR_RAY_TRACING_PIPELINE_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && GVulkanRayTracingCVar.GetValueOnAnyThread() && !FParse::Param(FCommandLine::Get(), TEXT("noraytracing"));
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(RayTracingPipelineFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_RAY_TRACING_PIPELINE_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, RayTracingPipelineFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (RayTracingPipelineFeatures.rayTracingPipeline == VK_TRUE) &&
			(RayTracingPipelineFeatures.rayTraversalPrimitiveCulling == VK_TRUE);

		ExtensionFlags.HasRayTracingPipeline = bRequirementsPassed;
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		VkPhysicalDeviceRayTracingPipelinePropertiesKHR& RayTracingPipeline = GetDeviceExtensionProperties().RayTracingPipelineProps;
		ZeroVulkanStruct(RayTracingPipeline, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_RAY_TRACING_PIPELINE_PROPERTIES_KHR);
		AddToPNext(PhysicalDeviceProperties2, RayTracingPipeline);
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, RayTracingPipelineFeatures);

			GRHISupportsRayTracingDispatchIndirect = (RayTracingPipelineFeatures.rayTracingPipelineTraceRaysIndirect == VK_TRUE);
		}
	}

private:
	VkPhysicalDeviceRayTracingPipelineFeaturesKHR RayTracingPipelineFeatures;
};



// ***** VK_KHR_ray_query
class FVulkanKHRRayQueryExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRRayQueryExtension(FVulkanDevice* InDevice) 
		: FVulkanDeviceExtension(InDevice, VK_KHR_RAY_QUERY_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && GVulkanRayTracingCVar.GetValueOnAnyThread() && !FParse::Param(FCommandLine::Get(), TEXT("noraytracing"));
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(RayQueryFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_RAY_QUERY_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, RayQueryFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (RayQueryFeatures.rayQuery == VK_TRUE);
		ExtensionFlags.HasRayQuery = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, RayQueryFeatures);
		}
	}

private:
	VkPhysicalDeviceRayQueryFeaturesKHR RayQueryFeatures;
};


// ***** VK_KHR_ray_tracing_position_fetch
class FVulkanKHRRayTracingPositionFetchExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRRayTracingPositionFetchExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_RAY_TRACING_POSITION_FETCH_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && GVulkanRayTracingCVar.GetValueOnAnyThread() && !FParse::Param(FCommandLine::Get(), TEXT("noraytracing"));
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(RayTracingPositionFetchFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_RAY_TRACING_POSITION_FETCH_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, RayTracingPositionFetchFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (RayTracingPositionFetchFeatures.rayTracingPositionFetch == VK_TRUE);
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, RayTracingPositionFetchFeatures);
		}
	}

private:
	VkPhysicalDeviceRayTracingPositionFetchFeaturesKHR  RayTracingPositionFetchFeatures;
};

// ***** VK_KHR_timeline_semaphore
class FVulkanKHRTimelineSemaphoreExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRTimelineSemaphoreExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_TIMELINE_SEMAPHORE_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_2)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(TimelineSemaphoreFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_TIMELINE_SEMAPHORE_FEATURES);
		AddToPNext(PhysicalDeviceFeatures2, TimelineSemaphoreFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (TimelineSemaphoreFeatures.timelineSemaphore == VK_TRUE);
		ExtensionFlags.HasKHRTimelineSemaphore = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, TimelineSemaphoreFeatures);
		}
	}

private:
	VkPhysicalDeviceTimelineSemaphoreFeatures TimelineSemaphoreFeatures;
};


// ***** VK_EXT_mesh_shader
class FVulkanEXTMeshShaderExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTMeshShaderExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_MESH_SHADER_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(MeshShaderFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_MESH_SHADER_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, MeshShaderFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (MeshShaderFeatures.meshShader == VK_TRUE) && (MeshShaderFeatures.multiviewMeshShader == VK_TRUE);
		ExtensionFlags.HasEXTMeshShader = bRequirementsPassed;
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		VkPhysicalDeviceMeshShaderPropertiesEXT& MeshShaderProperties = GetDeviceExtensionProperties().MeshShaderProperties;
		ZeroVulkanStruct(MeshShaderProperties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_MESH_SHADER_PROPERTIES_EXT);
		AddToPNext(PhysicalDeviceProperties2, MeshShaderProperties);
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			if (!GetDeviceExtensionProperties().FragmentShadingRateFeatures.primitiveFragmentShadingRate)
			{
				MeshShaderFeatures.primitiveFragmentShadingRateMeshShader = VK_FALSE;
			}

			AddToPNext(DeviceCreateInfo, MeshShaderFeatures);
		}
	}

private:
	VkPhysicalDeviceMeshShaderFeaturesEXT MeshShaderFeatures;
};


// ***** VK_AMD_buffer_marker (vendor)
class FVulkanAMDBufferMarkerExtension : public FVulkanDeviceExtension
{
public:

	FVulkanAMDBufferMarkerExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_AMD_BUFFER_MARKER_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && UE::RHI::UseGPUCrashBreadcrumbs() && UE::RHICore::AllowVendorDevice();
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		ExtensionFlags.HasAMDBufferMarker = 1; 
	}
};



// ***** VK_NV_device_diagnostic_checkpoints (vendor)
class FVulkanNVDeviceDiagnosticCheckpointsExtension : public FVulkanDeviceExtension
{
public:

	FVulkanNVDeviceDiagnosticCheckpointsExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_NV_DEVICE_DIAGNOSTIC_CHECKPOINTS_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && UE::RHI::UseGPUCrashDebugging() && UE::RHICore::AllowVendorDevice();
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		ExtensionFlags.HasNVDiagnosticCheckpoints = 1; 
	}
};



// ***** VK_NV_device_diagnostics_config (vendor)
class FVulkanNVDeviceDiagnosticConfigExtension : public FVulkanDeviceExtension
{
public:

	FVulkanNVDeviceDiagnosticConfigExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_NV_DEVICE_DIAGNOSTICS_CONFIG_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && UE::RHI::UseGPUCrashDebugging() && UE::RHICore::AllowVendorDevice();
	}


	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(DeviceDiagnosticsConfigFeaturesNV, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_DIAGNOSTICS_CONFIG_FEATURES_NV);
		AddToPNext(PhysicalDeviceFeatures2, DeviceDiagnosticsConfigFeaturesNV);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final 
	{
		bRequirementsPassed = (DeviceDiagnosticsConfigFeaturesNV.diagnosticsConfig == VK_TRUE);
		ExtensionFlags.HasNVDeviceDiagnosticConfig = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, DeviceDiagnosticsConfigFeaturesNV);

			ZeroVulkanStruct(DeviceDiagnosticsConfigCreateInfoNV, VK_STRUCTURE_TYPE_DEVICE_DIAGNOSTICS_CONFIG_CREATE_INFO_NV);
			DeviceDiagnosticsConfigCreateInfoNV.flags = VK_DEVICE_DIAGNOSTICS_CONFIG_ENABLE_SHADER_DEBUG_INFO_BIT_NV | VK_DEVICE_DIAGNOSTICS_CONFIG_ENABLE_RESOURCE_TRACKING_BIT_NV | VK_DEVICE_DIAGNOSTICS_CONFIG_ENABLE_AUTOMATIC_CHECKPOINTS_BIT_NV;
			AddToPNext(DeviceCreateInfo, DeviceDiagnosticsConfigCreateInfoNV);

#if NV_AFTERMATH
			UE::RHICore::Nvidia::Aftermath::InitializeBeforeDeviceCreation();

			extern void AftermathLateAssociate(float TimeLimitSeconds, uint32 FrameLimit);
			UE::RHICore::Nvidia::Aftermath::SetLateShaderAssociateCallback(AftermathLateAssociate);
#endif // NV_AFTERMATH
		}
	}

private:
	VkPhysicalDeviceDiagnosticsConfigFeaturesNV DeviceDiagnosticsConfigFeaturesNV;
	VkDeviceDiagnosticsConfigCreateInfoNV DeviceDiagnosticsConfigCreateInfoNV;
};

// ***** VK_NV_ray_tracing_validation (vendor)
class FVulkanNVRayTracingValidationExtension : public FVulkanDeviceExtension
{
public:

	FVulkanNVRayTracingValidationExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_NV_RAY_TRACING_VALIDATION_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
#if VULKAN_HAS_DEBUGGING_ENABLED
		bEnabledInCode = bEnabledInCode && (GValidationCvar.GetValueOnAnyThread() > 0) && UE::RHICore::AllowVendorDevice();
#else
		bEnabledInCode = false;
#endif
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(RayTracingValidationFeaturesNV, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_RAY_TRACING_VALIDATION_FEATURES_NV);
		AddToPNext(PhysicalDeviceFeatures2, RayTracingValidationFeaturesNV);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (RayTracingValidationFeaturesNV.rayTracingValidation == VK_TRUE);
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, RayTracingValidationFeaturesNV);
		}
	}

private:
	VkPhysicalDeviceRayTracingValidationFeaturesNV RayTracingValidationFeaturesNV;
};

// ***** VK_NV_ray_tracing_linear_swept_spheres (vendor)
class FVulkanNVRayTracingLinearSweptSphereExtension : public FVulkanDeviceExtension
{
public:

	FVulkanNVRayTracingLinearSweptSphereExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_NV_RAY_TRACING_LINEAR_SWEPT_SPHERES_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && UE::RHICore::AllowVendorDevice();
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(LinearSweptSphereFeaturesNV, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_RAY_TRACING_LINEAR_SWEPT_SPHERES_FEATURES_NV);
		AddToPNext(PhysicalDeviceFeatures2, LinearSweptSphereFeaturesNV);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (LinearSweptSphereFeaturesNV.linearSweptSpheres == VK_TRUE);
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		GRHIGlobals.RayTracing.SupportsLinearSweptSpheres = false;
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, LinearSweptSphereFeaturesNV);

			GRHIGlobals.RayTracing.SupportsLinearSweptSpheres = (LinearSweptSphereFeaturesNV.linearSweptSpheres == VK_TRUE);
		}
	}

private:
	VkPhysicalDeviceRayTracingLinearSweptSpheresFeaturesNV LinearSweptSphereFeaturesNV;
};

// ***** VK_EXT_device_fault
class FVulkanEXTDeviceFaultExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTDeviceFaultExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_DEVICE_FAULT_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(DeviceFaultFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_FAULT_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, DeviceFaultFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (DeviceFaultFeatures.deviceFault == VK_TRUE);
		ExtensionFlags.HasEXTDeviceFault = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, DeviceFaultFeatures);
		}
	}

private:
	VkPhysicalDeviceFaultFeaturesEXT DeviceFaultFeatures;
};

// ***** VK_EXT_device_address_binding_report
class FVulkanEXTDeviceAddressBindingReportExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTDeviceAddressBindingReportExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_DEVICE_ADDRESS_BINDING_REPORT_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && UE::RHI::UseGPUCrashDebugging() && GVulkanAllowAddressBindingReport.GetValueOnAnyThread();
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(DeviceAddressBindingReportFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_ADDRESS_BINDING_REPORT_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, DeviceAddressBindingReportFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (DeviceAddressBindingReportFeatures.reportAddressBinding == VK_TRUE);
		ExtensionFlags.HasEXTDeviceAddressBindingReport = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, DeviceAddressBindingReportFeatures);
		}
	}

private:
	VkPhysicalDeviceAddressBindingReportFeaturesEXT DeviceAddressBindingReportFeatures;
};

// ***** VK_EXT_host_query_reset
class FVulkanEXTHostQueryResetExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTHostQueryResetExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_HOST_QUERY_RESET_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_2)
	{
		bEnabledInCode = bEnabledInCode && (GVulkanAllowHostQueryResetCVar.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(HostQueryResetFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_HOST_QUERY_RESET_FEATURES);
		AddToPNext(PhysicalDeviceFeatures2, HostQueryResetFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (HostQueryResetFeatures.hostQueryReset == VK_TRUE);
		ExtensionFlags.HasEXTHostQueryReset = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, HostQueryResetFeatures);
		}
	}

private:
	VkPhysicalDeviceHostQueryResetFeaturesEXT HostQueryResetFeatures;
};

// ***** VK_EXT_subgroup_size_control
class FVulkanEXTSubgroupSizeControlExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTSubgroupSizeControlExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_SUBGROUP_SIZE_CONTROL_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_3)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(SubgroupSizeControlFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_SUBGROUP_SIZE_CONTROL_FEATURES);
		AddToPNext(PhysicalDeviceFeatures2, SubgroupSizeControlFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (SubgroupSizeControlFeatures.subgroupSizeControl == VK_TRUE);
		ExtensionFlags.HasEXTSubgroupSizeControl = bRequirementsPassed;
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		if (bRequirementsPassed)
		{
			VkPhysicalDeviceSubgroupSizeControlPropertiesEXT& SubgroupSizeControlProperties = GetDeviceExtensionProperties().SubgroupSizeControlProperties;
			ZeroVulkanStruct(SubgroupSizeControlProperties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_SUBGROUP_SIZE_CONTROL_PROPERTIES);
			AddToPNext(PhysicalDeviceProperties2, SubgroupSizeControlProperties);
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			VkPhysicalDeviceSubgroupSizeControlPropertiesEXT& SubgroupSizeControlProperties = GetDeviceExtensionProperties().SubgroupSizeControlProperties;

			GRHIMinimumWaveSize = SubgroupSizeControlProperties.minSubgroupSize;
			GRHIMaximumWaveSize = SubgroupSizeControlProperties.maxSubgroupSize;

			AddToPNext(DeviceCreateInfo, SubgroupSizeControlFeatures);
		}
	}

private:
	VkPhysicalDeviceSubgroupSizeControlFeaturesEXT SubgroupSizeControlFeatures;
};


// ***** VK_EXT_shader_demote_to_helper_invocation
class FVulkanEXTShaderDemoteToHelperInvocationExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTShaderDemoteToHelperInvocationExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_SHADER_DEMOTE_TO_HELPER_INVOCATION_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_3)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(ShaderDemoteToHelperInvocationFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_SHADER_DEMOTE_TO_HELPER_INVOCATION_FEATURES);
		AddToPNext(PhysicalDeviceFeatures2, ShaderDemoteToHelperInvocationFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (ShaderDemoteToHelperInvocationFeatures.shaderDemoteToHelperInvocation == VK_TRUE);
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, ShaderDemoteToHelperInvocationFeatures);
		}
	}

private:
	VkPhysicalDeviceShaderDemoteToHelperInvocationFeaturesEXT ShaderDemoteToHelperInvocationFeatures;
};


// ***** VK_EXT_calibrated_timestamps
class FVulkanEXTCalibratedTimestampsExtension : public FVulkanDeviceExtension
{
public:
	FVulkanEXTCalibratedTimestampsExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_CALIBRATED_TIMESTAMPS_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED)
	{
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		uint32 TimeDomainCount = 0;
		VulkanRHI::vkGetPhysicalDeviceCalibrateableTimeDomainsEXT(Device->GetPhysicalHandle(), &TimeDomainCount, nullptr);

		TArray<VkTimeDomainEXT, TInlineAllocator<4>> TimeDomains;
		TimeDomains.SetNumZeroed(TimeDomainCount);
		VulkanRHI::vkGetPhysicalDeviceCalibrateableTimeDomainsEXT(Device->GetPhysicalHandle(), &TimeDomainCount, &TimeDomains[0]);

		for (VkTimeDomainEXT TimeDomain : TimeDomains)
		{
			if (TimeDomain == VK_TIME_DOMAIN_DEVICE_EXT)
			{
				ExtensionFlags.HasEXTCalibratedTimestamps = 1;
				break;
			}
		}
	}
};


// ***** VK_EXT_descriptor_buffer
class FVulkanEXTDescriptorBuffer : public FVulkanDeviceExtension
{
public:
	FVulkanEXTDescriptorBuffer(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_DESCRIPTOR_BUFFER_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED)
	{
		// Sync2 is a prereq
		bEnabledInCode = bEnabledInCode && (GVulkanAllowSync2BarriersCVar.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(DescriptorBufferFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_DESCRIPTOR_BUFFER_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, DescriptorBufferFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		// only enable descriptor buffers if we also support mutable descriptor types (value filled prior)
		bRequirementsPassed = (DescriptorBufferFeatures.descriptorBuffer == VK_TRUE);
		ExtensionFlags.HasEXTDescriptorBuffer = bRequirementsPassed;
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		if (bRequirementsPassed)
		{
			VkPhysicalDeviceDescriptorBufferPropertiesEXT& DescriptorBufferProperties = GetDeviceExtensionProperties().DescriptorBufferProps;
			ZeroVulkanStruct(DescriptorBufferProperties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_DESCRIPTOR_BUFFER_PROPERTIES_EXT);
			AddToPNext(PhysicalDeviceProperties2, DescriptorBufferProperties);
		}
	}

	virtual void PostPhysicalDeviceProperties(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		if (bRequirementsPassed)
		{
			const VkPhysicalDeviceDescriptorBufferPropertiesEXT& DescriptorBufferProperties = GetDeviceExtensionProperties().DescriptorBufferProps;

			if (DescriptorBufferProperties.bufferlessPushDescriptors == VK_FALSE)
			{
				UE_LOGF(LogVulkanRHI, Display, "Disabling Vulkan Descriptor Buffers because bufferless isn't supported.");

				ExtensionFlags.HasEXTDescriptorBuffer = false;
			}
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			const VkPhysicalDeviceDescriptorBufferPropertiesEXT& DescriptorBufferProperties = GetDeviceExtensionProperties().DescriptorBufferProps;

			UE_LOGF(LogVulkanRHI, Display, "Enabling Vulkan Descriptor Buffers with: "
				"allowSamplerImageViewPostSubmitCreation=%u, maxDescriptorBufferBindings=%u, "
				"maxSamplerDescriptorBufferBindings=%u, maxResourceDescriptorBufferBindings=%u, "
				"samplerDescriptorBufferAddressSpaceSize=%llu, resourceDescriptorBufferAddressSpaceSize=%llu, "
				"maxSamplerDescriptorBufferRange=%llu, maxResourceDescriptorBufferRange=%llu, "
				"descriptorBufferAddressSpaceSize=%llu, descriptorBufferOffsetAlignment=%llu, "
				"samplerDescriptorSize=%llu",
				DescriptorBufferProperties.allowSamplerImageViewPostSubmitCreation, DescriptorBufferProperties.maxDescriptorBufferBindings,
				DescriptorBufferProperties.maxSamplerDescriptorBufferBindings, DescriptorBufferProperties.maxResourceDescriptorBufferBindings,
				DescriptorBufferProperties.samplerDescriptorBufferAddressSpaceSize, DescriptorBufferProperties.resourceDescriptorBufferAddressSpaceSize,
				DescriptorBufferProperties.maxSamplerDescriptorBufferRange, DescriptorBufferProperties.maxResourceDescriptorBufferRange,
				DescriptorBufferProperties.descriptorBufferAddressSpaceSize, DescriptorBufferProperties.descriptorBufferOffsetAlignment,
				DescriptorBufferProperties.samplerDescriptorSize);

			AddToPNext(DeviceCreateInfo, DescriptorBufferFeatures);
		}
	}

private:
	VkPhysicalDeviceDescriptorBufferFeaturesEXT DescriptorBufferFeatures;
};


// ***** VK_KHR_shader_untyped_pointers
class FVulkanKHRShaderUntypedPointersExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRShaderUntypedPointersExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_SHADER_UNTYPED_POINTERS_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(ShaderUntypedPointersFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_SHADER_UNTYPED_POINTERS_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, ShaderUntypedPointersFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = ShaderUntypedPointersFeatures.shaderUntypedPointers == VK_TRUE;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, ShaderUntypedPointersFeatures);
		}
	}

	VkPhysicalDeviceShaderUntypedPointersFeaturesKHR ShaderUntypedPointersFeatures;
};


// ***** VK_EXT_descriptor_heap
class FVulkanEXTDescriptorHeapExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTDescriptorHeapExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_DESCRIPTOR_HEAP_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(DescriptorHeapFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_DESCRIPTOR_HEAP_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, DescriptorHeapFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = DescriptorHeapFeatures.descriptorHeap == VK_TRUE;
		ExtensionFlags.HasEXTDescriptorHeap = bRequirementsPassed;
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		if (bRequirementsPassed)
		{
			VkPhysicalDeviceDescriptorHeapPropertiesEXT& DescriptorHeapProperties = GetDeviceExtensionProperties().DescriptorHeapProps;
			ZeroVulkanStruct(DescriptorHeapProperties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_DESCRIPTOR_HEAP_PROPERTIES_EXT);
			AddToPNext(PhysicalDeviceProperties2, DescriptorHeapProperties);
		}
	}

	virtual void PostPhysicalDeviceProperties(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		if (bRequirementsPassed)
		{
			const VkPhysicalDeviceDescriptorHeapPropertiesEXT& DescriptorBufferProperties = GetDeviceExtensionProperties().DescriptorHeapProps;
			UE_LOGF(LogVulkanRHI, Display, "Descriptor Heap is supported with following properties: "
				"maxSamplerHeapSize=%llu, maxResourceHeapSize=%llu, maxPushDataSize=%llu, "
				"samplerDescriptorAlignment=%llu, imageDescriptorAlignment=%llu, bufferDescriptorAlignment=%llu, "
				"samplerDescriptorSize=%llu, imageDescriptorSize=%llu, bufferDescriptorSize=%llu, ",
				DescriptorBufferProperties.maxSamplerHeapSize, DescriptorBufferProperties.maxResourceHeapSize, DescriptorBufferProperties.maxPushDataSize,
				DescriptorBufferProperties.samplerDescriptorAlignment, DescriptorBufferProperties.imageDescriptorAlignment, DescriptorBufferProperties.bufferDescriptorAlignment,
				DescriptorBufferProperties.samplerDescriptorSize, DescriptorBufferProperties.imageDescriptorSize, DescriptorBufferProperties.bufferDescriptorSize
				);
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, DescriptorHeapFeatures);
		}
	}

	VkPhysicalDeviceDescriptorHeapFeaturesEXT DescriptorHeapFeatures;
};


// ***** VK_KHR_16bit_storage
class FVulkanKHR16BitStorageExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHR16BitStorageExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_16BIT_STORAGE_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && (GRHIAllow16bitOps.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(Device16BitStorageFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_16BIT_STORAGE_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, Device16BitStorageFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (
			Device16BitStorageFeatures.storageBuffer16BitAccess == VK_TRUE &&
			Device16BitStorageFeatures.uniformAndStorageBuffer16BitAccess == VK_TRUE &&
			Device16BitStorageFeatures.storagePushConstant16 == VK_TRUE);
		ExtensionFlags.HasKHR16bitStorage = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, Device16BitStorageFeatures);
		}
	}

	VkPhysicalDevice16BitStorageFeaturesKHR Device16BitStorageFeatures;
};

// ***** VK_KHR_shader_float16_int8
class FVulkanKHRShaderFloat16Int8Extension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRShaderFloat16Int8Extension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_SHADER_FLOAT16_INT8_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && (GRHIAllow16bitOps.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(ShaderFloat16Int8Features, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_SHADER_FLOAT16_INT8_FEATURES_KHR);
		AddToPNext(PhysicalDeviceFeatures2, ShaderFloat16Int8Features);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (ShaderFloat16Int8Features.shaderFloat16 == VK_TRUE);
		ExtensionFlags.HasKHRShaderFloat16 = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		// The PreCreateDevice() call is after all extension have gone through PostPhysicalDeviceFeatures(), so ExtensionFlags will be filled for both.
		const FOptionalVulkanDeviceExtensions& ExtensionFlags = Device->GetOptionalExtensions();
		GRHIGlobals.SupportsNative16BitOps = ExtensionFlags.HasKHR16bitStorage && ExtensionFlags.HasKHRShaderFloat16 &&
			(FDataDrivenShaderPlatformInfo::GetSupportsRealTypes(GMaxRHIShaderPlatform) != ERHIFeatureSupport::Unsupported);

		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, ShaderFloat16Int8Features);
		}
	}

	VkPhysicalDeviceShaderFloat16Int8Features  ShaderFloat16Int8Features;
};

// ***** VK_KHR_depth_stencil_resolve
class FVulkanKHRDepthStencilResolveExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRDepthStencilResolveExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_DEPTH_STENCIL_RESOLVE_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		ExtensionFlags.HasKHRDepthStencilResolve = 1;
		GRHISupportsDepthStencilResolve = ExtensionFlags.HasKHRDepthStencilResolve;
	}
};

// ***** VK_EXT_image_robustness
class FVulkanEXTImageRobustnessExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTImageRobustnessExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_IMAGE_ROBUSTNESS_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && (GVulkanAllowImageRobustness.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(ImageRobustnessFeature, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_IMAGE_ROBUSTNESS_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, ImageRobustnessFeature);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (ImageRobustnessFeature.robustImageAccess == VK_TRUE);
		ExtensionFlags.HasEXTImageRobustness = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, ImageRobustnessFeature);
		}
	}

	VkPhysicalDeviceImageRobustnessFeaturesEXT ImageRobustnessFeature;
};

// ***** VK_EXT_pipeline_creation_cache_control
class FVulkanEXTPipelineCreationCacheControlExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTPipelineCreationCacheControlExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_PIPELINE_CREATION_CACHE_CONTROL_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_3)
	{}
	
	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(PhysicalDevicePipelineCreationCacheControlFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_PIPELINE_CREATION_CACHE_CONTROL_FEATURES);
		AddToPNext(PhysicalDeviceFeatures2, PhysicalDevicePipelineCreationCacheControlFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (PhysicalDevicePipelineCreationCacheControlFeatures.pipelineCreationCacheControl == VK_TRUE);
		ExtensionFlags.HasEXTPipelineCreationCacheControl = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, PhysicalDevicePipelineCreationCacheControlFeatures);
		}
	}

	VkPhysicalDevicePipelineCreationCacheControlFeatures PhysicalDevicePipelineCreationCacheControlFeatures;
};

// ***** VK_KHR_sampler_ycbcr_conversion
class FVulkanKHRSamplerYcbcrConversionExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRSamplerYcbcrConversionExtension(FVulkanDevice* InDevice) 
		: FVulkanDeviceExtension(InDevice, VK_KHR_SAMPLER_YCBCR_CONVERSION_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_1)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(PhysicalDeviceSamplerYcbcrConversionFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_SAMPLER_YCBCR_CONVERSION_FEATURES);
		AddToPNext(PhysicalDeviceFeatures2, PhysicalDeviceSamplerYcbcrConversionFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (PhysicalDeviceSamplerYcbcrConversionFeatures.samplerYcbcrConversion == VK_TRUE);
		ExtensionFlags.HasKHRSamplerYcbcrConversion = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, PhysicalDeviceSamplerYcbcrConversionFeatures);
		}
	}


private:
	VkPhysicalDeviceSamplerYcbcrConversionFeatures PhysicalDeviceSamplerYcbcrConversionFeatures;
};

// ***** VK_EXT_image_compression_control
class FVulkanEXTImageCompressionControlExtension : public FVulkanDeviceExtension
{
public:
	FVulkanEXTImageCompressionControlExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_IMAGE_COMPRESSION_CONTROL_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(PhysicalDeviceImageCompressionControlFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_IMAGE_COMPRESSION_CONTROL_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, PhysicalDeviceImageCompressionControlFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (PhysicalDeviceImageCompressionControlFeatures.imageCompressionControl == VK_TRUE);
		ExtensionFlags.HasEXTImageCompressionControl = bRequirementsPassed;
		if (ExtensionFlags.HasEXTImageCompressionControl && !ExtensionFlags.HasKHRMaintenance5 && !VulkanDynamicAPI::vkGetImageSubresourceLayout2KHR)
		{
			VulkanDynamicAPI::vkGetImageSubresourceLayout2KHR = VulkanDynamicAPI::vkGetImageSubresourceLayout2EXT;
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, PhysicalDeviceImageCompressionControlFeatures);
		}
	}


private:
	VkPhysicalDeviceImageCompressionControlFeaturesEXT PhysicalDeviceImageCompressionControlFeatures;
};

// ***** VK_EXT_graphics_pipeline_library
class FVulkanEXTGraphicsPipelineLibraryExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTGraphicsPipelineLibraryExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_GRAPHICS_PIPELINE_LIBRARY_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && (GVulkanAllowGraphicPipelineLibraryCVar.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(GraphicsPipelineLibraryFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_GRAPHICS_PIPELINE_LIBRARY_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, GraphicsPipelineLibraryFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (GraphicsPipelineLibraryFeatures.graphicsPipelineLibrary == VK_TRUE);
		ExtensionFlags.HasEXTGraphicsPipelineLibrary = bRequirementsPassed;
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		if (bRequirementsPassed)
		{
			VkPhysicalDeviceGraphicsPipelineLibraryPropertiesEXT& GraphicsPipelineLibraryProperties = GetDeviceExtensionProperties().GraphicsPipelineLibraryProperties;
			ZeroVulkanStruct(GraphicsPipelineLibraryProperties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_GRAPHICS_PIPELINE_LIBRARY_PROPERTIES_EXT);
			AddToPNext(PhysicalDeviceProperties2, GraphicsPipelineLibraryProperties);
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, GraphicsPipelineLibraryFeatures);
		}
	}

	VkPhysicalDeviceGraphicsPipelineLibraryFeaturesEXT GraphicsPipelineLibraryFeatures;
};

// ***** VK_EXT_mutable_descriptor_type
class FVulkanEXTMutableDescriptorTypeExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTMutableDescriptorTypeExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_MUTABLE_DESCRIPTOR_TYPE_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(MutableDescriptorTypeFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_MUTABLE_DESCRIPTOR_TYPE_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, MutableDescriptorTypeFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (MutableDescriptorTypeFeatures.mutableDescriptorType == VK_TRUE);
		ExtensionFlags.HasEXTMutableDescriptorType = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, MutableDescriptorTypeFeatures);
		}
	}

	VkPhysicalDeviceMutableDescriptorTypeFeaturesEXT MutableDescriptorTypeFeatures;
};

// ***** VK_KHR_dynamic_rendering
class FVulkanKHRDynamicRenderingExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRDynamicRenderingExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_DYNAMIC_RENDERING_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_3)
	{
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(DynamicRenderingFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_DYNAMIC_RENDERING_FEATURES);
		AddToPNext(PhysicalDeviceFeatures2, DynamicRenderingFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (DynamicRenderingFeatures.dynamicRendering == VK_TRUE);

		// We still want to load the extension if dynamic rendering isn't used (it's a dependency for Maintenance5).
		if (GVulkanAllowDynamicRenderingCVar.GetValueOnAnyThread() != 0)
		{
			ExtensionFlags.HasKHRDynamicRendering = bRequirementsPassed;
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, DynamicRenderingFeatures);
		}
	}

	VkPhysicalDeviceDynamicRenderingFeaturesKHR DynamicRenderingFeatures;
};

// ***** VK_KHR_dynamic_rendering_local_read
class FVulkanKHRDynamicRenderingLocalReadExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRDynamicRenderingLocalReadExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_KHR_DYNAMIC_RENDERING_LOCAL_READ_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_4)
	{
		bEnabledInCode = bEnabledInCode && (GVulkanAllowDynamicRenderingCVar.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(DynamicRenderingLocalReadFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_DYNAMIC_RENDERING_LOCAL_READ_FEATURES);
		AddToPNext(PhysicalDeviceFeatures2, DynamicRenderingLocalReadFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		// Depends on the base dynamic rendering extension 
		if (ExtensionFlags.HasKHRDynamicRendering)
		{
			bRequirementsPassed = (DynamicRenderingLocalReadFeatures.dynamicRenderingLocalRead == VK_TRUE);
			ExtensionFlags.HasKHRDynamicRendering = bRequirementsPassed;
		}
		else
		{
			bRequirementsPassed = false;
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, DynamicRenderingLocalReadFeatures);
		}
	}

	VkPhysicalDeviceDynamicRenderingLocalReadFeatures DynamicRenderingLocalReadFeatures;
};

// ***** VK_EXT_dynamic_rendering_unused_attachments
class FVulkanKHRDynamicRenderingUnusedAttachmentsExtension : public FVulkanDeviceExtension
{
public:

	FVulkanKHRDynamicRenderingUnusedAttachmentsExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_DYNAMIC_RENDERING_UNUSED_ATTACHMENTS_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && (GVulkanAllowDynamicRenderingCVar.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(DynamicRenderingUnusedAttachmentsFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_DYNAMIC_RENDERING_UNUSED_ATTACHMENTS_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, DynamicRenderingUnusedAttachmentsFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		// Depends on the base dynamic rendering extension 
		if (ExtensionFlags.HasKHRDynamicRendering)
		{
			bRequirementsPassed = (DynamicRenderingUnusedAttachmentsFeatures.dynamicRenderingUnusedAttachments == VK_TRUE);
			ExtensionFlags.HasKHRDynamicRendering = bRequirementsPassed;
		}
		else
		{
			bRequirementsPassed = false;
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, DynamicRenderingUnusedAttachmentsFeatures);
		}
	}

	VkPhysicalDeviceDynamicRenderingUnusedAttachmentsFeaturesEXT DynamicRenderingUnusedAttachmentsFeatures;
};

// ***** VK_EXT_extended_dynamic_state
class FVulkanEXTExtendedDynamicStateExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTExtendedDynamicStateExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_EXTENDED_DYNAMIC_STATE_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_3)
	{
		bEnabledInCode = bEnabledInCode && ((GVulkanAllowDynamicStatesCVar.GetValueOnAnyThread() & 1) != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(ExtendedDynamicStateFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_EXTENDED_DYNAMIC_STATE_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, ExtendedDynamicStateFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (ExtendedDynamicStateFeatures.extendedDynamicState == VK_TRUE);
		ExtensionFlags.HasEXTExtendedDynamicState1 = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, ExtendedDynamicStateFeatures);
		}
	}

	VkPhysicalDeviceExtendedDynamicStateFeaturesEXT ExtendedDynamicStateFeatures;
};

// ***** VK_EXT_extended_dynamic_state2
class FVulkanEXTExtendedDynamicState2Extension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTExtendedDynamicState2Extension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_EXTENDED_DYNAMIC_STATE_2_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_3)
	{
		bEnabledInCode = bEnabledInCode && ((GVulkanAllowDynamicStatesCVar.GetValueOnAnyThread() & 2) != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(ExtendedDynamicState2Features, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_EXTENDED_DYNAMIC_STATE_2_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, ExtendedDynamicState2Features);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (ExtendedDynamicState2Features.extendedDynamicState2 == VK_TRUE);
		ExtensionFlags.HasEXTExtendedDynamicState2 = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, ExtendedDynamicState2Features);
		}
	}

	VkPhysicalDeviceExtendedDynamicState2FeaturesEXT ExtendedDynamicState2Features;
};

// ***** VK_EXT_extended_dynamic_state3
class FVulkanEXTExtendedDynamicState3Extension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTExtendedDynamicState3Extension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_EXTENDED_DYNAMIC_STATE_3_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && ((GVulkanAllowDynamicStatesCVar.GetValueOnAnyThread() & 4) != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(ExtendedDynamicState3Features, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_EXTENDED_DYNAMIC_STATE_3_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, ExtendedDynamicState3Features);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = 
			(ExtendedDynamicState3Features.extendedDynamicState3PolygonMode == VK_TRUE) &&
			(ExtendedDynamicState3Features.extendedDynamicState3ColorBlendEnable == VK_TRUE) &&
			(ExtendedDynamicState3Features.extendedDynamicState3ColorBlendEquation == VK_TRUE) &&
			(ExtendedDynamicState3Features.extendedDynamicState3ColorWriteMask == VK_TRUE) &&
			(ExtendedDynamicState3Features.extendedDynamicState3RasterizationSamples == VK_TRUE) &&
			(ExtendedDynamicState3Features.extendedDynamicState3AlphaToCoverageEnable == VK_TRUE);
		ExtensionFlags.HasEXTExtendedDynamicState3 = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, ExtendedDynamicState3Features);
		}
	}

	VkPhysicalDeviceExtendedDynamicState3FeaturesEXT ExtendedDynamicState3Features;
};

// ***** VK_EXT_vertex_input_dynamic_state
class FVulkanEXTVertexInputDynamicStateExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTVertexInputDynamicStateExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_VERTEX_INPUT_DYNAMIC_STATE_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && ((GVulkanAllowDynamicStatesCVar.GetValueOnAnyThread() & 8) != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(VertexInputDynamicStateFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_VERTEX_INPUT_DYNAMIC_STATE_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, VertexInputDynamicStateFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (VertexInputDynamicStateFeatures.vertexInputDynamicState == VK_TRUE);
		ExtensionFlags.HasEXTVertexInputDynamicState = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, VertexInputDynamicStateFeatures);
		}
	}

	VkPhysicalDeviceVertexInputDynamicStateFeaturesEXT VertexInputDynamicStateFeatures;
};

// ***** VK_EXT_shader_object
class FVulkanEXTShaderObjectExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTShaderObjectExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_SHADER_OBJECT_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && (GVulkanAllowShaderObjectCVar.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(ShaderObjectFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_SHADER_OBJECT_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, ShaderObjectFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (ShaderObjectFeatures.shaderObject == VK_TRUE);
		ExtensionFlags.HasEXTShaderObject = bRequirementsPassed;
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, ShaderObjectFeatures);
		}
	}

	VkPhysicalDeviceShaderObjectFeaturesEXT ShaderObjectFeatures;
};


// ***** VK_EXT_host_image_copy
class FVulkanEXTHostImageCopyExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTHostImageCopyExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_HOST_IMAGE_COPY_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_4)
	{
		bEnabledInCode = bEnabledInCode && (GVulkanAllowHostImageCopyCVar.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(HostImageCopyFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_HOST_IMAGE_COPY_FEATURES);
		AddToPNext(PhysicalDeviceFeatures2, HostImageCopyFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (HostImageCopyFeatures.hostImageCopy == VK_TRUE);
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		if (bRequirementsPassed)
		{
			ZeroVulkanStruct(HostImageCopyProperties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_HOST_IMAGE_COPY_PROPERTIES);
			AddToPNext(PhysicalDeviceProperties2, HostImageCopyProperties);
		}
	}

	virtual void PostPhysicalDeviceProperties(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = bRequirementsPassed && (HostImageCopyProperties.identicalMemoryTypeRequirements == VK_TRUE);
		ExtensionFlags.HasEXTHostImageCopy = bRequirementsPassed;
		if (bRequirementsPassed)
		{
			UE_LOGF(LogVulkanRHI, Display, "Host Image Copy Enabled (SrcCount=%d, DstCount=%d)", 
				HostImageCopyProperties.copySrcLayoutCount, HostImageCopyProperties.copyDstLayoutCount);
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, HostImageCopyFeatures);
		}
	}

	VkPhysicalDeviceHostImageCopyFeatures HostImageCopyFeatures;
	VkPhysicalDeviceHostImageCopyProperties HostImageCopyProperties;
};


// ***** VK_EXT_device_generated_commands
class FVulkanEXTDeviceGeneratedCommandsExtension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTDeviceGeneratedCommandsExtension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_DEVICE_GENERATED_COMMANDS_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VK_API_VERSION_1_4)
		, DeviceGeneratedCommandsProperties(GetDeviceExtensionProperties().DeviceGeneratedCommandsProperties)
	{
		bEnabledInCode = bEnabledInCode && FDataDrivenShaderPlatformInfo::GetSupportsShaderBundleDispatch(GMaxRHIShaderPlatform);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(DeviceGeneratedCommandsFeatures, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_DEVICE_GENERATED_COMMANDS_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, DeviceGeneratedCommandsFeatures);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = (DeviceGeneratedCommandsFeatures.deviceGeneratedCommands == VK_TRUE);
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		if (bRequirementsPassed)
		{
			ZeroVulkanStruct(DeviceGeneratedCommandsProperties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_DEVICE_GENERATED_COMMANDS_PROPERTIES_EXT);
			AddToPNext(PhysicalDeviceProperties2, DeviceGeneratedCommandsProperties);
		}
	}

	virtual void PostPhysicalDeviceProperties(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = bRequirementsPassed && (DeviceGeneratedCommandsProperties.maxIndirectCommandsTokenCount >= 32);
		ExtensionFlags.HasEXTDeviceGeneratedCommands = bRequirementsPassed;
		if (bRequirementsPassed)
		{
			UE_LOGF(LogVulkanRHI, Display, "Device Generated Commands Enabled (PipelineCount=%d, SequenceCount=%d)",
				DeviceGeneratedCommandsProperties.maxIndirectPipelineCount, DeviceGeneratedCommandsProperties.maxIndirectSequenceCount);

			GRHISupportsShaderBundleDispatch = true;
			GRHISupportsShaderBundleParallel = false;
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, DeviceGeneratedCommandsFeatures);
		}
	}

	VkPhysicalDeviceDeviceGeneratedCommandsFeaturesEXT DeviceGeneratedCommandsFeatures;
	VkPhysicalDeviceDeviceGeneratedCommandsPropertiesEXT& DeviceGeneratedCommandsProperties;
};


// ***** VK_EXT_robustness2
class FVulkanEXTRobustness2Extension : public FVulkanDeviceExtension
{
public:

	FVulkanEXTRobustness2Extension(FVulkanDevice* InDevice)
		: FVulkanDeviceExtension(InDevice, VK_EXT_ROBUSTNESS_2_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED)
	{
		bEnabledInCode = bEnabledInCode && (GVulkanAllowRobustness2CVar.GetValueOnAnyThread() != 0);
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		ZeroVulkanStruct(Robustness2Features, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_ROBUSTNESS_2_FEATURES_EXT);
		AddToPNext(PhysicalDeviceFeatures2, Robustness2Features);
	}

	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		bRequirementsPassed = Robustness2Features.robustBufferAccess2 || Robustness2Features.robustImageAccess2 || Robustness2Features.nullDescriptor;
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		if (bRequirementsPassed)
		{
			ZeroVulkanStruct(Robustness2Properties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_ROBUSTNESS_2_PROPERTIES_EXT);
			AddToPNext(PhysicalDeviceProperties2, Robustness2Properties);
		}
	}

	virtual void PostPhysicalDeviceProperties(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		if (bRequirementsPassed)
		{
			UE_LOGF(LogVulkanRHI, Display, "Robustness2 Enabled: Buffer=%d (Align SB/UB=%u/%u), Image=%d, NullDescriptor=%d",
				Robustness2Features.robustBufferAccess2, 
				(uint32)Robustness2Properties.robustStorageBufferAccessSizeAlignment, (uint32)Robustness2Properties.robustUniformBufferAccessSizeAlignment,
				Robustness2Features.robustImageAccess2, Robustness2Features.nullDescriptor);
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo) override final
	{
		if (bRequirementsPassed)
		{
			AddToPNext(DeviceCreateInfo, Robustness2Features);
		}
	}

	VkPhysicalDeviceRobustness2FeaturesEXT Robustness2Features;
	VkPhysicalDeviceRobustness2PropertiesEXT Robustness2Properties;
};



template <typename ExtensionType>
static void FlagExtensionSupport(const TArray<VkExtensionProperties>& ExtensionProperties, TArray<TUniquePtr<ExtensionType>>& UEExtensions, uint32 ApiVersion, const ANSICHAR* ExtensionTypeName)
{
	// Flag the extension support
	UE_LOGF(LogVulkanRHI, Display, "Found %d available %s extensions :", ExtensionProperties.Num(), ExtensionTypeName);
	for (const VkExtensionProperties& Extension : ExtensionProperties)
	{
		const int32 ExtensionIndex = ExtensionType::FindExtension(UEExtensions, Extension.extensionName);
		const bool bFound = (ExtensionIndex != INDEX_NONE);
		bool bIsCore = false;
		if (bFound)
		{
			UEExtensions[ExtensionIndex]->SetSupported();

			// Set the core flag if the extension was promoted for our current api version
			bIsCore = UEExtensions[ExtensionIndex]->SetCore(ApiVersion);
		}

		UE_LOGF(LogVulkanRHI, Display, "  %s %s", bIsCore ? "*" : bFound ? "+" : "-", Extension.extensionName);
	}
}


class FVulkanExternalDeviceExtensionWrapper : public FVulkanDeviceExtension
{
public: 
	FVulkanExternalDeviceExtensionWrapper(FVulkanDevice* InDevice, TSharedRef<FVulkanRHIExternalDeviceExtensionBase> InExternalDeviceExtensionBase)
		: FVulkanDeviceExtension(InDevice, *InExternalDeviceExtensionBase->ExtensionName, VULKAN_EXTENSION_ENABLED)
		, ExternalDeviceExtensionBase(InExternalDeviceExtensionBase)
	{
	}

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) override final
	{
		VkBaseOutStructure* PropertiesStruct = bRequirementsPassed ? ExternalDeviceExtensionBase->GetPropertiesStruct() : nullptr;
		if (PropertiesStruct)
		{
			PropertiesStruct->pNext = (VkBaseOutStructure*)PhysicalDeviceProperties2.pNext;
			PhysicalDeviceProperties2.pNext = PropertiesStruct;
		}
	}
	
	virtual void PostPhysicalDeviceProperties(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		if (bRequirementsPassed)
		{
			bRequirementsPassed = ExternalDeviceExtensionBase->PropertiesCallback();
		}
	}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) override final
	{
		VkBaseOutStructure* FeaturesStruct = bRequirementsPassed ? ExternalDeviceExtensionBase->GetFeaturesStruct() : nullptr;
		if (FeaturesStruct)
		{
			FeaturesStruct->pNext = (VkBaseOutStructure*)PhysicalDeviceFeatures2.pNext;
			PhysicalDeviceFeatures2.pNext = FeaturesStruct;
		}
	}
	
	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags) override final
	{
		if (bRequirementsPassed)
		{
			bRequirementsPassed = ExternalDeviceExtensionBase->FeaturesCallback();
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceCreateInfo)  override final
	{
		VkBaseOutStructure* FeaturesStruct = bRequirementsPassed ? ExternalDeviceExtensionBase->GetFeaturesStruct() : nullptr;
		if (FeaturesStruct)
		{
			FeaturesStruct->pNext = (VkBaseOutStructure*)DeviceCreateInfo.pNext;
			DeviceCreateInfo.pNext = FeaturesStruct;
		}
	}

private:
	TSharedRef<FVulkanRHIExternalDeviceExtensionBase> ExternalDeviceExtensionBase;
};



FVulkanDeviceExtensionArray FVulkanDeviceExtension::GetUESupportedDeviceExtensions(FVulkanDevice* InDevice, uint32 ApiVersion)
{
	FVulkanDeviceExtensionArray OutUEDeviceExtensions;

	#define ADD_SIMPLE_EXTENSION(EXTENSION_NAME, ENABLED_IN_CODE, PROMOTED_VER, FLAG_SETTER) \
		OutUEDeviceExtensions.Add(MakeUnique<FVulkanDeviceExtension>(InDevice, EXTENSION_NAME, ENABLED_IN_CODE, PROMOTED_VER, FLAG_SETTER, FVulkanExtensionBase::AutoActivate))

	#define ADD_EXTERNAL_EXTENSION(EXTENSION_NAME, ENABLED_IN_CODE, PROMOTED_VER, FLAG_SETTER) \
		OutUEDeviceExtensions.Add(MakeUnique<FVulkanDeviceExtension>(InDevice, EXTENSION_NAME, ENABLED_IN_CODE, PROMOTED_VER, FLAG_SETTER, FVulkanExtensionBase::ManuallyActivate))

	#define ADD_CUSTOM_EXTENSION(EXTENSION_CLASS) \
		OutUEDeviceExtensions.Add(MakeUnique<EXTENSION_CLASS>(InDevice));


	// Generic simple extensions :

	ADD_SIMPLE_EXTENSION(VK_KHR_SWAPCHAIN_EXTENSION_NAME,                VULKAN_EXTENSION_ENABLED,             VULKAN_EXTENSION_NOT_PROMOTED, nullptr);
	ADD_SIMPLE_EXTENSION(VK_EXT_MEMORY_BUDGET_EXTENSION_NAME,            VULKAN_SUPPORTS_MEMORY_BUDGET,        VULKAN_EXTENSION_NOT_PROMOTED, DEVICE_EXT_FLAG_SETTER(HasMemoryBudget));
	ADD_SIMPLE_EXTENSION(VK_EXT_MEMORY_PRIORITY_EXTENSION_NAME,          VULKAN_SUPPORTS_MEMORY_PRIORITY,      VULKAN_EXTENSION_NOT_PROMOTED, DEVICE_EXT_FLAG_SETTER(HasMemoryPriority));
	ADD_SIMPLE_EXTENSION(VK_KHR_CREATE_RENDERPASS_2_EXTENSION_NAME,      VULKAN_SUPPORTS_RENDERPASS2,          VK_API_VERSION_1_2,            DEVICE_EXT_FLAG_SETTER(HasKHRRenderPass2));
	ADD_SIMPLE_EXTENSION(VK_KHR_DEFERRED_HOST_OPERATIONS_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED,             VULKAN_EXTENSION_NOT_PROMOTED, DEVICE_EXT_FLAG_SETTER(HasDeferredHostOperations));
	ADD_SIMPLE_EXTENSION(VK_KHR_SPIRV_1_4_EXTENSION_NAME,				 VULKAN_EXTENSION_ENABLED,             VK_API_VERSION_1_2,            DEVICE_EXT_FLAG_SETTER(HasSPIRV_14));
	ADD_SIMPLE_EXTENSION(VK_KHR_SHADER_FLOAT_CONTROLS_EXTENSION_NAME,	 VULKAN_EXTENSION_ENABLED,             VK_API_VERSION_1_2,            DEVICE_EXT_FLAG_SETTER(HasShaderFloatControls));
	ADD_SIMPLE_EXTENSION(VK_KHR_IMAGE_FORMAT_LIST_EXTENSION_NAME,        VULKAN_EXTENSION_ENABLED,             VK_API_VERSION_1_2,            DEVICE_EXT_FLAG_SETTER(HasKHRImageFormatList));
	ADD_SIMPLE_EXTENSION(VK_EXT_VALIDATION_CACHE_EXTENSION_NAME,         VULKAN_SUPPORTS_VALIDATION_CACHE,     VULKAN_EXTENSION_NOT_PROMOTED, DEVICE_EXT_FLAG_SETTER(HasEXTValidationCache));
	ADD_SIMPLE_EXTENSION(VK_EXT_LOAD_STORE_OP_NONE_EXTENSION_NAME,       VULKAN_EXTENSION_ENABLED,             VK_API_VERSION_1_4,            DEVICE_EXT_FLAG_SETTER(HasEXTLoadStoreOpNone));
	ADD_SIMPLE_EXTENSION(VK_QCOM_RENDER_PASS_SHADER_RESOLVE_EXTENSION_NAME, VULKAN_SUPPORTS_QCOM_RENDERPASS_SHADER_RESOLVE, VULKAN_EXTENSION_NOT_PROMOTED, DEVICE_EXT_FLAG_SETTER(HasQcomRenderPassShaderResolve));
	ADD_SIMPLE_EXTENSION(VK_EXT_TOOLING_INFO_EXTENSION_NAME, VULKAN_HAS_DEBUGGING_ENABLED || VULKAN_ENABLE_DRAW_MARKERS, VK_API_VERSION_1_3,  DEVICE_EXT_FLAG_SETTER(HasEXTToolingInfo));
	ADD_SIMPLE_EXTENSION(VK_KHR_PIPELINE_LIBRARY_EXTENSION_NAME,         VULKAN_EXTENSION_ENABLED,             VULKAN_EXTENSION_NOT_PROMOTED, DEVICE_EXT_FLAG_SETTER(HasKHRPipelineLibrary));
	ADD_SIMPLE_EXTENSION(VK_KHR_PUSH_DESCRIPTOR_EXTENSION_NAME,          VULKAN_EXTENSION_ENABLED,             VK_API_VERSION_1_4,            DEVICE_EXT_FLAG_SETTER(HasKHRPushDescriptors));

	// Externally activated extensions (supported by the engine, but enabled externally by plugin or other) :
	ADD_EXTERNAL_EXTENSION(VK_EXT_DEBUG_MARKER_EXTENSION_NAME, VULKAN_HAS_DEBUGGING_ENABLED || VULKAN_ENABLE_DRAW_MARKERS, VULKAN_EXTENSION_NOT_PROMOTED, nullptr);

	// Extensions with custom classes :

	ADD_CUSTOM_EXTENSION(FVulkanKHRDriverPropertiesExtension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRMaintenance4Extension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRMaintenance5Extension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRMaintenance6Extension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRMaintenance7Extension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRMaintenance9Extension);
	ADD_CUSTOM_EXTENSION(FVulkanShaderAtomicInt64Extension);
	ADD_CUSTOM_EXTENSION(FVulkanShaderImageAtomicInt64Extension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTScalarBlockLayoutExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTShaderViewportIndexLayerExtension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRSeparateDepthStencilLayoutsExtension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRSynchronization2);
	ADD_CUSTOM_EXTENSION(FVulkanKHRFragmentShadingRateExtension); // must be kept BEFORE DensityMap!
	ADD_CUSTOM_EXTENSION(FVulkanEXTFragmentDensityMapExtension);  // must be kept AFTER ShadingRate!
	ADD_CUSTOM_EXTENSION(FVulkanEXTFragmentDensityMap2Extension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTFragmentDensityMapOffsetExtension)
	ADD_CUSTOM_EXTENSION(FVulkanKHRMultiviewExtension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRGetMemoryRequirements2Extension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTDescriptorIndexingExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTHostQueryResetExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTSubgroupSizeControlExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTCalibratedTimestampsExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTDescriptorBuffer);
	ADD_CUSTOM_EXTENSION(FVulkanEXTDeviceFaultExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTDeviceAddressBindingReportExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTShaderDemoteToHelperInvocationExtension);
	ADD_CUSTOM_EXTENSION(FVulkanKHR16BitStorageExtension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRShaderFloat16Int8Extension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRDepthStencilResolveExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTImageRobustnessExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTPipelineCreationCacheControlExtension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRFragmentShaderBarycentricExtension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRComputeShaderDerivatives);
	ADD_CUSTOM_EXTENSION(FVulkanKHRSamplerYcbcrConversionExtension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRTimelineSemaphoreExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTMeshShaderExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTImageCompressionControlExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTGraphicsPipelineLibraryExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTMutableDescriptorTypeExtension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRDynamicRenderingExtension);			// must be kept BEFORE DynamicRenderingLocalRead!
	ADD_CUSTOM_EXTENSION(FVulkanKHRDynamicRenderingLocalReadExtension);	// must be kept AFTER DynamicRendering!
	ADD_CUSTOM_EXTENSION(FVulkanKHRDynamicRenderingUnusedAttachmentsExtension);	// must be kept AFTER DynamicRendering!
	ADD_CUSTOM_EXTENSION(FVulkanEXTExtendedDynamicStateExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTExtendedDynamicState2Extension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTExtendedDynamicState3Extension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTVertexInputDynamicStateExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTShaderObjectExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTHostImageCopyExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTDeviceGeneratedCommandsExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTRobustness2Extension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRShaderUntypedPointersExtension);
	ADD_CUSTOM_EXTENSION(FVulkanEXTDescriptorHeapExtension);

	// Needed for Raytracing
	ADD_CUSTOM_EXTENSION(FVulkanKHRBufferDeviceAddressExtension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRAccelerationStructureExtension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRRayTracingPipelineExtension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRRayQueryExtension);
	ADD_CUSTOM_EXTENSION(FVulkanKHRRayTracingPositionFetchExtension);

	// Vendor extensions
	ADD_CUSTOM_EXTENSION(FVulkanAMDBufferMarkerExtension);
	ADD_CUSTOM_EXTENSION(FVulkanNVDeviceDiagnosticCheckpointsExtension);
	ADD_CUSTOM_EXTENSION(FVulkanNVDeviceDiagnosticConfigExtension);
	ADD_CUSTOM_EXTENSION(FVulkanNVRayTracingValidationExtension);
	ADD_CUSTOM_EXTENSION(FVulkanNVRayTracingLinearSweptSphereExtension);

	// Add in platform specific extensions
	FVulkanPlatform::GetDeviceExtensions(InDevice, OutUEDeviceExtensions);

	// Merge extensions from outside the RHI into the main array using the wrapper
	for (TSharedRef<FVulkanRHIExternalDeviceExtensionBase>& ExtensionBase : ExternalExtensions)
	{
		const int32 ExtensionIndex = FindExtension(OutUEDeviceExtensions, *ExtensionBase->ExtensionName);
		if (ExtensionIndex == INDEX_NONE)
		{
			OutUEDeviceExtensions.Add(MakeUnique<FVulkanExternalDeviceExtensionWrapper>(InDevice, ExtensionBase));
			UE_LOGF(LogVulkanRHI, Warning, "Adding externally requested extension: %s", *ExtensionBase->ExtensionName);
		}
		else
		{
			OutUEDeviceExtensions[ExtensionIndex]->SetActivated();
			ExtensionBase->DuplicateCallback();
		}
	}

	// Release the external extensions array, the device extensions are holding the references now
	FVulkanDeviceExtension::ExternalExtensions.Empty();

	// Add HMD requested extensions
	{
		if (IHeadMountedDisplayModule::IsAvailable())
		{
			FVulkanDynamicRHI::HMDVulkanExtensions = IHeadMountedDisplayModule::Get().GetVulkanExtensions();
		}
		if (FVulkanDynamicRHI::HMDVulkanExtensions.IsValid())
		{
			TArray<const ANSICHAR*> HMDExtensions;
			FVulkanDynamicRHI::HMDVulkanExtensions->GetVulkanDeviceExtensionsRequired(InDevice->GetPhysicalHandle(), HMDExtensions);

			for (const ANSICHAR* ExtensionName : HMDExtensions)
			{
				const int32 ExtensionIndex = FindExtension(OutUEDeviceExtensions, ExtensionName);
				if (ExtensionIndex == INDEX_NONE)
				{
					OutUEDeviceExtensions.Add(MakeUnique<FVulkanExternalDeviceExtensionWrapper>(InDevice, MakeShared<FVulkanRHIExternalDeviceExtensionBase>(ExtensionName)));
					UE_LOGF(LogVulkanRHI, Warning, "Adding HMD requested extension: %s", ExtensionName);
				}
			}
		}
	}

	// Now that all the extensions are listed, update their support flags
	FlagExtensionSupport(GetDriverSupportedDeviceExtensions(InDevice->GetPhysicalHandle()), OutUEDeviceExtensions, ApiVersion, "device");

	#undef ADD_SIMPLE_EXTENSION
	#undef ADD_EXTERNAL_EXTENSION
	#undef ADD_CUSTOM_EXTENSION

	return OutUEDeviceExtensions;
}











// *** Vulkan Instance Extension support ***
// Typical flow:
// 1- The engine will query the supported extensions at RHI creation (FVulkanDeviceExtension are constructed and support is queried)
// 2- The supported extensions are added at VkInstance creation (PreCreateInstance)



// ***** VK_EXT_validation_features
class FVulkanEXTValidationFeaturesExtension : public FVulkanInstanceExtension
{
public:

	FVulkanEXTValidationFeaturesExtension()
		: FVulkanInstanceExtension(VK_EXT_VALIDATION_FEATURES_EXTENSION_NAME, VULKAN_HAS_DEBUGGING_ENABLED & VULKAN_HAS_VALIDATION_FEATURES, VULKAN_EXTENSION_NOT_PROMOTED, nullptr, FVulkanExtensionBase::ManuallyActivate)
	{}

	virtual void PreCreateInstance(VkInstanceCreateInfo& InstanceCreateInfo, FOptionalVulkanInstanceExtensions& ExtensionFlags) override final 
	{
#if VULKAN_HAS_DEBUGGING_ENABLED
		check(GValidationCvar.GetValueOnAnyThread() > 0);

		auto GetValidationFeaturesEnabled = []()
		{
			TArray<VkValidationFeatureEnableEXT> Features;
			const int32 GPUValidationValue = GGPUValidationCvar.GetValueOnAnyThread();
			if (GPUValidationValue > 0)
			{
				Features.Add(VK_VALIDATION_FEATURE_ENABLE_GPU_ASSISTED_EXT);
				if (GPUValidationValue > 1)
				{
					Features.Add(VK_VALIDATION_FEATURE_ENABLE_GPU_ASSISTED_RESERVE_BINDING_SLOT_EXT);
				}
			}

			if (FParse::Param(FCommandLine::Get(), TEXT("vulkanbestpractices")))
			{
				Features.Add(VK_VALIDATION_FEATURE_ENABLE_BEST_PRACTICES_EXT);
			}

			if (FParse::Param(FCommandLine::Get(), TEXT("vulkandebugsync")))
			{
				Features.Add(VK_VALIDATION_FEATURE_ENABLE_SYNCHRONIZATION_VALIDATION_EXT);
			}

			return Features;
		};

		ValidationFeaturesEnabled = GetValidationFeaturesEnabled();

		if (ValidationFeaturesEnabled.Num())
		{
			ZeroVulkanStruct(ValidationFeatures, VK_STRUCTURE_TYPE_VALIDATION_FEATURES_EXT);
			ValidationFeatures.enabledValidationFeatureCount = (uint32)ValidationFeaturesEnabled.Num();
			ValidationFeatures.pEnabledValidationFeatures = ValidationFeaturesEnabled.GetData();
			AddToPNext(InstanceCreateInfo, ValidationFeatures);
		}
#endif 
	}

private:
	VkValidationFeaturesEXT ValidationFeatures;
	TArray<VkValidationFeatureEnableEXT> ValidationFeaturesEnabled;
};




FVulkanInstanceExtensionArray FVulkanInstanceExtension::GetUESupportedInstanceExtensions(uint32 ApiVersion)
{
	FVulkanInstanceExtensionArray OutUEInstanceExtensions;

	// Generic simple extensions :
	OutUEInstanceExtensions.Add(MakeUnique<FVulkanInstanceExtension>(VK_KHR_SURFACE_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));
	OutUEInstanceExtensions.Add(MakeUnique<FVulkanInstanceExtension>(VK_EXT_SWAPCHAIN_COLOR_SPACE_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));

	// Debug extensions :
	OutUEInstanceExtensions.Add(MakeUnique<FVulkanInstanceExtension>(VK_EXT_DEBUG_UTILS_EXTENSION_NAME, VULKAN_HAS_DEBUGGING_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED, nullptr, FVulkanExtensionBase::ManuallyActivate));

	// Extensions with custom classes :
	OutUEInstanceExtensions.Add(MakeUnique<FVulkanEXTValidationFeaturesExtension>());


	// Add in platform specific extensions
	FVulkanPlatform::GetInstanceExtensions(OutUEInstanceExtensions);

	// Helper function to go through a list of extensions and activate them (or add them)
	auto ActivateExternalExtensions = [&] (TArray<const ANSICHAR*> Extensions, const ANSICHAR* Requester) {
		for (const ANSICHAR* ExtensionName : Extensions)
		{
			const int32 ExtensionIndex = FindExtension(OutUEInstanceExtensions, ExtensionName);
			if (ExtensionIndex == INDEX_NONE)
			{
				OutUEInstanceExtensions.Add(MakeUnique<FVulkanInstanceExtension>(ExtensionName, VULKAN_EXTENSION_ENABLED));
				UE_LOGF(LogVulkanRHI, Warning, "%s requested instance extension [%s] isn't part of the engine's core extension list. Adding it on-the-fly...", Requester, ExtensionName);
			}
			else
			{
				OutUEInstanceExtensions[ExtensionIndex]->SetActivated();
			}
		}
	};


	// Add HMD requested extensions
	{
		if (IHeadMountedDisplayModule::IsAvailable())
		{
			FVulkanDynamicRHI::HMDVulkanExtensions = IHeadMountedDisplayModule::Get().GetVulkanExtensions();
		}
		if (FVulkanDynamicRHI::HMDVulkanExtensions.IsValid())
		{
			TArray<const ANSICHAR*> HMDExtensions;
			FVulkanDynamicRHI::HMDVulkanExtensions->GetVulkanInstanceExtensionsRequired(HMDExtensions);
			ActivateExternalExtensions(HMDExtensions, "HMD");
		}
	}

	// Add extensions added outside the RHI (eg plugins)
	ActivateExternalExtensions(ExternalExtensions, "Externally");

	// Now that all the extensions are listed, update their support flags
	FlagExtensionSupport(GetDriverSupportedInstanceExtensions(), OutUEInstanceExtensions, ApiVersion, "instance");

	return OutUEInstanceExtensions;
}
