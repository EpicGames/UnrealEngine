// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanExtensions.h: Vulkan extension definitions.
=============================================================================*/

#pragma once

#include "VulkanConfiguration.h"

struct FOptionalVulkanInstanceExtensions;


#define VULKAN_EXTENSION_NOT_PROMOTED UINT32_MAX
#define VULKAN_EXTENSION_ENABLED      1
#define VULKAN_EXTENSION_DISABLED     0

class FVulkanDevice;

enum class EVulkanVariableRateShadingPreference : uint8
{
	PreferFSR = 0, // This can be used to prefer FSR over FDM when both are available.
	UseFSROnlyIfAvailable, // This will only print a message if FSR is not available.
	RequireFSR, // This will print an error if FSR is not available.
	PreferFDM, // This can be used to prefer FDM over FSR when both are available.
	UseFDMOnlyIfAvailable, // This will only print a message if FDM is not available.
	RequireFDM, // This will print an error if FDM is not available.
};

class FVulkanExtensionBase
{
public:
	enum EExtensionActivation : uint8
	{
		// If the extension is enabled in code and supported by the driver, it will automatically get used.
		AutoActivate,

		// The extension will not be used unless SetActivated() is called in the extension.  
		// Can be used to add engine support for an extension, but not use it by default unless externally requested.
		// Useful for extensions only activated by plugins, or only present when a certain layer is loaded.
		ManuallyActivate
	};

	FVulkanExtensionBase(const ANSICHAR* InExtensionName, int32 InEnabledInCode, uint32 InPromotedVersion, EExtensionActivation InActivation)
		: ExtensionName(InExtensionName)
		, PromotedVersion(InPromotedVersion)
		, bEnabledInCode(InEnabledInCode == VULKAN_EXTENSION_ENABLED)
		, bSupported(false)
		, bActivated(InActivation == EExtensionActivation::AutoActivate)
		, bCore(false)
	{}

	virtual ~FVulkanExtensionBase() = default;

	inline void SetSupported() { bSupported = true; }
	inline void SetActivated() { bActivated = true; }
	inline bool SetCore(uint32 ApiVersion) { return (bCore = (ApiVersion >= PromotedVersion)); }

	inline const ANSICHAR* GetExtensionName() const { return ExtensionName; }
	inline bool IsEnabled() const { return bEnabledInCode; }
	inline bool IsSupported() const { return bSupported; }
	inline bool IsCore() { return bCore; }

	inline bool InUse() const
	{
		return bEnabledInCode && bSupported && bActivated;
	}

	template <typename ExtensionType>
	static int32 FindExtension(const TArray<TUniquePtr<ExtensionType>>& UEExtensions, const ANSICHAR* ExtensionName)
	{
		for (int32 Index = 0; Index < UEExtensions.Num(); ++Index)
		{
			if (!FCStringAnsi::Strcmp(UEExtensions[Index]->GetExtensionName(), ExtensionName))
			{
				return Index;
			}
		}
		return INDEX_NONE;
	}

protected:
	const ANSICHAR* ExtensionName;
	const uint32 PromotedVersion;

	// Extension is enabled in code if the VULKAN_SUPPORTS_* define is TRUE in VulkanConfiguration.h or Vulkan*Platform.h.
	bool bEnabledInCode;

	// Signals if the extension is support by the driver of the device being used (evaluated at runtime).
	bool bSupported;

	// Signals if the runtime currently wants this extension to be loaded.
	bool bActivated;

	// Signals if the extension has been promoted to core Vulkan for the current instance version
	bool bCore;
};



class FVulkanDeviceExtension : public FVulkanExtensionBase
{
public:

	FVulkanDeviceExtension(FVulkanDevice* InDevice, const ANSICHAR* InExtensionName, int32 InEnabledInCode, uint32 InPromotedVersion = VULKAN_EXTENSION_NOT_PROMOTED, TUniqueFunction<void(FOptionalVulkanDeviceExtensions& ExtensionFlags)>&& InFlagSetter = nullptr, EExtensionActivation InActivation = FVulkanExtensionBase::AutoActivate)
		: FVulkanExtensionBase(InExtensionName, InEnabledInCode, InPromotedVersion, InActivation)
		, Device(InDevice)
		, FlagSetter(MoveTemp(InFlagSetter))
		, bRequirementsPassed(true)
	{}

	static FVulkanDeviceExtensionArray GetUESupportedDeviceExtensions(FVulkanDevice* InDevice, uint32 ApiVersion);
	static TArray<VkExtensionProperties> GetDriverSupportedDeviceExtensions(VkPhysicalDevice Gpu, const ANSICHAR* LayerName = nullptr);

	virtual void PrePhysicalDeviceProperties(VkPhysicalDeviceProperties2KHR& PhysicalDeviceProperties2) {}
	virtual void PostPhysicalDeviceProperties(FOptionalVulkanDeviceExtensions& ExtensionFlags) {}

	virtual void PrePhysicalDeviceFeatures(VkPhysicalDeviceFeatures2KHR& PhysicalDeviceFeatures2) {}
	virtual void PostPhysicalDeviceFeatures(FOptionalVulkanDeviceExtensions& ExtensionFlags)
	{
		if (FlagSetter)
		{
			FlagSetter(ExtensionFlags);
		}
	}

	virtual void PreCreateDevice(VkDeviceCreateInfo& DeviceInfo) {}

	// Holds extensions requested externally (eg plugins)
	static TArray<TSharedRef<class FVulkanRHIExternalDeviceExtensionBase>> ExternalExtensions;

protected:
	struct FOptionalVulkanDeviceExtensionProperties& GetDeviceExtensionProperties();

	FVulkanDevice* Device;
	TUniqueFunction<void(FOptionalVulkanDeviceExtensions& ExtensionFlags)> FlagSetter;

	// Signals that this extension has passed any specific feature and/or property requirements
	bool bRequirementsPassed;
};



class FVulkanInstanceExtension : public FVulkanExtensionBase
{
public:

	FVulkanInstanceExtension(const ANSICHAR* InExtensionName, int32 InEnabledInCode, uint32 InPromotedVersion = VULKAN_EXTENSION_NOT_PROMOTED, TUniqueFunction<void(FOptionalVulkanInstanceExtensions& ExtensionFlags)>&& InFlagSetter = nullptr, EExtensionActivation InActivation = FVulkanExtensionBase::AutoActivate)
		: FVulkanExtensionBase(InExtensionName, InEnabledInCode, InPromotedVersion, InActivation)
		, FlagSetter(MoveTemp(InFlagSetter))
	{}

	static FVulkanInstanceExtensionArray GetUESupportedInstanceExtensions(uint32 ApiVersion);
	static TArray<VkExtensionProperties> GetDriverSupportedInstanceExtensions(const ANSICHAR* LayerName = nullptr);

	virtual void PreCreateInstance(VkInstanceCreateInfo& CreateInfo, FOptionalVulkanInstanceExtensions& ExtensionFlags)
	{
		if (FlagSetter)
		{
			FlagSetter(ExtensionFlags);
		}
	}

	static TArray<const ANSICHAR*> ExternalExtensions;

protected:
	TUniqueFunction<void(FOptionalVulkanInstanceExtensions& ExtensionFlags)> FlagSetter;
};

// Helpers to create simple extensions that set a flag
#define DEVICE_EXT_FLAG_SETTER(FLAG_NAME) [](FOptionalVulkanDeviceExtensions& ExtensionFlags) { ExtensionFlags.FLAG_NAME = 1; }
#define INSTANCE_EXT_FLAG_SETTER(FLAG_NAME) [](FOptionalVulkanInstanceExtensions& ExtensionFlags) { ExtensionFlags.FLAG_NAME = 1; }
