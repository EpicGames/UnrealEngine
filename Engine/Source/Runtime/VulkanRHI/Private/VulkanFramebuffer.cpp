// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanFramebuffer.h"
#include "VulkanDevice.h"
#include "VulkanRenderpass.h"
#include "VulkanRenderTargetLayout.h"

FVulkanFramebuffer::FVulkanFramebuffer(FVulkanDevice& Device, const FRHISetRenderTargetsInfo& InRTInfo, const FVulkanRenderTargetLayout& RTLayout, const FVulkanRenderPass& RenderPass)
	: Framebuffer(VK_NULL_HANDLE)
	, NumColorRenderTargets(InRTInfo.NumColorRenderTargets)
	, NumColorAttachments(0)
	, DepthStencilRenderTargetImage(VK_NULL_HANDLE)
	, DepthStencilResolveRenderTargetImage(VK_NULL_HANDLE)
	, FragmentDensityImage(VK_NULL_HANDLE)
{
	FMemory::Memzero(ColorRenderTargetImages);
	FMemory::Memzero(ColorResolveTargetImages);

	AttachmentTextureViews.Empty(RTLayout.GetNumAttachmentDescriptions());

	uint32 MipIndex = 0;

	const VkExtent3D& RTExtents = RTLayout.GetExtent3D();
	// Adreno does not like zero size RTs
	check(RTExtents.width != 0 && RTExtents.height != 0);
	uint32 NumLayers = RTExtents.depth;

	for (int32 Index = 0; Index < InRTInfo.NumColorRenderTargets; ++Index)
	{
		FRHITexture* RHITexture = InRTInfo.ColorRenderTarget[Index].Texture;
		if (!RHITexture)
		{
			continue;
		}

		FVulkanTexture* Texture = ResourceCast(RHITexture);

		ColorRenderTargetImages[Index] = Texture->Image;
		MipIndex = InRTInfo.ColorRenderTarget[Index].MipIndex;

		FVulkanView* View = GetColorRenderTargetViewDesc(Texture, MipIndex, InRTInfo.ColorRenderTarget[Index].ArraySliceIndex, RTLayout.GetMultiViewCount(), NumLayers);
		AttachmentTextureViews.Add(View);

		++NumColorAttachments;

		// Check the RTLayout as well to make sure the resolve attachment is needed (Vulkan and Feature level specific)
		// See: FVulkanRenderTargetLayout constructor with FRHIRenderPassInfo
		if (InRTInfo.bHasResolveAttachments && RTLayout.GetHasResolveAttachments() && RTLayout.GetResolveAttachmentReferences()[Index].layout != VK_IMAGE_LAYOUT_UNDEFINED)
		{
			FRHITexture* ResolveRHITexture = InRTInfo.ColorResolveRenderTarget[Index].Texture;
			FVulkanTexture* ResolveTexture = ResourceCast(ResolveRHITexture);
			ColorResolveTargetImages[Index] = ResolveTexture->Image;

			if (FVulkanView* ResolveView = GetColorResolveTargetViewDesc(ResolveTexture, MipIndex, InRTInfo.ColorRenderTarget[Index].ArraySliceIndex))
			{
				AttachmentTextureViews.Add(ResolveView);
			}
		}
	}

	if (RTLayout.GetHasDepthStencil())
	{
		FVulkanTexture* Texture = ResourceCast(InRTInfo.DepthStencilRenderTarget.Texture);
		const FRHITextureDesc& Desc = Texture->GetDesc();
		DepthStencilRenderTargetImage = Texture->Image;

		check(Texture->PartialView);
		PartialDepthTextureView = Texture->PartialView;

		FVulkanView* View = GetDepthStencilTargetViewDesc(Texture, InRTInfo.NumColorRenderTargets, MipIndex, RTLayout.GetMultiViewCount(), NumLayers);
		AttachmentTextureViews.Add(View);

		if (RTLayout.GetHasDepthStencilResolve() && RTLayout.GetDepthStencilResolveAttachmentReference()->layout != VK_IMAGE_LAYOUT_UNDEFINED)
		{
			FRHITexture* ResolveRHITexture = InRTInfo.DepthStencilResolveRenderTarget.Texture;
			FVulkanTexture* ResolveTexture = ResourceCast(ResolveRHITexture);
			DepthStencilResolveRenderTargetImage = ResolveTexture->Image;

			if (FVulkanView* ResolveView = GetDepthStencilResolveTargetViewDesc(ResolveTexture, MipIndex))
			{
				AttachmentTextureViews.Add(ResolveView);
			}
		}
	}

	if (GRHISupportsAttachmentVariableRateShading && RTLayout.GetHasFragmentDensityAttachment())
	{
		FVulkanTexture* Texture = ResourceCast(InRTInfo.ShadingRateTexture);
		FragmentDensityImage = Texture->Image;
		FVulkanView* View = GetFragmentDensityAttachmentViewDesc(Texture, MipIndex);
		AttachmentTextureViews.Add(View);
	}

	TArray<VkImageView> AttachmentViews;
	AttachmentViews.Reserve(AttachmentTextureViews.Num());
	for (FVulkanView const* View : AttachmentTextureViews)
	{
		AttachmentViews.Add(View->GetTextureView().View);
	}

	VkFramebufferCreateInfo CreateInfo;
	ZeroVulkanStruct(CreateInfo, VK_STRUCTURE_TYPE_FRAMEBUFFER_CREATE_INFO);
	CreateInfo.renderPass = RenderPass.GetHandle();
	CreateInfo.attachmentCount = AttachmentViews.Num();
	CreateInfo.pAttachments = AttachmentViews.GetData();
	CreateInfo.width = RTExtents.width;
	CreateInfo.height = RTExtents.height;
	CreateInfo.layers = NumLayers;

	VERIFYVULKANRESULT_EXPANDED(VulkanRHI::vkCreateFramebuffer(Device.GetHandle(), &CreateInfo, VULKAN_CPU_ALLOCATOR, &Framebuffer));

	RenderArea.offset.x = 0;
	RenderArea.offset.y = 0;
	RenderArea.extent.width = RTExtents.width;
	RenderArea.extent.height = RTExtents.height;

	INC_DWORD_STAT(STAT_VulkanNumFrameBuffers);
}

FVulkanFramebuffer::~FVulkanFramebuffer()
{
	ensure(Framebuffer == VK_NULL_HANDLE);
}

FVulkanView* FVulkanFramebuffer::GetColorRenderTargetViewDesc(FVulkanTexture* Texture, uint32 MipIndex, int32 SliceIndex, uint32 MultiViewCount, uint32& InOutNumLayers)
{
	if (!ensure(Texture->Image != VK_NULL_HANDLE))
	{
		UE_LOGF(LogVulkanRHI, Error, "[%hs] VK_NULL_HANDLE", __FUNCTION__);
		return nullptr;
	}
	
	FVulkanTextureViewDesc ViewDesc = GetDefaultRenderTargetViewDesc(*Texture, MipIndex, SliceIndex);
	const VkImageViewType TextureType = Texture->GetViewType();
	if (TextureType == VK_IMAGE_VIEW_TYPE_2D || TextureType == VK_IMAGE_VIEW_TYPE_2D_ARRAY)
	{
		if (SliceIndex == -1)
		{
			ViewDesc.ArraySliceIndex = 0;
		}
		else
		{
			ViewDesc.NumArraySlices = 1;
			check(static_cast<uint32>(SliceIndex) < Texture->GetNumberOfArrayLevels());
		}

		// About !RTLayout.GetIsMultiView(), from https://registry.khronos.org/vulkan/specs/1.3-extensions/man/html/VkFramebufferCreateInfo.html: 
		// If the render pass uses multiview, then layers must be one
		InOutNumLayers = MultiViewCount == 0 && SliceIndex == -1 ? Texture->GetNumberOfArrayLevels() : 1;
	}
	else if (TextureType == VK_IMAGE_VIEW_TYPE_CUBE || TextureType == VK_IMAGE_VIEW_TYPE_CUBE_ARRAY)
	{
		// Cube always renders one face at a time
		INC_DWORD_STAT(STAT_VulkanNumImageViews);

		ViewDesc.ViewType = VK_IMAGE_VIEW_TYPE_2D;
		ViewDesc.ArraySliceIndex = SliceIndex;
		ViewDesc.NumArraySlices = 1;
	}
	else if (TextureType == VK_IMAGE_VIEW_TYPE_3D)
	{
		ViewDesc.ViewType = VK_IMAGE_VIEW_TYPE_2D_ARRAY;
		ViewDesc.ArraySliceIndex = 0;
		ViewDesc.NumArraySlices = Texture->GetDesc().Depth;
	}
	else
	{
		ensure(false);
		UE_LOGF(LogVulkanRHI, Error, "[%hs] Invalid texture type", __FUNCTION__);
		return nullptr;
	}
	
	const VkImageUsageFlags ImageUsageFlags = VK_IMAGE_USAGE_COLOR_ATTACHMENT_BIT | (Texture->ImageUsageFlags & VK_IMAGE_USAGE_INPUT_ATTACHMENT_BIT);
	ViewDesc.ImageUsageFlags = ImageUsageFlags;
	return Texture->FindOrAddInternalView(ViewDesc);
}

FVulkanView* FVulkanFramebuffer::GetColorResolveTargetViewDesc(FVulkanTexture* ResolveTexture, uint32 MipIndex, int32 ArraySliceIndex)
{
	// Resolve attachments only supported for 2d/2d array textures
	return
		ensure(ResolveTexture->GetViewType() == VK_IMAGE_VIEW_TYPE_2D || ResolveTexture->GetViewType() == VK_IMAGE_VIEW_TYPE_2D_ARRAY) ? 
		ResolveTexture->FindOrAddInternalView(GetDefaultRenderTargetViewDesc(*ResolveTexture, MipIndex, FMath::Max(0, ArraySliceIndex))) :
		nullptr;
}

FVulkanView* FVulkanFramebuffer::GetDepthStencilTargetViewDesc(FVulkanTexture* Texture, uint32 NumColorAttachments, uint32 MipIndex, uint32 MultiViewCount, uint32& InOutNumLayers)
{
	const VkImageViewType TextureType = Texture->GetViewType();
	if (!ensure(TextureType == VK_IMAGE_VIEW_TYPE_2D || TextureType == VK_IMAGE_VIEW_TYPE_2D_ARRAY || TextureType == VK_IMAGE_VIEW_TYPE_CUBE))
	{
		return nullptr;
	}
	
	FVulkanTextureViewDesc ViewDesc = GetDefaultRenderTargetViewDesc(*Texture, MipIndex, MultiViewCount);
	if (NumColorAttachments == 0 && TextureType == VK_IMAGE_VIEW_TYPE_CUBE)
	{
		InOutNumLayers = 6;
		ViewDesc.ArraySliceIndex = 0;
		ViewDesc.NumArraySlices = 6;
		return Texture->FindOrAddInternalView(ViewDesc);
	}
	else if (TextureType == VK_IMAGE_VIEW_TYPE_2D || TextureType == VK_IMAGE_VIEW_TYPE_2D_ARRAY)
	{
		// Depth attachments need a separate view to have no swizzle components, for validation correctness
		ViewDesc.ArraySliceIndex = 0;

		// About !RTLayout.GetIsMultiView(), from https://registry.khronos.org/vulkan/specs/1.3-extensions/man/html/VkFramebufferCreateInfo.html: 
		// If the render pass uses multiview, then layers must be one
		InOutNumLayers = MultiViewCount == 0 ? Texture->GetNumberOfArrayLevels() : 1;

		return Texture->FindOrAddInternalView(ViewDesc);
	}
	else
	{
		return Texture->DefaultView;
	}
}

FVulkanView* FVulkanFramebuffer::GetDepthStencilResolveTargetViewDesc(FVulkanTexture* ResolveTexture, uint32 MipIndex)
{
	// Resolve attachments only supported for 2d/2d array textures
	return
		ensure(ResolveTexture->GetViewType() == VK_IMAGE_VIEW_TYPE_2D || ResolveTexture->GetViewType() == VK_IMAGE_VIEW_TYPE_2D_ARRAY) ?
		ResolveTexture->FindOrAddInternalView(GetDefaultRenderTargetViewDesc(*ResolveTexture, MipIndex, 0)) :
		nullptr;
}

FVulkanView* FVulkanFramebuffer::GetFragmentDensityAttachmentViewDesc(FVulkanTexture* Texture, uint32 MipIndex)
{
	return 
		ensure(Texture->GetViewType() == VK_IMAGE_VIEW_TYPE_2D || Texture->GetViewType() == VK_IMAGE_VIEW_TYPE_2D_ARRAY) ? 
		Texture->FindOrAddInternalView(GetDefaultRenderTargetViewDesc(*Texture, MipIndex, 0)) :
		nullptr;
}

FVulkanTextureViewDesc FVulkanFramebuffer::GetDefaultRenderTargetViewDesc(const FVulkanTexture& Texture, const uint32 MipIndex, const int32 SliceIndex)
{
	FVulkanTextureViewDesc ViewDesc;
	ViewDesc.ViewType = Texture.GetViewType();
	ViewDesc.AspectFlags = Texture.GetFullAspectMask();
	ViewDesc.UEFormat = Texture.GetDesc().Format;
	ViewDesc.Format = Texture.ViewFormat;
	ViewDesc.FirstMip = MipIndex;
	ViewDesc.NumMips = 1;
	ViewDesc.ArraySliceIndex = SliceIndex;
	ViewDesc.NumArraySlices = Texture.GetNumberOfArrayLevels();
	ViewDesc.bUseIdentitySwizzle = true;
	return ViewDesc;
}

void FVulkanFramebuffer::Destroy(FVulkanDevice& Device)
{
	VulkanRHI::FDeferredDeletionQueue2& Queue = Device.GetDeferredDeletionQueue();

	// will be deleted in reverse order
	Queue.EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::Framebuffer, Framebuffer);
	Framebuffer = VK_NULL_HANDLE;

	DEC_DWORD_STAT(STAT_VulkanNumFrameBuffers);
}

bool FVulkanFramebuffer::Matches(const FRHISetRenderTargetsInfo& InRTInfo) const
{
	if (NumColorRenderTargets != InRTInfo.NumColorRenderTargets)
	{
		return false;
	}

	{
		const FRHIDepthRenderTargetView& B = InRTInfo.DepthStencilRenderTarget;
		if (B.Texture)
		{
			VkImage AImage = DepthStencilRenderTargetImage;
			VkImage BImage = ResourceCast(B.Texture)->Image;
			if (AImage != BImage)
			{
				return false;
			}
		}
	}

	{
		const FRHIDepthRenderTargetView& R = InRTInfo.DepthStencilResolveRenderTarget;
		if (R.Texture)
		{
			VkImage AImage = DepthStencilResolveRenderTargetImage;
			VkImage BImage = ResourceCast(R.Texture)->Image;
			if (AImage != BImage)
			{
				return false;
			}
		}
	}

	{
		FRHITexture* Texture = InRTInfo.ShadingRateTexture;
		if (Texture)
		{
			VkImage AImage = FragmentDensityImage;
			VkImage BImage = ResourceCast(Texture)->Image;
			if (AImage != BImage)
			{
				return false;
			}
		}
	}

	int32 AttachementIndex = 0;
	for (int32 Index = 0; Index < InRTInfo.NumColorRenderTargets; ++Index)
	{
		if (InRTInfo.bHasResolveAttachments)
		{
			const FRHIRenderTargetView& R = InRTInfo.ColorResolveRenderTarget[Index];
			if (R.Texture)
			{
				VkImage AImage = ColorResolveTargetImages[AttachementIndex];
				VkImage BImage = ResourceCast(R.Texture)->Image;
				if (AImage != BImage)
				{
					return false;
				}
			}
		}

		const FRHIRenderTargetView& B = InRTInfo.ColorRenderTarget[Index];
		if (B.Texture)
		{
			VkImage AImage = ColorRenderTargetImages[AttachementIndex];
			VkImage BImage = ResourceCast(B.Texture)->Image;
			if (AImage != BImage)
			{
				return false;
			}
			AttachementIndex++;
		}
	}

	return true;
}





