// Copyright Epic Games, Inc. All Rights Reserved..

#pragma once

#include "VulkanRHIPrivate.h"

class FVulkanFramebuffer
{
public:
	FVulkanFramebuffer(FVulkanDevice& Device, const FRHISetRenderTargetsInfo& InRTInfo, const FVulkanRenderTargetLayout& RTLayout, const FVulkanRenderPass& RenderPass);
	~FVulkanFramebuffer();

	bool Matches(const FRHISetRenderTargetsInfo& RTInfo) const;

	uint32 GetNumColorAttachments() const
	{
		return NumColorAttachments;
	}

	void Destroy(FVulkanDevice& Device);

	VkFramebuffer GetHandle()
	{
		return Framebuffer;
	}

	const FVulkanView::FTextureView& GetPartialDepthTextureView() const
	{
		check(PartialDepthTextureView);
		return PartialDepthTextureView->GetTextureView();
	}

	TArray<FVulkanView const*> AttachmentTextureViews;

	// Copy from the Depth render target partial view
	FVulkanView const* PartialDepthTextureView = nullptr;

	bool ContainsRenderTarget(FRHITexture* Texture) const
	{
		ensure(Texture);
		FVulkanTexture* VulkanTexture = ResourceCast(Texture);
		return ContainsRenderTarget(VulkanTexture->Image);
	}

	bool ContainsRenderTarget(VkImage Image) const
	{
		ensure(Image != VK_NULL_HANDLE);
		for (uint32 Index = 0; Index < NumColorAttachments; ++Index)
		{
			if (ColorRenderTargetImages[Index] == Image)
			{
				return true;
			}
		}

		return (DepthStencilRenderTargetImage == Image);
	}

	VkRect2D GetRenderArea() const
	{
		return RenderArea;
	}

	// Expose the view creation logic so it can be reused for dynamic rendering
	static FVulkanView* GetColorRenderTargetViewDesc(FVulkanTexture* Texture, uint32 MipIndex, int32 ArraySliceIndex, uint32 MultiViewCount, uint32& InOutNumLayers);
	static FVulkanView* GetColorResolveTargetViewDesc(FVulkanTexture* ResolveTexture, uint32 MipIndex, int32 ArraySliceIndex);
	static FVulkanView* GetDepthStencilTargetViewDesc(FVulkanTexture* Texture, uint32 NumColorAttachments, uint32 MipIndex, uint32 MultiViewCount, uint32& InOutNumLayers);
	static FVulkanView* GetDepthStencilResolveTargetViewDesc(FVulkanTexture* ResolveTexture, uint32 MipIndex);
	static FVulkanView* GetFragmentDensityAttachmentViewDesc(FVulkanTexture* Texture, uint32 MipIndex);
	
private:
	[[nodiscard]] static FVulkanTextureViewDesc GetDefaultRenderTargetViewDesc(const FVulkanTexture& Texture, const uint32 MipIndex, const int32 SliceIndex);

private:
	VkFramebuffer Framebuffer;
	VkRect2D RenderArea;

	// Unadjusted number of color render targets as in FRHISetRenderTargetsInfo 
	uint32 NumColorRenderTargets;

	// Save image off for comparison, in case it gets aliased.
	uint32 NumColorAttachments;
	VkImage ColorRenderTargetImages[MaxSimultaneousRenderTargets];
	VkImage ColorResolveTargetImages[MaxSimultaneousRenderTargets];
	VkImage DepthStencilRenderTargetImage;
	VkImage DepthStencilResolveRenderTargetImage;
	VkImage FragmentDensityImage;

	// Predefined set of barriers, when executes ensuring all writes are finished
	TArray<VkImageMemoryBarrier> WriteBarriers;

	friend class FVulkanCommandListContext;
};
