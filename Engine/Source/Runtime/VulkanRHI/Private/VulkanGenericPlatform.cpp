// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanGenericPlatform.h"
#include "VulkanCommandWrappers.h"
#include "VulkanDevice.h"
#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "Misc/CommandLine.h"
#include "VulkanCommandBuffer.h"

static TAutoConsoleVariable<int32> CVarVulkanUseProfileCheck(
	TEXT("r.Vulkan.UseProfileCheck"),
	1,
	TEXT("0 to assume all requested feature levels are supported.\n")
	TEXT("1 to verify feature level support using a profile check (default)\n"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> CVarVulkanAllowSparseResidency(
	TEXT("r.Vulkan.AllowSparseResidency"),
	1,
	TEXT("Toggles the use of sparse residency for buffers and images:\n")
	TEXT("0 to disable\n")
	TEXT("1 to allow when supported (default)\n"),
	ECVF_ReadOnly
);


void FVulkanGenericPlatform::SetupFeatureLevels(TArrayView<EShaderPlatform> ShaderPlatformForFeatureLevel)
{
	checkf(ERHIFeatureLevel::Num == ShaderPlatformForFeatureLevel.Num(), TEXT("ShaderPlatformForFeatureLevel is not the right size to fit all Feature Level values."));

	ShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES2_REMOVED] = SP_NumPlatforms;
	ShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES3_1] = SP_VULKAN_PCES3_1;
	ShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM4_REMOVED] = SP_NumPlatforms;
	ShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM5] = SP_VULKAN_SM5;
	ShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM6] = SP_VULKAN_SM6;
}

ERHIFeatureLevel::Type FVulkanGenericPlatform::GetFeatureLevel(ERHIFeatureLevel::Type InRequestedFeatureLevel)
{
	const bool bForceES3_1 = (FVulkanPlatform::RequiresMobileRenderer() ||
		(InRequestedFeatureLevel == ERHIFeatureLevel::ES3_1) ||
		FParse::Param(FCommandLine::Get(), TEXT("FeatureLevelES31")) || FParse::Param(FCommandLine::Get(), TEXT("FeatureLevelES3_1")));

	return (!GIsEditor && bForceES3_1) ? ERHIFeatureLevel::ES3_1 : InRequestedFeatureLevel;
}

bool FVulkanGenericPlatform::PSOBinaryCacheMatches(FVulkanDevice* Device, const TArray<uint8>& DeviceCache)
{
	if (DeviceCache.Num() > 4)
	{
		uint32* Data = (uint32*)DeviceCache.GetData();
		uint32 HeaderSize = *Data++;
		// 16 is HeaderSize + HeaderVersion
		if (HeaderSize == 16 + VK_UUID_SIZE)
		{
			uint32 HeaderVersion = *Data++;
			if (HeaderVersion == VK_PIPELINE_CACHE_HEADER_VERSION_ONE)
			{
				uint32 VendorID = *Data++;
				const VkPhysicalDeviceProperties& DeviceProperties = Device->GetDeviceProperties();
				if (VendorID == DeviceProperties.vendorID)
				{
					uint32 DeviceID = *Data++;
					if (DeviceID == DeviceProperties.deviceID)
					{
						uint8* Uuid = (uint8*)Data;
						if (FMemory::Memcmp(DeviceProperties.pipelineCacheUUID, Uuid, VK_UUID_SIZE) == 0)
						{
							// This particular binary cache matches this device
							return true;
						}
					}
				}
			}
		}
	}

	return false;
}

FString FVulkanGenericPlatform::CreatePSOBinaryCacheFilename(FVulkanDevice* Device, FString CacheFilename)
{
	const VkPhysicalDeviceProperties& DeviceProperties = Device->GetDeviceProperties();
	FString BinaryCacheAppendage = FString::Printf(TEXT(".%x.%x"), DeviceProperties.vendorID, DeviceProperties.deviceID);
	if (!CacheFilename.EndsWith(BinaryCacheAppendage))
	{
		CacheFilename += BinaryCacheAppendage;
	}

	return CacheFilename;
}

FString FVulkanGenericPlatform::GetCompiledPSOCacheTopFolderPath()
{
	return FPaths::ProjectSavedDir() / TEXT("VulkanProgramBinaryCache");
}

TArray<FString> FVulkanGenericPlatform::GetPSOCacheFilenames()
{
	TArray<FString> CacheFilenames;
	FString StagedCacheDirectory = FPaths::ProjectDir() / TEXT("Build") / TEXT("ShaderCaches") / FPlatformProperties::IniPlatformName();

	// look for any staged caches
	TArray<FString> StagedCaches;
	IFileManager::Get().FindFiles(StagedCaches, *StagedCacheDirectory, TEXT("cache"));
	// FindFiles returns the filenames without directory, so prepend the stage directory
	for (const FString& Filename : StagedCaches)
	{
		CacheFilenames.Add(StagedCacheDirectory / Filename);
	}

	return CacheFilenames;
}

void FVulkanGenericPlatform::RestrictEnabledPhysicalDeviceFeatures(FVulkanPhysicalDeviceFeatures* InOutFeaturesToEnable)
{
	if (CVarVulkanAllowSparseResidency.GetValueOnAnyThread() == 0)
	{
		// Disable everything sparse-related
		InOutFeaturesToEnable->Core_1_0.shaderResourceResidency = VK_FALSE;
		InOutFeaturesToEnable->Core_1_0.shaderResourceMinLod = VK_FALSE;
		InOutFeaturesToEnable->Core_1_0.sparseBinding = VK_FALSE;
		InOutFeaturesToEnable->Core_1_0.sparseResidencyBuffer = VK_FALSE;
		InOutFeaturesToEnable->Core_1_0.sparseResidencyImage2D = VK_FALSE;
		InOutFeaturesToEnable->Core_1_0.sparseResidencyImage3D = VK_FALSE;
		InOutFeaturesToEnable->Core_1_0.sparseResidency2Samples = VK_FALSE;
		InOutFeaturesToEnable->Core_1_0.sparseResidency4Samples = VK_FALSE;
		InOutFeaturesToEnable->Core_1_0.sparseResidency8Samples = VK_FALSE;
		InOutFeaturesToEnable->Core_1_0.sparseResidencyAliased = VK_FALSE;
	}
}

VkResult FVulkanGenericPlatform::Present(VkQueue Queue, VkPresentInfoKHR& PresentInfo)
{
	return VulkanRHI::vkQueuePresentKHR(Queue, &PresentInfo);
}

VkResult FVulkanGenericPlatform::CreateSwapchainKHR(FVulkanGenericPlatformWindowContext& WindowContext, VkPhysicalDevice PhysicalDevice, VkDevice Device, const VkSwapchainCreateInfoKHR* CreateInfo, const VkAllocationCallbacks* Allocator, VkSwapchainKHR* Swapchain)
{
	return VulkanRHI::vkCreateSwapchainKHR(Device, CreateInfo, Allocator, Swapchain);
}

void FVulkanGenericPlatform::DestroySwapchainKHR(VkDevice Device, VkSwapchainKHR Swapchain, const VkAllocationCallbacks* Allocator)
{
	VulkanRHI::vkDestroySwapchainKHR(Device, Swapchain, Allocator);
}

void FVulkanGenericPlatform::DestroySurface(uint32 /*WindowId*/, VkInstance Instance, VkSurfaceKHR Surface)
{
	VulkanRHI::vkDestroySurfaceKHR(Instance, Surface, VULKAN_CPU_ALLOCATOR);
}

PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
bool FVulkanGenericPlatform::LoadVulkanInstanceFunctions(VkInstance InInstance)
{
	if (VulkanDynamicAPI::vkGetInstanceProcAddr != nullptr)
	{
		bool bFoundAllEntryPoints = true;

#define GETINSTANCE_VK_ENTRYPOINTS(Type, Func) VulkanDynamicAPI::Func = (Type)VulkanDynamicAPI::vkGetInstanceProcAddr(InInstance, #Func);
#define CHECK_VK_ENTRYPOINTS(Type,Func) if (VulkanDynamicAPI::Func == nullptr) { bFoundAllEntryPoints = false; UE_LOGF(LogRHI, Warning, "Failed to find entry point for %ls", TEXT(#Func)); }

		// Initialize basic common instance entrypoints and verify they are all present
		ENUM_VK_ENTRYPOINTS_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
		ENUM_VK_ENTRYPOINTS_INSTANCE(CHECK_VK_ENTRYPOINTS);

		// Initialize optional instance entrypoints
		ENUM_VK_ENTRYPOINTS_OPTIONAL_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);

#undef GETINSTANCE_VK_ENTRYPOINTS
#undef CHECK_VK_ENTRYPOINTS

		return bFoundAllEntryPoints;
	}
	return false;
}
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS


void FVulkanGenericPlatform::ClearVulkanInstanceFunctions()
{
	// Initialize all of the entry points we have to query manually
#define CLEAR_VK_ENTRYPOINTS(Type, Func) VulkanDynamicAPI::Func = nullptr;
	ENUM_VK_ENTRYPOINTS_INSTANCE(CLEAR_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_OPTIONAL_INSTANCE(CLEAR_VK_ENTRYPOINTS);
#undef CLEAR_VK_ENTRYPOINTS
}

bool FVulkanGenericPlatform::SupportsProfileChecks()
{
	return (CVarVulkanUseProfileCheck.GetValueOnAnyThread() != 0) && 
		!FParse::Param(FCommandLine::Get(), TEXT("SkipVulkanProfileCheck"));
}

FString FVulkanGenericPlatform::GetVulkanProfileNameForFeatureLevel(ERHIFeatureLevel::Type FeatureLevel, bool bRaytracing)
{
	FString ProfileName = TEXT("VP_UE_Vulkan_") + LexToString(FeatureLevel);
	if (bRaytracing)
	{
		ProfileName += TEXT("_RT");
	}
	return ProfileName;
}

#if WITH_ADDITIONAL_CRASH_CONTEXTS
void FVulkanGenericPlatform::OnCreatePipelineCrash(FCrashContextExtendedWriter& Writer, TConstArrayView<FVulkanShader*> Shaders, TStringView<TCHAR> ExtraInfo, bool bIsPrecache)
{
	auto GetHash = [](const FVulkanShader* Shader)
	{
			switch (Shader->Frequency)
			{
			case SF_Vertex: return static_cast<const FVulkanVertexShader*>(Shader)->GetHash();
			case SF_Pixel: return static_cast<const FVulkanPixelShader*>(Shader)->GetHash();
			case SF_Mesh: return static_cast<const FVulkanMeshShader*>(Shader)->GetHash();
			case SF_Amplification: return static_cast<const FVulkanTaskShader*>(Shader)->GetHash();
			case SF_Geometry: return static_cast<const FVulkanGeometryShader*>(Shader)->GetHash();

			case SF_Compute: return static_cast<const FVulkanComputeShader*>(Shader)->GetHash();

			case SF_RayGen:
			case SF_RayMiss:
			case SF_RayHitGroup:
			case SF_RayCallable:
				return static_cast<const FVulkanRayTracingShader*>(Shader)->GetHash();

			default: return FShaderHash();
			}
	};

	constexpr int32 SizeOfHashString = (sizeof(uint64) * 2) + 1;

	// write hash strings out, trying to avoid allocating
	auto HashToString = [SizeOfHashString](const FShaderHash& ShaderHash, TCHAR(&Output)[SizeOfHashString])
	{
		uint8 Bytes[sizeof(uint64)];
		ShaderHash.ToByteArray(Bytes);
		static_assert(sizeof(Bytes) * 2 < SizeOfHashString);
		UE::String::BytesToHex(MakeArrayView(Bytes), Output);
		Output[sizeof(Bytes) * 2] = '\0';
	};

	bool bFirstShader = true;
	int32 StrLength = 0;
	TCHAR CrashString[4096] = { 0 };
	for (const FVulkanShader* Shader : Shaders)
	{
		if (Shader)
		{
			const FShaderHash Hash = GetHash(Shader);
			TCHAR HashString[SizeOfHashString] = { 0 };
			HashToString(Hash, HashString);
			const int32 Ret = FCString::Snprintf(
				&CrashString[StrLength], UE_ARRAY_COUNT(CrashString) - StrLength, 
				TEXT("%s%s=%s"),
				bFirstShader ? TEXT("") : TEXT(", "),
				GetShaderFrequencyString(Shader->Frequency, false),
				HashString
			);

			// Only move forward if call succeeded
			if (Ret > 0)
			{
				StrLength += Ret;
			}

			bFirstShader = false;
		}
	}

	if (!ExtraInfo.IsEmpty())
	{
		const int32 Ret = FCString::Snprintf(
			&CrashString[StrLength], UE_ARRAY_COUNT(CrashString) - StrLength,
			TEXT(" %.*s"),
			ExtraInfo.Len(), ExtraInfo.GetData());

		if (Ret > 0)
		{
			StrLength += Ret;
		}
	}

	Writer.AddString(TEXT("VKhashes"), CrashString);
	UE_LOGF(LogVulkanRHI, Log, "CreatePipeline: %ls", CrashString);
}
#endif // WITH_ADDITIONAL_CRASH_CONTEXTS
