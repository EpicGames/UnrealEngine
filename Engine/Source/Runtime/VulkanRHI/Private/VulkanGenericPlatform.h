// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "PixelFormat.h"
#include "RHIResources.h"
#include "Templates/UniquePtr.h"
#include "VulkanThirdParty.h"

struct FOptionalVulkanDeviceExtensions;
class FVulkanDevice;
class FVulkanRenderTargetLayout;
struct FGfxPipelineDesc;
class FVulkanPhysicalDeviceFeatures;
class FVulkanCommandBuffer;
class FVulkanShader;

enum EShaderPlatform : uint16;
enum class EGpuVendorId : uint32;
namespace ERHIFeatureLevel { enum Type : int; }

using FVulkanDeviceExtensionArray = TArray<TUniquePtr<class FVulkanDeviceExtension>>;
using FVulkanInstanceExtensionArray = TArray<TUniquePtr<class FVulkanInstanceExtension>>;

// the platform interface, and empty implementations for platforms that don't need em

class FVulkanGenericPlatformWindowContext
{
public:
	FVulkanGenericPlatformWindowContext(void* WindowHandleIn) : WindowHandle(WindowHandleIn) { }
	void* GetWindowHandle() const { return WindowHandle; }
	// Platform-defined opaque window id captured at context construction (0 if the platform does not
	// track one). Used to release per-window RHI resources by a stable id rather than by a pointer
	// that may be stale/reused by destroy time.
	uint32 GetWindowId() const { return 0; }
	// whether the platform can attempt to create a swapchains during present
	static bool CanCreateSwapchainOnDemand() { return true; }
	bool IsValid() const { return true; }
protected:
	void* WindowHandle = nullptr;
};

class FVulkanGenericPlatform
{
public:
	static bool IsSupported() { return true; }

	static bool LoadVulkanLibrary() { return true; }
	static bool LoadVulkanInstanceFunctions(VkInstance inInstance);
	static void ClearVulkanInstanceFunctions();
	static void FreeVulkanLibrary() {}

	static void InitDevice(FVulkanDevice* InDevice) {}
	static void PostInitGPU(const FVulkanDevice& InDevice) {}

	// Called after querying all the available extensions and layers
	static void NotifyFoundInstanceLayersAndExtensions(const TArray<const ANSICHAR*>& Layers, const TArray<const ANSICHAR*>& Extensions) {}
	static void NotifyFoundDeviceLayersAndExtensions(VkPhysicalDevice PhysicalDevice, const TArray<const ANSICHAR*>& Layers, const TArray<const ANSICHAR*>& Extensions) {}

	// Array of required extensions for the platform (Required!)
	static void GetInstanceExtensions(FVulkanInstanceExtensionArray& OutExtensions);
	static void GetInstanceLayers(TArray<const ANSICHAR*>& OutLayers) {}
	static void GetDeviceExtensions(FVulkanDevice* Device, FVulkanDeviceExtensionArray& OutExtensions);
	static void GetDeviceLayers(TArray<const ANSICHAR*>& OutLayers) {}

	// Allows platform specific GPU crash handling
	static void OnGPUCrash(const TCHAR* Message) {}

	// create/destroy the platform-specific surface object - required
	static void CreateSurface(VkSurfaceKHR* OutSurface);
	static void DestroySurface(uint32 WindowId, VkInstance Instance, VkSurfaceKHR Surface);

	// most platforms support BC* but not ASTC* or ETC2*
	static bool SupportsBCTextureFormats() { return true; }
	static bool SupportsASTCTextureFormats() { return false; }
	static bool SupportsETC2TextureFormats() { return false; }
	static bool SupportsR16UnormTextureFormat() { return true; }

	// most platforms can query the surface for the present mode, and size, etc
	static bool SupportsQuerySurfaceProperties() { return true; }

	static void SetupFeatureLevels(TArrayView<EShaderPlatform> ShaderPlatformForFeatureLevel);

	static bool SupportsTimestampRenderQueries() { return true; }

	static bool RequiresMobileRenderer() { return false; }

	static ERHIFeatureLevel::Type GetFeatureLevel(ERHIFeatureLevel::Type RequestedFeatureLevel);

	// bInit=1 called at RHI init time, bInit=0 at RHI deinit time
	static void OverridePlatformHandlers(bool bInit) {}

	// Some platforms have issues with the access flags for the Present layout
	static bool RequiresPresentLayoutFix() { return false; }

	static bool HasUnifiedMemory() { return false; }

	// Allow the platform code to restrict the device features
	static void RestrictEnabledPhysicalDeviceFeatures(FVulkanPhysicalDeviceFeatures* InOutFeaturesToEnable);

	static bool SupportParallelRenderingTasks() { return true; }

	/** The status quo is true, Platforms might opt out if GPU timing queries are not supported. */
	static bool SupportsDynamicResolution() { return true; }

	static bool SupportsVolumeTextureRendering() { return true; }

	static bool RequiresSwapchainGeneralInitialLayout() { return false; }

	// Allow platforms to perform their own frame pacing, called before Present. Returns true if the platform has done framepacing, false otherwise.
	static bool FramePace(FVulkanDevice& Device, void* WindowHandle, VkSwapchainKHR Swapchain, uint32 PresentID, VkPresentInfoKHR& Info) { return false; }

	// Allow platforms to do extra work on present
	static VkResult Present(VkQueue Queue, VkPresentInfoKHR& PresentInfo);

	// Allow platforms to track swapchain creation
	static VkResult CreateSwapchainKHR(FVulkanGenericPlatformWindowContext& WindowContext, VkPhysicalDevice PhysicalDevice, VkDevice Device, const VkSwapchainCreateInfoKHR* CreateInfo, const VkAllocationCallbacks* Allocator, VkSwapchainKHR* Swapchain);
	
	// Allow platforms to track swapchain destruction
	static void DestroySwapchainKHR(VkDevice Device, VkSwapchainKHR Swapchain, const VkAllocationCallbacks* Allocator);

	// Whether to attempt recreate swapchain when present or acqurire operations fail
	static bool RecreateSwapchainOnFail() { return true; }

	// Check if a native window handle is still valid for presentation.
	// Platforms where surfaces can be invalidated externally (e.g. Wayland) should override this.
	static bool IsWindowValid(void* WindowHandle) { return WindowHandle != nullptr; }

	// Test-and-clear a platform-side swapchain-recreate request.
	static bool TryConsumeSwapchainRecreateRequest(void* WindowHandle) { return false; }

	// Ensure the last frame completed on the GPU
	static bool RequiresWaitingForFrameCompletionEvent() { return true; }

	// Does the platform allow a nullptr Pixelshader on the pipeline
	static bool SupportsNullPixelShader() { return true; }

	// Does the platform require depth to be written on stencil clear
	static bool RequiresDepthStencilFullWrite() { return false; }

	// Does the platform require to avoid VkAttachmentDescriptionStencilLayout in favor of merged depth-stencil layouts
	static bool RequiresMergedDepthStencilLayout() { return false; }

	// Checks if the PSO cache matches the expected vulkan device properties
	static bool PSOBinaryCacheMatches(FVulkanDevice* Device, const TArray<uint8>& DeviceCache);

	// Will create the correct format from a generic pso filename
	static FString CreatePSOBinaryCacheFilename(FVulkanDevice* Device, FString CacheFilename);

	// Get top-level path for pso cache files (defaults to ProjectSavedDir/VulkanProgramBinaryCache)
	static FString GetCompiledPSOCacheTopFolderPath();

	// Gathers a list of pso cache filenames to attempt to load
	static TArray<FString> GetPSOCacheFilenames();

	// Gives platform a chance to handle precompile of PSOs, returns VK_NULL_HANDLE if unsupported
	static VkPipelineCache PrecompilePSO(FVulkanDevice* Device, const TArrayView<const uint8> ContextData, const TArrayView<const uint8> OptionalPSOCacheData, FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType PSOCompileType, const VkGraphicsPipelineCreateInfo* PipelineInfo, const FGfxPipelineDesc* GfxEntry, const FVulkanRenderTargetLayout* RTLayout, TArrayView<const uint32_t> VS, TArrayView<const uint32_t> PS, size_t& AfterSize, FString* FailureMessageOUT = nullptr) { return VK_NULL_HANDLE;}

	// Return VK_FALSE if platform wants to suppress the given debug report from the validation layers, VK_TRUE to print it.
	static VkBool32 DebugReportFunction(VkDebugReportFlagsEXT MsgFlags, VkDebugReportObjectTypeEXT ObjType, uint64_t SrcObject, size_t Location, int32 MsgCode, const ANSICHAR* LayerPrefix, const ANSICHAR* Msg, void* UserData) { return VK_TRUE; }

	// Setup platform to use a workaround to reduce textures memory requirements
	static void SetImageMemoryRequirementWorkaround(VkImageCreateInfo& ImageCreateInfo) {};

	// Returns the profile name to look up for a given feature level on a platform
	static bool SupportsProfileChecks();
	static FString GetVulkanProfileNameForFeatureLevel(ERHIFeatureLevel::Type FeatureLevel, bool bRaytracing);

	// Returns the shader stages for which we need support for subgroup ops.
	static VkShaderStageFlags RequiredWaveOpsShaderStageFlags(VkShaderStageFlags VulkanDeviceShaderStageFlags) { return VulkanDeviceShaderStageFlags; }

	static VkTimeDomainKHR GetTimeDomain() { return VK_TIME_DOMAIN_DEVICE_KHR; }

#if WITH_ADDITIONAL_CRASH_CONTEXTS
	static void OnCreatePipelineCrash(struct FCrashContextExtendedWriter& Writer, TConstArrayView<FVulkanShader*> Shaders, TStringView<TCHAR> ExtraInfo, bool bIsPrecache);
#endif
};
