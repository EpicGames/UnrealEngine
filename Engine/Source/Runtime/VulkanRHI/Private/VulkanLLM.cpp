// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanLLM.cpp: Vulkan LLM implementation.
=============================================================================*/


#include "VulkanRHIPrivate.h"
#include "VulkanLLM.h"
#include "HAL/LowLevelMemStats.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

uint64 GVulkanLLMAllocationID = 0x0;

struct FLLMTagInfoVulkan
{
	const TCHAR* Name;
	FName StatName;				// shows in the LLMFULL stat group
	FName SummaryStatName;		// shows in the LLM summary stat group
};

DECLARE_LLM_MEMORY_STAT(TEXT("VulkanMisc"), STAT_VulkanMiscLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("VulkanUniformBuffers"), STAT_VulkanUniformBuffersLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("VulkanBuffers"), STAT_VulkanBuffersLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("VulkanTextures"), STAT_VulkanTexturesLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("VulkanRenderTargets"), STAT_VulkanRenderTargetsLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("VulkanSpareMemoryGPU"), STAT_VulkanSpareMemoryGPULLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("VulkanShaders"), STAT_VulkanShadersLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("VulkanFrameTemp"), STAT_VulkanFrameTempLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("VulkanStagingBuffers"), STAT_VulkanStagingBuffersLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("VulkanDriverMemoryCPU"), STAT_VulkanDriverMemoryCPULLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("VulkanDriverMemoryGPU"), STAT_VulkanDriverMemoryGPULLM, STATGROUP_LLMPlatform);


// *** order must match ELLMTagVulkan enum ***
static const FLLMTagInfoVulkan ELLMTagNamesVulkan[] =
{
	// csv name							// stat name												// summary stat name						// enum value
	{ TEXT("VulkanMisc"),				GET_STATFNAME(STAT_VulkanMiscLLM),							GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagVulkan::VulkanMisc
	{ TEXT("VulkanUniformBuffers"),		GET_STATFNAME(STAT_VulkanUniformBuffersLLM),				GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagVulkan::VulkanUniformBuffers
	{ TEXT("VulkanBuffers"),			GET_STATFNAME(STAT_VulkanBuffersLLM),						GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagVulkan::VulkanBuffers
	{ TEXT("VulkanTextures"),			GET_STATFNAME(STAT_VulkanTexturesLLM),						GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagVulkan::VulkanTextures
	{ TEXT("VulkanRenderTargets"),		GET_STATFNAME(STAT_VulkanRenderTargetsLLM),					GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagVulkan::VulkanRenderTargets
	{ TEXT("VulkanSpareMemoryGPU"),		GET_STATFNAME(STAT_VulkanSpareMemoryGPULLM),				GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagVulkan::VulkanSpareMemoryGPU
	{ TEXT("VulkanShaders"),			GET_STATFNAME(STAT_VulkanShadersLLM),						GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagVulkan::VulkanShaders
	{ TEXT("VulkanFrameTemp"),			GET_STATFNAME(STAT_VulkanFrameTempLLM),						GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagVulkan::VulkanFrameTempGPU
	{ TEXT("VulkanStagingBuffers"),		GET_STATFNAME(STAT_VulkanStagingBuffersLLM),				GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagVulkan::VulkanStagingBuffersGPU
	{ TEXT("VulkanDriverMemoryCPU"),	GET_STATFNAME(STAT_VulkanDriverMemoryCPULLM),				GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagVulkan::VulkanDriverMemoryCPU
	{ TEXT("VulkanDriverMemoryGPU"),	GET_STATFNAME(STAT_VulkanDriverMemoryGPULLM),				GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagVulkan::VulkanDriverMemoryGPU
};

/*
 * Register Vulkan tags with LLM
 */
namespace VulkanLLM
{
	void Initialize()
	{
		const int32 TagCount = sizeof(ELLMTagNamesVulkan) / sizeof(FLLMTagInfoVulkan);

		for (int32 Index = 0; Index < TagCount; ++Index)
		{
			int32 Tag = (int32)ELLMTag::PlatformTagStart + Index;
			const FLLMTagInfoVulkan& TagInfo = ELLMTagNamesVulkan[Index];

			FLowLevelMemTracker::Get().RegisterPlatformTag(Tag, TagInfo.Name, TagInfo.StatName, TagInfo.SummaryStatName);
		}
	}
}

#endif		// #if ENABLE_LOW_LEVEL_MEM_TRACKER
