// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanLLM.h: Vulkan LLM definitions.
=============================================================================*/

#pragma once

#include "HAL/LowLevelMemTracker.h"
#include "VulkanConfiguration.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

#if VULKAN_USE_LLM

// Add custom Tags
enum class ELLMTagVulkan : LLM_TAG_TYPE
{
	VulkanMisc = (LLM_TAG_TYPE)ELLMTag::PlatformTagStart,
	VulkanUniformBuffers,
	VulkanBuffers,
	VulkanTextures,
	VulkanRenderTargets,
	VulkanSpareMemoryGPU,
	VulkanShaders,
	VulkanFrameTemp,
	VulkanStagingBuffers,
	VulkanDriverMemoryCPU,
	VulkanDriverMemoryGPU,
	Count,
};

extern uint64 GVulkanLLMAllocationID;
#define LLM_TRACK_VULKAN_HIGH_LEVEL_ALLOC(AllocObj, Size) { AllocObj.SetLLMTrackerID(0xDEAD | (++GVulkanLLMAllocationID << 16)); FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, (void*)AllocObj.GetLLMTrackerID(), Size, (ELLMTag)ELLMTagVulkan::VulkanMisc); }
#define LLM_TRACK_VULKAN_HIGH_LEVEL_FREE(AllocObj) { FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, (void*)AllocObj.GetLLMTrackerID()); }
#define LLM_TRACK_VULKAN_SPARE_MEMORY_GPU(Size)	{ LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT((ELLMTag)ELLMTagVulkan::VulkanSpareMemoryGPU, Size, ELLMTracker::Default, ELLMAllocType::None); }

#define LLM_TRACK_VULKAN_HIGH_LEVEL_ALLOCATION

static_assert((int32)ELLMTagVulkan::Count <= (int32)ELLMTag::PlatformTagEnd, "too many ELLMTagVulkan tags");

namespace VulkanLLM
{
	void Initialize();
}

#else		// #if VULKAN_USE_LLM

// Translate VulkanTag to regular Tag
enum class ELLMTagVulkan : LLM_TAG_TYPE
{
	VulkanMisc				= (LLM_TAG_TYPE)ELLMTag::RHIMisc,
	VulkanUniformBuffers	= (LLM_TAG_TYPE)ELLMTag::RHIMisc,
	VulkanBuffers			= (LLM_TAG_TYPE)ELLMTag::RHIMisc,
	VulkanTextures			= (LLM_TAG_TYPE)ELLMTag::Textures,
	VulkanRenderTargets		= (LLM_TAG_TYPE)ELLMTag::RHIMisc,
	VulkanSpareMemoryGPU	= (LLM_TAG_TYPE)ELLMTag::RHIMisc,
	VulkanShaders			= (LLM_TAG_TYPE)ELLMTag::Shaders,
	VulkanFrameTemp			= (LLM_TAG_TYPE)ELLMTag::RHIMisc,
	VulkanStagingBuffers	= (LLM_TAG_TYPE)ELLMTag::RHIMisc,
	VulkanDriverMemoryCPU	= (LLM_TAG_TYPE)ELLMTag::RHIMisc,
	VulkanDriverMemoryGPU	= (LLM_TAG_TYPE)ELLMTag::RHIMisc,
	Count					= (LLM_TAG_TYPE)ELLMTag::PlatformTagStart + 11,
};

#define LLM_TRACK_VULKAN_HIGH_LEVEL_ALLOC(...)
#define LLM_TRACK_VULKAN_HIGH_LEVEL_FREE(...)
#define LLM_TRACK_VULKAN_SPARE_MEMORY_GPU(...)

#endif		// #if VULKAN_USE_LLM

#define LLM_SCOPE_VULKAN(Tag)			LLM_SCOPE((ELLMTag)Tag)
#define LLM_PLATFORM_SCOPE_VULKAN(Tag)	LLM_PLATFORM_SCOPE((ELLMTag)Tag)
static_assert((int32)ELLMTagVulkan::Count == (int32)ELLMTag::PlatformTagStart + 11, "There needs to be a 1 to 1 mapping between VulkanTag and Translation");

#else		// #if ENABLE_LOW_LEVEL_MEM_TRACKER

#define LLM_SCOPE_VULKAN(...)
#define LLM_PLATFORM_SCOPE_VULKAN(...)
#define LLM_TRACK_VULKAN_HIGH_LEVEL_ALLOC(...)
#define LLM_TRACK_VULKAN_HIGH_LEVEL_FREE(...)
#define LLM_TRACK_VULKAN_SPARE_MEMORY_GPU(...)

#endif		// #if ENABLE_LOW_LEVEL_MEM_TRACKER