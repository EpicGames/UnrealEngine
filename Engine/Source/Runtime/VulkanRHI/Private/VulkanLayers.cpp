// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanLayers.cpp: Vulkan device layers implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanDevice.h"
#include "VulkanExtensions.h"
#include "IHeadMountedDisplayModule.h"
#include "IHeadMountedDisplayVulkanExtensions.h"
#include "Misc/CommandLine.h"

#if VULKAN_HAS_DEBUGGING_ENABLED
bool GRenderDocFound = false;
#endif

#if VULKAN_ENABLE_DESKTOP_HMD_SUPPORT
#include "IHeadMountedDisplayModule.h"
#endif

#if VULKAN_HAS_DEBUGGING_ENABLED
TAutoConsoleVariable<int32> GValidationCvar(
	TEXT("r.Vulkan.EnableValidation"),
	VULKAN_VALIDATION_DEFAULT_VALUE,
	TEXT("0 to disable validation layers\n")
	TEXT("1 to enable errors\n")
	TEXT("2 to enable errors & warnings\n")
	TEXT("3 to enable errors, warnings & performance warnings\n")
	TEXT("4 to enable errors, warnings, performance & information messages\n")
	TEXT("5 to enable all messages"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> GGPUValidationCvar(
	TEXT("r.Vulkan.GPUValidation"),
	0,
	TEXT("2 to use enable GPU assisted validation AND extra binding slot when using validation layers\n")
	TEXT("1 to use enable GPU assisted validation when using validation layers, or\n")
	TEXT("0 to not use (default)"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

#if VULKAN_ENABLE_DRAW_MARKERS
	#define RENDERDOC_LAYER_NAME				"VK_LAYER_RENDERDOC_Capture"
#endif

#define KHRONOS_STANDARD_VALIDATION_LAYER_NAME	"VK_LAYER_KHRONOS_validation"

#endif // VULKAN_HAS_DEBUGGING_ENABLED



#define VERIFYVULKANRESULT_INIT(VkFunction)	{ const VkResult ScopedResult = VkFunction; \
											if (ScopedResult == VK_ERROR_INITIALIZATION_FAILED) { \
												UE_LOG(LogVulkanRHI, Error, \
												TEXT("%s failed\n at %s:%u\nThis typically means Vulkan is not properly set up in your system; try running vulkaninfo from the Vulkan SDK."), \
												ANSI_TO_TCHAR(#VkFunction), ANSI_TO_TCHAR(__FILE__), __LINE__); } \
											else if (ScopedResult < VK_SUCCESS) { \
												VulkanRHI::VerifyVulkanResult(ScopedResult, #VkFunction, __FILE__, __LINE__); }}


// Package a layer with its extensions
struct FLayerWithExtensions
{
	VkLayerProperties LayerProperties;
	TArray<VkExtensionProperties> ExtensionProperties;
};



class FVulkanIntanceSetupHelper
{
public:
	TArray<FLayerWithExtensions> EnumerateLayerProperties() const
	{
		TArray<FLayerWithExtensions> OutLayerProperties;
		TArray<VkLayerProperties> TempLayerProperties;
		uint32 Count = 0;
		VERIFYVULKANRESULT_INIT(VulkanRHI::vkEnumerateInstanceLayerProperties(&Count, nullptr));
		if (Count > 0)
		{
			TempLayerProperties.AddZeroed(Count);
			VERIFYVULKANRESULT_INIT(VulkanRHI::vkEnumerateInstanceLayerProperties(&Count, TempLayerProperties.GetData()));
			OutLayerProperties.SetNum(Count);
			for (uint32 i=0; i < Count; ++i)
			{
				OutLayerProperties[i].LayerProperties = TempLayerProperties[i];
				OutLayerProperties[i].ExtensionProperties = FVulkanInstanceExtension::GetDriverSupportedInstanceExtensions(TempLayerProperties[i].layerName);

				UE_LOGF(LogVulkanRHI, Display, "Intance Layer %ls provides %d extensions:", ANSI_TO_TCHAR(TempLayerProperties[i].layerName), (int)OutLayerProperties[i].ExtensionProperties.Num());
				for (VkExtensionProperties ep : OutLayerProperties[i].ExtensionProperties)
				{
					UE_LOGF(LogVulkanRHI, Display, "  %ls", ANSI_TO_TCHAR(ep.extensionName));
				}
			}
		}
		return OutLayerProperties;
	}

	TArray<const ANSICHAR*> GetPlatformLayers()
	{
		TArray<const ANSICHAR*> OutPlatformLayers;
		FVulkanPlatform::GetInstanceLayers(OutPlatformLayers);
		return OutPlatformLayers;
	}

	void AddDebugLayers(const TArray<FLayerWithExtensions>& LayerProperties, FVulkanInstanceExtensionArray& UEExtensions, TArray<const ANSICHAR*>& OutLayers);

	const TCHAR* HelperTypeName = TEXT("instance");
	static TArray<const ANSICHAR*> ExternalLayers;
	FVulkanDynamicRHI::EActiveDebugLayerExtension ActiveDebugLayerExtension = FVulkanDynamicRHI::EActiveDebugLayerExtension::None;
};

TArray<const ANSICHAR*> FVulkanIntanceSetupHelper::ExternalLayers;


class FVulkanDeviceSetupHelper
{
public:
	TArray<FLayerWithExtensions> EnumerateLayerProperties() const
	{
		TArray<FLayerWithExtensions> OutLayerProperties;
		TArray<VkLayerProperties> TempLayerProperties;
		uint32 Count = 0;
		VERIFYVULKANRESULT_INIT(VulkanRHI::vkEnumerateDeviceLayerProperties(Gpu, &Count, nullptr));
		if (Count > 0)
		{
			TempLayerProperties.AddZeroed(Count);
			VERIFYVULKANRESULT_INIT(VulkanRHI::vkEnumerateDeviceLayerProperties(Gpu, &Count, TempLayerProperties.GetData()));
			OutLayerProperties.SetNum(Count);
			for (uint32 i = 0; i < Count; ++i)
			{
				OutLayerProperties[i].LayerProperties = TempLayerProperties[i];
				OutLayerProperties[i].ExtensionProperties = FVulkanDeviceExtension::GetDriverSupportedDeviceExtensions(Gpu, TempLayerProperties[i].layerName);

				UE_LOGF(LogVulkanRHI, Display, "Device Layer %ls provides %d extensions:", ANSI_TO_TCHAR(TempLayerProperties[i].layerName), (int)OutLayerProperties[i].ExtensionProperties.Num());
				for (VkExtensionProperties ep : OutLayerProperties[i].ExtensionProperties)
				{
					UE_LOGF(LogVulkanRHI, Display, "  %ls", ANSI_TO_TCHAR(ep.extensionName));
				}
			}
		}
		return OutLayerProperties;
	}

	TArray<const ANSICHAR*> GetPlatformLayers()
	{
		TArray<const ANSICHAR*> OutPlatformLayers;
		FVulkanPlatform::GetDeviceLayers(OutPlatformLayers);
		return OutPlatformLayers;
	}

	void AddDebugLayers(const TArray<FLayerWithExtensions>& LayerProperties, FVulkanDeviceExtensionArray& UEExtensions, TArray<const ANSICHAR*>& OutLayers);

	const TCHAR* HelperTypeName = TEXT("device");
	static TArray<const ANSICHAR*> ExternalLayers;
	VkPhysicalDevice Gpu = VK_NULL_HANDLE;
	bool bSupportsDebugUtilsExt = false;
	bool bSupportsDebugMarkerExt = false;
};

TArray<const ANSICHAR*> FVulkanDeviceSetupHelper::ExternalLayers;

#undef VERIFYVULKANRESULT_INIT




// Helper function to find a layer name in a list of LayerProperties
static inline int32 FindLayerIndexInList(const char* LayerName, const TArray<FLayerWithExtensions>& LayerProperties)
{
	for (int32 Index = 0; Index < LayerProperties.Num(); ++Index)
	{
		if (!FCStringAnsi::Strcmp(LayerProperties[Index].LayerProperties.layerName, LayerName))
		{
			return Index;
		}
	}

	return INDEX_NONE;
}



// Helper function to add a layer to a list and flag its extensions if it's found
template <typename ExtensionType>
static inline bool AddRequestedLayer(const ANSICHAR* LayerName, const TArray<FLayerWithExtensions>& LayerProperties, TArray<TUniquePtr<ExtensionType>>& UEExtensions, TArray<const ANSICHAR*>& OutLayers)
{
	// Find the layer in the list
	int32 LayerIndex = FindLayerIndexInList(LayerName, LayerProperties);
	if (LayerIndex == INDEX_NONE)
	{
		return false;
	}

	// Add it to the list of used layers
	OutLayers.Add(LayerName);


	// Helper function to flag an extension as supported by the driver
	auto FlagExtensionSupport = [](TArray<TUniquePtr<ExtensionType>>& UEExtensions, const ANSICHAR* ExtensionName)
	{
		for (TUniquePtr<ExtensionType>& Extension : UEExtensions)
		{
			if (!FCStringAnsi::Strcmp(Extension->GetExtensionName(), ExtensionName))
			{
				Extension->SetSupported();
				return true;
			}
		}
		return false;
	};

	// Flag its extensions as usable
	for (const VkExtensionProperties& ExtensionProperties : LayerProperties[LayerIndex].ExtensionProperties)
	{
		FlagExtensionSupport(UEExtensions, ExtensionProperties.extensionName);
	}

	return true;
}



template <typename SetupHelperType, typename ExtensionType>
static TArray<const ANSICHAR*> SetupLayers(SetupHelperType& VulkanSetupHelper, TArray<TUniquePtr<ExtensionType>>& UEExtensions)
{
	TArray<const ANSICHAR*> OutLayers;

	// Fetch the list of supported layers
	TArray<FLayerWithExtensions> LayerProperties = VulkanSetupHelper.EnumerateLayerProperties();
	LayerProperties.Sort([](const FLayerWithExtensions& A, const FLayerWithExtensions& B) { return FCStringAnsi::Strcmp(A.LayerProperties.layerName, B.LayerProperties.layerName) < 0; });

	UE_LOGF(LogVulkanRHI, Display, "Found %d available %ls layers %ls", LayerProperties.Num(), VulkanSetupHelper.HelperTypeName, LayerProperties.Num() ? TEXT(":") : TEXT("!"));
	for (const FLayerWithExtensions& Prop : LayerProperties)
	{
		UE_LOGF(LogVulkanRHI, Display, "  * %ls", ANSI_TO_TCHAR(Prop.LayerProperties.layerName));
	}

	// Check for layers added outside the RHI (eg plugins)
	for (const ANSICHAR* VulkanBridgeLayer : VulkanSetupHelper.ExternalLayers)
	{
		if (!AddRequestedLayer(VulkanBridgeLayer, LayerProperties, UEExtensions, OutLayers))
		{
			UE_LOGF(LogVulkanRHI, Warning, "Unable to find VulkanExternalExtensions %ls layer '%ls'", VulkanSetupHelper.HelperTypeName, ANSI_TO_TCHAR(VulkanBridgeLayer));
		}
	}

	// Check for platform specific layers
	TArray<const ANSICHAR*> PlatformLayers = VulkanSetupHelper.GetPlatformLayers();
	for (const ANSICHAR* PlatformLayer : PlatformLayers)
	{
		if (!AddRequestedLayer(PlatformLayer, LayerProperties, UEExtensions, OutLayers))
		{
			UE_LOGF(LogVulkanRHI, Warning, "Unable to find platform %ls layer '%ls'", VulkanSetupHelper.HelperTypeName, ANSI_TO_TCHAR(PlatformLayer));
		}
	}

	// Check for any requested debug layers
	VulkanSetupHelper.AddDebugLayers(LayerProperties, UEExtensions, OutLayers);

	// Clean up the resulting array
	if (OutLayers.Num() > 0)
	{
		auto TrimDuplicates = [](TArray<const ANSICHAR*>& Array)
		{
			for (int32 OuterIndex = Array.Num() - 1; OuterIndex >= 0; --OuterIndex)
			{
				bool bFound = false;
				for (int32 InnerIndex = OuterIndex - 1; InnerIndex >= 0; --InnerIndex)
				{
					if (!FCStringAnsi::Strcmp(Array[OuterIndex], Array[InnerIndex]))
					{
						bFound = true;
						break;
					}
				}

				if (bFound)
				{
					Array.RemoveAtSwap(OuterIndex, EAllowShrinking::No);
				}
			}
		};

		TrimDuplicates(OutLayers);
	}
	OutLayers.Sort();

	return OutLayers;
}


void IVulkanDynamicRHI::AddEnabledInstanceExtensionsAndLayers(TArrayView<const ANSICHAR* const> InInstanceExtensions, TArrayView<const ANSICHAR* const> InInstanceLayers)
{
	checkf(!GVulkanRHI, TEXT("AddEnabledInstanceExtensionsAndLayers should be called before the VulkanRHI has been created"));
	FVulkanInstanceExtension::ExternalExtensions.Append(InInstanceExtensions.GetData(), InInstanceExtensions.Num());
	FVulkanIntanceSetupHelper::ExternalLayers.Append(InInstanceLayers.GetData(), InInstanceLayers.Num());
}

void IVulkanDynamicRHI::AddEnabledDeviceExtensionsAndLayers(TArrayView<TSharedRef<FVulkanRHIExternalDeviceExtensionBase>> InDeviceExtensions, TArrayView<const ANSICHAR* const> InDeviceLayers)
{
	checkf(!GVulkanRHI, TEXT("AddEnabledDeviceExtensionsAndLayers should be called before the VulkanRHI has been created"));
	FVulkanDeviceExtension::ExternalExtensions.Append(InDeviceExtensions.GetData(), InDeviceExtensions.Num());
	FVulkanDeviceSetupHelper::ExternalLayers.Append(InDeviceLayers.GetData(), InDeviceLayers.Num());
}

TArray<const ANSICHAR*> FVulkanDynamicRHI::SetupInstanceLayers(FVulkanInstanceExtensionArray& UEExtensions)
{
	FVulkanIntanceSetupHelper InstanceHelper;
	TArray<const ANSICHAR*> OutInstanceLayers = SetupLayers(InstanceHelper, UEExtensions);
	ActiveDebugLayerExtension = InstanceHelper.ActiveDebugLayerExtension;
	return OutInstanceLayers;
}

TArray<const ANSICHAR*> FVulkanDevice::SetupDeviceLayers(FVulkanDeviceExtensionArray& UEExtensions)
{
	FVulkanDeviceSetupHelper DeviceHelper;
	DeviceHelper.Gpu = Gpu;
	DeviceHelper.bSupportsDebugUtilsExt = RHI->SupportsDebugUtilsExt();
	TArray<const ANSICHAR*> OutDeviceLayers = SetupLayers(DeviceHelper, UEExtensions);
#if VULKAN_ENABLE_DRAW_MARKERS
	bUseLegacyDebugMarkerExt = !DeviceHelper.bSupportsDebugUtilsExt && DeviceHelper.bSupportsDebugMarkerExt;
#endif
	return OutDeviceLayers;
}

void FVulkanDynamicRHI::SetupValidationRequests()
{
#if VULKAN_HAS_DEBUGGING_ENABLED
	int32 VulkanValidationOption = GValidationCvar.GetValueOnAnyThread();

	// Command line overrides Cvar
	if (FParse::Param(FCommandLine::Get(), TEXT("vulkandebug")))
	{
		// Match D3D and GL
		GValidationCvar->Set(2, ECVF_SetByCommandline);
	}
	else if (FParse::Value(FCommandLine::Get(), TEXT("vulkanvalidation="), VulkanValidationOption))
	{
		GValidationCvar->Set(VulkanValidationOption, ECVF_SetByCommandline);
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("gpuvalidation")))
	{
		if (GValidationCvar->GetInt() < 2)
		{
			GValidationCvar->Set(2, ECVF_SetByCommandline);
		}
		GGPUValidationCvar->Set(2, ECVF_SetByCommandline);
	}
	GRHIGlobals.IsDebugLayerEnabled = (GValidationCvar.GetValueOnAnyThread() > 0);
#endif
}


// Get a list of debugging layers to activate
void FVulkanIntanceSetupHelper::AddDebugLayers(const TArray<FLayerWithExtensions>& LayerProperties, FVulkanInstanceExtensionArray& UEExtensions, TArray<const ANSICHAR*>& OutLayers)
{
	if (FParse::Param(FCommandLine::Get(), TEXT("vktrace")))
	{
		const char* GfxReconstructName = "VK_LAYER_LUNARG_gfxreconstruct";
		if (AddRequestedLayer(GfxReconstructName, LayerProperties, UEExtensions, OutLayers))
		{
			ActiveDebugLayerExtension = FVulkanDynamicRHI::EActiveDebugLayerExtension::GfxReconstructLayer;
		}
		else
		{
			const char* VkTraceName = "VK_LAYER_LUNARG_vktrace";
			if (AddRequestedLayer(VkTraceName, LayerProperties, UEExtensions, OutLayers))
			{
				ActiveDebugLayerExtension = FVulkanDynamicRHI::EActiveDebugLayerExtension::VkTraceLayer;
			}
		}
	}

	const bool bGfxReconstructOrVkTrace = (ActiveDebugLayerExtension != FVulkanDynamicRHI::EActiveDebugLayerExtension::None);

#if VULKAN_HAS_DEBUGGING_ENABLED
	if (FParse::Param(FCommandLine::Get(), TEXT("vulkanapidump")))
	{
		if (bGfxReconstructOrVkTrace)
		{
			UE_LOGF(LogVulkanRHI, Warning, "Can't enable api_dump when GfxReconstruct/VkTrace is enabled.");
		}
		else
		{
			const char* VkApiDumpName = "VK_LAYER_LUNARG_api_dump";
			const bool bApiDumpFound = AddRequestedLayer(VkApiDumpName, LayerProperties, UEExtensions, OutLayers);
			if (bApiDumpFound)
			{
				const FString ApiDumpFileName = FString::Printf(TEXT("%s/vk_apidump.%s.txt"), *FPaths::ProjectLogDir(), *FDateTime::Now().ToString());
				FPlatformMisc::SetEnvironmentVar(TEXT("VK_APIDUMP_LOG_FILENAME"), *ApiDumpFileName);
				FPlatformMisc::SetEnvironmentVar(TEXT("VK_APIDUMP_DETAILED"), TEXT("true"));
				FPlatformMisc::SetEnvironmentVar(TEXT("VK_APIDUMP_FLUSH"), TEXT("true"));
				FPlatformMisc::SetEnvironmentVar(TEXT("VK_APIDUMP_OUTPUT_FORMAT"), TEXT("text"));
			}
			else
			{
				UE_LOGF(LogVulkanRHI, Warning, "Unable to find Vulkan instance layer %ls", ANSI_TO_TCHAR(VkApiDumpName));
			}
		}
	}

	// At this point the CVar holds the final value
	const bool bUseVulkanValidation = GRHIGlobals.IsDebugLayerEnabled;
	if (!bGfxReconstructOrVkTrace && bUseVulkanValidation)
	{
		if (!AddRequestedLayer(KHRONOS_STANDARD_VALIDATION_LAYER_NAME, LayerProperties, UEExtensions, OutLayers))
		{
#if PLATFORM_WINDOWS || PLATFORM_LINUX
			//#todo-rco: We don't package DLLs so if this fails it means no DLL was found anywhere, so don't try to load standard validation layers
			UE_LOGF(LogVulkanRHI, Warning, "Unable to find Vulkan instance validation layer %ls;  Do you have the Vulkan SDK Installed?", TEXT(KHRONOS_STANDARD_VALIDATION_LAYER_NAME));
#else
			UE_LOGF(LogVulkanRHI, Warning, "Unable to find Vulkan instance validation layer %ls", TEXT(KHRONOS_STANDARD_VALIDATION_LAYER_NAME));
#endif
		}
	}

	bool bVulkanEnableDrawMarkers = VULKAN_ENABLE_DRAW_MARKERS;
	const bool bForceDebugUtils = bVulkanEnableDrawMarkers || FParse::Param(FCommandLine::Get(), TEXT("vulkandebugutils"));
	if ((bUseVulkanValidation || bForceDebugUtils) && (ActiveDebugLayerExtension == FVulkanDynamicRHI::EActiveDebugLayerExtension::None))
	{
		auto FindLayerContainingExtension = [](const ANSICHAR* ExtensionName, const TArray<FLayerWithExtensions>& LayerProperties)
		{
			for (int32 LayerIndex = 0; LayerIndex < LayerProperties.Num(); ++LayerIndex)
			{
				for (int32 ExtIndex = 0; ExtIndex < LayerProperties[LayerIndex].ExtensionProperties.Num(); ++ExtIndex)
				{
					if (!FCStringAnsi::Strcmp(LayerProperties[LayerIndex].ExtensionProperties[ExtIndex].extensionName, ExtensionName))
					{
						return LayerIndex;
					}
				}
			}
			return (int32)INDEX_NONE;
		};

		auto ActivateDebuggingExtension = [&](const ANSICHAR* ExtensionName) {

			const int32 ExtensionIndex = FVulkanInstanceExtension::FindExtension(UEExtensions, ExtensionName);
			check(ExtensionIndex != INDEX_NONE);

			// If the extension is supported, activate it and set it as our active debugging extension
			if (UEExtensions[ExtensionIndex]->IsSupported())
			{
				UEExtensions[ExtensionIndex]->SetActivated();
				return true;
			}
			return false;
		};

		if (ActivateDebuggingExtension(VK_EXT_DEBUG_UTILS_EXTENSION_NAME))
		{
			ActiveDebugLayerExtension = FVulkanDynamicRHI::EActiveDebugLayerExtension::DebugUtilsExtension;
		}

		const bool bRequiresValidationFeatures = (GGPUValidationCvar.GetValueOnAnyThread() != 0) ||
			(FParse::Param(FCommandLine::Get(), TEXT("vulkanbestpractices"))) ||
			(FParse::Param(FCommandLine::Get(), TEXT("vulkandebugsync")));
		if (bRequiresValidationFeatures)
		{
			ActivateDebuggingExtension(VK_EXT_VALIDATION_FEATURES_EXTENSION_NAME);
		}
	}
#endif	// VULKAN_HAS_DEBUGGING_ENABLED
}

// Return a list of debug layers to activate
void FVulkanDeviceSetupHelper::AddDebugLayers(const TArray<FLayerWithExtensions>& LayerProperties, FVulkanDeviceExtensionArray& UEExtensions, TArray<const ANSICHAR*>& OutLayers)
{
#if VULKAN_HAS_DEBUGGING_ENABLED
#if VULKAN_ENABLE_DRAW_MARKERS
	GRenderDocFound = (FindLayerIndexInList(RENDERDOC_LAYER_NAME, LayerProperties) != INDEX_NONE);
#else
	GRenderDocFound = false;
#endif // VULKAN_ENABLE_DRAW_MARKERS
#endif // VULKAN_HAS_DEBUGGING_ENABLED

#if VULKAN_ENABLE_DRAW_MARKERS
	// If debug_utils is supported we don't need the legacy extension debug_marker
	if (!bSupportsDebugUtilsExt)
	{
		const int32 ExtensionIndex = FVulkanDeviceExtension::FindExtension(UEExtensions, VK_EXT_DEBUG_MARKER_EXTENSION_NAME);
		check(ExtensionIndex != INDEX_NONE);

		// If the extension is supported, activate it and set it as our active debugging extension
		if (UEExtensions[ExtensionIndex]->IsSupported())
		{
			UEExtensions[ExtensionIndex]->SetActivated();
			bSupportsDebugMarkerExt = true;
		}
	}
#endif // VULKAN_ENABLE_DRAW_MARKERS
}
