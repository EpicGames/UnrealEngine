// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanLayout.h"
#include "VulkanBindlessDescriptorManager.h"

FVulkanLayout::FVulkanLayout(FVulkanDevice& InDevice, bool InGfxLayout, bool InUsesBindless)
	: Device(InDevice)
	, DescriptorSetLayout(&InDevice)
	, PipelineLayout(VK_NULL_HANDLE)
	, bIsGfxLayout(InGfxLayout)
	, bUsesBindless(InUsesBindless)
{
}

FVulkanLayout::~FVulkanLayout()
{
	if (!bUsesBindless && PipelineLayout != VK_NULL_HANDLE)
	{
		Device.GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::PipelineLayout, PipelineLayout);
		PipelineLayout = VK_NULL_HANDLE;
	}
}

void FVulkanLayout::Compile(FVulkanDescriptorSetLayoutMap& DSetLayoutMap)
{
	check(PipelineLayout == VK_NULL_HANDLE);

	DescriptorSetLayout.Compile(DSetLayoutMap);

	if (bUsesBindless)
	{
		// Get the bindless manager's layout
		PipelineLayout = Device.GetBindlessDescriptorManager()->GetPipelineLayout();
	}
	else
	{
		VkPipelineLayoutCreateInfo PipelineLayoutCreateInfo;
		ZeroVulkanStruct(PipelineLayoutCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_LAYOUT_CREATE_INFO);

		const TArray<VkDescriptorSetLayout>& LayoutHandles = DescriptorSetLayout.GetHandles();
		PipelineLayoutCreateInfo.setLayoutCount = LayoutHandles.Num();
		PipelineLayoutCreateInfo.pSetLayouts = LayoutHandles.GetData();
		//PipelineLayoutCreateInfo.pushConstantRangeCount = 0;
		VERIFYVULKANRESULT(VulkanRHI::vkCreatePipelineLayout(Device.GetHandle(), &PipelineLayoutCreateInfo, VULKAN_CPU_ALLOCATOR, &PipelineLayout));
	}
}
