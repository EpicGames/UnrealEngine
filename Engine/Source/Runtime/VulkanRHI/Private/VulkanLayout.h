// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VulkanMemory.h"
#include "VulkanDescriptorSets.h"

// Layout for a Pipeline, also includes DescriptorSets layout
class FVulkanLayout
{
public:
	FVulkanLayout(FVulkanDevice& InDevice, bool InGfxLayout, bool InUsesBindless);
	virtual ~FVulkanLayout();

	const FVulkanDescriptorSetsLayout& GetDescriptorSetsLayout() const
	{
		return DescriptorSetLayout;
	}

	VkPipelineLayout GetPipelineLayout() const
	{
		return PipelineLayout;
	}

	bool HasDescriptors() const
	{
		return DescriptorSetLayout.GetLayouts().Num() > 0;
	}

	uint32 GetDescriptorSetLayoutHash() const
	{
		return DescriptorSetLayout.GetHash();
	}

	bool IsGfxLayout() const
	{
		return bIsGfxLayout;
	}

protected:
	FVulkanDevice& Device;
	FVulkanDescriptorSetsLayout	DescriptorSetLayout;
	VkPipelineLayout			PipelineLayout;

	const bool bIsGfxLayout;
	const bool bUsesBindless;

	void Compile(FVulkanDescriptorSetLayoutMap& DSetLayoutMap);

	friend class FVulkanComputePipeline;
	friend class FVulkanGfxPipeline;
	friend class FVulkanPipelineStateCacheManager;
	friend class FVulkanRayTracingPipelineState;
};
