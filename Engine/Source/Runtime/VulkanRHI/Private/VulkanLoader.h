// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VulkanPlatform.h"
#include "VulkanThirdParty.h"

// List all instance Vulkan entry points used by Unreal that need to be loaded manually
#define ENUM_VK_ENTRYPOINTS_INSTANCE(EnumMacro) \
	EnumMacro(PFN_vkDestroyInstance, vkDestroyInstance) \
	EnumMacro(PFN_vkEnumeratePhysicalDevices, vkEnumeratePhysicalDevices) \
	EnumMacro(PFN_vkGetPhysicalDeviceFeatures, vkGetPhysicalDeviceFeatures) \
	EnumMacro(PFN_vkGetPhysicalDeviceFormatProperties, vkGetPhysicalDeviceFormatProperties) \
	EnumMacro(PFN_vkGetPhysicalDeviceFormatProperties2, vkGetPhysicalDeviceFormatProperties2) \
	EnumMacro(PFN_vkGetPhysicalDeviceImageFormatProperties, vkGetPhysicalDeviceImageFormatProperties) \
	EnumMacro(PFN_vkGetPhysicalDeviceImageFormatProperties2, vkGetPhysicalDeviceImageFormatProperties2) \
	EnumMacro(PFN_vkGetPhysicalDeviceProperties, vkGetPhysicalDeviceProperties) \
	EnumMacro(PFN_vkGetPhysicalDeviceQueueFamilyProperties, vkGetPhysicalDeviceQueueFamilyProperties) \
	EnumMacro(PFN_vkGetPhysicalDeviceQueueFamilyProperties2, vkGetPhysicalDeviceQueueFamilyProperties2) \
	EnumMacro(PFN_vkGetPhysicalDeviceMemoryProperties, vkGetPhysicalDeviceMemoryProperties) \
	EnumMacro(PFN_vkGetPhysicalDeviceProperties2, vkGetPhysicalDeviceProperties2) \
	EnumMacro(PFN_vkGetPhysicalDeviceFeatures2, vkGetPhysicalDeviceFeatures2) \
	EnumMacro(PFN_vkGetPhysicalDeviceMemoryProperties2, vkGetPhysicalDeviceMemoryProperties2) \
	EnumMacro(PFN_vkCreateDevice, vkCreateDevice) \
	EnumMacro(PFN_vkDestroyDevice, vkDestroyDevice) \
	EnumMacro(PFN_vkEnumerateDeviceExtensionProperties, vkEnumerateDeviceExtensionProperties) \
	EnumMacro(PFN_vkEnumerateDeviceLayerProperties, vkEnumerateDeviceLayerProperties) \
	EnumMacro(PFN_vkGetDeviceQueue, vkGetDeviceQueue) \
	EnumMacro(PFN_vkQueueSubmit, vkQueueSubmit) \
	EnumMacro(PFN_vkQueueWaitIdle, vkQueueWaitIdle) \
	EnumMacro(PFN_vkDeviceWaitIdle, vkDeviceWaitIdle) \
	EnumMacro(PFN_vkAllocateMemory, vkAllocateMemory) \
	EnumMacro(PFN_vkFreeMemory, vkFreeMemory) \
	EnumMacro(PFN_vkMapMemory, vkMapMemory) \
	EnumMacro(PFN_vkUnmapMemory, vkUnmapMemory) \
	EnumMacro(PFN_vkFlushMappedMemoryRanges, vkFlushMappedMemoryRanges) \
	EnumMacro(PFN_vkInvalidateMappedMemoryRanges, vkInvalidateMappedMemoryRanges) \
	EnumMacro(PFN_vkGetDeviceMemoryCommitment, vkGetDeviceMemoryCommitment) \
	EnumMacro(PFN_vkBindBufferMemory, vkBindBufferMemory) \
	EnumMacro(PFN_vkBindImageMemory, vkBindImageMemory) \
	EnumMacro(PFN_vkBindBufferMemory2, vkBindBufferMemory2) \
	EnumMacro(PFN_vkBindImageMemory2, vkBindImageMemory2) \
	EnumMacro(PFN_vkGetBufferMemoryRequirements, vkGetBufferMemoryRequirements) \
	EnumMacro(PFN_vkGetImageMemoryRequirements, vkGetImageMemoryRequirements) \
	EnumMacro(PFN_vkGetImageSparseMemoryRequirements, vkGetImageSparseMemoryRequirements) \
	EnumMacro(PFN_vkGetImageMemoryRequirements2, vkGetImageMemoryRequirements2) \
	EnumMacro(PFN_vkGetBufferMemoryRequirements2, vkGetBufferMemoryRequirements2) \
	EnumMacro(PFN_vkGetPhysicalDeviceSparseImageFormatProperties, vkGetPhysicalDeviceSparseImageFormatProperties) \
	EnumMacro(PFN_vkQueueBindSparse, vkQueueBindSparse) \
	EnumMacro(PFN_vkCreateFence, vkCreateFence) \
	EnumMacro(PFN_vkDestroyFence, vkDestroyFence) \
	EnumMacro(PFN_vkResetFences, vkResetFences) \
	EnumMacro(PFN_vkGetFenceStatus, vkGetFenceStatus) \
	EnumMacro(PFN_vkWaitForFences, vkWaitForFences) \
	EnumMacro(PFN_vkCreateSemaphore, vkCreateSemaphore) \
	EnumMacro(PFN_vkDestroySemaphore, vkDestroySemaphore) \
	EnumMacro(PFN_vkCreateEvent, vkCreateEvent) \
	EnumMacro(PFN_vkDestroyEvent, vkDestroyEvent) \
	EnumMacro(PFN_vkGetEventStatus, vkGetEventStatus) \
	EnumMacro(PFN_vkSetEvent, vkSetEvent) \
	EnumMacro(PFN_vkResetEvent, vkResetEvent) \
	EnumMacro(PFN_vkCreateQueryPool, vkCreateQueryPool) \
	EnumMacro(PFN_vkDestroyQueryPool, vkDestroyQueryPool) \
	EnumMacro(PFN_vkGetQueryPoolResults, vkGetQueryPoolResults) \
	EnumMacro(PFN_vkCreateBuffer, vkCreateBuffer) \
	EnumMacro(PFN_vkDestroyBuffer, vkDestroyBuffer) \
	EnumMacro(PFN_vkCreateBufferView, vkCreateBufferView) \
	EnumMacro(PFN_vkDestroyBufferView, vkDestroyBufferView) \
	EnumMacro(PFN_vkCreateImage, vkCreateImage) \
	EnumMacro(PFN_vkDestroyImage, vkDestroyImage) \
	EnumMacro(PFN_vkGetImageSubresourceLayout, vkGetImageSubresourceLayout) \
	EnumMacro(PFN_vkCreateImageView, vkCreateImageView) \
	EnumMacro(PFN_vkDestroyImageView, vkDestroyImageView) \
	EnumMacro(PFN_vkCreateShaderModule, vkCreateShaderModule) \
	EnumMacro(PFN_vkDestroyShaderModule, vkDestroyShaderModule) \
	EnumMacro(PFN_vkCreatePipelineCache, vkCreatePipelineCache) \
	EnumMacro(PFN_vkDestroyPipelineCache, vkDestroyPipelineCache) \
	EnumMacro(PFN_vkGetPipelineCacheData, vkGetPipelineCacheData) \
	EnumMacro(PFN_vkMergePipelineCaches, vkMergePipelineCaches) \
	EnumMacro(PFN_vkCreateGraphicsPipelines, vkCreateGraphicsPipelines) \
	EnumMacro(PFN_vkCreateComputePipelines, vkCreateComputePipelines) \
	EnumMacro(PFN_vkDestroyPipeline, vkDestroyPipeline) \
	EnumMacro(PFN_vkCreatePipelineLayout, vkCreatePipelineLayout) \
	EnumMacro(PFN_vkDestroyPipelineLayout, vkDestroyPipelineLayout) \
	EnumMacro(PFN_vkCreateSampler, vkCreateSampler) \
	EnumMacro(PFN_vkDestroySampler, vkDestroySampler) \
	EnumMacro(PFN_vkCreateDescriptorSetLayout, vkCreateDescriptorSetLayout) \
	EnumMacro(PFN_vkDestroyDescriptorSetLayout, vkDestroyDescriptorSetLayout) \
	EnumMacro(PFN_vkCreateDescriptorPool, vkCreateDescriptorPool) \
	EnumMacro(PFN_vkDestroyDescriptorPool, vkDestroyDescriptorPool) \
	EnumMacro(PFN_vkResetDescriptorPool, vkResetDescriptorPool) \
	EnumMacro(PFN_vkAllocateDescriptorSets, vkAllocateDescriptorSets) \
	EnumMacro(PFN_vkFreeDescriptorSets, vkFreeDescriptorSets) \
	EnumMacro(PFN_vkUpdateDescriptorSets, vkUpdateDescriptorSets) \
	EnumMacro(PFN_vkCreateFramebuffer, vkCreateFramebuffer) \
	EnumMacro(PFN_vkDestroyFramebuffer, vkDestroyFramebuffer) \
	EnumMacro(PFN_vkCreateRenderPass, vkCreateRenderPass) \
	EnumMacro(PFN_vkDestroyRenderPass, vkDestroyRenderPass) \
	EnumMacro(PFN_vkGetRenderAreaGranularity, vkGetRenderAreaGranularity) \
	EnumMacro(PFN_vkCreateCommandPool, vkCreateCommandPool) \
	EnumMacro(PFN_vkDestroyCommandPool, vkDestroyCommandPool) \
	EnumMacro(PFN_vkResetCommandPool, vkResetCommandPool) \
	EnumMacro(PFN_vkTrimCommandPool, vkTrimCommandPool) \
	EnumMacro(PFN_vkAllocateCommandBuffers, vkAllocateCommandBuffers) \
	EnumMacro(PFN_vkFreeCommandBuffers, vkFreeCommandBuffers) \
	EnumMacro(PFN_vkBeginCommandBuffer, vkBeginCommandBuffer) \
	EnumMacro(PFN_vkEndCommandBuffer, vkEndCommandBuffer) \
	EnumMacro(PFN_vkResetCommandBuffer, vkResetCommandBuffer) \
	EnumMacro(PFN_vkCmdBindPipeline, vkCmdBindPipeline) \
	EnumMacro(PFN_vkCmdSetViewport, vkCmdSetViewport) \
	EnumMacro(PFN_vkCmdSetScissor, vkCmdSetScissor) \
	EnumMacro(PFN_vkCmdSetLineWidth, vkCmdSetLineWidth) \
	EnumMacro(PFN_vkCmdSetDepthBias, vkCmdSetDepthBias) \
	EnumMacro(PFN_vkCmdSetBlendConstants, vkCmdSetBlendConstants) \
	EnumMacro(PFN_vkCmdSetDepthBounds, vkCmdSetDepthBounds) \
	EnumMacro(PFN_vkCmdSetStencilCompareMask, vkCmdSetStencilCompareMask) \
	EnumMacro(PFN_vkCmdSetStencilWriteMask, vkCmdSetStencilWriteMask) \
	EnumMacro(PFN_vkCmdSetStencilReference, vkCmdSetStencilReference) \
	EnumMacro(PFN_vkCmdBindDescriptorSets, vkCmdBindDescriptorSets) \
	EnumMacro(PFN_vkCmdBindIndexBuffer, vkCmdBindIndexBuffer) \
	EnumMacro(PFN_vkCmdBindVertexBuffers, vkCmdBindVertexBuffers) \
	EnumMacro(PFN_vkCmdDraw, vkCmdDraw) \
	EnumMacro(PFN_vkCmdDrawIndexed, vkCmdDrawIndexed) \
	EnumMacro(PFN_vkCmdDrawIndirect, vkCmdDrawIndirect) \
	EnumMacro(PFN_vkCmdDrawIndexedIndirect, vkCmdDrawIndexedIndirect) \
	EnumMacro(PFN_vkCmdDispatch, vkCmdDispatch) \
	EnumMacro(PFN_vkCmdDispatchIndirect, vkCmdDispatchIndirect) \
	EnumMacro(PFN_vkCmdCopyBuffer, vkCmdCopyBuffer) \
	EnumMacro(PFN_vkCmdCopyImage, vkCmdCopyImage) \
	EnumMacro(PFN_vkCmdBlitImage, vkCmdBlitImage) \
	EnumMacro(PFN_vkCmdCopyBufferToImage, vkCmdCopyBufferToImage) \
	EnumMacro(PFN_vkCmdCopyImageToBuffer, vkCmdCopyImageToBuffer) \
	EnumMacro(PFN_vkCmdUpdateBuffer, vkCmdUpdateBuffer) \
	EnumMacro(PFN_vkCmdFillBuffer, vkCmdFillBuffer) \
	EnumMacro(PFN_vkCmdClearColorImage, vkCmdClearColorImage) \
	EnumMacro(PFN_vkCmdClearDepthStencilImage, vkCmdClearDepthStencilImage) \
	EnumMacro(PFN_vkCmdClearAttachments, vkCmdClearAttachments) \
	EnumMacro(PFN_vkCmdResolveImage, vkCmdResolveImage) \
	EnumMacro(PFN_vkCmdSetEvent, vkCmdSetEvent) \
	EnumMacro(PFN_vkCmdResetEvent, vkCmdResetEvent) \
	EnumMacro(PFN_vkCmdWaitEvents, vkCmdWaitEvents) \
	EnumMacro(PFN_vkCmdPipelineBarrier, vkCmdPipelineBarrier) \
	EnumMacro(PFN_vkCmdBeginQuery, vkCmdBeginQuery) \
	EnumMacro(PFN_vkCmdEndQuery, vkCmdEndQuery) \
	EnumMacro(PFN_vkCmdResetQueryPool, vkCmdResetQueryPool) \
	EnumMacro(PFN_vkCmdWriteTimestamp, vkCmdWriteTimestamp) \
	EnumMacro(PFN_vkCmdCopyQueryPoolResults, vkCmdCopyQueryPoolResults) \
	EnumMacro(PFN_vkCmdPushConstants, vkCmdPushConstants) \
	EnumMacro(PFN_vkCmdBeginRenderPass, vkCmdBeginRenderPass) \
	EnumMacro(PFN_vkCmdNextSubpass, vkCmdNextSubpass) \
	EnumMacro(PFN_vkCmdEndRenderPass, vkCmdEndRenderPass) \
	EnumMacro(PFN_vkCmdEndRenderPass2KHR, vkCmdEndRenderPass2KHR) \
	EnumMacro(PFN_vkCmdExecuteCommands, vkCmdExecuteCommands) \
	EnumMacro(PFN_vkCreateSwapchainKHR, vkCreateSwapchainKHR) \
	EnumMacro(PFN_vkDestroySwapchainKHR, vkDestroySwapchainKHR) \
	EnumMacro(PFN_vkGetSwapchainImagesKHR, vkGetSwapchainImagesKHR) \
	EnumMacro(PFN_vkAcquireNextImageKHR, vkAcquireNextImageKHR) \
	EnumMacro(PFN_vkQueuePresentKHR, vkQueuePresentKHR) \
	EnumMacro(PFN_vkCreateSamplerYcbcrConversion, vkCreateSamplerYcbcrConversion) \
	EnumMacro(PFN_vkDestroySamplerYcbcrConversion, vkDestroySamplerYcbcrConversion)

// List all surface Vulkan entry points used by Unreal that need to be loaded manually
#define ENUM_VK_ENTRYPOINTS_SURFACE_INSTANCE(EnumMacro) \
	EnumMacro(PFN_vkDestroySurfaceKHR, vkDestroySurfaceKHR) \
	EnumMacro(PFN_vkGetPhysicalDeviceSurfaceSupportKHR, vkGetPhysicalDeviceSurfaceSupportKHR) \
	EnumMacro(PFN_vkGetPhysicalDeviceSurfaceCapabilitiesKHR, vkGetPhysicalDeviceSurfaceCapabilitiesKHR) \
	EnumMacro(PFN_vkGetPhysicalDeviceSurfaceFormatsKHR, vkGetPhysicalDeviceSurfaceFormatsKHR) \
	EnumMacro(PFN_vkGetPhysicalDeviceSurfacePresentModesKHR, vkGetPhysicalDeviceSurfacePresentModesKHR)

// List all base Vulkan entry points used by Unreal that need to be loaded manually
#define ENUM_VK_ENTRYPOINTS_BASE(EnumMacro) \
	EnumMacro(PFN_vkCreateInstance, vkCreateInstance) \
	EnumMacro(PFN_vkGetInstanceProcAddr, vkGetInstanceProcAddr) \
	EnumMacro(PFN_vkGetDeviceProcAddr, vkGetDeviceProcAddr) \
	EnumMacro(PFN_vkEnumerateInstanceExtensionProperties, vkEnumerateInstanceExtensionProperties) \
	EnumMacro(PFN_vkEnumerateInstanceLayerProperties, vkEnumerateInstanceLayerProperties)

// List all optional Vulkan entry points used by Unreal that need to be loaded manually
#define ENUM_VK_ENTRYPOINTS_OPTIONAL_BASE(EnumMacro) \
	EnumMacro(PFN_vkEnumerateInstanceVersion, vkEnumerateInstanceVersion) \
	EnumMacro(PFN_vkGetPhysicalDeviceDisplayPropertiesKHR, vkGetPhysicalDeviceDisplayPropertiesKHR) \
	EnumMacro(PFN_vkGetPhysicalDeviceDisplayPlanePropertiesKHR, vkGetPhysicalDeviceDisplayPlanePropertiesKHR) \
	EnumMacro(PFN_vkGetDisplayPlaneSupportedDisplaysKHR, vkGetDisplayPlaneSupportedDisplaysKHR) \
	EnumMacro(PFN_vkGetDisplayModePropertiesKHR, vkGetDisplayModePropertiesKHR) \
	EnumMacro(PFN_vkCreateDisplayModeKHR, vkCreateDisplayModeKHR) \
	EnumMacro(PFN_vkGetDisplayPlaneCapabilitiesKHR, vkGetDisplayPlaneCapabilitiesKHR)

// List all optional Vulkan raytracing entrypoints used by Unreal
#define ENUM_VK_ENTRYPOINTS_RAYTRACING(EnumMacro) \
	EnumMacro(PFN_vkCreateAccelerationStructureKHR, vkCreateAccelerationStructureKHR) \
	EnumMacro(PFN_vkDestroyAccelerationStructureKHR, vkDestroyAccelerationStructureKHR) \
	EnumMacro(PFN_vkCmdBuildAccelerationStructuresKHR, vkCmdBuildAccelerationStructuresKHR) \
	EnumMacro(PFN_vkGetAccelerationStructureBuildSizesKHR, vkGetAccelerationStructureBuildSizesKHR) \
	EnumMacro(PFN_vkGetAccelerationStructureDeviceAddressKHR, vkGetAccelerationStructureDeviceAddressKHR) \
	EnumMacro(PFN_vkCmdTraceRaysKHR, vkCmdTraceRaysKHR) \
	EnumMacro(PFN_vkCmdTraceRaysIndirectKHR, vkCmdTraceRaysIndirectKHR) \
	EnumMacro(PFN_vkCmdTraceRaysIndirect2KHR, vkCmdTraceRaysIndirect2KHR) \
	EnumMacro(PFN_vkCreateRayTracingPipelinesKHR, vkCreateRayTracingPipelinesKHR) \
	EnumMacro(PFN_vkGetRayTracingShaderGroupHandlesKHR, vkGetRayTracingShaderGroupHandlesKHR) \
	EnumMacro(PFN_vkCmdWriteAccelerationStructuresPropertiesKHR, vkCmdWriteAccelerationStructuresPropertiesKHR) \
	EnumMacro(PFN_vkCmdCopyAccelerationStructureKHR, vkCmdCopyAccelerationStructureKHR)

// List all optional Vulkan entry points used by Unreal that need to be loaded manually
#define ENUM_VK_ENTRYPOINTS_OPTIONAL_INSTANCE(EnumMacro) \
	ENUM_VK_ENTRYPOINTS_RAYTRACING(EnumMacro) \
	EnumMacro(PFN_vkCreateRenderPass2KHR, vkCreateRenderPass2KHR) \
	EnumMacro(PFN_vkCmdBeginRenderPass2KHR, vkCmdBeginRenderPass2KHR) \
	EnumMacro(PFN_vkCmdBeginRenderingKHR, vkCmdBeginRenderingKHR) \
	EnumMacro(PFN_vkCmdEndRenderingKHR, vkCmdEndRenderingKHR) \
	EnumMacro(PFN_vkCmdEndRendering2KHR, vkCmdEndRendering2KHR) \
	EnumMacro(PFN_vkGetDeviceImageMemoryRequirementsKHR, vkGetDeviceImageMemoryRequirementsKHR) \
	EnumMacro(PFN_vkGetDeviceBufferMemoryRequirementsKHR, vkGetDeviceBufferMemoryRequirementsKHR) \
	EnumMacro(PFN_vkResetQueryPoolEXT, vkResetQueryPoolEXT) \
	EnumMacro(PFN_vkCmdPipelineBarrier2KHR, vkCmdPipelineBarrier2KHR) \
	EnumMacro(PFN_vkCmdResetEvent2KHR, vkCmdResetEvent2KHR) \
	EnumMacro(PFN_vkCmdSetEvent2KHR, vkCmdSetEvent2KHR) \
	EnumMacro(PFN_vkCmdWaitEvents2KHR, vkCmdWaitEvents2KHR) \
	EnumMacro(PFN_vkQueueSubmit2KHR, vkQueueSubmit2KHR) \
	EnumMacro(PFN_vkCreateSharedSwapchainsKHR, vkCreateSharedSwapchainsKHR) \
	EnumMacro(PFN_vkGetPhysicalDeviceCalibrateableTimeDomainsEXT, vkGetPhysicalDeviceCalibrateableTimeDomainsEXT) \
	EnumMacro(PFN_vkGetCalibratedTimestampsEXT, vkGetCalibratedTimestampsEXT) \
	EnumMacro(PFN_vkGetBufferDeviceAddressKHR, vkGetBufferDeviceAddressKHR) \
	EnumMacro(PFN_vkGetDescriptorSetLayoutSizeEXT, vkGetDescriptorSetLayoutSizeEXT) \
	EnumMacro(PFN_vkGetDescriptorSetLayoutBindingOffsetEXT, vkGetDescriptorSetLayoutBindingOffsetEXT) \
	EnumMacro(PFN_vkCmdBindDescriptorBuffersEXT, vkCmdBindDescriptorBuffersEXT) \
	EnumMacro(PFN_vkCmdSetDescriptorBufferOffsetsEXT, vkCmdSetDescriptorBufferOffsetsEXT) \
	EnumMacro(PFN_vkCmdSetDescriptorBufferOffsets2EXT, vkCmdSetDescriptorBufferOffsets2EXT) \
	EnumMacro(PFN_vkGetDescriptorEXT, vkGetDescriptorEXT) \
	EnumMacro(PFN_vkCreateDeferredOperationKHR, vkCreateDeferredOperationKHR) \
	EnumMacro(PFN_vkDestroyDeferredOperationKHR, vkDestroyDeferredOperationKHR) \
	EnumMacro(PFN_vkDeferredOperationJoinKHR, vkDeferredOperationJoinKHR) \
	EnumMacro(PFN_vkGetDeferredOperationMaxConcurrencyKHR, vkGetDeferredOperationMaxConcurrencyKHR) \
	EnumMacro(PFN_vkGetDeferredOperationResultKHR, vkGetDeferredOperationResultKHR) \
	EnumMacro(PFN_vkCmdDrawMeshTasksEXT, vkCmdDrawMeshTasksEXT) \
	EnumMacro(PFN_vkCmdDrawMeshTasksIndirectCountEXT, vkCmdDrawMeshTasksIndirectCountEXT) \
	EnumMacro(PFN_vkCmdDrawMeshTasksIndirectEXT, vkCmdDrawMeshTasksIndirectEXT) \
	EnumMacro(PFN_vkCmdWriteBufferMarkerAMD, vkCmdWriteBufferMarkerAMD) \
	EnumMacro(PFN_vkCmdSetCheckpointNV, vkCmdSetCheckpointNV) \
	EnumMacro(PFN_vkGetQueueCheckpointDataNV, vkGetQueueCheckpointDataNV) \
	EnumMacro(PFN_vkGetPhysicalDeviceFragmentShadingRatesKHR, vkGetPhysicalDeviceFragmentShadingRatesKHR) \
	EnumMacro(PFN_vkGetDeviceFaultInfoEXT, vkGetDeviceFaultInfoEXT) \
	EnumMacro(PFN_vkGetPhysicalDeviceToolPropertiesEXT, vkGetPhysicalDeviceToolPropertiesEXT) \
	EnumMacro(PFN_vkGetImageSubresourceLayout2EXT, vkGetImageSubresourceLayout2EXT) \
	EnumMacro(PFN_vkGetImageSubresourceLayout2KHR, vkGetImageSubresourceLayout2KHR) \
	EnumMacro(PFN_vkGetSemaphoreCounterValue, vkGetSemaphoreCounterValueKHR) \
	EnumMacro(PFN_vkWaitSemaphores, vkWaitSemaphoresKHR) \
	EnumMacro(PFN_vkSignalSemaphore, vkSignalSemaphoreKHR) \
	EnumMacro(PFN_vkCreateShadersEXT, vkCreateShadersEXT) \
	EnumMacro(PFN_vkDestroyShaderEXT, vkDestroyShaderEXT) \
	EnumMacro(PFN_vkGetShaderBinaryDataEXT, vkGetShaderBinaryDataEXT) \
	EnumMacro(PFN_vkCmdBindShadersEXT, vkCmdBindShadersEXT) \
	EnumMacro(PFN_vkCmdBindVertexBuffers2EXT, vkCmdBindVertexBuffers2EXT) \
	EnumMacro(PFN_vkCmdSetCullModeEXT, vkCmdSetCullModeEXT) \
	EnumMacro(PFN_vkCmdSetColorBlendEnableEXT, vkCmdSetColorBlendEnableEXT) \
	EnumMacro(PFN_vkCmdSetColorBlendEquationEXT, vkCmdSetColorBlendEquationEXT) \
	EnumMacro(PFN_vkCmdSetColorWriteMaskEXT, vkCmdSetColorWriteMaskEXT) \
	EnumMacro(PFN_vkCmdSetFrontFaceEXT, vkCmdSetFrontFaceEXT) \
	EnumMacro(PFN_vkCmdSetPolygonModeEXT, vkCmdSetPolygonModeEXT) \
	EnumMacro(PFN_vkCmdSetPrimitiveTopologyEXT, vkCmdSetPrimitiveTopologyEXT) \
	EnumMacro(PFN_vkCmdSetRasterizationSamplesEXT, vkCmdSetRasterizationSamplesEXT) \
	EnumMacro(PFN_vkCmdSetScissorWithCountEXT, vkCmdSetScissorWithCountEXT) \
	EnumMacro(PFN_vkCmdSetStencilOpEXT, vkCmdSetStencilOpEXT) \
	EnumMacro(PFN_vkCmdSetStencilTestEnableEXT, vkCmdSetStencilTestEnableEXT) \
	EnumMacro(PFN_vkCmdSetVertexInputEXT, vkCmdSetVertexInputEXT) \
	EnumMacro(PFN_vkCmdSetViewportWithCountEXT, vkCmdSetViewportWithCountEXT) \
	EnumMacro(PFN_vkCmdSetDepthBiasEnableEXT, vkCmdSetDepthBiasEnableEXT) \
	EnumMacro(PFN_vkCmdSetDepthBoundsTestEnableEXT, vkCmdSetDepthBoundsTestEnableEXT) \
	EnumMacro(PFN_vkCmdSetDepthClampEnableEXT, vkCmdSetDepthClampEnableEXT) \
	EnumMacro(PFN_vkCmdSetDepthCompareOpEXT, vkCmdSetDepthCompareOpEXT) \
	EnumMacro(PFN_vkCmdSetDepthTestEnableEXT, vkCmdSetDepthTestEnableEXT) \
	EnumMacro(PFN_vkCmdSetDepthWriteEnableEXT, vkCmdSetDepthWriteEnableEXT) \
	EnumMacro(PFN_vkCmdSetRasterizerDiscardEnableEXT, vkCmdSetRasterizerDiscardEnableEXT) \
	EnumMacro(PFN_vkCmdSetPrimitiveRestartEnableEXT, vkCmdSetPrimitiveRestartEnableEXT) \
	EnumMacro(PFN_vkCmdSetAlphaToCoverageEnableEXT, vkCmdSetAlphaToCoverageEnableEXT) \
	EnumMacro(PFN_vkCmdSetAlphaToOneEnableEXT, vkCmdSetAlphaToOneEnableEXT) \
	EnumMacro(PFN_vkCmdSetLogicOpEnableEXT, vkCmdSetLogicOpEnableEXT) \
	EnumMacro(PFN_vkCmdSetSampleMaskEXT, vkCmdSetSampleMaskEXT) \
	EnumMacro(PFN_vkCmdSetFragmentShadingRateKHR, vkCmdSetFragmentShadingRateKHR) \
	EnumMacro(PFN_vkCopyMemoryToImageEXT, vkCopyMemoryToImageEXT) \
	EnumMacro(PFN_vkTransitionImageLayoutEXT, vkTransitionImageLayoutEXT) \
	EnumMacro(PFN_vkCmdPushDescriptorSetKHR, vkCmdPushDescriptorSetKHR) \
	EnumMacro(PFN_vkCmdPushDescriptorSet2KHR, vkCmdPushDescriptorSet2KHR) \
	EnumMacro(PFN_vkCmdExecuteGeneratedCommandsEXT, vkCmdExecuteGeneratedCommandsEXT) \
	EnumMacro(PFN_vkCmdPreprocessGeneratedCommandsEXT, vkCmdPreprocessGeneratedCommandsEXT) \
	EnumMacro(PFN_vkCreateIndirectCommandsLayoutEXT, vkCreateIndirectCommandsLayoutEXT) \
	EnumMacro(PFN_vkCreateIndirectExecutionSetEXT, vkCreateIndirectExecutionSetEXT) \
	EnumMacro(PFN_vkDestroyIndirectCommandsLayoutEXT, vkDestroyIndirectCommandsLayoutEXT) \
	EnumMacro(PFN_vkDestroyIndirectExecutionSetEXT, vkDestroyIndirectExecutionSetEXT) \
	EnumMacro(PFN_vkGetGeneratedCommandsMemoryRequirementsEXT, vkGetGeneratedCommandsMemoryRequirementsEXT) \
	EnumMacro(PFN_vkUpdateIndirectExecutionSetPipelineEXT, vkUpdateIndirectExecutionSetPipelineEXT) \
	EnumMacro(PFN_vkUpdateIndirectExecutionSetShaderEXT, vkUpdateIndirectExecutionSetShaderEXT) \
	EnumMacro(PFN_vkGetPastPresentationTimingEXT, vkGetPastPresentationTimingEXT) \
	EnumMacro(PFN_vkGetSwapchainTimeDomainPropertiesEXT, vkGetSwapchainTimeDomainPropertiesEXT) \
	EnumMacro(PFN_vkGetSwapchainTimingPropertiesEXT, vkGetSwapchainTimingPropertiesEXT) \
	EnumMacro(PFN_vkSetSwapchainPresentTimingQueueSizeEXT, vkSetSwapchainPresentTimingQueueSizeEXT) \
	EnumMacro(PFN_vkCmdBindResourceHeapEXT, vkCmdBindResourceHeapEXT) \
	EnumMacro(PFN_vkCmdBindSamplerHeapEXT, vkCmdBindSamplerHeapEXT) \
	EnumMacro(PFN_vkCmdPushDataEXT, vkCmdPushDataEXT) \
	EnumMacro(PFN_vkGetPhysicalDeviceDescriptorSizeEXT, vkGetPhysicalDeviceDescriptorSizeEXT) \
	EnumMacro(PFN_vkWriteResourceDescriptorsEXT, vkWriteResourceDescriptorsEXT) \
	EnumMacro(PFN_vkWriteSamplerDescriptorsEXT, vkWriteSamplerDescriptorsEXT) \
	EnumMacro(PFN_vkCmdSetRenderingAttachmentLocationsKHR, vkCmdSetRenderingAttachmentLocationsKHR) \
	EnumMacro(PFN_vkCmdSetRenderingInputAttachmentIndicesKHR, vkCmdSetRenderingInputAttachmentIndicesKHR) \
	EnumMacro(PFN_vkGetDescriptorSetLayoutSupport, vkGetDescriptorSetLayoutSupport)


// List of all Vulkan entry points
#define ENUM_VK_ENTRYPOINTS_ALL(EnumMacro) \
	ENUM_VK_ENTRYPOINTS_BASE(EnumMacro) \
	ENUM_VK_ENTRYPOINTS_OPTIONAL_BASE(EnumMacro) \
	ENUM_VK_ENTRYPOINTS_INSTANCE(EnumMacro) \
	ENUM_VK_ENTRYPOINTS_SURFACE_INSTANCE(EnumMacro) \
	ENUM_VK_ENTRYPOINTS_OPTIONAL_INSTANCE(EnumMacro) \
	ENUM_VK_ENTRYPOINTS_PLATFORM_BASE(EnumMacro) \
	ENUM_VK_ENTRYPOINTS_OPTIONAL_PLATFORM_INSTANCE(EnumMacro) \
	ENUM_VK_ENTRYPOINTS_PLATFORM_INSTANCE(EnumMacro)

// Declare all Vulkan functions
#define DECLARE_VK_ENTRYPOINTS(Type,Func) extern VULKANRHI_API Type Func;
namespace VulkanDynamicAPI
{
	ENUM_VK_ENTRYPOINTS_ALL(DECLARE_VK_ENTRYPOINTS);
}
using namespace VulkanDynamicAPI;
