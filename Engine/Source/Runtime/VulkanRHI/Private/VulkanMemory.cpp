// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanMemory.cpp: Vulkan memory RHI implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanMemory.h"
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformStackWalk.h"
#include "VulkanContext.h"
#include "VulkanLLM.h"
#include "VulkanDescriptorSets.h"
#include "VulkanBindlessDescriptorManager.h"
#include "Containers/SortedMap.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <VersionHelpers.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#include "VulkanRayTracing.h"

// This 'frame number' should only be used for the deletion queue
uint32 GVulkanRHIDeletionFrameNumber = 0;


int32 GVulkanNumFramesToWaitForResourceDelete = 2;
static FAutoConsoleVariableRef CVarVulkanNumFramesToWaitForResourceDelete(
	TEXT("r.Vulkan.NumFramesToWaitForResourceDelete"),
	GVulkanNumFramesToWaitForResourceDelete,
	TEXT("The number of frames to wait before deleting a resource. Used for debugging. (default:2)\n"),
	ECVF_ReadOnly
);

#define UE_VK_MEMORY_MAX_SUB_ALLOCATION (64llu << 20llu) // set to 0 to disable

#define UE_VK_MEMORY_KEEP_FREELIST_SORTED					1
#define UE_VK_MEMORY_JOIN_FREELIST_ON_THE_FLY				(UE_VK_MEMORY_KEEP_FREELIST_SORTED && 1)
#define UE_VK_MEMORY_KEEP_FREELIST_SORTED_CATCHBUGS			0 // debugging

#define VULKAN_LOG_MEMORY_UELOG 1 //in case of debugging, it is useful to be able to log directly to LowLevelPrintf, as this is easier to diff. Please do not delete this code.

#if VULKAN_LOG_MEMORY_UELOG
#define VULKAN_LOGMEMORY(fmt, ...) UE_LOG(LogVulkanRHI, Display, fmt, ##__VA_ARGS__)
#else
#define VULKAN_LOGMEMORY(fmt, ...) FPlatformMisc::LowLevelOutputDebugStringf(fmt TEXT("\n"), ##__VA_ARGS__)
#endif


DECLARE_STATS_GROUP_SORTBYNAME(TEXT("Vulkan Memory Raw"), STATGROUP_VulkanMemoryRaw, STATCAT_Advanced);
DECLARE_MEMORY_STAT(TEXT("Dedicated Memory"), STAT_VulkanDedicatedMemory, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("MemoryPool 0"), STAT_VulkanMemory0, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("MemoryPool 1"), STAT_VulkanMemory1, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("MemoryPool 2"), STAT_VulkanMemory2, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("MemoryPool 3"), STAT_VulkanMemory3, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("MemoryPool 4"), STAT_VulkanMemory4, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("MemoryPool 5"), STAT_VulkanMemory5, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("MemoryPool (remaining)"), STAT_VulkanMemoryX, STATGROUP_VulkanMemoryRaw);

DECLARE_MEMORY_STAT(TEXT("MemoryPool 0 Reserved"), STAT_VulkanMemory0Reserved, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("MemoryPool 1 Reserved"), STAT_VulkanMemory1Reserved, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("MemoryPool 2 Reserved"), STAT_VulkanMemory2Reserved, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("MemoryPool 3 Reserved"), STAT_VulkanMemory3Reserved, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("MemoryPool 4 Reserved"), STAT_VulkanMemory4Reserved, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("MemoryPool 5 Reserved"), STAT_VulkanMemory5Reserved, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("MemoryPool (remaining) Reserved"), STAT_VulkanMemoryXReserved, STATGROUP_VulkanMemoryRaw);

DECLARE_MEMORY_STAT(TEXT("_Total Allocated"), STAT_VulkanMemoryTotal, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("_Reserved"), STAT_VulkanMemoryReserved, STATGROUP_VulkanMemoryRaw);

DECLARE_MEMORY_STAT(TEXT("Memory Heap 0 Budget"), STAT_VulkanMemoryBudget0, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("Memory Heap 1 Budget"), STAT_VulkanMemoryBudget1, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("Memory Heap 2 Budget"), STAT_VulkanMemoryBudget2, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("Memory Heap 3 Budget"), STAT_VulkanMemoryBudget3, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("Memory Heap 4 Budget"), STAT_VulkanMemoryBudget4, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("Memory Heap 5 Budget"), STAT_VulkanMemoryBudget5, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("Memory Heap (remaining) Budget"), STAT_VulkanMemoryBudgetX, STATGROUP_VulkanMemoryRaw);

DECLARE_MEMORY_STAT(TEXT("Memory Heap 0 Usage"), STAT_VulkanMemoryUsage0, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("Memory Heap 1 Usage"), STAT_VulkanMemoryUsage1, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("Memory Heap 2 Usage"), STAT_VulkanMemoryUsage2, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("Memory Heap 3 Usage"), STAT_VulkanMemoryUsage3, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("Memory Heap 4 Usage"), STAT_VulkanMemoryUsage4, STATGROUP_VulkanMemoryRaw);
DECLARE_MEMORY_STAT(TEXT("Memory Heap 5 Usage"), STAT_VulkanMemoryUsage5, STATGROUP_VulkanMemoryRaw);


DECLARE_STATS_GROUP(TEXT("Vulkan Memory"), STATGROUP_VulkanMemory, STATCAT_Advanced);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Unknown"), STAT_VulkanAllocation_Unknown, STATGROUP_VulkanMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("UniformBuffer"), STAT_VulkanAllocation_UniformBuffer, STATGROUP_VulkanMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("MultiBuffer"), STAT_VulkanAllocation_MultiBuffer, STATGROUP_VulkanMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("RingBuffer"), STAT_VulkanAllocation_RingBuffer, STATGROUP_VulkanMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("FrameTempBuffer"), STAT_VulkanAllocation_FrameTempBuffer, STATGROUP_VulkanMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("ImageRenderTarget"), STAT_VulkanAllocation_ImageRenderTarget, STATGROUP_VulkanMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("ImageOther"), STAT_VulkanAllocation_ImageOther, STATGROUP_VulkanMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("BufferUAV"), STAT_VulkanAllocation_BufferUAV, STATGROUP_VulkanMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("BufferStaging"), STAT_VulkanAllocation_BufferStaging, STATGROUP_VulkanMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("BufferOther"), STAT_VulkanAllocation_BufferOther, STATGROUP_VulkanMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("TempBlocks"), STAT_VulkanAllocation_TempBlocks, STATGROUP_VulkanMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("_Total"), STAT_VulkanAllocation_Allocated, STATGROUP_VulkanMemory, );

DEFINE_STAT(STAT_VulkanAllocation_UniformBuffer);
DEFINE_STAT(STAT_VulkanAllocation_MultiBuffer);
DEFINE_STAT(STAT_VulkanAllocation_RingBuffer);
DEFINE_STAT(STAT_VulkanAllocation_FrameTempBuffer);
DEFINE_STAT(STAT_VulkanAllocation_ImageRenderTarget);
DEFINE_STAT(STAT_VulkanAllocation_ImageOther);
DEFINE_STAT(STAT_VulkanAllocation_BufferUAV);
DEFINE_STAT(STAT_VulkanAllocation_BufferStaging);
DEFINE_STAT(STAT_VulkanAllocation_BufferOther);
DEFINE_STAT(STAT_VulkanAllocation_TempBlocks);
DEFINE_STAT(STAT_VulkanAllocation_Allocated);

int32 GVulkanLogDefrag = 0;
static FAutoConsoleVariableRef CVarVulkanLogDefrag(
	TEXT("r.vulkan.LogDefrag"),
	GVulkanLogDefrag,
	TEXT("Whether to log all defrag moves & evictions\n")
	TEXT("0: Off\n")
	TEXT("1: On\n"),
	ECVF_Default
);

static int32 GVulkanMemoryBackTrace = 10;
static FAutoConsoleVariableRef CVarVulkanMemoryBackTrace(
	TEXT("r.Vulkan.MemoryBacktrace"),
	GVulkanMemoryBackTrace,
	TEXT("0: Disable, store __FILE__ and __LINE__\n")
	TEXT("N: Enable, n is # of steps to go back\n"),
	ECVF_ReadOnly
);

static int32 GVulkanMemoryMemoryFallbackToHost = 1;
static FAutoConsoleVariableRef CVarVulkanMemoryMemoryFallbackToHost(
	TEXT("r.Vulkan.MemoryFallbackToHost"),
	GVulkanMemoryMemoryFallbackToHost,
	TEXT("0: Legacy, will crash when oom for rendertargets\n")
	TEXT("1: Fallback to Host memory on oom\n"),
	ECVF_ReadOnly
);



float GVulkanDefragSizeFactor = 1.3f;
static FAutoConsoleVariableRef CVarVulkanDefragSizeFactor(
	TEXT("r.Vulkan.DefragSizeFactor"),
	GVulkanDefragSizeFactor,
	TEXT("Amount of space required to be free, on other pages, before attempting to do a defrag of a page"),
	ECVF_RenderThreadSafe
);

float GVulkanDefragSizeFraction = .7f;
static FAutoConsoleVariableRef CVarGVulkanDefragSizeFraction(
	TEXT("r.Vulkan.DefragSizeFraction"),
	GVulkanDefragSizeFraction,
	TEXT("Fill threshold, dont attempt defrag if free space is less than this fraction\n"),
	ECVF_RenderThreadSafe
);

int32 GVulkanDefragAgeDelay = 100;
static FAutoConsoleVariableRef CVarVulkanDefragAgeDelay(
	TEXT("r.Vulkan.DefragAgeDelay"),
	GVulkanDefragAgeDelay,
	TEXT("Delay in Frames that needs to pass before attempting to defrag a page again\n"),
	ECVF_RenderThreadSafe
);
int32 GVulkanDefragPaused = 0;
static FAutoConsoleVariableRef CVarVulkanDefragPaused(
	TEXT("r.Vulkan.DefragPaused"),
	GVulkanDefragPaused,
	TEXT("Pause Any defragging. Only for debugging defragmentation code"),
	ECVF_RenderThreadSafe
);

int32 GVulkanDefragAutoPause = 0;
static FAutoConsoleVariableRef CVarVulkanDefragAutoPause(
	TEXT("r.Vulkan.DefragAutoPause"),
	GVulkanDefragAutoPause,
	TEXT("Automatically Pause defragging after a single page has been defragged. Only for debugging the defragmentation code."),
	ECVF_RenderThreadSafe
);

int32 GVulkanUseBufferBinning = 0;
static FAutoConsoleVariableRef CVarVulkanUseBufferBinning(
	TEXT("r.Vulkan.UseBufferBinning"),
	GVulkanUseBufferBinning,
	TEXT("Enable binning sub-allocations within buffers to help reduce fragmentation at the expense of higher high watermark [read-only]\n"),
	ECVF_ReadOnly
);

static int32 GVulkanLogEvictStatus = 0;
static FAutoConsoleVariableRef GVarVulkanLogEvictStatus(
	TEXT("r.Vulkan.LogEvictStatus"),
	GVulkanLogEvictStatus,
	TEXT("Log Eviction status every frame"),
	ECVF_RenderThreadSafe
);

int32 GVulkanBudgetPercentageScale = 100;
static FAutoConsoleVariableRef CVarVulkanBudgetPercentageScale(
	TEXT("r.Vulkan.BudgetScale"),
	GVulkanBudgetPercentageScale,
	TEXT("Percentage Scaling of MemoryBudget. Valid range is [0-100]. Only has an effect if VK_EXT_memory_budget is available"),
	ECVF_RenderThreadSafe
);




int32 GVulkanEnableDedicatedImageMemory = 1;
static FAutoConsoleVariableRef CVarVulkanEnableDedicatedImageMemory(
	TEXT("r.Vulkan.EnableDedicatedImageMemory"),
	GVulkanEnableDedicatedImageMemory,
	TEXT("Enable to use Dedidcated Image memory on devices that prefer it."),
	ECVF_RenderThreadSafe
);


int32 GVulkanSingleAllocationPerResource = VULKAN_SINGLE_ALLOCATION_PER_RESOURCE;
static FAutoConsoleVariableRef CVarVulkanSingleAllocationPerResource(
	TEXT("r.Vulkan.SingleAllocationPerResource"),
	GVulkanSingleAllocationPerResource,
	TEXT("Enable to do a single allocation per resource"),
	ECVF_RenderThreadSafe
);

//debug variable to force evict one page
int32 GVulkanEvictOnePage = 0;
static FAutoConsoleVariableRef CVarVulkanEvictbLilleHestyMusOne(
	TEXT("r.Vulkan.EvictOnePageDebug"),
	GVulkanEvictOnePage,
	TEXT("Set to 1 to test evict one page to host"),
	ECVF_RenderThreadSafe
);
#if !UE_BUILD_SHIPPING
static int32 GVulkanFakeMemoryLimit = 0;
static FAutoConsoleVariableRef CVarVulkanFakeMemoryLimit(
	TEXT("r.Vulkan.FakeMemoryLimit"),
	GVulkanFakeMemoryLimit,
	TEXT("set to artificially limit to # MB. 0 is disabled"),
	ECVF_RenderThreadSafe
);
#endif

int32 GVulkanEnableDefrag = 1;
static FAutoConsoleVariableRef CVarVulkanEnableDefrag(
	TEXT("r.Vulkan.EnableDefrag"),
	GVulkanEnableDefrag,
	TEXT("Whether to enable defrag moves & evictions\n")
	TEXT("0: Off\n")
	TEXT("1: On\n"),
	ECVF_RenderThreadSafe
);

int32 GVulkanDefragOnce = 0;
static FAutoConsoleVariableRef CVarVulkanDefragOnce(
	TEXT("r.Vulkan.DefragOnceDebug"),
	GVulkanDefragOnce,
	TEXT("Set to 1 to test defrag"),
	ECVF_RenderThreadSafe
);



static float GVulkanEvictionLimitPercentage = 70.f;
static FAutoConsoleVariableRef CVarVulkanEvictionLimitPercentage(
	TEXT("r.Vulkan.EvictionLimitPercentage"),
	GVulkanEvictionLimitPercentage,
	TEXT("When more than x% of local memory is used, evict resources to host memory"),
	ECVF_RenderThreadSafe
);


static float GVulkanEvictionLimitPercentageReenableLimit = 60.f;
static FAutoConsoleVariableRef CVarVulkanEvictionLimitPercentageReenableLimit(
	TEXT("r.Vulkan.EvictionLimitPercentageRenableLimit"),
	GVulkanEvictionLimitPercentageReenableLimit,
	TEXT("After eviction has occurred, only start using local mem for textures after memory usage is less than this(Relative to Eviction percentage)"),
	ECVF_RenderThreadSafe
);

#if (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)
static int32 GVulkanEvictionAllowBuffers = 0;
static FAutoConsoleVariableRef CVarVulkanEvictionAllowBuffers(
	TEXT("r.Vulkan.EvictionAllowBuffers"),
	GVulkanEvictionAllowBuffers,
	TEXT("Allow for buffers to be evicted.  Used to debug memory peaks. (default: 0)"),
	ECVF_RenderThreadSafe
);

static int32 GVulkanEvictionAllowRenderTargets = 0;
static FAutoConsoleVariableRef CVarVulkanEvictionAllowRenderTargets(
	TEXT("r.Vulkan.EvictionAllowRenderTargets"),
	GVulkanEvictionAllowRenderTargets,
	TEXT("Allow for render targets to be evicted.  Used to debug memory peaks. (default: 0)"),
	ECVF_RenderThreadSafe
);
#endif

RENDERCORE_API	void DumpRenderTargetPoolMemory(FOutputDevice& OutputDevice);

#if (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)
static int32 GForceCoherent = 0;
static FAutoConsoleVariableRef CVarForceCoherentOperations(
	TEXT("r.Vulkan.ForceCoherentOperations"),
	GForceCoherent,
	TEXT("1 forces memory invalidation and flushing of coherent memory\n"),
	ECVF_ReadOnly
);
#else
constexpr int32 GForceCoherent = 0;
#endif


FVulkanTrackInfo::FVulkanTrackInfo()
	: Data(0)
	, SizeOrLine(0)
{
}

#if VULKAN_MEMORY_TRACK
#define VULKAN_FILL_TRACK_INFO(...) do{VulkanTrackFillInfo(__VA_ARGS__);}while(0)
#define VULKAN_FREE_TRACK_INFO(...) do{VulkanTrackFreeInfo(__VA_ARGS__);}while(0)
#define VULKAN_TRACK_STRING(s) VulkanTrackGetString(s)
#else
#define VULKAN_FILL_TRACK_INFO(...) do{}while(0)
#define VULKAN_FREE_TRACK_INFO(...) do{}while(0)
#define VULKAN_TRACK_STRING(s) FString()
#endif

FString VulkanTrackGetString(FVulkanTrackInfo& Track)
{
	if (Track.SizeOrLine < 0)
	{
		const size_t STRING_SIZE = 16 * 1024;
		ANSICHAR StackTraceString[STRING_SIZE];
		uint64* Stack = (uint64*)Track.Data;
		FMemory::Memset(StackTraceString, 0, sizeof(StackTraceString));
		SIZE_T StringSize = STRING_SIZE;
		for (int32 Index = 0; Index < -Track.SizeOrLine; ++Index)
		{
			FPlatformStackWalk::ProgramCounterToHumanReadableString(Index, Stack[Index], StackTraceString, StringSize, 0);
			FCStringAnsi::StrncatTruncateDest(StackTraceString, (int32)StringSize, LINE_TERMINATOR_ANSI);
		}
		FString Out = FString::Printf(TEXT("\n%S\n"), StackTraceString);
		return Out;
	}
	else
	{
		return FString::Printf(TEXT("\n%S:%d\n"), (const char*)Track.Data, Track.SizeOrLine);
	}
}

void VulkanTrackFillInfo(FVulkanTrackInfo& Track, const char* File, uint32 Line)
{
	if (GVulkanMemoryBackTrace > 0)
	{
		uint64* Stack = ((Track.Data != nullptr) && (Track.SizeOrLine < 0)) ? (uint64*)Track.Data : new uint64[GVulkanMemoryBackTrace];
		int32 Depth = FPlatformStackWalk::CaptureStackBackTrace(Stack, GVulkanMemoryBackTrace);
		Track.SizeOrLine = -Depth;
		Track.Data = Stack;
	}
	else
	{
		Track.Data = (void*)File;
		Track.SizeOrLine = Line;
	}
}
void VulkanTrackFreeInfo(FVulkanTrackInfo& Track)
{
	if(Track.SizeOrLine < 0)
	{
		delete[] (uint64*)Track.Data;
	}
	Track.Data = 0;
	Track.SizeOrLine = 0;
}

namespace VulkanRHI
{
	struct FVulkanMemoryAllocation
	{
		const TCHAR* Name;
		FName ResourceName;
		void* Address;
		void* RHIResouce;
		uint32 Size;
		uint32 Width;
		uint32 Height;
		uint32 Depth;
		uint32 BytesPerPixel;
	};

	struct FVulkanMemoryBucket
	{
		TArray<FVulkanMemoryAllocation> Allocations;
	};

	struct FResourceHeapStats
	{
		uint64 BufferAllocations = 0;
		uint64 ImageAllocations = 0;
		uint64 UsedImageMemory = 0;
		uint64 UsedBufferMemory = 0;
		uint64 TotalMemory = 0;
		uint64 Pages = 0;
		uint64 ImagePages = 0;
		uint64 BufferPages = 0;
		VkMemoryPropertyFlags MemoryFlags = (VkMemoryPropertyFlags)0;

		FResourceHeapStats& operator += (const FResourceHeapStats& Other)
		{
			BufferAllocations += Other.BufferAllocations;
			ImageAllocations += Other.ImageAllocations;
			UsedImageMemory += Other.UsedImageMemory;
			UsedBufferMemory += Other.UsedBufferMemory;
			TotalMemory += Other.TotalMemory;
			Pages += Other.Pages;
			ImagePages += Other.ImagePages;
			BufferPages += Other.BufferPages;
			return *this;
		}
	};

	template<typename Callback>
	void IterateVulkanAllocations(Callback F, uint32 AllocatorIndex)
	{
		checkNoEntry();
	}

	enum
	{
		GPU_ONLY_HEAP_PAGE_SIZE = 128 * 1024 * 1024,
		STAGING_HEAP_PAGE_SIZE = 32 * 1024 * 1024,
		ANDROID_MAX_HEAP_PAGE_SIZE = 16 * 1024 * 1024,
		ANDROID_MAX_HEAP_IMAGE_PAGE_SIZE = 16 * 1024 * 1024,
		ANDROID_MAX_HEAP_BUFFER_PAGE_SIZE = 4 * 1024 * 1024,
	};


	constexpr uint32 FMemoryManager::PoolSizes[(int32)FMemoryManager::EPoolSizes::SizesCount];
	constexpr uint32 FMemoryManager::BufferSizes[(int32)FMemoryManager::EPoolSizes::SizesCount + 1];


	const TCHAR* VulkanAllocationTypeToString(EVulkanAllocationType Type)
	{
		switch (Type)
		{
		case EVulkanAllocationEmpty: return TEXT("Empty");
		case EVulkanAllocationPooledBuffer: return TEXT("PooledBuffer");
		case EVulkanAllocationBuffer: return TEXT("Buffer");
		case EVulkanAllocationImage: return TEXT("Image");
		case EVulkanAllocationImageDedicated: return TEXT("ImageDedicated");
		default:
			checkNoEntry();
		}
		return TEXT("");
	}
	const TCHAR* VulkanAllocationMetaTypeToString(EVulkanAllocationMetaType MetaType)
	{
		switch(MetaType)
		{
		case EVulkanAllocationMetaUnknown: return TEXT("Unknown");
		case EVulkanAllocationMetaUniformBuffer: return TEXT("UBO");
		case EVulkanAllocationMetaMultiBuffer: return TEXT("MultiBuf");
		case EVulkanAllocationMetaRingBuffer: return TEXT("RingBuf");
		case EVulkanAllocationMetaFrameTempBuffer: return TEXT("FrameTemp");
		case EVulkanAllocationMetaImageRenderTarget: return TEXT("ImageRT");
		case EVulkanAllocationMetaImageOther: return TEXT("Image");
		case EVulkanAllocationMetaBufferUAV: return TEXT("BufferUAV");
		case EVulkanAllocationMetaBufferStaging: return TEXT("BufferStg");
		case EVulkanAllocationMetaBufferOther: return TEXT("BufOthr");
		default:
			checkNoEntry();
		}
		return TEXT("");
	}

	static void DecMetaStats(EVulkanAllocationMetaType MetaType, uint32 Size)
	{
		DEC_DWORD_STAT_BY(STAT_VulkanAllocation_Allocated, Size);
		switch (MetaType)
		{
		case EVulkanAllocationMetaUniformBuffer:
			DEC_DWORD_STAT_BY(STAT_VulkanAllocation_UniformBuffer, Size);
			break;
		case EVulkanAllocationMetaMultiBuffer:
			DEC_DWORD_STAT_BY(STAT_VulkanAllocation_MultiBuffer, Size);
			break;
		case EVulkanAllocationMetaRingBuffer:
			DEC_DWORD_STAT_BY(STAT_VulkanAllocation_RingBuffer, Size);
			break;
		case EVulkanAllocationMetaFrameTempBuffer:
			DEC_DWORD_STAT_BY(STAT_VulkanAllocation_FrameTempBuffer, Size);
			break;
		case EVulkanAllocationMetaImageRenderTarget:
			DEC_DWORD_STAT_BY(STAT_VulkanAllocation_ImageRenderTarget, Size);
			break;
		case EVulkanAllocationMetaImageOther:
			DEC_DWORD_STAT_BY(STAT_VulkanAllocation_ImageOther, Size);
			break;
		case EVulkanAllocationMetaBufferUAV:
			DEC_DWORD_STAT_BY(STAT_VulkanAllocation_BufferUAV, Size);
			break;
		case EVulkanAllocationMetaBufferStaging:
			DEC_DWORD_STAT_BY(STAT_VulkanAllocation_BufferStaging, Size);
			break;
		case EVulkanAllocationMetaBufferOther:
			DEC_DWORD_STAT_BY(STAT_VulkanAllocation_BufferOther, Size);
			break;
		default:
			checkNoEntry();
		}
	}
	static void IncMetaStats(EVulkanAllocationMetaType MetaType, uint32 Size)
	{
		INC_DWORD_STAT_BY(STAT_VulkanAllocation_Allocated, Size);

		switch (MetaType)
		{
		case EVulkanAllocationMetaUniformBuffer:
			INC_DWORD_STAT_BY(STAT_VulkanAllocation_UniformBuffer, Size);
			break;
		case EVulkanAllocationMetaMultiBuffer:
			INC_DWORD_STAT_BY(STAT_VulkanAllocation_MultiBuffer, Size);
			break;
		case EVulkanAllocationMetaRingBuffer:
			INC_DWORD_STAT_BY(STAT_VulkanAllocation_RingBuffer, Size);
			break;
		case EVulkanAllocationMetaFrameTempBuffer:
			INC_DWORD_STAT_BY(STAT_VulkanAllocation_FrameTempBuffer, Size);
			break;
		case EVulkanAllocationMetaImageRenderTarget:
			INC_DWORD_STAT_BY(STAT_VulkanAllocation_ImageRenderTarget, Size);
			break;
		case EVulkanAllocationMetaImageOther:
			INC_DWORD_STAT_BY(STAT_VulkanAllocation_ImageOther, Size);
			break;
		case EVulkanAllocationMetaBufferUAV:
			INC_DWORD_STAT_BY(STAT_VulkanAllocation_BufferUAV, Size);
			break;
		case EVulkanAllocationMetaBufferStaging:
			INC_DWORD_STAT_BY(STAT_VulkanAllocation_BufferStaging, Size);
			break;
		case EVulkanAllocationMetaBufferOther:
			INC_DWORD_STAT_BY(STAT_VulkanAllocation_BufferOther, Size);
			break;
		default:
			checkNoEntry();
		}
	}

	FDeviceMemoryManager::FDeviceMemoryManager() :
		MemoryUpdateTime(0.0),
		DeviceHandle(VK_NULL_HANDLE),
		Device(nullptr),
		NumAllocations(0),
		PeakNumAllocations(0),
		bHasUnifiedMemory(false),
		bSupportsMemoryless(false),
		PrimaryHeapIndex(-1)
	{
		FMemory::Memzero(MemoryBudget);
		FMemory::Memzero(MemoryProperties);
	}

	FDeviceMemoryManager::~FDeviceMemoryManager()
	{
		Deinit();
	}

	void FDeviceMemoryManager::UpdateMemoryProperties()
	{
		const bool bHasMemoryBudget = Device->GetOptionalExtensions().HasMemoryBudget;

		if (bHasMemoryBudget)
		{
			VkPhysicalDeviceMemoryProperties2 MemoryProperties2;
			ZeroVulkanStruct(MemoryBudget, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_MEMORY_BUDGET_PROPERTIES_EXT);
			ZeroVulkanStruct(MemoryProperties2, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_MEMORY_PROPERTIES_2);
			MemoryProperties2.pNext = &MemoryBudget;
			VulkanRHI::vkGetPhysicalDeviceMemoryProperties2(Device->GetPhysicalHandle(), &MemoryProperties2);
			FMemory::Memcpy(MemoryProperties, MemoryProperties2.memoryProperties);

			for (uint32 Heap = 0; Heap < VK_MAX_MEMORY_HEAPS; ++Heap)
			{
				MemoryBudget.heapBudget[Heap] = GVulkanBudgetPercentageScale * MemoryBudget.heapBudget[Heap] / 100;
			}

			VkDeviceSize BudgetX = 0;
			for (uint32 Heap = 6; Heap < VK_MAX_MEMORY_HEAPS; ++Heap)
			{
				BudgetX += MemoryBudget.heapBudget[Heap];
			}
			SET_DWORD_STAT(STAT_VulkanMemoryBudget0, MemoryBudget.heapBudget[0]);
			SET_DWORD_STAT(STAT_VulkanMemoryBudget1, MemoryBudget.heapBudget[1]);
			SET_DWORD_STAT(STAT_VulkanMemoryBudget2, MemoryBudget.heapBudget[2]);
			SET_DWORD_STAT(STAT_VulkanMemoryBudget3, MemoryBudget.heapBudget[3]);
			SET_DWORD_STAT(STAT_VulkanMemoryBudget4, MemoryBudget.heapBudget[4]);
			SET_DWORD_STAT(STAT_VulkanMemoryBudget5, MemoryBudget.heapBudget[5]);
			SET_DWORD_STAT(STAT_VulkanMemoryBudgetX, BudgetX);

			SET_DWORD_STAT(STAT_VulkanMemoryUsage0, MemoryBudget.heapUsage[0]);
			SET_DWORD_STAT(STAT_VulkanMemoryUsage1, MemoryBudget.heapUsage[1]);
			SET_DWORD_STAT(STAT_VulkanMemoryUsage2, MemoryBudget.heapUsage[2]);
			SET_DWORD_STAT(STAT_VulkanMemoryUsage3, MemoryBudget.heapUsage[3]);
			SET_DWORD_STAT(STAT_VulkanMemoryUsage4, MemoryBudget.heapUsage[4]);
			SET_DWORD_STAT(STAT_VulkanMemoryUsage5, MemoryBudget.heapUsage[5]);
		}
		else
		{
			VulkanRHI::vkGetPhysicalDeviceMemoryProperties(Device->GetPhysicalHandle(), &MemoryProperties);
		}

		// Save Memory stats in FRHIMemoryStats format
		MemoryStats = FRHIMemoryStats{};
		uint64 HeapBudget, HeapUsed;

		for (uint32 HeapIndex = 0; HeapIndex < MemoryProperties.memoryHeapCount; ++HeapIndex)
		{
			const bool bLocalHeap = (MemoryProperties.memoryHeaps[HeapIndex].flags & VK_MEMORY_HEAP_DEVICE_LOCAL_BIT) != 0;

			if (bHasMemoryBudget)
			{
				HeapBudget = MemoryBudget.heapBudget[HeapIndex];
				HeapUsed = MemoryBudget.heapUsage[HeapIndex];
			}
			else
			{
				HeapBudget = MemoryProperties.memoryHeaps[HeapIndex].size;
				
				if (HeapIndex < (uint32)HeapInfos.Num())
				{
					HeapUsed = HeapInfos[HeapIndex].UsedSize;
				}
				else
				{
					HeapUsed = 0;
				}
			}

			if (bLocalHeap)
			{
				MemoryStats.BudgetLocal += HeapBudget;
				MemoryStats.UsedLocal += HeapUsed;
			}
			else
			{
				MemoryStats.BudgetSystem += HeapBudget;
				MemoryStats.UsedSystem += HeapUsed;
			}
		}

		MemoryStats.AvailableLocal = (MemoryStats.UsedLocal < MemoryStats.BudgetLocal) ? (MemoryStats.BudgetLocal - MemoryStats.UsedLocal) : 0;
		MemoryStats.AvailableSystem = (MemoryStats.UsedSystem < MemoryStats.BudgetSystem) ? (MemoryStats.BudgetSystem - MemoryStats.UsedSystem) : 0;

		MemoryStats.DemotedLocal = (MemoryStats.UsedLocal > MemoryStats.BudgetLocal) ? (MemoryStats.UsedLocal - MemoryStats.BudgetLocal) : 0;
		MemoryStats.DemotedSystem = (MemoryStats.UsedSystem > MemoryStats.BudgetSystem) ? (MemoryStats.UsedSystem - MemoryStats.BudgetSystem) : 0;
	}

	void FDeviceMemoryManager::Init(FVulkanDevice* InDevice)
	{
		check(Device == nullptr);
		Device = InDevice;
		NumAllocations = 0;
		PeakNumAllocations = 0;

		bHasUnifiedMemory = FVulkanPlatform::HasUnifiedMemory();

		DeviceHandle = Device->GetHandle();
		UpdateMemoryProperties();

		PrimaryHeapIndex = -1;
		uint64 PrimaryHeapSize = 0;
		uint32 NonLocalHeaps = 0;

		for(uint32 i = 0; i < MemoryProperties.memoryHeapCount; ++i)
		{
			
			if (VKHasAllFlags(MemoryProperties.memoryHeaps[i].flags, VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT))
			{
				if(MemoryProperties.memoryHeaps[i].size > PrimaryHeapSize)
				{
					PrimaryHeapIndex = i;
					PrimaryHeapSize = MemoryProperties.memoryHeaps[i].size;
				}
			}
			else
			{
				NonLocalHeaps++;
			}
		}
		if(0 == NonLocalHeaps)
		{
			PrimaryHeapIndex = -1; // if there are no non-local heaps, disable eviction and defragmentation
		}

		// Update bMemoryless support
		bSupportsMemoryless = false;
		for (uint32 i = 0; i < MemoryProperties.memoryTypeCount && !bSupportsMemoryless; ++i)
		{
			bSupportsMemoryless = VKHasAllFlags(MemoryProperties.memoryTypes[i].propertyFlags, VK_MEMORY_PROPERTY_LAZILY_ALLOCATED_BIT);
		}

		HeapInfos.AddDefaulted(MemoryProperties.memoryHeapCount);

		PrintMemInfo();
	}

	bool MetaTypeCanEvict(EVulkanAllocationMetaType MetaType)
	{
		switch(MetaType)
		{
			case EVulkanAllocationMetaImageOther: return true;

#if (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)
			case EVulkanAllocationMetaMultiBuffer: return (GVulkanEvictionAllowBuffers != 0);
			case EVulkanAllocationMetaImageRenderTarget: return (GVulkanEvictionAllowRenderTargets != 0);
#endif

			default: return false;
		}
	}



	void FDeviceMemoryManager::PrintMemInfo() const
	{
		auto ToMB = [](uint64 ByteSize) { return (float)ByteSize / (1024.f * 1024.f); };
		auto ToPct = [](uint64 Used, uint64 Total) { return 100.f * (float)Used / (float)Total; };

		const uint32 MaxAllocations = Device->GetLimits().maxMemoryAllocationCount;
		VULKAN_LOGMEMORY(TEXT("Max memory allocations %u."), MaxAllocations);

		VULKAN_LOGMEMORY(TEXT("%d Device Memory Heaps:"), MemoryProperties.memoryHeapCount);
		for (uint32 HeapIndex = 0; HeapIndex < MemoryProperties.memoryHeapCount; ++HeapIndex)
		{
			const bool bIsGPUHeap = VKHasAllFlags(MemoryProperties.memoryHeaps[HeapIndex].flags, VK_MEMORY_HEAP_DEVICE_LOCAL_BIT);
			const VkDeviceSize UsedMemory = (HeapIndex < (uint32)HeapInfos.Num()) ? HeapInfos[HeapIndex].UsedSize : 0;

			VULKAN_LOGMEMORY(TEXT(" %2d: Flags 0x%x - Size %llu (%.2f MB) - Used %llu (%.2f%%)%s%s"),
				HeapIndex,
				MemoryProperties.memoryHeaps[HeapIndex].flags,
				MemoryProperties.memoryHeaps[HeapIndex].size,
				ToMB(MemoryProperties.memoryHeaps[HeapIndex].size),
				UsedMemory,
				ToPct(UsedMemory, MemoryProperties.memoryHeaps[HeapIndex].size),
				bIsGPUHeap ? TEXT(" - DeviceLocal") : TEXT(""),
				(PrimaryHeapIndex == (int32)HeapIndex) ? TEXT(" - PrimaryHeap") : TEXT(""));
		}

		VULKAN_LOGMEMORY(TEXT("%d Device Memory Types (%sunified):"), MemoryProperties.memoryTypeCount, bHasUnifiedMemory ? TEXT("") : TEXT("Not "));
		for (uint32 HeapIndex = 0; HeapIndex < MemoryProperties.memoryHeapCount; ++HeapIndex)
		{
			for (uint32 TypeIndex = 0; TypeIndex < MemoryProperties.memoryTypeCount; ++TypeIndex)
			{
				if(HeapIndex == MemoryProperties.memoryTypes[TypeIndex].heapIndex)
				{
					const VkMemoryPropertyFlags MemoryPropertyFlags = MemoryProperties.memoryTypes[TypeIndex].propertyFlags;
					VULKAN_LOGMEMORY(TEXT(" %2d: Flags 0x%05x - Heap %2d - %s"),
						TypeIndex, MemoryPropertyFlags, HeapIndex,
						MemoryPropertyFlags ? VK_FLAGS_TO_STRING(VkMemoryPropertyFlags, MemoryPropertyFlags) : TEXT(""));
				}
			}
		}

		if(Device->GetOptionalExtensions().HasMemoryBudget)
		{
			VULKAN_LOGMEMORY(TEXT("Memory Budget Extension:"));
			VULKAN_LOGMEMORY(TEXT("\t         | Usage                     | Budget          | Size            |"));
			VULKAN_LOGMEMORY(TEXT("\t---------|------------------------------------------------------------------|"));
			for(uint32 HeapIndex = 0; HeapIndex < VK_MAX_MEMORY_HEAPS; ++HeapIndex)
			{
				const VkDeviceSize Budget = MemoryBudget.heapBudget[HeapIndex];
				const VkDeviceSize Usage = MemoryBudget.heapUsage[HeapIndex];
				if(0 != Budget || 0 != Usage)
				{
					const VkDeviceSize Size = MemoryProperties.memoryHeaps[HeapIndex].size;
					VULKAN_LOGMEMORY(TEXT("\t HEAP %02d | %6.2f%% / %13.2f MB | %13.2f MB | %13.2f MB |"),
						HeapIndex, ToPct(Usage, Size), ToMB(Usage), ToMB(Budget), ToMB(Size));
				}
			}
			VULKAN_LOGMEMORY(TEXT("\t---------|------------------------------------------------------------------|"));
		}
		else
		{
			VULKAN_LOGMEMORY(TEXT("Memory Budget unavailable"));
		}
	}

	uint32 FDeviceMemoryManager::GetEvictedMemoryProperties()
	{
		if (Device->GetVendorId() == EGpuVendorId::Amd)
		{
			return VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT;
		}
		else if (Device->GetVendorId() == EGpuVendorId::Nvidia)
		{
			return 0;
		}
		else
		{
			return 0;
		}
	}


	void FDeviceMemoryManager::Deinit()
	{
		TrimMemory(true);
		for (int32 Index = 0; Index < HeapInfos.Num(); ++Index)
		{
			if (HeapInfos[Index].Allocations.Num())
			{
				UE_LOGF(LogVulkanRHI, Warning, "Found %d unfreed allocations!", HeapInfos[Index].Allocations.Num());
				DumpMemory();
			}
		}
		NumAllocations = 0;
	}

	bool FDeviceMemoryManager::SupportsMemoryType(VkMemoryPropertyFlags Properties) const
	{
		for (uint32 Index = 0; Index < MemoryProperties.memoryTypeCount; ++Index)
		{
			if (MemoryProperties.memoryTypes[Index].propertyFlags == Properties)
			{
				return true;
			}
		}
		return false;
	}

	void FDeviceMemoryManager::GetPrimaryHeapStatus(uint64& OutAllocated, uint64& OutLimit)
	{
		if (PrimaryHeapIndex < 0)
		{
			OutAllocated = 0;
			OutLimit = 1;
		}
		else
		{
			OutAllocated = HeapInfos[PrimaryHeapIndex].UsedSize;
			OutLimit = GetBaseHeapSize(PrimaryHeapIndex);
	}
	}

	FDeviceMemoryAllocation* FDeviceMemoryManager::Alloc(bool bCanFail, VkDeviceSize AllocationSize, uint32 MemoryTypeBits, VkMemoryPropertyFlags MemoryPropertyFlags, void* DedicatedAllocateInfo, float Priority, bool bExternal, const char* File, uint32 Line)
	{
		uint32 MemoryTypeIndex = ~0;
		VERIFYVULKANRESULT(this->GetMemoryTypeFromProperties(MemoryTypeBits, MemoryPropertyFlags, &MemoryTypeIndex));
		return Alloc(bCanFail, AllocationSize, MemoryTypeIndex, DedicatedAllocateInfo, Priority, bExternal, File, Line);
	}

	FDeviceMemoryAllocation* FDeviceMemoryManager::Alloc(bool bCanFail, VkDeviceSize AllocationSize, uint32 MemoryTypeIndex, void* DedicatedAllocateInfo, float Priority, bool bExternal, const char* File, uint32 Line)
	{
		SCOPED_NAMED_EVENT(FDeviceMemoryManager_Alloc, FColor::Cyan);
		FScopeLock Lock(&DeviceMemLock);

		if(!DedicatedAllocateInfo)
		{
			FDeviceMemoryBlockKey Key = {MemoryTypeIndex, AllocationSize};
			FDeviceMemoryBlock& Block = Allocations.FindOrAdd(Key);
			if(Block.Allocations.Num() > 0)
			{
				FDeviceMemoryBlock::FFreeBlock Alloc = Block.Allocations.Pop();

				switch (MemoryTypeIndex)
				{
				case 0:
					INC_DWORD_STAT_BY(STAT_VulkanMemory0, AllocationSize);
					DEC_DWORD_STAT_BY(STAT_VulkanMemory0Reserved, AllocationSize);
					break;
				case 1:
					INC_DWORD_STAT_BY(STAT_VulkanMemory1, AllocationSize);
					DEC_DWORD_STAT_BY(STAT_VulkanMemory1Reserved, AllocationSize);
					break;
				case 2:
					INC_DWORD_STAT_BY(STAT_VulkanMemory2, AllocationSize);
					DEC_DWORD_STAT_BY(STAT_VulkanMemory2Reserved, AllocationSize);
					break;
				case 3:
					INC_DWORD_STAT_BY(STAT_VulkanMemory3, AllocationSize);
					DEC_DWORD_STAT_BY(STAT_VulkanMemory3Reserved, AllocationSize);
					break;
				case 4:
					INC_DWORD_STAT_BY(STAT_VulkanMemory4, AllocationSize);
					DEC_DWORD_STAT_BY(STAT_VulkanMemory4Reserved, AllocationSize);
					break;
				case 5:
					INC_DWORD_STAT_BY(STAT_VulkanMemory5, AllocationSize);
					DEC_DWORD_STAT_BY(STAT_VulkanMemory5Reserved, AllocationSize);
					break;
				default:
					INC_DWORD_STAT_BY(STAT_VulkanMemoryX, AllocationSize);
					DEC_DWORD_STAT_BY(STAT_VulkanMemoryXReserved, AllocationSize);
					break;
				}
				DEC_DWORD_STAT_BY(STAT_VulkanMemoryReserved, AllocationSize);
				return Alloc.Allocation;
			}
		}

		check(AllocationSize > 0);
		check(MemoryTypeIndex < MemoryProperties.memoryTypeCount);

		VkMemoryAllocateInfo Info;
		ZeroVulkanStruct(Info, VK_STRUCTURE_TYPE_MEMORY_ALLOCATE_INFO);
		Info.allocationSize = AllocationSize;
		Info.memoryTypeIndex = MemoryTypeIndex;


#if VULKAN_SUPPORTS_MEMORY_PRIORITY
		VkMemoryPriorityAllocateInfoEXT Prio;
		if (Device->GetOptionalExtensions().HasMemoryPriority)
		{
			ZeroVulkanStruct(Prio, VK_STRUCTURE_TYPE_MEMORY_PRIORITY_ALLOCATE_INFO_EXT);
			Prio.priority = Priority;
			Info.pNext = &Prio;
		}
#endif

#if VULKAN_SUPPORTS_DEDICATED_ALLOCATION
		if (DedicatedAllocateInfo)
		{
			((VkMemoryDedicatedAllocateInfoKHR*)DedicatedAllocateInfo)->pNext = Info.pNext;
			Info.pNext = DedicatedAllocateInfo;
			INC_DWORD_STAT_BY(STAT_VulkanDedicatedMemory, AllocationSize);
			IncMetaStats(EVulkanAllocationMetaImageRenderTarget, AllocationSize);
		}
#endif

		VkExportMemoryAllocateInfoKHR VulkanExportMemoryAllocateInfoKHR = {};
#if PLATFORM_WINDOWS
		VkExportMemoryWin32HandleInfoKHR VulkanExportMemoryWin32HandleInfoKHR = {};
#endif // PLATFORM_WINDOWS
		if (bExternal)
		{
			ZeroVulkanStruct(VulkanExportMemoryAllocateInfoKHR, VK_STRUCTURE_TYPE_EXPORT_MEMORY_ALLOCATE_INFO_KHR);
#if PLATFORM_WINDOWS
			ZeroVulkanStruct(VulkanExportMemoryWin32HandleInfoKHR, VK_STRUCTURE_TYPE_EXPORT_MEMORY_WIN32_HANDLE_INFO_KHR);
			VulkanExportMemoryWin32HandleInfoKHR.pNext = Info.pNext;
			VulkanExportMemoryWin32HandleInfoKHR.pAttributes = NULL;
			VulkanExportMemoryWin32HandleInfoKHR.dwAccess =	GENERIC_ALL;
			VulkanExportMemoryWin32HandleInfoKHR.name = (LPCWSTR)nullptr;
			VulkanExportMemoryAllocateInfoKHR.pNext = IsWindows8OrGreater() ? &VulkanExportMemoryWin32HandleInfoKHR : nullptr;
			VulkanExportMemoryAllocateInfoKHR.handleTypes = IsWindows8OrGreater() ? VK_EXTERNAL_MEMORY_HANDLE_TYPE_OPAQUE_WIN32_BIT : VK_EXTERNAL_MEMORY_HANDLE_TYPE_OPAQUE_WIN32_KMT_BIT;
#else
			VulkanExportMemoryAllocateInfoKHR.pNext = Info.pNext;
			VulkanExportMemoryAllocateInfoKHR.handleTypes = VK_EXTERNAL_MEMORY_HANDLE_TYPE_OPAQUE_FD_BIT_KHR;
#endif // PLATFORM_WINDOWS
			Info.pNext = &VulkanExportMemoryAllocateInfoKHR;
		}

		VkMemoryAllocateFlagsInfo MemoryAllocateFlagsInfo;
		if (Device->GetOptionalExtensions().HasBufferDeviceAddress)
		{
			ZeroVulkanStruct(MemoryAllocateFlagsInfo, VK_STRUCTURE_TYPE_MEMORY_ALLOCATE_FLAGS_INFO);
			MemoryAllocateFlagsInfo.flags = VK_MEMORY_ALLOCATE_DEVICE_ADDRESS_BIT_KHR;
			MemoryAllocateFlagsInfo.pNext = Info.pNext;
			Info.pNext = &MemoryAllocateFlagsInfo;
		}

		VkDeviceMemory Handle;
		VkResult Result;

#if !UE_BUILD_SHIPPING
		if (MemoryTypeIndex == PrimaryHeapIndex && GVulkanFakeMemoryLimit && ((uint64)GVulkanFakeMemoryLimit << 20llu) < HeapInfos[PrimaryHeapIndex].UsedSize )
		{
			Handle = 0;
			Result = VK_ERROR_OUT_OF_DEVICE_MEMORY;
		}
		else
#endif
		{
			SCOPED_NAMED_EVENT(vkAllocateMemory, FColor::Cyan);
			Result = VulkanRHI::vkAllocateMemory(DeviceHandle, &Info, VULKAN_CPU_ALLOCATOR, &Handle);
		}

		if (Result == VK_ERROR_OUT_OF_DEVICE_MEMORY || Result == VK_ERROR_OUT_OF_HOST_MEMORY)
		{
			if (bCanFail)
			{
				UE_LOGF(LogVulkanRHI, Warning, "Failed to allocate Device Memory, Requested=%.2fKb MemTypeIndex=%d", (float)Info.allocationSize / 1024.0f, Info.memoryTypeIndex);
				return nullptr;
			}
			const TCHAR* MemoryType = TEXT("?");
			switch (Result)
			{
			case VK_ERROR_OUT_OF_HOST_MEMORY: MemoryType = TEXT("Host"); break;
			case VK_ERROR_OUT_OF_DEVICE_MEMORY: MemoryType = TEXT("Local"); break;
			}
			DumpRenderTargetPoolMemory(*GLog);
			Device->GetMemoryManager().DumpMemory();
			GLog->Panic();

			UE_LOGF(LogVulkanRHI, Fatal, "Out of %ls Memory, Requested%.2fKB MemTypeIndex=%d\n", MemoryType, AllocationSize / 1024.f, MemoryTypeIndex);
		}
		else
		{
			VERIFYVULKANRESULT(Result);
		}

		FDeviceMemoryAllocation* NewAllocation = new FDeviceMemoryAllocation;
		NewAllocation->DeviceHandle = DeviceHandle;
		NewAllocation->Handle = Handle;
		NewAllocation->Size = AllocationSize;
		NewAllocation->MemoryTypeIndex = MemoryTypeIndex;
		NewAllocation->bCanBeMapped = VKHasAllFlags(MemoryProperties.memoryTypes[MemoryTypeIndex].propertyFlags, VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT);
		NewAllocation->bIsCoherent = VKHasAllFlags(MemoryProperties.memoryTypes[MemoryTypeIndex].propertyFlags, VK_MEMORY_PROPERTY_HOST_COHERENT_BIT);
		NewAllocation->bIsCached = VKHasAllFlags(MemoryProperties.memoryTypes[MemoryTypeIndex].propertyFlags, VK_MEMORY_PROPERTY_HOST_CACHED_BIT);
#if VULKAN_SUPPORTS_DEDICATED_ALLOCATION
		NewAllocation->bDedicatedMemory = DedicatedAllocateInfo != 0;
#else
		NewAllocation->bDedicatedMemory = 0;
#endif
		VULKAN_FILL_TRACK_INFO(NewAllocation->Track, File, Line);
		++NumAllocations;
		PeakNumAllocations = FMath::Max(NumAllocations, PeakNumAllocations);

		if (NumAllocations == Device->GetLimits().maxMemoryAllocationCount && !GVulkanSingleAllocationPerResource)
		{
			UE_LOGF(LogVulkanRHI, Warning, "Hit Maximum # of allocations (%d) reported by device!", NumAllocations);
		}

		uint32 HeapIndex = MemoryProperties.memoryTypes[MemoryTypeIndex].heapIndex;
		HeapInfos[HeapIndex].Allocations.Add(NewAllocation);
		HeapInfos[HeapIndex].UsedSize += AllocationSize;
		HeapInfos[HeapIndex].PeakSize = FMath::Max(HeapInfos[HeapIndex].PeakSize, HeapInfos[HeapIndex].UsedSize);

#if VULKAN_USE_LLM
		LLM_PLATFORM_SCOPE_VULKAN(ELLMTagVulkan::VulkanDriverMemoryGPU);
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, (void*)NewAllocation->Handle, AllocationSize, ELLMTag::GraphicsPlatform, ELLMAllocType::System));
		LLM_TRACK_VULKAN_SPARE_MEMORY_GPU((int64)AllocationSize);
#else
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, (void*)NewAllocation->Handle, AllocationSize, ELLMTag::GraphicsPlatform));
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, (void*)NewAllocation->Handle, AllocationSize, ELLMTag::Untagged));
#endif

		INC_DWORD_STAT(STAT_VulkanNumPhysicalMemAllocations);
		switch(MemoryTypeIndex)
		{
		case 0:  INC_DWORD_STAT_BY(STAT_VulkanMemory0, AllocationSize); break;
		case 1:  INC_DWORD_STAT_BY(STAT_VulkanMemory1, AllocationSize); break;
		case 2:  INC_DWORD_STAT_BY(STAT_VulkanMemory2, AllocationSize); break;
		case 3:  INC_DWORD_STAT_BY(STAT_VulkanMemory3, AllocationSize); break;
		case 4:  INC_DWORD_STAT_BY(STAT_VulkanMemory4, AllocationSize); break;
		case 5:  INC_DWORD_STAT_BY(STAT_VulkanMemory5, AllocationSize); break;
		default: INC_DWORD_STAT_BY(STAT_VulkanMemoryX, AllocationSize); break;
		}
		INC_DWORD_STAT_BY(STAT_VulkanMemoryTotal, AllocationSize);

		return NewAllocation;
	}

	void FDeviceMemoryManager::Free(FDeviceMemoryAllocation*& Allocation)
	{
		SCOPED_NAMED_EVENT(FDeviceMemoryManager_Free, FColor::Cyan);
		FScopeLock Lock(&DeviceMemLock);

		check(Allocation);
		check(Allocation->Handle != VK_NULL_HANDLE);
		check(!Allocation->bFreedBySystem);
		if (Allocation->bDedicatedMemory)
		{
			DEC_DWORD_STAT_BY(STAT_VulkanDedicatedMemory, Allocation->Size);
			DecMetaStats(EVulkanAllocationMetaImageRenderTarget, Allocation->Size);
		}
		switch (Allocation->MemoryTypeIndex)
		{
		case 0:  DEC_DWORD_STAT_BY(STAT_VulkanMemory0, Allocation->Size); break;
		case 1:  DEC_DWORD_STAT_BY(STAT_VulkanMemory1, Allocation->Size); break;
		case 2:  DEC_DWORD_STAT_BY(STAT_VulkanMemory2, Allocation->Size); break;
		case 3:  DEC_DWORD_STAT_BY(STAT_VulkanMemory3, Allocation->Size); break;
		case 4:  DEC_DWORD_STAT_BY(STAT_VulkanMemory4, Allocation->Size); break;
		case 5:  DEC_DWORD_STAT_BY(STAT_VulkanMemory5, Allocation->Size); break;
		default: DEC_DWORD_STAT_BY(STAT_VulkanMemoryX, Allocation->Size); break;
		}
		if(!Allocation->bDedicatedMemory)
		{
			VkDeviceSize AllocationSize = Allocation->Size;
			FDeviceMemoryBlockKey Key = { Allocation->MemoryTypeIndex, AllocationSize };
			FDeviceMemoryBlock& Block = Allocations.FindOrAdd(Key);
			FDeviceMemoryBlock::FFreeBlock FreeBlock = {Allocation, GFrameNumberRenderThread};
			Block.Allocations.Add(FreeBlock);


			switch (Allocation->MemoryTypeIndex)
			{
			case 0:	INC_DWORD_STAT_BY(STAT_VulkanMemory0Reserved, AllocationSize); break;
			case 1: INC_DWORD_STAT_BY(STAT_VulkanMemory1Reserved, AllocationSize); break;
			case 2:	INC_DWORD_STAT_BY(STAT_VulkanMemory2Reserved, AllocationSize); break;
			case 3:	INC_DWORD_STAT_BY(STAT_VulkanMemory3Reserved, AllocationSize); break;
			case 4:	INC_DWORD_STAT_BY(STAT_VulkanMemory4Reserved, AllocationSize); break;
			case 5:	INC_DWORD_STAT_BY(STAT_VulkanMemory5Reserved, AllocationSize); break;
			default:
				INC_DWORD_STAT_BY(STAT_VulkanMemoryXReserved, AllocationSize);
				break;
			}
			INC_DWORD_STAT_BY(STAT_VulkanMemoryReserved, AllocationSize);
			

			Allocation = nullptr;
			return;
		}

		FreeInternal(Allocation);
		Allocation = nullptr;
	}
	void FDeviceMemoryManager::FreeInternal(FDeviceMemoryAllocation* Allocation)
	{
		DEC_DWORD_STAT_BY(STAT_VulkanMemoryTotal, Allocation->Size);
		VulkanRHI::vkFreeMemory(DeviceHandle, Allocation->Handle, VULKAN_CPU_ALLOCATOR);

#if VULKAN_USE_LLM
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, (void*)Allocation->Handle, ELLMAllocType::System));
		LLM_TRACK_VULKAN_SPARE_MEMORY_GPU(-(int64)Allocation->Size);
#else
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, (void*)Allocation->Handle));
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, (void*)Allocation->Handle));
#endif

		--NumAllocations;

		DEC_DWORD_STAT(STAT_VulkanNumPhysicalMemAllocations);

		uint32 HeapIndex = MemoryProperties.memoryTypes[Allocation->MemoryTypeIndex].heapIndex;

		HeapInfos[HeapIndex].UsedSize -= Allocation->Size;
		HeapInfos[HeapIndex].Allocations.RemoveSwap(Allocation);
		Allocation->bFreedBySystem = true;

		delete Allocation;
		Allocation = nullptr;

	}
	void FDeviceMemoryManager::TrimMemory(bool bFullTrim)
	{
		FScopeLock Lock(&DeviceMemLock);
		//blocks are always freed after being reserved for FrameThresholdFull frames.
		const uint32 FrameThresholdFull = 100; 
		// After being held for FrameThresholdPartial frames, only LargeThresholdPartial/SmallThresholdPartial pages are kept reserved
		// SmallPageSize defines when Large/Small is used.
		const uint32 FrameThresholdPartial = 10;
		const uint32 LargeThresholdPartial = 5;
		const uint32 SmallThresholdPartial = 10;
		const uint64 SmallPageSize = (8llu << 20); 

		uint32 Frame = GFrameNumberRenderThread;
		for(TPair<FDeviceMemoryBlockKey, FDeviceMemoryBlock>& Pair : Allocations)
		{
			FDeviceMemoryBlock& Block = Pair.Value;
			const uint32 ThresholdPartial = Block.Key.BlockSize <= SmallPageSize ? SmallThresholdPartial : LargeThresholdPartial;

			uint32 AbovePartialThreshold = 0;
			int32 Index = 0;
			while(Index < Block.Allocations.Num())
			{
				FDeviceMemoryBlock::FFreeBlock& FreeBlock = Block.Allocations[Index];
				if (FreeBlock.FrameFreed + FrameThresholdFull < Frame || bFullTrim)
				{
					VkDeviceSize AllocationSize = FreeBlock.Allocation->GetSize();
					switch (FreeBlock.Allocation->MemoryTypeIndex)
					{
					case 0:	DEC_DWORD_STAT_BY(STAT_VulkanMemory0Reserved, AllocationSize); break;
					case 1: DEC_DWORD_STAT_BY(STAT_VulkanMemory1Reserved, AllocationSize); break;
					case 2:	DEC_DWORD_STAT_BY(STAT_VulkanMemory2Reserved, AllocationSize); break;
					case 3:	DEC_DWORD_STAT_BY(STAT_VulkanMemory3Reserved, AllocationSize); break;
					case 4:	DEC_DWORD_STAT_BY(STAT_VulkanMemory4Reserved, AllocationSize); break;
					case 5:	DEC_DWORD_STAT_BY(STAT_VulkanMemory5Reserved, AllocationSize); break;
					default:
						DEC_DWORD_STAT_BY(STAT_VulkanMemoryXReserved, AllocationSize);
						break;
					}
					DEC_DWORD_STAT_BY(STAT_VulkanMemoryReserved, AllocationSize);

					FreeInternal(FreeBlock.Allocation);
					Block.Allocations.RemoveAt(Index);
					continue;
				}
				else if(FreeBlock.FrameFreed + FrameThresholdPartial < Frame)
				{
					AbovePartialThreshold++;
				}
				Index++;
			}
			if(AbovePartialThreshold > ThresholdPartial)
			{
				uint32 BlocksToFree = AbovePartialThreshold - ThresholdPartial;
				Index = 0;
				while(Index < Block.Allocations.Num() && BlocksToFree > 0)
				{
					FDeviceMemoryBlock::FFreeBlock& FreeBlock = Block.Allocations[Index];
					if (FreeBlock.FrameFreed + FrameThresholdPartial < Frame)
					{
						VkDeviceSize AllocationSize = FreeBlock.Allocation->GetSize();
						switch (FreeBlock.Allocation->MemoryTypeIndex)
						{
						case 0:	DEC_DWORD_STAT_BY(STAT_VulkanMemory0Reserved, AllocationSize); break;
						case 1: DEC_DWORD_STAT_BY(STAT_VulkanMemory1Reserved, AllocationSize); break;
						case 2:	DEC_DWORD_STAT_BY(STAT_VulkanMemory2Reserved, AllocationSize); break;
						case 3:	DEC_DWORD_STAT_BY(STAT_VulkanMemory3Reserved, AllocationSize); break;
						case 4:	DEC_DWORD_STAT_BY(STAT_VulkanMemory4Reserved, AllocationSize); break;
						case 5:	DEC_DWORD_STAT_BY(STAT_VulkanMemory5Reserved, AllocationSize); break;
						default:
							DEC_DWORD_STAT_BY(STAT_VulkanMemoryXReserved, AllocationSize);
							break;
						}
						DEC_DWORD_STAT_BY(STAT_VulkanMemoryReserved, AllocationSize);


						FreeInternal(FreeBlock.Allocation);
						Block.Allocations.RemoveAt(Index);
						BlocksToFree--;
						continue;
					}
					else
					{
						Index++;
					}
				}
			}
		}
	}
	void FDeviceMemoryManager::GetMemoryDump(TArray<FResourceHeapStats>& OutDeviceHeapsStats)
	{
		OutDeviceHeapsStats.SetNum(0);
		for (int32 Index = 0; Index < HeapInfos.Num(); ++Index)
		{
			FResourceHeapStats Stat;
			Stat.MemoryFlags = 0;
			FHeapInfo& HeapInfo = HeapInfos[Index];
			Stat.TotalMemory = MemoryProperties.memoryHeaps[Index].size;
			for (uint32 TypeIndex = 0; TypeIndex < MemoryProperties.memoryTypeCount; ++TypeIndex)
			{
				if (MemoryProperties.memoryTypes[TypeIndex].heapIndex == Index)
				{
					Stat.MemoryFlags |= MemoryProperties.memoryTypes[TypeIndex].propertyFlags;
				}
			}

			for (int32 SubIndex = 0; SubIndex < HeapInfo.Allocations.Num(); ++SubIndex)
			{
				FDeviceMemoryAllocation* Allocation = HeapInfo.Allocations[SubIndex];
				Stat.BufferAllocations += 1;
				Stat.UsedBufferMemory += Allocation->Size;
				Stat.Pages += 1;
			}


			for (TPair<FDeviceMemoryBlockKey, FDeviceMemoryBlock>& Pair : Allocations)
			{
				uint32 MemoryType = Pair.Key.MemoryTypeIndex;
				if(MemoryProperties.memoryTypes[MemoryType].heapIndex == Index)
				{
					for(FDeviceMemoryBlock::FFreeBlock& FreeBlock : Pair.Value.Allocations)
					{
						Stat.ImageAllocations += 1;
						Stat.UsedImageMemory += FreeBlock.Allocation->GetSize();
						Stat.Pages += 1;
					}
				}
			}
			OutDeviceHeapsStats.Add(Stat);
		}
	}


	void FDeviceMemoryManager::DumpMemory()
	{
		VULKAN_LOGMEMORY(TEXT("/******************************************* Device Memory ********************************************\\"));
		PrintMemInfo();
		VULKAN_LOGMEMORY(TEXT("Device Memory: %d allocations on %d heaps"), NumAllocations, HeapInfos.Num());
		for (int32 Index = 0; Index < HeapInfos.Num(); ++Index)
		{
			FHeapInfo& HeapInfo = HeapInfos[Index];
			VULKAN_LOGMEMORY(TEXT("\tHeap %d, %d allocations"), Index, HeapInfo.Allocations.Num());
			uint64 TotalSize = 0;

			if (HeapInfo.Allocations.Num() > 0)
			{
				VULKAN_LOGMEMORY(TEXT("\t\tAlloc AllocSize(MB) TotalSize(MB)    Handle"));
			}

			for (int32 SubIndex = 0; SubIndex < HeapInfo.Allocations.Num(); ++SubIndex)
			{
				FDeviceMemoryAllocation* Allocation = HeapInfo.Allocations[SubIndex];
				VULKAN_LOGMEMORY(TEXT("\t\t%5d %13.3f %13.3f %p"), SubIndex, Allocation->Size / 1024.f / 1024.f, TotalSize / 1024.0f / 1024.0f, (void*)Allocation->Handle);
				TotalSize += Allocation->Size;
			}
			VULKAN_LOGMEMORY(TEXT("\t\tTotal Allocated %.2f MB, Peak %.2f MB"), TotalSize / 1024.0f / 1024.0f, HeapInfo.PeakSize / 1024.0f / 1024.0f);
		}
		VULKAN_LOGMEMORY(TEXT("Blocks Kept alive Current Frame (%d) "), GFrameNumberRenderThread);
		VULKAN_LOGMEMORY(TEXT("Free Blocks in Allocations:"));
		for (TPair<FDeviceMemoryBlockKey, FDeviceMemoryBlock>& Pair : Allocations)
		{
			FDeviceMemoryBlock& Block = Pair.Value;
			FDeviceMemoryBlockKey& Key = Pair.Key;
			for(FDeviceMemoryBlock::FFreeBlock& FreeBlock : Block.Allocations)
			{
				VULKAN_LOGMEMORY(TEXT("%02d %12lld : %p/Frame %05d :: Size %6.2fMB"), Key.MemoryTypeIndex, Key.BlockSize, FreeBlock.Allocation, FreeBlock.FrameFreed, FreeBlock.Allocation->GetSize() / (1024.f * 1024.f));
			}
		}

#if VULKAN_OBJECT_TRACKING
		{
			TSortedMap<uint32, FVulkanMemoryBucket> AllocationBuckets;
			auto Collector = [&](const TCHAR* Name, FName ResourceName, void* Address, void* RHIRes, uint32 Width, uint32 Height, uint32 Depth, uint32 Format)
			{
				uint32 BytesPerPixel = (Format != VK_FORMAT_UNDEFINED ? GetNumBitsPerPixel((VkFormat)Format) : 8) / 8;
				uint32 Size = FPlatformMath::Max(Width,1u) * FPlatformMath::Max(Height,1u) * FPlatformMath::Max(Depth, 1u) * BytesPerPixel;
				uint32 Bucket = Size;
				if(Bucket >= (1<<20))
				{
					Bucket = (Bucket + ((1 << 20) - 1)) & ~((1 << 20)-1);
				}
				else
				{
					Bucket = (Bucket + ((1 << 10) - 1)) & ~((1 << 10)-1);
				}
				FVulkanMemoryAllocation Allocation =
				{
					Name, ResourceName, Address, RHIRes, Size, Width, Height, Depth, BytesPerPixel
				};
				FVulkanMemoryBucket& ActualBucket = AllocationBuckets.FindOrAdd(Bucket);
				ActualBucket.Allocations.Add(Allocation);
			};


			TVulkanTrackBase<FVulkanTexture>::CollectAll(Collector);
			TVulkanTrackBase<FVulkanBuffer>::CollectAll(Collector);
			for(auto& Itr : AllocationBuckets)
			{
				VULKAN_LOGMEMORY(TEXT("***** BUCKET < %d kb *****"), Itr.Key/1024);
				FVulkanMemoryBucket& B = Itr.Value;
				uint32 Size = 0;
				for (FVulkanMemoryAllocation& A : B.Allocations)
				{
					Size += A.Size;
				}
				VULKAN_LOGMEMORY(TEXT("\t\t%d / %d kb"), B.Allocations.Num(), Size / 1024);


				B.Allocations.Sort([](const FVulkanMemoryAllocation& L, const FVulkanMemoryAllocation& R)
				{
					return L.Address < R.Address;
				}
				);
				for(FVulkanMemoryAllocation& A : B.Allocations)
				{
					VULKAN_LOGMEMORY(TEXT("\t\t%p/%p %6.2fkb (%d) %5d/%5d/%5d %s ::: %s"), A.Address, A.RHIResouce, A.Size / 1024.f, A.Size, A.Width, A.Height, A.Depth, A.Name, *A.ResourceName.ToString());
				}
			}
		}
#endif

	}

	uint64 FDeviceMemoryManager::GetTotalMemory(bool bGPU) const
	{
		uint64 TotalMemory = 0;
		for (uint32 Index = 0; Index < MemoryProperties.memoryHeapCount; ++Index)
		{
			const bool bIsGPUHeap = ((MemoryProperties.memoryHeaps[Index].flags & VK_MEMORY_HEAP_DEVICE_LOCAL_BIT) == VK_MEMORY_HEAP_DEVICE_LOCAL_BIT);

			if (bIsGPUHeap == bGPU)
			{
				TotalMemory += MemoryProperties.memoryHeaps[Index].size;
			}
		}
		return TotalMemory;
	}

	FDeviceMemoryAllocation::~FDeviceMemoryAllocation()
	{
		checkf(bFreedBySystem, TEXT("Memory has to released calling FDeviceMemory::Free()!"));
	}

	void* FDeviceMemoryAllocation::Map(VkDeviceSize InSize, VkDeviceSize Offset)
	{
		check(bCanBeMapped);
		if(!MappedPointer)
		{
			check(!MappedPointer);
			checkf(InSize == VK_WHOLE_SIZE || InSize + Offset <= Size, TEXT("Failed to Map %llu bytes, Offset %llu, AllocSize %llu bytes"), InSize, Offset, Size);
			VERIFYVULKANRESULT(VulkanRHI::vkMapMemory(DeviceHandle, Handle, Offset, InSize, 0, &MappedPointer));
		}
		return MappedPointer;
	}

	void FDeviceMemoryAllocation::Unmap()
	{
		check(MappedPointer);
		VulkanRHI::vkUnmapMemory(DeviceHandle, Handle);
		MappedPointer = nullptr;
	}

	void FDeviceMemoryAllocation::FlushMappedMemory(VkDeviceSize InOffset, VkDeviceSize InSize)
	{
		if (!IsCoherent() || GForceCoherent != 0)
		{
			check(IsMapped());
			check(InOffset + InSize <= Size);
			const VkMappedMemoryRange Range = {
				.sType = VK_STRUCTURE_TYPE_MAPPED_MEMORY_RANGE,
				.pNext = nullptr,
				.memory = Handle,
				.offset = InOffset,
				.size = InSize
			};
			VERIFYVULKANRESULT(VulkanRHI::vkFlushMappedMemoryRanges(DeviceHandle, 1, &Range));
		}
	}

	void FDeviceMemoryAllocation::InvalidateMappedMemory(VkDeviceSize InOffset, VkDeviceSize InSize)
	{
		if (!IsCoherent() || GForceCoherent != 0)
		{
			check(IsMapped());
			check(InOffset + InSize <= Size);
			const VkMappedMemoryRange Range = {
				.sType = VK_STRUCTURE_TYPE_MAPPED_MEMORY_RANGE,
				.pNext = nullptr,
				.memory = Handle,
				.offset = InOffset,
				.size = InSize
			};
			VERIFYVULKANRESULT(VulkanRHI::vkInvalidateMappedMemoryRanges(DeviceHandle, 1, &Range));
		}
	}

	void FRange::JoinConsecutiveRanges(TArray<FRange>& Ranges)
	{
		if (Ranges.Num() > 1)
		{
#if !UE_VK_MEMORY_KEEP_FREELIST_SORTED
			Ranges.Sort();
#else
	#if UE_VK_MEMORY_KEEP_FREELIST_SORTED_CATCHBUGS
			SanityCheck(Ranges);
	#endif
#endif

#if !UE_VK_MEMORY_JOIN_FREELIST_ON_THE_FLY
			for (int32 Index = Ranges.Num() - 1; Index > 0; --Index)
			{
				FRange& Current = Ranges[Index];
				FRange& Prev = Ranges[Index - 1];
				if (Prev.Offset + Prev.Size == Current.Offset)
				{
					Prev.Size += Current.Size;
					Ranges.RemoveAt(Index, 1, false);
				}
			}
#endif
		}
	}

	int32 FRange::InsertAndTryToMerge(TArray<FRange>& Ranges, const FRange& Item, int32 ProposedIndex)
	{
#if !UE_VK_MEMORY_JOIN_FREELIST_ON_THE_FLY
		int32 Ret = Ranges.Insert(Item, ProposedIndex);
#else
		// there are four cases here
		// 1) nothing can be merged (distinct ranges)		 XXXX YYY ZZZZZ  =>   XXXX YYY ZZZZZ
		// 2) new range can be merged with the previous one: XXXXYYY  ZZZZZ  =>   XXXXXXX  ZZZZZ
		// 3) new range can be merged with the next one:     XXXX  YYYZZZZZ  =>   XXXX  ZZZZZZZZ
		// 4) new range perfectly fills the gap:             XXXXYYYYYZZZZZ  =>   XXXXXXXXXXXXXX

		// note: we can have a case where we're inserting at the beginning of the array (no previous element), but we won't have a case
		// where we're inserting at the end (no next element) - AppendAndTryToMerge() should be called instead
		checkf(Item.Offset < Ranges[ProposedIndex].Offset, TEXT("FRange::InsertAndTryToMerge() was called to append an element - internal logic error, FRange::AppendAndTryToMerge() should have been called instead."))
		int32 Ret = ProposedIndex;
		if (UNLIKELY(ProposedIndex == 0))
		{
			// only cases 1 and 3 apply
			FRange& NextRange = Ranges[Ret];

			if (UNLIKELY(NextRange.Offset == Item.Offset + Item.Size))
			{
				NextRange.Offset = Item.Offset;
				NextRange.Size += Item.Size;
			}
			else
			{
				Ret = Ranges.Insert(Item, ProposedIndex);
			}
		}
		else
		{
			// all cases apply
			FRange& NextRange = Ranges[ProposedIndex];
			FRange& PrevRange = Ranges[ProposedIndex - 1];

			// see if we can merge with previous
			if (UNLIKELY(PrevRange.Offset + PrevRange.Size == Item.Offset))
			{
				// case 2, can still end up being case 4
				PrevRange.Size += Item.Size;

				if (UNLIKELY(PrevRange.Offset + PrevRange.Size == NextRange.Offset))
				{
					// case 4
					PrevRange.Size += NextRange.Size;
					Ranges.RemoveAt(ProposedIndex);
					Ret = ProposedIndex - 1;
				}
			}
			else if (UNLIKELY(Item.Offset + Item.Size == NextRange.Offset))
			{
				// case 3
				NextRange.Offset = Item.Offset;
				NextRange.Size += Item.Size;
			}
			else
			{
				// case 1 - the new range is disjoint with both
				Ret = Ranges.Insert(Item, ProposedIndex);	// this can invalidate NextRange/PrevRange references, don't touch them after this
			}
		}
#endif

#if UE_VK_MEMORY_KEEP_FREELIST_SORTED_CATCHBUGS
		SanityCheck(Ranges);
#endif
		return Ret;
	}

	int32 FRange::AppendAndTryToMerge(TArray<FRange>& Ranges, const FRange& Item)
	{
#if !UE_VK_MEMORY_JOIN_FREELIST_ON_THE_FLY
		int32 Ret = Ranges.Add(Item);
#else
		int32 Ret = Ranges.Num() - 1;
		// we only get here when we have an element in front of us
		checkf(Ret >= 0, TEXT("FRange::AppendAndTryToMerge() was called on an empty array."));
		FRange& PrevRange = Ranges[Ret];
		if (UNLIKELY(PrevRange.Offset + PrevRange.Size == Item.Offset))
		{
			PrevRange.Size += Item.Size;
		}
		else
		{
			Ret = Ranges.Add(Item);
		}
#endif

#if UE_VK_MEMORY_KEEP_FREELIST_SORTED_CATCHBUGS
		SanityCheck(Ranges);
#endif
		return Ret;
	}

	void FRange::AllocateFromEntry(TArray<FRange>& Ranges, int32 Index, uint32 SizeToAllocate)
	{
		FRange& Entry = Ranges[Index];
		if (SizeToAllocate < Entry.Size)
		{
			// Modify current free entry in-place.
			Entry.Size -= SizeToAllocate;
			Entry.Offset += SizeToAllocate;
		}
		else
		{
			// Remove this free entry.
			Ranges.RemoveAt(Index, EAllowShrinking::No);
#if UE_VK_MEMORY_KEEP_FREELIST_SORTED_CATCHBUGS
			SanityCheck(Ranges);
#endif
		}
	}

	void FRange::SanityCheck(TArray<FRange>& Ranges)
	{
		if (UE_VK_MEMORY_KEEP_FREELIST_SORTED_CATCHBUGS)	// keeping the check code visible to the compiler
		{
			int32 Num = Ranges.Num();
			if (Num > 1)
			{
				for (int32 ChkIndex = 0; ChkIndex < Num - 1; ++ChkIndex)
				{
					checkf(Ranges[ChkIndex].Offset < Ranges[ChkIndex + 1].Offset, TEXT("Array is not sorted!"));
					// if we're joining on the fly, then there cannot be any adjoining ranges, so use < instead of <=
#if UE_VK_MEMORY_JOIN_FREELIST_ON_THE_FLY
					checkf(Ranges[ChkIndex].Offset + Ranges[ChkIndex].Size < Ranges[ChkIndex + 1].Offset, TEXT("Ranges are overlapping or adjoining!"));
#else
					checkf(Ranges[ChkIndex].Offset + Ranges[ChkIndex].Size <= Ranges[ChkIndex + 1].Offset, TEXT("Ranges are overlapping!"));
#endif
				}
			}
		}
	}


	int32 FRange::Add(TArray<FRange>& Ranges, const FRange & Item)
	{
#if UE_VK_MEMORY_KEEP_FREELIST_SORTED
		// find the right place to add
		int32 NumRanges = Ranges.Num();
		if (LIKELY(NumRanges <= 0))
		{
			return Ranges.Add(Item);
		}

		FRange* Data = Ranges.GetData();
		for (int32 Index = 0; Index < NumRanges; ++Index)
		{
			if (UNLIKELY(Data[Index].Offset > Item.Offset))
			{
				return InsertAndTryToMerge(Ranges, Item, Index);
			}
		}

		// if we got this far and still haven't inserted, we're a new element
		return AppendAndTryToMerge(Ranges, Item);
#else
		return Ranges.Add(Item);
#endif
	}

	VkDeviceSize FDeviceMemoryManager::GetBaseHeapSize(uint32 HeapIndex) const
	{
		VkDeviceSize HeapSize = MemoryProperties.memoryHeaps[HeapIndex].size;
#if !UE_BUILD_SHIPPING
		if (GVulkanFakeMemoryLimit && PrimaryHeapIndex == HeapIndex)
		{
			HeapSize = FMath::Min<VkDeviceSize>((uint64)GVulkanFakeMemoryLimit << 20llu, HeapSize);
		}
#endif
		return HeapSize;
	}


	uint32 FVulkanResourceHeap::GetPageSizeBucket(FVulkanPageSizeBucket& BucketOut, EType Type, uint32 AllocationSize, bool bForceSingleAllocation)
	{
		if(bForceSingleAllocation)
		{
			uint32 Bucket = PageSizeBuckets.Num()-1;
			BucketOut = PageSizeBuckets[Bucket];
			return Bucket;
		}
		uint32 Mask = 0;
		Mask |= (Type == EType::Image) ? FVulkanPageSizeBucket::BUCKET_MASK_IMAGE : 0;
		Mask |= (Type == EType::Buffer) ? FVulkanPageSizeBucket::BUCKET_MASK_BUFFER: 0;
		for(FVulkanPageSizeBucket& B : PageSizeBuckets)
		{
			if(Mask == (B.BucketMask & Mask) && AllocationSize <= B.AllocationMax)
			{
				BucketOut = B;
				return &B - &PageSizeBuckets[0];
			}
		}
		checkNoEntry();
		return 0xffffffff;
	}

	uint32 FDeviceMemoryManager::GetHeapIndex(uint32 MemoryTypeIndex)
	{
		return MemoryProperties.memoryTypes[MemoryTypeIndex].heapIndex;
	}




	FVulkanResourceHeap::FVulkanResourceHeap(FMemoryManager* InOwner, uint32 InMemoryTypeIndex, uint32 InOverridePageSize)
		: Owner(InOwner)
		, MemoryTypeIndex((uint16)InMemoryTypeIndex)
		, HeapIndex((uint16)InOwner->GetDevice().GetDeviceMemoryManager().GetHeapIndex(InMemoryTypeIndex))
		, bIsHostCachedSupported(false)
		, bIsLazilyAllocatedSupported(false)
		, OverridePageSize(InOverridePageSize)
		, PeakPageSize(0)
		, UsedMemory(0)
		, PageIDCounter(0)
	{
	}

	FVulkanResourceHeap::~FVulkanResourceHeap()
	{
		auto DeletePages = [&](TArray<FVulkanSubresourceAllocator*>& UsedPages, const TCHAR* Name)
		{
			bool bLeak = false;
			for (int32 Index = UsedPages.Num() - 1; Index >= 0; --Index)
			{
				FVulkanSubresourceAllocator* Page = UsedPages[Index];
				bLeak |= !Page->JoinFreeBlocks();
				Owner->GetDevice().GetDeviceMemoryManager().Free(Page->MemoryAllocation);
				delete Page;
			}
			UsedPages.Reset(0);
			return bLeak;
		};
		bool bDump = false;
		for(TArray<FVulkanSubresourceAllocator*>& Pages : ActivePages)
		{
			bDump = bDump || DeletePages(Pages, TEXT("Pages"));
		}
		if (bDump)
		{
			Owner->GetDevice().GetMemoryManager().DumpMemory();
			GLog->Flush();
		}
	}
	void FVulkanResourceHeap::ReleasePage(FVulkanSubresourceAllocator* InPage)
	{
		Owner->UnregisterSubresourceAllocator(InPage);
		FDeviceMemoryAllocation* Allocation = InPage->MemoryAllocation;
		InPage->MemoryAllocation = 0;
		Owner->GetDevice().GetDeviceMemoryManager().Free(Allocation );
		UsedMemory -= InPage->MaxSize;
		delete InPage;
	}

	void FVulkanResourceHeap::FreePage(FVulkanSubresourceAllocator* InPage)
	{
		FScopeLock ScopeLock(&PagesLock);
		check(InPage->JoinFreeBlocks());
		int32 Index = -1;

		InPage->FrameFreed = GFrameNumberRenderThread;

		if(InPage->GetType() == EVulkanAllocationImageDedicated)
		{
			if (UsedDedicatedImagePages.Find(InPage, Index))
			{
				UsedDedicatedImagePages.RemoveAtSwap(Index, EAllowShrinking::No);
			}
			else
			{
				checkNoEntry();
			}
		}
		else
		{
			uint8 BucketId = InPage->BucketId;

			TArray<FVulkanSubresourceAllocator*>& Pages = ActivePages[BucketId];
			if (Pages.Find(InPage, Index))
			{
				Pages.RemoveAtSwap(Index, EAllowShrinking::No);
			}
			else
			{
				checkNoEntry();
			}
			check(!Pages.Find(InPage, Index));
		}

		ReleasePage(InPage);
	}

	uint64 FVulkanResourceHeap::EvictOne(FVulkanDevice& Device, const FVulkanContextArray& Contexts)
	{
		FScopeLock ScopeLock(&PagesLock);
		for(int32 Index = MAX_BUCKETS - 2; Index >= 0; Index--)
		{
			TArray<FVulkanSubresourceAllocator*>& Pages = ActivePages[Index];
			for (int32 Index2 = 0; Index2 < Pages.Num(); ++Index2)
			{
				FVulkanSubresourceAllocator* Allocator = Pages[Index2];
				if (!Allocator->bIsEvicting && Allocator->GetSubresourceAllocatorFlags() & VulkanAllocationFlagsCanEvict)
				{
					return Allocator->EvictToHost(Device, Contexts);
				}
			}
		}
		return 0;
	}

	void FVulkanResourceHeap::SetDefragging(FVulkanSubresourceAllocator* Allocator)
	{
		uint32 Flags = Allocator->GetSubresourceAllocatorFlags();
		int32 CurrentIndex = -1;
		int32 LastIndex = -1;
		uint32 BucketId = Allocator->BucketId;
		TArray<FVulkanSubresourceAllocator*>& Pages = ActivePages[BucketId];
		for(int32 Index = 0; Index < Pages.Num(); ++Index)
		{
			FVulkanSubresourceAllocator* Used = Pages[Index];
			if(Flags == (Allocator->GetSubresourceAllocatorFlags()))
			{
				if(Used == Allocator)
				{
					CurrentIndex = Index;
					Used->SetIsDefragging(true);
				}
				else
				{
					LastIndex = Index;
					Used->SetIsDefragging(false);
				}
			}
		}
		check(CurrentIndex != -1);

		//make sure heap being defragged is last, so we don't allocate from it unless its the only option
		if(LastIndex > CurrentIndex)
		{
			Pages[CurrentIndex] = Pages[LastIndex];
			Pages[LastIndex] = Allocator;
		}
	}

	bool FVulkanResourceHeap::GetIsDefragging(FVulkanSubresourceAllocator* Allocator)
	{
		return Allocator->GetIsDefragging();
	}

	void FVulkanResourceHeap::DefragTick(FVulkanDevice& Device, const FVulkanContextArray& Contexts, uint32 Count)
	{
		SCOPED_NAMED_EVENT(FVulkanSubresourceAllocator_DefragTick, FColor::Cyan);

		FScopeLock ScopeLock(&PagesLock);

		FVulkanSubresourceAllocator* CurrentDefragTarget = 0;
		//Continue if defrag currently in progress.
		for(TArray<FVulkanSubresourceAllocator*>& Pages : ActivePages) 
		{
			for (int32 Index = 0; Index < Pages.Num(); ++Index)
			{
				FVulkanSubresourceAllocator* Allocator = Pages[Index];
				if(GetIsDefragging(Allocator))
				{
					CurrentDefragTarget = Allocator;
				}
			}
		}


		if(!GVulkanDefragPaused && !CurrentDefragTarget && (GVulkanDefragSizeFactor > 0.f || GVulkanDefragOnce == 1))
		{
			//    If no defragger is currently in progress, search for a new candidate.
			//    search for a candiate that
			// 1) is defraggable
			// 2) has not recently been defragged
			struct FPotentialDefragMove
			{
				uint64 Key;
				FVulkanSubresourceAllocator* Allocator;
			};
			const uint32 FrameNumber = GFrameNumberRenderThread;
			TArray<FPotentialDefragMove> PotentialDefrag;
			for (TArray<FVulkanSubresourceAllocator*>& Pages : ActivePages)
			{
				uint32 FreeSpace = 0;
				for (int32 Index = 0; Index < Pages.Num(); ++Index)
				{
					FVulkanSubresourceAllocator* Allocator = Pages[Index];
					if (0 == (Allocator->GetSubresourceAllocatorFlags() & VulkanAllocationFlagsCanEvict))
					{

						uint32 MaxSize = Allocator->GetMaxSize();
						uint32 UsedSize = Allocator->GetUsedSize();
						FreeSpace += MaxSize - UsedSize;
					}
				}

				for (int32 Index = 0; Index < Pages.Num(); ++Index)
				{
					FVulkanSubresourceAllocator* Allocator = Pages[Index];
					if (0 == (Allocator->GetSubresourceAllocatorFlags() & VulkanAllocationFlagsCanEvict) && Allocator->CanDefrag())
					{
						uint32 MaxSize = Allocator->GetMaxSize();
						uint32 UsedSize = Allocator->GetUsedSize();
						uint32 Age = FrameNumber - Allocator->LastDefragFrame;
						if(    FreeSpace > UsedSize
							&& UsedSize * GVulkanDefragSizeFactor < FreeSpace - UsedSize
							&& UsedSize < MaxSize * GVulkanDefragSizeFraction
							&& Age > (uint32)GVulkanDefragAgeDelay)
						{
							float FillPrc = 1.f - (float(UsedSize) / MaxSize);
							uint32 SortLow = uint16(FillPrc * 0xffff);
							uint64 SortKey = (uint64(Age) << 32) | SortLow;
							FPotentialDefragMove Potential = {SortKey, Allocator};
							PotentialDefrag.Add(Potential);
						}
					}
				}
			}
			if (PotentialDefrag.Num())
			{
				PotentialDefrag.Sort([](const FPotentialDefragMove& LHS, const FPotentialDefragMove& RHS)
					{
						return LHS.Key < RHS.Key;
					});
				CurrentDefragTarget = PotentialDefrag[0].Allocator;
				SetDefragging(CurrentDefragTarget);
				DefragCountDown = 3;
			}
		}


		if (CurrentDefragTarget)
		{
			uint32 MaxSize = CurrentDefragTarget->GetMaxSize();
			uint32 UsedSize = CurrentDefragTarget->GetUsedSize();
			uint32 FreeSpace = MaxSize - UsedSize;

			if (GVulkanLogDefrag)
			{
				VULKAN_LOGMEMORY(TEXT("Defragging heap [%d] Max %6.2fMB Free %6.2fMB   %2d"), CurrentDefragTarget->AllocatorIndex, MaxSize / (1024.f*1024.f), FreeSpace / (1024.f * 1024.f), DefragCountDown);
			}
			if (CurrentDefragTarget->DefragTick(Device, Contexts, this, Count) > 0)
			{
				DefragCountDown = 3;
			}
			else
			{
				if (0 == --DefragCountDown)
				{
					if(GVulkanDefragAutoPause)
					{
						GVulkanDefragPaused = 1;
					}
					CurrentDefragTarget->SetIsDefragging(false);
				}
			}
		}
	}


	void FVulkanResourceHeap::DumpMemory(FResourceHeapStats& Stats)
	{
		auto DumpPages = [&](TArray<FVulkanSubresourceAllocator*>& UsedPages, const TCHAR* TypeName, bool bIsImage)
		{
			uint64 SubAllocUsedMemory = 0;
			uint64 SubAllocAllocatedMemory = 0;
			uint32 NumSuballocations = 0;
			for (int32 Index = 0; Index < UsedPages.Num(); ++Index)
			{
				Stats.Pages++;
				Stats.TotalMemory += UsedPages[Index]->MaxSize;
				if(bIsImage)
				{
					Stats.UsedImageMemory += UsedPages[Index]->UsedSize;
					Stats.ImageAllocations += UsedPages[Index]->NumSubAllocations;
					Stats.ImagePages++;
				}
				else
				{
					Stats.UsedBufferMemory += UsedPages[Index]->UsedSize;
					Stats.BufferAllocations += UsedPages[Index]->NumSubAllocations;
					Stats.BufferPages++;
				}

				SubAllocUsedMemory += UsedPages[Index]->UsedSize;
				SubAllocAllocatedMemory += UsedPages[Index]->MaxSize;
				NumSuballocations += UsedPages[Index]->NumSubAllocations;

				VULKAN_LOGMEMORY(TEXT("\t\t%s%d:(%6.2fmb/%6.2fmb) ID %4d %4d suballocs, %4d free chunks,DeviceMemory %p"),
					TypeName,
					Index,
					UsedPages[Index]->UsedSize / (1024.f*1024.f),
					UsedPages[Index]->MaxSize / (1024.f*1024.f),
					UsedPages[Index]->GetHandleId(),
					UsedPages[Index]->NumSubAllocations,
					UsedPages[Index]->FreeList.Num(),
					(void*)UsedPages[Index]->MemoryAllocation->GetHandle());
			}
		};

		int32 Index = 0;

		for(TArray<FVulkanSubresourceAllocator*>& Page : ActivePages)
		{
			bool bIsImage = Index < PageSizeBuckets.Num() && (PageSizeBuckets[Index].BucketMask & FVulkanPageSizeBucket::BUCKET_MASK_IMAGE) == FVulkanPageSizeBucket::BUCKET_MASK_IMAGE;
			Index++;
			DumpPages(Page, TEXT("ActivePages"), bIsImage);
		}
	}

	//Try to reallocate an allocation on a different page. Used When defragging
	bool FVulkanResourceHeap::TryRealloc(FVulkanAllocation& OutAllocation, FVulkanEvictable* AllocationOwner, EType Type, uint32 Size, uint32 Alignment, EVulkanAllocationMetaType MetaType)
	{
		FDeviceMemoryManager& DeviceMemoryManager = Owner->GetDevice().GetDeviceMemoryManager();
		FVulkanPageSizeBucket MemoryBucket;
		uint32 BucketId = GetPageSizeBucket(MemoryBucket, Type, Size, false);


		bool bHasUnifiedMemory = DeviceMemoryManager.HasUnifiedMemory();

		TArray<FVulkanSubresourceAllocator*>& UsedPages = ActivePages[BucketId];
		EVulkanAllocationType AllocationType = (Type == EType::Image) ? EVulkanAllocationImage : EVulkanAllocationBuffer;
		uint8 AllocationFlags = (!bHasUnifiedMemory && MetaTypeCanEvict(MetaType)) ? VulkanAllocationFlagsCanEvict : 0;

		check(Size <= MemoryBucket.AllocationMax);
		{
			// Check Used pages to see if we can fit this in
			for (int32 Index = 0; Index < UsedPages.Num(); ++Index)
			{
				FVulkanSubresourceAllocator* Page = UsedPages[Index];
				if(!Page->bLocked)
				{
					if (Page->GetSubresourceAllocatorFlags() == AllocationFlags)
					{
						if (Page->TryAllocate2(OutAllocation, AllocationOwner, Size, Alignment, MetaType, __FILE__, __LINE__))
						{
							IncMetaStats(MetaType, OutAllocation.Size);
							return true;
						}
					}
				}
			}
		}
		return false;
	}


	bool FVulkanResourceHeap::AllocateResource(FVulkanAllocation& OutAllocation, FVulkanEvictable* AllocationOwner, EType Type, uint32 Size, uint32 Alignment, bool bMapAllocation, bool bForceSeparateAllocation, EVulkanAllocationMetaType MetaType, bool bExternal, const char* File, uint32 Line)
	{
		SCOPED_NAMED_EVENT(FResourceHeap_AllocateResource, FColor::Cyan);
		FScopeLock ScopeLock(&PagesLock);

		bForceSeparateAllocation = bForceSeparateAllocation || GVulkanSingleAllocationPerResource != 0;

		FDeviceMemoryManager& DeviceMemoryManager = Owner->GetDevice().GetDeviceMemoryManager();
		FVulkanPageSizeBucket MemoryBucket;
		uint32 BucketId = GetPageSizeBucket(MemoryBucket, Type, Size, bForceSeparateAllocation);

		bool bHasUnifiedMemory = DeviceMemoryManager.HasUnifiedMemory();
		TArray<FVulkanSubresourceAllocator*>& UsedPages = ActivePages[BucketId];
		EVulkanAllocationType AllocationType = (Type == EType::Image) ? EVulkanAllocationImage : EVulkanAllocationBuffer;
		uint8 AllocationFlags = (!bHasUnifiedMemory && MetaTypeCanEvict(MetaType)) ? VulkanAllocationFlagsCanEvict : 0;
		if(bMapAllocation)
		{
			AllocationFlags |= VulkanAllocationFlagsMapped;
		}
		
		uint32 AllocationSize;

		if (!bForceSeparateAllocation)
		{
			if(Size < MemoryBucket.PageSize) // Last bucket, for dedicated allocations has max size set to 0, preventing reuse
			{
				// Check Used pages to see if we can fit this in
				for (int32 Index = 0; Index < UsedPages.Num(); ++Index)
				{
					FVulkanSubresourceAllocator* Page = UsedPages[Index];
					if(Page->GetSubresourceAllocatorFlags() == AllocationFlags)
					{
						check(Page->MemoryAllocation->IsMapped() == bMapAllocation);
						if(Page->TryAllocate2(OutAllocation, AllocationOwner, Size, Alignment, MetaType, File, Line))
						{
							IncMetaStats(MetaType, OutAllocation.Size);
							return true;
						}
					}
				}
			}
			AllocationSize = FMath::Max(Size, MemoryBucket.PageSize); // for allocations above max, which are forced to be separate allocations
		}
		else
		{
			// We get here when bForceSeparateAllocation is true, which is used for lazy allocations, since pooling those doesn't make sense.
			AllocationSize = Size;
		}

		FDeviceMemoryAllocation* DeviceMemoryAllocation = DeviceMemoryManager.Alloc(true, AllocationSize, MemoryTypeIndex, nullptr, VULKAN_MEMORY_HIGHEST_PRIORITY, bExternal, File, Line);
		if (!DeviceMemoryAllocation && Size != AllocationSize)
		{
			// Retry with a smaller size
			DeviceMemoryAllocation = DeviceMemoryManager.Alloc(true, Size, MemoryTypeIndex, nullptr, VULKAN_MEMORY_HIGHEST_PRIORITY, bExternal, File, Line);
			if(!DeviceMemoryAllocation)
			{
				return false;
			}
		}
		if (!DeviceMemoryAllocation)
		{
			UE_LOGF(LogVulkanRHI, Warning, "Out of memory on Vulkan; MemoryTypeIndex=%d, AllocSize=%0.3fMB", MemoryTypeIndex, (float)AllocationSize / 1048576.0f);
			return false;
		}
		if (bMapAllocation)
		{
			DeviceMemoryAllocation->Map(AllocationSize, 0);
		}

		uint32 BufferId = 0;
		if (UseVulkanDescriptorCache())
		{
			BufferId = ++GVulkanBufferHandleIdCounter;
		}
		
		++PageIDCounter;
		FVulkanSubresourceAllocator* Page = new FVulkanSubresourceAllocator(AllocationType, Owner, AllocationFlags, DeviceMemoryAllocation, MemoryTypeIndex, BufferId);
		Owner->RegisterSubresourceAllocator(Page);
		Page->BucketId = BucketId;
		ActivePages[BucketId].Add(Page);

		UsedMemory += AllocationSize;

		PeakPageSize = FMath::Max(PeakPageSize, AllocationSize);


		bool bOk = Page->TryAllocate2(OutAllocation, AllocationOwner, Size, Alignment, MetaType, File, Line);
		if(bOk)
		{
			IncMetaStats(MetaType, OutAllocation.Size);
		}
		return bOk;
	}

	bool FVulkanResourceHeap::AllocateDedicatedImage(FVulkanAllocation& OutAllocation, FVulkanEvictable* AllocationOwner, VkImage Image, uint32 Size, uint32 Alignment, EVulkanAllocationMetaType MetaType, bool bExternal, const char* File, uint32 Line)
	{
#if VULKAN_SUPPORTS_DEDICATED_ALLOCATION
		FScopeLock ScopeLock(&PagesLock);

		uint32 AllocationSize = Size;

		check(Image != VK_NULL_HANDLE);
		VkMemoryDedicatedAllocateInfoKHR DedicatedAllocInfo;
		ZeroVulkanStruct(DedicatedAllocInfo, VK_STRUCTURE_TYPE_MEMORY_DEDICATED_ALLOCATE_INFO_KHR);
		DedicatedAllocInfo.image = Image;
		FDeviceMemoryAllocation* DeviceMemoryAllocation = Owner->GetDevice().GetDeviceMemoryManager().Alloc(true, AllocationSize, MemoryTypeIndex, &DedicatedAllocInfo, VULKAN_MEMORY_HIGHEST_PRIORITY, bExternal, File, Line);
		if(!DeviceMemoryAllocation)
		{
			return false;
		}

		
		uint32 BufferId = 0;
		if (UseVulkanDescriptorCache())
		{
			BufferId = ++GVulkanBufferHandleIdCounter;
		}

		++PageIDCounter;
		FVulkanSubresourceAllocator* NewPage = new FVulkanSubresourceAllocator(EVulkanAllocationImageDedicated, Owner, 0, DeviceMemoryAllocation, MemoryTypeIndex, BufferId);
		Owner->RegisterSubresourceAllocator(NewPage);
		NewPage->BucketId = 0xff;
		UsedDedicatedImagePages.Add(NewPage);

		UsedMemory += AllocationSize;

		PeakPageSize = FMath::Max(PeakPageSize, AllocationSize);
		return NewPage->TryAllocate2(OutAllocation, AllocationOwner, Size, Alignment, MetaType, File, Line);
#else
		checkNoEntry();
		return false;
#endif
	}

	FMemoryManager::FMemoryManager(FVulkanDevice& InDevice)
		: Device(InDevice)
		, DeviceMemoryManager(&InDevice.GetDeviceMemoryManager())
		, AllBufferAllocationsFreeListHead(-1)
	{
	}

	FMemoryManager::~FMemoryManager()
	{
		Deinit();
	}

	void FMemoryManager::Init()
	{
		const uint32 TypeBits = (1 << DeviceMemoryManager->GetNumMemoryTypes()) - 1;

		const VkPhysicalDeviceMemoryProperties& MemoryProperties = DeviceMemoryManager->GetMemoryProperties();

		ResourceTypeHeaps.AddZeroed(MemoryProperties.memoryTypeCount);

		// Upload heap. Spec requires this combination to exist.
		{
			uint32 TypeIndex = 0;
			VERIFYVULKANRESULT(DeviceMemoryManager->GetMemoryTypeFromProperties(TypeBits, VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT | VK_MEMORY_PROPERTY_HOST_COHERENT_BIT, &TypeIndex));
			uint64 HeapSize = MemoryProperties.memoryHeaps[MemoryProperties.memoryTypes[TypeIndex].heapIndex].size;
			ResourceTypeHeaps[TypeIndex] = new FVulkanResourceHeap(this, TypeIndex, STAGING_HEAP_PAGE_SIZE);

			auto& PageSizeBuckets = ResourceTypeHeaps[TypeIndex]->PageSizeBuckets;
			FVulkanPageSizeBucket Bucket0 = {STAGING_HEAP_PAGE_SIZE, STAGING_HEAP_PAGE_SIZE, FVulkanPageSizeBucket::BUCKET_MASK_IMAGE | FVulkanPageSizeBucket::BUCKET_MASK_BUFFER};
			FVulkanPageSizeBucket Bucket1 = { UINT64_MAX, 0, FVulkanPageSizeBucket::BUCKET_MASK_IMAGE | FVulkanPageSizeBucket::BUCKET_MASK_BUFFER};
			PageSizeBuckets.Add(Bucket0);
			PageSizeBuckets.Add(Bucket1);
		}

		// Download heap. Optional type per the spec.
		{
			uint32 TypeIndex = 0;
			{
				uint32 HostVisCachedIndex = 0;
				VkResult HostCachedResult = DeviceMemoryManager->GetMemoryTypeFromProperties(TypeBits, VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT | VK_MEMORY_PROPERTY_HOST_CACHED_BIT, &HostVisCachedIndex);
				uint32 HostVisIndex = 0;
				VkResult HostResult = DeviceMemoryManager->GetMemoryTypeFromProperties(TypeBits, VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT, &HostVisIndex);
				if (HostCachedResult == VK_SUCCESS)
				{
					TypeIndex = HostVisCachedIndex;
				}
				else if (HostResult == VK_SUCCESS)
				{
					TypeIndex = HostVisIndex;
				}
				else
				{
					// Redundant as it would have asserted above...
					UE_LOGF(LogVulkanRHI, Fatal, "No Memory Type found supporting VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT!");
				}
			}
			uint64 HeapSize = MemoryProperties.memoryHeaps[MemoryProperties.memoryTypes[TypeIndex].heapIndex].size;
			ResourceTypeHeaps[TypeIndex] = new FVulkanResourceHeap(this, TypeIndex, STAGING_HEAP_PAGE_SIZE);

			auto& PageSizeBuckets = ResourceTypeHeaps[TypeIndex]->PageSizeBuckets;
			FVulkanPageSizeBucket Bucket0 = { STAGING_HEAP_PAGE_SIZE, STAGING_HEAP_PAGE_SIZE, FVulkanPageSizeBucket::BUCKET_MASK_IMAGE | FVulkanPageSizeBucket::BUCKET_MASK_BUFFER };
			FVulkanPageSizeBucket Bucket1 = { UINT64_MAX, 0, FVulkanPageSizeBucket::BUCKET_MASK_IMAGE | FVulkanPageSizeBucket::BUCKET_MASK_BUFFER };
			PageSizeBuckets.Add(Bucket0);
			PageSizeBuckets.Add(Bucket1);
		}


		// Setup main GPU heap
		{
			uint32 Index;
			check(DeviceMemoryManager->GetMemoryTypeFromProperties(TypeBits, VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT, &Index) == VK_SUCCESS);

			for (Index = 0; Index < MemoryProperties.memoryTypeCount; ++Index)
			{
				const int32 HeapIndex = MemoryProperties.memoryTypes[Index].heapIndex;
				const VkDeviceSize HeapSize = MemoryProperties.memoryHeaps[HeapIndex].size;
				if(!ResourceTypeHeaps[Index] )
				{
					ResourceTypeHeaps[Index] = new FVulkanResourceHeap(this, Index);
					ResourceTypeHeaps[Index]->bIsHostCachedSupported = VKHasAllFlags(MemoryProperties.memoryTypes[Index].propertyFlags, VK_MEMORY_PROPERTY_HOST_CACHED_BIT);
					ResourceTypeHeaps[Index]->bIsLazilyAllocatedSupported = VKHasAllFlags(MemoryProperties.memoryTypes[Index].propertyFlags, VK_MEMORY_PROPERTY_LAZILY_ALLOCATED_BIT);
					auto& PageSizeBuckets = ResourceTypeHeaps[Index]->PageSizeBuckets;

#if PLATFORM_ANDROID
					FVulkanPageSizeBucket BucketImage= { UINT64_MAX, (uint32)ANDROID_MAX_HEAP_IMAGE_PAGE_SIZE, FVulkanPageSizeBucket::BUCKET_MASK_IMAGE };
					FVulkanPageSizeBucket BucketBuffer = { UINT64_MAX, (uint32)ANDROID_MAX_HEAP_BUFFER_PAGE_SIZE, FVulkanPageSizeBucket::BUCKET_MASK_BUFFER };
					PageSizeBuckets.Add(BucketImage);
					PageSizeBuckets.Add(BucketBuffer);
#else
					uint32 SmallAllocationThreshold = 2 << 20;
					uint32 LargeAllocationThreshold = UE_VK_MEMORY_MAX_SUB_ALLOCATION;
					VkDeviceSize SmallPageSize = 8llu << 20;
					VkDeviceSize LargePageSize = FMath::Min<VkDeviceSize>(HeapSize / 8, GPU_ONLY_HEAP_PAGE_SIZE);

				
					FVulkanPageSizeBucket BucketSmallImage = { SmallAllocationThreshold, (uint32)SmallPageSize, FVulkanPageSizeBucket::BUCKET_MASK_IMAGE };
					FVulkanPageSizeBucket BucketLargeImage = { LargeAllocationThreshold, (uint32)LargePageSize, FVulkanPageSizeBucket::BUCKET_MASK_IMAGE };
					FVulkanPageSizeBucket BucketSmallBuffer = { SmallAllocationThreshold, (uint32)SmallPageSize, FVulkanPageSizeBucket::BUCKET_MASK_BUFFER };
					FVulkanPageSizeBucket BucketLargeBuffer = { LargeAllocationThreshold, (uint32)LargePageSize, FVulkanPageSizeBucket::BUCKET_MASK_BUFFER };
					FVulkanPageSizeBucket BucketRemainder = { UINT64_MAX, 0, FVulkanPageSizeBucket::BUCKET_MASK_BUFFER|FVulkanPageSizeBucket::BUCKET_MASK_IMAGE };
					PageSizeBuckets.Add(BucketSmallImage);
					PageSizeBuckets.Add(BucketLargeImage);
					PageSizeBuckets.Add(BucketSmallBuffer);
					PageSizeBuckets.Add(BucketLargeBuffer);
					PageSizeBuckets.Add(BucketRemainder);
#endif
				}
			}
		}
	}

	void FMemoryManager::Deinit()
	{
		{
			ProcessPendingUBFreesNoLock(true);
			check(UBAllocations.PendingFree.Num() == 0);
		}
		DestroyResourceAllocations();

		for (int32 Index = 0; Index < ResourceTypeHeaps.Num(); ++Index)
		{
			delete ResourceTypeHeaps[Index];
			ResourceTypeHeaps[Index] = nullptr;
		}
		ResourceTypeHeaps.Empty(0);
	}

	void FMemoryManager::DestroyResourceAllocations()
	{
		ReleaseFreedResources(true);

		for (auto& UsedAllocations : UsedBufferAllocations)
		{
			for (int32 Index = UsedAllocations.Num() - 1; Index >= 0; --Index)
			{
				FVulkanSubresourceAllocator* BufferAllocation = UsedAllocations[Index];
				if (!BufferAllocation->JoinFreeBlocks())
				{
					UE_LOGF(LogVulkanRHI, Warning, "Suballocation(s) for Buffer %p were not released.%ls", (void*)BufferAllocation->Buffer, *VULKAN_TRACK_STRING(BufferAllocation->Track));
				}

				BufferAllocation->Destroy(&Device);
				Device.GetDeviceMemoryManager().Free(BufferAllocation->MemoryAllocation);
				delete BufferAllocation;
			}
			UsedAllocations.Empty(0);
		}

		for (auto& FreeAllocations : FreeBufferAllocations)
		{
			for (int32 Index = 0; Index < FreeAllocations.Num(); ++Index)
			{
				FVulkanSubresourceAllocator* BufferAllocation = FreeAllocations[Index];
				BufferAllocation->Destroy(&Device);
				Device.GetDeviceMemoryManager().Free(BufferAllocation->MemoryAllocation);
				delete BufferAllocation;
			}
			FreeAllocations.Empty(0);
		}
	}

	void FMemoryManager::ReleaseFreedResources(bool bImmediately)
	{
		TArray<FVulkanSubresourceAllocator*> BufferAllocationsToRelease;
		{
			FScopeLock ScopeLock(&UsedFreeBufferAllocationsLock);
			for (auto& FreeAllocations : FreeBufferAllocations)
			{
				for (int32 Index = 0; Index < FreeAllocations.Num(); ++Index)
				{
					FVulkanSubresourceAllocator* BufferAllocation = FreeAllocations[Index];
					if (bImmediately || BufferAllocation->FrameFreed + GVulkanNumFramesToWaitForResourceDelete < GFrameNumberRenderThread)
					{
						BufferAllocationsToRelease.Add(BufferAllocation);
						FreeAllocations.RemoveAtSwap(Index, EAllowShrinking::No);
					}
				}
			}
		}

		for (int32 Index = 0; Index < BufferAllocationsToRelease.Num(); ++Index)
		{
			FVulkanSubresourceAllocator* BufferAllocation = BufferAllocationsToRelease[Index];

			BufferAllocation->Destroy(&Device);

			FVulkanResourceHeap* Heap = ResourceTypeHeaps[BufferAllocation->MemoryTypeIndex];
			Heap->ReleasePage(BufferAllocation);
		}

		DeviceMemoryManager->TrimMemory(bImmediately);
	}

	void FMemoryManager::ReleaseFreedPages(const FVulkanContextArray& Contexts)
	{
		auto CanDefragHeap = [](FVulkanResourceHeap* Heap)
		{
			if (!GVulkanEnableDefrag)
			{
				return false;
			}

			for(TArray<FVulkanSubresourceAllocator*>& Pages : Heap->ActivePages) 
			{
				for(FVulkanSubresourceAllocator* Allocator : Pages) 
				{
					if(0 == (Allocator->GetSubresourceAllocatorFlags() & VulkanAllocationFlagsCanEvict) && Allocator->CanDefrag())
					{
						return true;
					}
				}
			}
			return false;
		};

		auto CanEvictHeap = [](FVulkanResourceHeap* Heap)
		{
			if (!GVulkanEnableDefrag)
			{
				return false;
			}

			for(TArray<FVulkanSubresourceAllocator*>& Pages : Heap->ActivePages) 
			{
				for(FVulkanSubresourceAllocator* Allocator : Pages) 
				{
					if(VulkanAllocationFlagsCanEvict == (Allocator->GetSubresourceAllocatorFlags() & VulkanAllocationFlagsCanEvict))
					{
						return true;
					}
				}
			}
			return false;
		};

		ReleaseFreedResources(false);

		const int32 PrimaryHeapIndex = DeviceMemoryManager->PrimaryHeapIndex;
		FVulkanResourceHeap* BestEvictHeap = 0;
		uint64 BestEvictHeapSize = 0;

		FVulkanResourceHeap* BestDefragHeap = 0;
		uint64 BestDefragHeapSize = 0;

		{
			for (FVulkanResourceHeap* Heap : ResourceTypeHeaps)
			{
				if (Heap->HeapIndex == PrimaryHeapIndex)
				{
					FScopeLock ScopeLock(&Heap->PagesLock);

					uint64 UsedSize = Heap->UsedMemory;
					if (CanDefragHeap(Heap) && BestDefragHeapSize < UsedSize)
					{
						BestDefragHeap = Heap;
						BestDefragHeapSize = UsedSize;
					}
					if (CanEvictHeap(Heap) && BestEvictHeapSize < UsedSize)
					{
						BestEvictHeap = Heap;
						BestEvictHeapSize = UsedSize;
					}
				}
			}
		}

		if(BestEvictHeap && ((GVulkanEvictOnePage || UpdateEvictThreshold(true)) && PrimaryHeapIndex >= 0))
		{
			GVulkanEvictOnePage = 0;
			PendingEvictBytes += BestEvictHeap->EvictOne(Device, Contexts);
		}

#if !PLATFORM_ANDROID
		if(BestDefragHeap)
		{
			uint32 Count = 1;
			if(GVulkanDefragOnce)
			{
				Count = 0x7fffffff;
			}

			if(GVulkanDefragOnce)
			{
				DumpMemory();
			}
			BestDefragHeap->DefragTick(Device, Contexts, Count);

			if(GVulkanDefragOnce)
			{
				DumpMemory();
				GVulkanDefragOnce = 0;
			}
		}
#endif
		DeviceMemoryManager->TrimMemory(false);
	}

	void FMemoryManager::FreeVulkanAllocationPooledBuffer(FVulkanAllocation& Allocation)
	{
		SCOPED_NAMED_EVENT(FResourceHeapManager_FreeVulkanAllocationPooledBuffer, FColor::Cyan);
		DecMetaStats(Allocation.MetaType, Allocation.Size);
		const uint32 Index = Allocation.AllocatorIndex;
		GetSubresourceAllocator(Index)->Free(Allocation);
	}
	void FMemoryManager::FreeVulkanAllocationBuffer(FVulkanAllocation& Allocation)
	{
		SCOPED_NAMED_EVENT(FResourceHeapManager_FreeVulkanAllocationBuffer, FColor::Cyan);
		DecMetaStats(Allocation.MetaType, Allocation.Size);
		const uint32 Index = Allocation.AllocatorIndex;
		GetSubresourceAllocator(Index)->Free(Allocation);
	}

	void FMemoryManager::FreeVulkanAllocationImage(FVulkanAllocation& Allocation)
	{
		SCOPED_NAMED_EVENT(FResourceHeapManager_FreeVulkanAllocationImage, FColor::Cyan);
		DecMetaStats(Allocation.MetaType, Allocation.Size);
		const uint32 Index = Allocation.AllocatorIndex;
		GetSubresourceAllocator(Index)->Free(Allocation);
	}
	void FMemoryManager::FreeVulkanAllocationImageDedicated(FVulkanAllocation& Allocation)
	{
		SCOPED_NAMED_EVENT(FResourceHeapManager_FreeVulkanAllocationImageDedicated, FColor::Cyan);
		DecMetaStats(Allocation.MetaType, Allocation.Size);
		const uint32 Index = Allocation.AllocatorIndex;
		GetSubresourceAllocator(Index)->Free(Allocation);
	}

	void FVulkanSubresourceAllocator::SetFreePending(FVulkanAllocation& Allocation)
	{
		check(Allocation.GetType() == Type);
		check(Allocation.AllocatorIndex == GetAllocatorIndex());
		{
			FScopeLock ScopeLock(&SubresourceAllocatorCS);
			FVulkanAllocationInternal& Data = InternalData[Allocation.AllocationIndex];
			Data.State = FVulkanAllocationInternal::EFREEPENDING;
		}
	}



	void FVulkanSubresourceAllocator::Free(FVulkanAllocation& Allocation)
	{
		check(Allocation.Type == Type);
		check(Allocation.AllocatorIndex == GetAllocatorIndex());
		bool bTryFree = false;
		{
			FScopeLock ScopeLock(&SubresourceAllocatorCS);
			FreeCalls++;
			uint32 AllocationOffset;
			uint32 AllocationSize;
			{
				FVulkanAllocationInternal& Data = InternalData[Allocation.AllocationIndex];
				bool bWasDiscarded = Data.State == FVulkanAllocationInternal::EFREEDISCARDED;
				check(Data.State == FVulkanAllocationInternal::EALLOCATED || Data.State == FVulkanAllocationInternal::EFREEPENDING || Data.State == FVulkanAllocationInternal::EFREEDISCARDED);
				AllocationOffset = Data.AllocationOffset;
				AllocationSize = Data.AllocationSize;
				if(!bWasDiscarded)
				{
					MemoryUsed[Allocation.MetaType] -= AllocationSize;
					LLM_TRACK_VULKAN_HIGH_LEVEL_FREE(Data);
					LLM_TRACK_VULKAN_SPARE_MEMORY_GPU((int64)Allocation.Size);
					VULKAN_FREE_TRACK_INFO(Data.Track);
				}
				Data.State = FVulkanAllocationInternal::EFREED;
				FreeInternalData(Allocation.AllocationIndex);
				Allocation.AllocationIndex = -1;
				if(bWasDiscarded)
				{
					//this occurs if we do full defrag when there are pending frees. in that case the memory is just not moved to the new block.
					return;
				}
			}
			FRange NewFree;
			NewFree.Offset = AllocationOffset;
			NewFree.Size = AllocationSize;
			check(NewFree.Offset <= GetMaxSize());
			check(NewFree.Offset + NewFree.Size <= GetMaxSize());
			FRange::Add(FreeList, NewFree);
			UsedSize -= AllocationSize;
			NumSubAllocations--;
			check(UsedSize >= 0);
			if (JoinFreeBlocks())
			{
				bTryFree = true; //cannot free here as it will cause incorrect lock ordering
			}
		}

		if (bTryFree)
		{
			Owner->ReleaseSubresourceAllocator(this);
		}
	}

	void FMemoryManager::FreeVulkanAllocation(FVulkanAllocation& Allocation, EVulkanFreeFlags FreeFlags)
	{
		//by default, all allocations are implicitly deferred, unless manually handled.
		if(FreeFlags & EVulkanFreeFlag_DontDefer)
		{
			switch (Allocation.Type)
			{
			case EVulkanAllocationEmpty:
				break;
			case EVulkanAllocationPooledBuffer:
				FreeVulkanAllocationPooledBuffer(Allocation);
				break;
			case EVulkanAllocationBuffer:
				FreeVulkanAllocationBuffer(Allocation);
				break;
			case EVulkanAllocationImage:
				FreeVulkanAllocationImage(Allocation);
				break;
			case EVulkanAllocationImageDedicated:
				FreeVulkanAllocationImageDedicated(Allocation);
				break;
			}
			FMemory::Memzero(&Allocation, sizeof(Allocation));
			Allocation.Type = EVulkanAllocationEmpty;
		}
		else
		{
			GetSubresourceAllocator(Allocation.AllocatorIndex)->SetFreePending(Allocation);
			Device.GetDeferredDeletionQueue().EnqueueResourceAllocation(Allocation);
		}
		check(!Allocation.HasAllocation());
	}


	void FVulkanSubresourceAllocator::Destroy(FVulkanDevice* Device)
	{
		// Does not need to go in the deferred deletion queue
		if(Buffer != VK_NULL_HANDLE)
		{
			VulkanRHI::vkDestroyBuffer(Device->GetHandle(), Buffer, VULKAN_CPU_ALLOCATOR);
			Buffer = VK_NULL_HANDLE;
		}
	}

	FVulkanSubresourceAllocator::FVulkanSubresourceAllocator(EVulkanAllocationType InType, FMemoryManager* InOwner, uint8 InSubresourceAllocatorFlags, FDeviceMemoryAllocation* InDeviceMemoryAllocation,
		uint32 InMemoryTypeIndex, VkMemoryPropertyFlags InMemoryPropertyFlags,
		uint32 InAlignment, VkBuffer InBuffer, uint32 InBufferSize, uint32 InBufferId, VkBufferUsageFlags InBufferUsageFlags, int32 InPoolSizeIndex)
		: Type(InType)
		, Owner(InOwner)
		, MemoryTypeIndex(InMemoryTypeIndex)
		, MemoryPropertyFlags(InMemoryPropertyFlags)
		, MemoryAllocation(InDeviceMemoryAllocation)
		, MaxSize(InBufferSize)
		, Alignment(InAlignment)
		, FrameFreed(0)
		, UsedSize(0)
		, BufferUsageFlags(InBufferUsageFlags)
		, Buffer(InBuffer)
		, BufferId(InBufferId)
		, PoolSizeIndex(InPoolSizeIndex)
		, AllocatorIndex(0xffffffff)
		, SubresourceAllocatorFlags(InSubresourceAllocatorFlags)

	{
		FMemory::Memzero(MemoryUsed);

		if(InDeviceMemoryAllocation->IsMapped())
		{
			SubresourceAllocatorFlags |= VulkanAllocationFlagsMapped;
		}
		else
		{
			SubresourceAllocatorFlags &= ~VulkanAllocationFlagsMapped;
		}

		FRange FullRange;
		FullRange.Offset = 0;
		FullRange.Size = MaxSize;
		FreeList.Add(FullRange);

		VULKAN_FILL_TRACK_INFO(Track, __FILE__, __LINE__);
	}

	FVulkanSubresourceAllocator::FVulkanSubresourceAllocator(EVulkanAllocationType InType, FMemoryManager* InOwner, uint8 InSubresourceAllocatorFlags, FDeviceMemoryAllocation* InDeviceMemoryAllocation,
		uint32 InMemoryTypeIndex, uint32 BufferId)
		: Type(InType)
		, Owner(InOwner)
		, MemoryTypeIndex(InMemoryTypeIndex)
		, MemoryPropertyFlags(0)
		, MemoryAllocation(InDeviceMemoryAllocation)
		, Alignment(0)
		, FrameFreed(0)
		, UsedSize(0)
		, BufferUsageFlags(0)
		, Buffer(VK_NULL_HANDLE)
		, BufferId(BufferId)
		, PoolSizeIndex(0x7fffffff)
		, AllocatorIndex(0xffffffff)
		, SubresourceAllocatorFlags(InSubresourceAllocatorFlags)

	{
		FMemory::Memzero(MemoryUsed);
		MaxSize = InDeviceMemoryAllocation->GetSize();

		if (InDeviceMemoryAllocation->IsMapped())
		{
			SubresourceAllocatorFlags |= VulkanAllocationFlagsMapped;
		}
		else
		{
			SubresourceAllocatorFlags &= ~VulkanAllocationFlagsMapped;
		}

		FRange FullRange;
		FullRange.Offset = 0;
		FullRange.Size = MaxSize;
		FreeList.Add(FullRange);

		VULKAN_FILL_TRACK_INFO(Track, __FILE__, __LINE__);
	}

	FVulkanSubresourceAllocator::~FVulkanSubresourceAllocator()
	{
		if (!JoinFreeBlocks())
		{
			UE_LOGF(LogVulkanRHI, Warning, "FVulkanSubresourceAllocator %p has unfreed %ls resources %ls", (void*)this, VulkanAllocationTypeToString(Type), *VULKAN_TRACK_STRING(Track));
			uint32 LeakCount = 0;
			for (FVulkanAllocationInternal& Data : InternalData)
			{
				if (Data.State == FVulkanAllocationInternal::EALLOCATED)
				{
					UE_LOGF(LogVulkanRHI, Warning, " ** LEAK %03d [%08x-%08x] %u  %ls \n%ls", LeakCount++, Data.AllocationOffset, Data.AllocationSize,  Data.Size, VulkanAllocationMetaTypeToString(Data.MetaType), *VULKAN_TRACK_STRING(Data.Track));
				}
			}
		}
		check(0 == MemoryAllocation);
		VULKAN_FREE_TRACK_INFO(Track);
	}

	static uint32 CalculateBufferAlignmentFromVKUsageFlags(FVulkanDevice& InDevice, const VkBufferUsageFlags BufferUsageFlags)
	{
		const VkPhysicalDeviceLimits& Limits = InDevice.GetLimits();

		const bool bIsTexelBuffer = VKHasAnyFlags(BufferUsageFlags, (VK_BUFFER_USAGE_UNIFORM_TEXEL_BUFFER_BIT | VK_BUFFER_USAGE_STORAGE_TEXEL_BUFFER_BIT));
		const bool bIsStorageBuffer = VKHasAnyFlags(BufferUsageFlags, VK_BUFFER_USAGE_STORAGE_BUFFER_BIT);
		const bool bIsVertexOrIndexBuffer = VKHasAnyFlags(BufferUsageFlags, (VK_BUFFER_USAGE_VERTEX_BUFFER_BIT | VK_BUFFER_USAGE_INDEX_BUFFER_BIT));
		const bool bIsAccelerationStructureBuffer = VKHasAnyFlags(BufferUsageFlags, VK_BUFFER_USAGE_ACCELERATION_STRUCTURE_STORAGE_BIT_KHR);
		const bool bIsUniformBuffer = VKHasAnyFlags(BufferUsageFlags, VK_BUFFER_USAGE_UNIFORM_BUFFER_BIT);

		// Buffers are sometimes directly cast into classes with 16byte alignment expectations (like FVector3f)
		uint32 Alignment = 16u;

		if (bIsTexelBuffer || bIsStorageBuffer)
		{
			Alignment = FMath::Max(Alignment, (uint32)Limits.minTexelBufferOffsetAlignment);
			Alignment = FMath::Max(Alignment, (uint32)Limits.minStorageBufferOffsetAlignment);
		}
		else if (bIsVertexOrIndexBuffer)
		{
			// No alignment restrictions on Vertex or Index buffers, leave it at 1
		}
		else if (bIsAccelerationStructureBuffer)
		{
			Alignment = FMath::Max(Alignment, GRHIRayTracingAccelerationStructureAlignment);
		}
		else if (bIsUniformBuffer)
		{
			Alignment = FMath::Max(Alignment, (uint32)Limits.minUniformBufferOffsetAlignment);
		}
		else
		{
			checkf(false, TEXT("Unknown buffer alignment for VkBufferUsageFlags combination: 0x%x (%s)"), BufferUsageFlags, VK_FLAGS_TO_STRING(VkBufferUsageFlags, BufferUsageFlags));
		}

		return Alignment;
	}

	uint32 FMemoryManager::CalculateBufferAlignment(FVulkanDevice& InDevice, EBufferUsageFlags InUEUsage, bool bZeroSize)
	{
		const VkBufferUsageFlags VulkanBufferUsage = FVulkanBuffer::UEToVKBufferUsageFlags(InDevice, InUEUsage, bZeroSize);

		uint32 Alignment = CalculateBufferAlignmentFromVKUsageFlags(InDevice, VulkanBufferUsage);

		if (EnumHasAllFlags(InUEUsage, BUF_RayTracingScratch))
		{
			Alignment = GRHIRayTracingScratchBufferAlignment;
		}

		return Alignment;
	}

	float FMemoryManager::CalculateBufferPriority(const VkBufferUsageFlags BufferUsageFlags)
	{
		float Priority = VULKAN_MEMORY_MEDIUM_PRIORITY;

		if (VKHasAnyFlags(BufferUsageFlags, (VK_BUFFER_USAGE_STORAGE_TEXEL_BUFFER_BIT | VK_BUFFER_USAGE_INDIRECT_BUFFER_BIT)))
		{
			Priority = VULKAN_MEMORY_HIGHEST_PRIORITY;
		}
		else if (VKHasAnyFlags(BufferUsageFlags, (VK_BUFFER_USAGE_UNIFORM_TEXEL_BUFFER_BIT | VK_BUFFER_USAGE_STORAGE_BUFFER_BIT)))
		{
			Priority = VULKAN_MEMORY_MEDIUM_PRIORITY;
		}
		else if (VKHasAnyFlags(BufferUsageFlags, 
			(VK_BUFFER_USAGE_ACCELERATION_STRUCTURE_STORAGE_BIT_KHR |
			VK_BUFFER_USAGE_SHADER_BINDING_TABLE_BIT_KHR |
			VK_BUFFER_USAGE_ACCELERATION_STRUCTURE_BUILD_INPUT_READ_ONLY_BIT_KHR)))
		{
			Priority = VULKAN_MEMORY_MEDIUM_PRIORITY;
		}
		else if (VKHasAnyFlags(BufferUsageFlags, (VK_BUFFER_USAGE_VERTEX_BUFFER_BIT | VK_BUFFER_USAGE_INDEX_BUFFER_BIT)))
		{
			Priority = VULKAN_MEMORY_LOW_PRIORITY;
		}
		else if (VKHasAnyFlags(BufferUsageFlags, VK_BUFFER_USAGE_UNIFORM_BUFFER_BIT))
		{
			Priority = VULKAN_MEMORY_HIGHER_PRIORITY;
		}
		else
		{
			checkf(false, TEXT("Unknown priority for VkBufferUsageFlags combination: 0x%x (%s)"), BufferUsageFlags, VK_FLAGS_TO_STRING(VkBufferUsageFlags, BufferUsageFlags));
		}

		return Priority;
	}

	static VkMemoryPropertyFlags GetMemoryPropertyFlags(EVulkanAllocationFlags AllocFlags, bool bHasUnifiedMemory)
	{
		VkMemoryPropertyFlags MemFlags = 0;

		checkf(!(EnumHasAnyFlags(AllocFlags, EVulkanAllocationFlags::PreferBAR) && !EnumHasAnyFlags(AllocFlags, EVulkanAllocationFlags::HostVisible)), TEXT("PreferBAR should always be used with HostVisible."));

		if (EnumHasAnyFlags(AllocFlags, EVulkanAllocationFlags::HostCached))
		{
			MemFlags = VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT | VK_MEMORY_PROPERTY_HOST_CACHED_BIT;
		}
		else if (EnumHasAnyFlags(AllocFlags, EVulkanAllocationFlags::HostVisible))
		{
			MemFlags = VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT | VK_MEMORY_PROPERTY_HOST_COHERENT_BIT;

			if (EnumHasAnyFlags(AllocFlags, EVulkanAllocationFlags::PreferBAR))
			{
				MemFlags |= VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT;
			}
		}
		else
		{
			MemFlags = VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT;

			if (EnumHasAnyFlags(AllocFlags, EVulkanAllocationFlags::Protected))
			{
				MemFlags |= VK_MEMORY_PROPERTY_PROTECTED_BIT;
			}
			else if (bHasUnifiedMemory)
			{
				MemFlags |= (VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT | VK_MEMORY_PROPERTY_HOST_COHERENT_BIT);
			}
		}

		if (EnumHasAnyFlags(AllocFlags, EVulkanAllocationFlags::Memoryless))
		{
			MemFlags = VK_MEMORY_PROPERTY_LAZILY_ALLOCATED_BIT;
		}

		return MemFlags;
	}

	bool FMemoryManager::AllocateBufferPooled(FVulkanAllocation& OutAllocation, FVulkanEvictable* AllocationOwner, uint32 Size, uint32 MinAlignment, VkBufferUsageFlags BufferUsageFlags, VkMemoryPropertyFlags MemoryPropertyFlags, EVulkanAllocationMetaType MetaType, const char* File, uint32 Line)
	{
		SCOPED_NAMED_EVENT(FResourceHeapManager_AllocateBufferPooled, FColor::Cyan);
		check(OutAllocation.Type == EVulkanAllocationEmpty);

		uint32 Alignment = FMath::Max(MinAlignment, CalculateBufferAlignmentFromVKUsageFlags(Device, BufferUsageFlags));
		const float Priority = CalculateBufferPriority(BufferUsageFlags);

		const int32 PoolSize = (int32)GetPoolTypeForAlloc(Size, Alignment);
		if (PoolSize != (int32)EPoolSizes::SizesCount)
		{
			Size = PoolSizes[PoolSize];
		}

		const bool bHasUnifiedMemory = DeviceMemoryManager->HasUnifiedMemory();

#if (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)
		const bool bCanEvict = MetaTypeCanEvict(MetaType);
		if (!bHasUnifiedMemory && bCanEvict && (MemoryPropertyFlags == VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT) && UpdateEvictThreshold(false))
		{
			MemoryPropertyFlags = DeviceMemoryManager->GetEvictedMemoryProperties();
		}
#endif

		FScopeLock ScopeLock(&UsedFreeBufferAllocationsLock);

		for (int32 Index = 0; Index < UsedBufferAllocations[PoolSize].Num(); ++Index)
		{
			FVulkanSubresourceAllocator* SubresourceAllocator = UsedBufferAllocations[PoolSize][Index];
			if ((SubresourceAllocator->BufferUsageFlags & BufferUsageFlags) == BufferUsageFlags &&
				(SubresourceAllocator->MemoryPropertyFlags & MemoryPropertyFlags) == MemoryPropertyFlags)
			{

				if(SubresourceAllocator->TryAllocate2(OutAllocation, AllocationOwner, Size, Alignment, MetaType, File, Line))
				{
					IncMetaStats(MetaType, OutAllocation.Size);
					return true;
				}
			}
		}

		for (int32 Index = 0; Index < FreeBufferAllocations[PoolSize].Num(); ++Index)
		{
			FVulkanSubresourceAllocator* SubresourceAllocator = FreeBufferAllocations[PoolSize][Index];
			if ((SubresourceAllocator->BufferUsageFlags & BufferUsageFlags) == BufferUsageFlags &&
				(SubresourceAllocator->MemoryPropertyFlags & MemoryPropertyFlags) == MemoryPropertyFlags)
			{
				if(SubresourceAllocator->TryAllocate2(OutAllocation, AllocationOwner, Size, Alignment, MetaType, File, Line))
				{
					IncMetaStats(MetaType, OutAllocation.Size);
					FreeBufferAllocations[PoolSize].RemoveAtSwap(Index, EAllowShrinking::No);
					UsedBufferAllocations[PoolSize].Add(SubresourceAllocator);
					return true;
				}
			}
		}

		// New Buffer
		const uint32 BufferSize = FMath::Max(Size, BufferSizes[PoolSize]);
		VkBuffer Buffer = Device.CreateBuffer(BufferSize, BufferUsageFlags);

		VkMemoryRequirements MemReqs;
		VulkanRHI::vkGetBufferMemoryRequirements(Device.GetHandle(), Buffer, &MemReqs);
		Alignment = FMath::Max((uint32)MemReqs.alignment, Alignment);
		ensure(MemReqs.size >= BufferSize);

		uint32 MemoryTypeIndex;	
		VERIFYVULKANRESULT(Device.GetDeviceMemoryManager().GetMemoryTypeFromProperties(MemReqs.memoryTypeBits, MemoryPropertyFlags, &MemoryTypeIndex));

		FDeviceMemoryAllocation* DeviceMemoryAllocation = DeviceMemoryManager->Alloc(true, MemReqs.size, MemoryTypeIndex, nullptr, Priority, false, File, Line);
		if(!DeviceMemoryAllocation)
		{
			MemoryPropertyFlags &= (~VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT);
			const uint32 ForbiddenMemoryTypeIndex = MemoryTypeIndex;
			if (GVulkanMemoryMemoryFallbackToHost && VK_SUCCESS == Device.GetDeviceMemoryManager().GetMemoryTypeFromPropertiesExcluding(MemReqs.memoryTypeBits, MemoryPropertyFlags, ForbiddenMemoryTypeIndex, &MemoryTypeIndex))
			{
				DeviceMemoryAllocation = DeviceMemoryManager->Alloc(false, MemReqs.size, MemoryTypeIndex, nullptr, Priority, false, File, Line);
			}
		}
		if(!DeviceMemoryAllocation)
		{
			HandleOOM(false);
			checkNoEntry();
		}
		VERIFYVULKANRESULT(VulkanRHI::vkBindBufferMemory(Device.GetHandle(), Buffer, DeviceMemoryAllocation->GetHandle(), 0));
		uint8 AllocationFlags = 0;
		if(!bHasUnifiedMemory && MetaTypeCanEvict(MetaType))
		{
			AllocationFlags |= VulkanAllocationFlagsCanEvict;
		}
		if (DeviceMemoryAllocation->CanBeMapped())
		{
			DeviceMemoryAllocation->Map(BufferSize, 0);
		}

		uint32 BufferId = 0;
		if (UseVulkanDescriptorCache())
		{
			BufferId = ++GVulkanBufferHandleIdCounter;
		}
		FVulkanSubresourceAllocator* SubresourceAllocator = new FVulkanSubresourceAllocator(EVulkanAllocationPooledBuffer, this, AllocationFlags, DeviceMemoryAllocation, MemoryTypeIndex,
			MemoryPropertyFlags, MemReqs.alignment, Buffer, BufferSize, BufferId, BufferUsageFlags, PoolSize);

		RegisterSubresourceAllocator(SubresourceAllocator);
		UsedBufferAllocations[PoolSize].Add(SubresourceAllocator);

		if(SubresourceAllocator->TryAllocate2(OutAllocation, AllocationOwner, Size, Alignment, MetaType, File, Line))
		{
			IncMetaStats(MetaType, OutAllocation.Size);
			return true;
		}
		HandleOOM(false);
		checkNoEntry();
		return false;
	}

	void FMemoryManager::RegisterSubresourceAllocator(FVulkanSubresourceAllocator* SubresourceAllocator)
	{
		check(SubresourceAllocator->AllocatorIndex == 0xffffffff);
		FRWScopeLock ScopedLock(AllBufferAllocationsLock, SLT_Write);
		if (AllBufferAllocationsFreeListHead != (PTRINT)-1)
		{
			const uint32 Index = AllBufferAllocationsFreeListHead;
			AllBufferAllocationsFreeListHead = (PTRINT)AllBufferAllocations[Index];
			SubresourceAllocator->AllocatorIndex = Index;
			AllBufferAllocations[Index] = SubresourceAllocator;
		}
		else
		{
			SubresourceAllocator->AllocatorIndex = AllBufferAllocations.Num();
			AllBufferAllocations.Add(SubresourceAllocator);
		}

	}

	void FMemoryManager::UnregisterSubresourceAllocator(FVulkanSubresourceAllocator* SubresourceAllocator)
	{
		if (SubresourceAllocator->bIsEvicting)
		{
			PendingEvictBytes -= SubresourceAllocator->GetMemoryAllocation()->GetSize();
		}
		FRWScopeLock ScopedLock(AllBufferAllocationsLock, SLT_Write);
		const uint32 Index = SubresourceAllocator->AllocatorIndex;
		check(Index != 0xffffffff);
		AllBufferAllocations[Index] = (FVulkanSubresourceAllocator*)AllBufferAllocationsFreeListHead;
		AllBufferAllocationsFreeListHead = Index;
	}

	bool FMemoryManager::ReleaseSubresourceAllocator(FVulkanSubresourceAllocator* SubresourceAllocator)
	{
		// Lock everything to make sure no one adds anything while we get rid of it
		FScopeLock ScopeLockBufferAllocations(&UsedFreeBufferAllocationsLock);
		FScopeLock ScopeLockPages(&ResourceTypeHeaps[SubresourceAllocator->MemoryTypeIndex]->PagesLock);

		if (SubresourceAllocator->JoinFreeBlocks())
		{
			if (SubresourceAllocator->Type == EVulkanAllocationPooledBuffer)
			{
				UsedBufferAllocations[SubresourceAllocator->PoolSizeIndex].RemoveSingleSwap(SubresourceAllocator, EAllowShrinking::No);
				SubresourceAllocator->FrameFreed = GFrameNumberRenderThread;
				FreeBufferAllocations[SubresourceAllocator->PoolSizeIndex].Add(SubresourceAllocator);
			}
			else
			{
				FVulkanResourceHeap* Heap = ResourceTypeHeaps[SubresourceAllocator->MemoryTypeIndex];
				Heap->FreePage(SubresourceAllocator);
			}

			return true;
		}
		return false;
	}

	bool FMemoryManager::UpdateEvictThreshold(bool bLog)
	{
		uint64 PrimaryAllocated, PrimaryLimit;
		DeviceMemoryManager->GetPrimaryHeapStatus(PrimaryAllocated, PrimaryLimit);
		double AllocatedPercentage = 100.0 * (PrimaryAllocated - PendingEvictBytes) / PrimaryLimit;

		double EvictionLimit = GVulkanEvictionLimitPercentage;
		double EvictionLimitLowered = EvictionLimit * (GVulkanEvictionLimitPercentageReenableLimit / 100.f);
		if(bIsEvicting) //once eviction is started, further lower the limit, to avoid reclaiming memory we just free up
		{
			EvictionLimit = EvictionLimitLowered;
		}
		if(bLog && GVulkanLogEvictStatus)
		{
			VULKAN_LOGMEMORY(TEXT("EVICT STATUS %6.2f%%/%6.2f%% :: A:%8.3fMB / E:%8.3fMB / T:%8.3fMB\n"), AllocatedPercentage, EvictionLimit, PrimaryAllocated / (1024.f*1024.f), PendingEvictBytes/ (1024.f*1024.f), PrimaryLimit / (1024.f*1024.f));
		}

		bIsEvicting = AllocatedPercentage > EvictionLimit;
		return bIsEvicting;
	}

	bool FMemoryManager::AllocateImageMemory(FVulkanAllocation& OutAllocation, FVulkanEvictable* AllocationOwner, const VkMemoryRequirements& MemoryReqs, VkMemoryPropertyFlags MemoryPropertyFlags, EVulkanAllocationMetaType MetaType, bool bExternal, const char* File, uint32 Line)
	{
		const bool bHasUnifiedMemory = DeviceMemoryManager->HasUnifiedMemory();
		const bool bCanEvict = MetaTypeCanEvict(MetaType);
		if(!bHasUnifiedMemory && bCanEvict && MemoryPropertyFlags == VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT && UpdateEvictThreshold(false))
		{
			MemoryPropertyFlags = DeviceMemoryManager->GetEvictedMemoryProperties();
		}
		bool bMapped = VKHasAllFlags(MemoryPropertyFlags, VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT);
		uint32 TypeIndex = 0;

		if (DeviceMemoryManager->GetMemoryTypeFromProperties(MemoryReqs.memoryTypeBits, MemoryPropertyFlags, &TypeIndex) != VK_SUCCESS)
		{
			if (VKHasAllFlags(MemoryPropertyFlags, VK_MEMORY_PROPERTY_LAZILY_ALLOCATED_BIT))
			{
				// If lazy allocations are not supported, we can fall back to real allocations.
				MemoryPropertyFlags = MemoryPropertyFlags & ~VK_MEMORY_PROPERTY_LAZILY_ALLOCATED_BIT;
				VERIFYVULKANRESULT(DeviceMemoryManager->GetMemoryTypeFromProperties(MemoryReqs.memoryTypeBits, MemoryPropertyFlags, &TypeIndex));
			}
			else
			{
				UE_LOGF(LogVulkanRHI, Fatal, "Cannot find memory type for MemSize %d, MemPropTypeBits %u, MemPropertyFlags %u, %ls(%d)", (uint32)MemoryReqs.size, (uint32)MemoryReqs.memoryTypeBits, (uint32)MemoryPropertyFlags, ANSI_TO_TCHAR(File), Line);
			}
		}
		if (!ResourceTypeHeaps[TypeIndex])
		{
			UE_LOGF(LogVulkanRHI, Fatal, "Missing memory type index %d, MemSize %d, MemPropTypeBits %u, MemPropertyFlags %u, %ls(%d)", TypeIndex, (uint32)MemoryReqs.size, (uint32)MemoryReqs.memoryTypeBits, (uint32)MemoryPropertyFlags, ANSI_TO_TCHAR(File), Line);
		}

		check(MemoryReqs.size <= (uint64)MAX_uint32);

		const bool bForceSeparateAllocation = bExternal || VKHasAllFlags(MemoryPropertyFlags, VK_MEMORY_PROPERTY_LAZILY_ALLOCATED_BIT);
		if (!ResourceTypeHeaps[TypeIndex]->AllocateResource(OutAllocation, AllocationOwner, EType::Image, MemoryReqs.size, MemoryReqs.alignment, bMapped, bForceSeparateAllocation, MetaType, bExternal, File, Line))
		{
			if(GVulkanMemoryMemoryFallbackToHost)
			{
				MemoryPropertyFlags &= (~VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT);
			}

			VERIFYVULKANRESULT(DeviceMemoryManager->GetMemoryTypeFromPropertiesExcluding(MemoryReqs.memoryTypeBits, MemoryPropertyFlags, TypeIndex, &TypeIndex));
			bMapped = VKHasAllFlags(MemoryPropertyFlags, VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT);
			if (!ResourceTypeHeaps[TypeIndex]->AllocateResource(OutAllocation, AllocationOwner, EType::Image, MemoryReqs.size, MemoryReqs.alignment, bMapped, bForceSeparateAllocation, MetaType, bExternal, File, Line))
			{
				DumpMemory();
				FPlatformMemory::bIsOOM = true;
				UE_LOGF(LogVulkanRHI, Fatal, "Out Of Memory, trying to allocate %lld bytes\n", MemoryReqs.size);
				return false;
			}
		}
		return true;
	}

	bool FMemoryManager::AllocateBufferMemory(FVulkanAllocation& OutAllocation, FVulkanEvictable* AllocationOwner, const VkMemoryRequirements& MemoryReqs, VkMemoryPropertyFlags MemoryPropertyFlags, EVulkanAllocationMetaType MetaType, bool bExternal, bool bForceSeparateAllocation, const char* File, uint32 Line)
	{
		SCOPED_NAMED_EVENT(FResourceHeapManager_AllocateBufferMemory, FColor::Cyan);
		uint32 TypeIndex = 0;
		VkResult Result = DeviceMemoryManager->GetMemoryTypeFromProperties(MemoryReqs.memoryTypeBits, MemoryPropertyFlags, &TypeIndex);
		const bool bPreferBAR = VKHasAllFlags(MemoryPropertyFlags, VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT | VK_MEMORY_PROPERTY_HOST_COHERENT_BIT | VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT);
		bool bMapped = VKHasAllFlags(MemoryPropertyFlags, VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT);
		if ((Result != VK_SUCCESS) || !ResourceTypeHeaps[TypeIndex])
		{
			if (VKHasAllFlags(MemoryPropertyFlags, VK_MEMORY_PROPERTY_HOST_CACHED_BIT))
			{
				// Try non-cached flag
				MemoryPropertyFlags = MemoryPropertyFlags & ~VK_MEMORY_PROPERTY_HOST_CACHED_BIT;
			}

			if (VKHasAllFlags(MemoryPropertyFlags, VK_MEMORY_PROPERTY_LAZILY_ALLOCATED_BIT))
			{
				// Try non-lazy flag
				MemoryPropertyFlags = MemoryPropertyFlags & ~VK_MEMORY_PROPERTY_LAZILY_ALLOCATED_BIT;
			}

			if (bPreferBAR)
			{
				// Try regular host memory if local+host is not available
				MemoryPropertyFlags = MemoryPropertyFlags & ~VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT;
			}

			// Try another heap type
			uint32 OriginalTypeIndex = TypeIndex;
			if (DeviceMemoryManager->GetMemoryTypeFromPropertiesExcluding(MemoryReqs.memoryTypeBits, MemoryPropertyFlags, (Result == VK_SUCCESS) ? TypeIndex : (uint32)-1, &TypeIndex) != VK_SUCCESS)
			{
				UE_LOGF(LogVulkanRHI, Fatal, "Unable to find alternate type for index %d, MemSize %d, MemPropTypeBits %u, MemPropertyFlags %u, %ls(%d)",
					OriginalTypeIndex, (uint32)MemoryReqs.size, (uint32)MemoryReqs.memoryTypeBits, (uint32)MemoryPropertyFlags, ANSI_TO_TCHAR(File), Line);
			}
			if (!ResourceTypeHeaps[TypeIndex])
			{
				DumpMemory();
				UE_LOGF(LogVulkanRHI, Fatal, "Missing memory type index %d (originally requested %d), MemSize %d, MemPropTypeBits %u, MemPropertyFlags %u, %ls(%d)", TypeIndex, OriginalTypeIndex, (uint32)MemoryReqs.size, (uint32)MemoryReqs.memoryTypeBits, (uint32)MemoryPropertyFlags, ANSI_TO_TCHAR(File), Line);
			}
		}

		check(MemoryReqs.size <= (uint64)MAX_uint32);

		if (!ResourceTypeHeaps[TypeIndex]->AllocateResource(OutAllocation, AllocationOwner, EType::Buffer, MemoryReqs.size, MemoryReqs.alignment, bMapped, bForceSeparateAllocation, MetaType, bExternal, File, Line))
		{
			// Try another memory type if the allocation failed
			if (bPreferBAR)
			{
				// Try regular host memory if local+host is not available
				MemoryPropertyFlags = MemoryPropertyFlags & ~VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT;
			}
			VERIFYVULKANRESULT(DeviceMemoryManager->GetMemoryTypeFromPropertiesExcluding(MemoryReqs.memoryTypeBits, MemoryPropertyFlags, TypeIndex, &TypeIndex));
			bMapped = (MemoryPropertyFlags & VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT) == VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT;
			if (!ResourceTypeHeaps[TypeIndex])
			{
				UE_LOGF(LogVulkanRHI, Fatal, "Missing memory type index %d, MemSize %d, MemPropTypeBits %u, MemPropertyFlags %u, %ls(%d)", TypeIndex, (uint32)MemoryReqs.size, (uint32)MemoryReqs.memoryTypeBits, (uint32)MemoryPropertyFlags, ANSI_TO_TCHAR(File), Line);
			}
			if (!ResourceTypeHeaps[TypeIndex]->AllocateResource(OutAllocation, AllocationOwner, EType::Buffer, MemoryReqs.size, MemoryReqs.alignment, bMapped, bForceSeparateAllocation, MetaType, bExternal, File, Line))
			{
				DumpMemory();
				FPlatformMemory::bIsOOM = true;
				UE_LOGF(LogVulkanRHI, Fatal, "Out Of Memory, trying to allocate %lld bytes\n", MemoryReqs.size);
				return false;
			}
		}
		return true;
	}

	bool FMemoryManager::AllocateBufferMemory(FVulkanAllocation& OutAllocation, VkBuffer InBuffer, EVulkanAllocationFlags InAllocFlags, const TCHAR* InDebugName, uint32 InForceMinAlignment)
	{
		SCOPED_NAMED_EVENT(FMemoryManager_AllocateBufferMemory, FColor::Cyan);

		VkBufferMemoryRequirementsInfo2 BufferMemoryRequirementsInfo;
		ZeroVulkanStruct(BufferMemoryRequirementsInfo, VK_STRUCTURE_TYPE_BUFFER_MEMORY_REQUIREMENTS_INFO_2);
		BufferMemoryRequirementsInfo.buffer = InBuffer;

		VkMemoryDedicatedRequirements DedicatedRequirements;
		ZeroVulkanStruct(DedicatedRequirements, VK_STRUCTURE_TYPE_MEMORY_DEDICATED_REQUIREMENTS);

		VkMemoryRequirements2 MemoryRequirements;
		ZeroVulkanStruct(MemoryRequirements, VK_STRUCTURE_TYPE_MEMORY_REQUIREMENTS_2);
		MemoryRequirements.pNext = &DedicatedRequirements;

		VulkanRHI::vkGetBufferMemoryRequirements2(Device.GetHandle(), &BufferMemoryRequirementsInfo, &MemoryRequirements);

		// Allow caller to force his own alignment requirements
		MemoryRequirements.memoryRequirements.alignment = FMath::Max(MemoryRequirements.memoryRequirements.alignment, (VkDeviceSize)InForceMinAlignment);

		if (DedicatedRequirements.requiresDedicatedAllocation || DedicatedRequirements.prefersDedicatedAllocation)
		{
			InAllocFlags |= EVulkanAllocationFlags::Dedicated;
		}

		// For now, translate all the flags into a call to the legacy AllocateBufferMemory() function
		const VkMemoryPropertyFlags MemoryPropertyFlags = GetMemoryPropertyFlags(InAllocFlags, DeviceMemoryManager->HasUnifiedMemory());
		const bool bExternal = EnumHasAllFlags(InAllocFlags, EVulkanAllocationFlags::External);
		const bool bForceSeparateAllocation = EnumHasAllFlags(InAllocFlags, EVulkanAllocationFlags::Dedicated);
		AllocateBufferMemory(OutAllocation, nullptr, MemoryRequirements.memoryRequirements, MemoryPropertyFlags, EVulkanAllocationMetaBufferOther, bExternal, bForceSeparateAllocation, __FILE__, __LINE__);

		if (OutAllocation.IsValid())
		{
			if (EnumHasAllFlags(InAllocFlags, EVulkanAllocationFlags::AutoBind))
			{
				VkBindBufferMemoryInfo BindBufferMemoryInfo;
				ZeroVulkanStruct(BindBufferMemoryInfo, VK_STRUCTURE_TYPE_BIND_BUFFER_MEMORY_INFO);
				BindBufferMemoryInfo.buffer = InBuffer;
				BindBufferMemoryInfo.memory = OutAllocation.GetDeviceMemoryHandle(&Device);
				BindBufferMemoryInfo.memoryOffset = OutAllocation.Offset;

				VERIFYVULKANRESULT(VulkanRHI::vkBindBufferMemory2(Device.GetHandle(), 1, &BindBufferMemoryInfo));
			}

			if (InDebugName)
			{
				VULKAN_SET_DEBUG_NAME(Device, VK_OBJECT_TYPE_BUFFER, InBuffer, TEXT("%s"), InDebugName);
			}
		}
		else
		{
			if (!EnumHasAllFlags(InAllocFlags, EVulkanAllocationFlags::NoError))
			{
				const bool IsHostMemory = EnumHasAnyFlags(InAllocFlags, EVulkanAllocationFlags::HostVisible | EVulkanAllocationFlags::HostCached);
				HandleOOM(false, IsHostMemory ? VK_ERROR_OUT_OF_HOST_MEMORY : VK_ERROR_OUT_OF_DEVICE_MEMORY, MemoryRequirements.memoryRequirements.size);
			}
		}

		return OutAllocation.IsValid();
	}

	bool FMemoryManager::AllocateDedicatedImageMemory(FVulkanAllocation& OutAllocation, FVulkanEvictable* AllocationOwner, VkImage Image, VkMemoryPropertyFlags MemoryPropertyFlags, EVulkanAllocationMetaType MetaType, bool bExternal, bool bForceDedicatedAllocation, const char* File, uint32 Line)
	{
		SCOPED_NAMED_EVENT(FVulkanMemoryManager_AllocateDedicatedImageMemory, FColor::Cyan);
		VkImageMemoryRequirementsInfo2KHR ImageMemoryReqs2;
		ZeroVulkanStruct(ImageMemoryReqs2, VK_STRUCTURE_TYPE_IMAGE_MEMORY_REQUIREMENTS_INFO_2_KHR);
		ImageMemoryReqs2.image = Image;

		VkMemoryDedicatedRequirementsKHR DedMemoryReqs;
		ZeroVulkanStruct(DedMemoryReqs, VK_STRUCTURE_TYPE_MEMORY_DEDICATED_REQUIREMENTS_KHR);

		VkMemoryRequirements2 MemoryReqs2;
		ZeroVulkanStruct(MemoryReqs2, VK_STRUCTURE_TYPE_MEMORY_REQUIREMENTS_2_KHR);
		MemoryReqs2.pNext = &DedMemoryReqs;

		VulkanRHI::vkGetImageMemoryRequirements2(Device.GetHandle(), &ImageMemoryReqs2, &MemoryReqs2);
		const VkMemoryRequirements& MemoryReqs = MemoryReqs2.memoryRequirements;

		const bool bUseDedicated = bForceDedicatedAllocation || (DedMemoryReqs.prefersDedicatedAllocation != VK_FALSE || DedMemoryReqs.requiresDedicatedAllocation != VK_FALSE);
		if (bUseDedicated)
		{
			uint32 TypeIndex = 0;
			VERIFYVULKANRESULT(DeviceMemoryManager->GetMemoryTypeFromProperties(MemoryReqs.memoryTypeBits, MemoryPropertyFlags, &TypeIndex));
			ensure((MemoryPropertyFlags & VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT) != VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT);
			check(MemoryReqs.size <= (uint64)MAX_uint32);
			if (!ResourceTypeHeaps[TypeIndex])
			{
				UE_LOGF(LogVulkanRHI, Fatal, "Missing memory type index %d, MemSize %d, MemPropTypeBits %u, MemPropertyFlags %u, %ls(%d)", TypeIndex, (uint32)MemoryReqs.size, (uint32)MemoryReqs.memoryTypeBits, (uint32)MemoryPropertyFlags, ANSI_TO_TCHAR(File), Line);
			}
			if (!ResourceTypeHeaps[TypeIndex]->AllocateDedicatedImage(OutAllocation, AllocationOwner, Image, MemoryReqs.size, MemoryReqs.alignment, MetaType, bExternal, File, Line))
			{
				if(GVulkanMemoryMemoryFallbackToHost)
				{
					MemoryPropertyFlags &= (~VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT);
				}
				if(VK_SUCCESS == DeviceMemoryManager->GetMemoryTypeFromPropertiesExcluding(MemoryReqs.memoryTypeBits, MemoryPropertyFlags, TypeIndex, &TypeIndex))
				{
					if (ResourceTypeHeaps[TypeIndex]->AllocateDedicatedImage(OutAllocation, AllocationOwner, Image, MemoryReqs.size, MemoryReqs.alignment, MetaType, bExternal, File, Line))
					{
						return true;
					}
				}
				return false;
			}
			return true;
		}
		else
		{
			return AllocateImageMemory(OutAllocation, AllocationOwner, MemoryReqs, MemoryPropertyFlags, MetaType, bExternal, File, Line);
		}
	}


	void FMemoryManager::DumpMemory(bool bFullDump)
	{
		FScopeLock ScopeLock(&UsedFreeBufferAllocationsLock);
		Device.GetDeviceMemoryManager().DumpMemory();
		VULKAN_LOGMEMORY(TEXT("/******************************************* FMemoryManager ********************************************\\"));
		VULKAN_LOGMEMORY(TEXT("HEAP DUMP"));

		const VkPhysicalDeviceMemoryProperties& MemoryProperties = DeviceMemoryManager->GetMemoryProperties();

		TArray<FResourceHeapStats> HeapSummary;
		HeapSummary.SetNum(MemoryProperties.memoryHeapCount);
		for(uint32 HeapIndex = 0; HeapIndex < MemoryProperties.memoryHeapCount; ++HeapIndex)
		{
			HeapSummary[HeapIndex].MemoryFlags = 0;
			for (uint32 TypeIndex = 0; TypeIndex < MemoryProperties.memoryTypeCount; ++TypeIndex)
			{
				if (MemoryProperties.memoryTypes[TypeIndex].heapIndex == HeapIndex)
				{
					HeapSummary[HeapIndex].MemoryFlags |= MemoryProperties.memoryTypes[TypeIndex].propertyFlags; //since it can be different, just set to the bitwise or of all flags.
				}
			}
		}

		TArray<FResourceHeapStats> OverallSummary;
		const uint32 NumSmallAllocators = UE_ARRAY_COUNT(UsedBufferAllocations);
		const uint32 NumResourceHeaps = ResourceTypeHeaps.Num();
		
		OverallSummary.SetNum(NumResourceHeaps + NumSmallAllocators * 2 + 1);
		const uint32 SmallAllocatorsBegin = NumResourceHeaps;
		const uint32 SmallAllocatorsEnd = NumResourceHeaps + NumSmallAllocators * 2;
		const uint32 DedicatedAllocatorSummary = SmallAllocatorsEnd;


		for (int32 TypeIndex = 0; TypeIndex < ResourceTypeHeaps.Num(); ++TypeIndex)
		{
			if (ResourceTypeHeaps[TypeIndex])
			{
				const uint32 MemoryTypeIndex = ResourceTypeHeaps[TypeIndex]->MemoryTypeIndex;
				VULKAN_LOGMEMORY(TEXT("ResourceHeap %d, Memory Type Index %d"), TypeIndex, MemoryTypeIndex);
				OverallSummary[TypeIndex].MemoryFlags = MemoryProperties.memoryTypes[MemoryTypeIndex].propertyFlags;
				ResourceTypeHeaps[TypeIndex]->DumpMemory(OverallSummary[TypeIndex]);
				const uint32 HeapIndex = MemoryProperties.memoryTypes[MemoryTypeIndex].heapIndex;
				HeapSummary[HeapIndex] += OverallSummary[TypeIndex];
			}
			else
			{
				VULKAN_LOGMEMORY(TEXT("ResourceHeap %d, NOT USED"), TypeIndex);
			}
		}

		VULKAN_LOGMEMORY(TEXT("BUFFER DUMP"));
		uint64 UsedBinnedTotal = 0;
		uint64 AllocBinnedTotal = 0;
		uint64 UsedLargeTotal = 0;
		uint64 AllocLargeTotal = 0;
		for (int32 PoolSizeIndex = 0; PoolSizeIndex < UE_ARRAY_COUNT(UsedBufferAllocations); PoolSizeIndex++)
		{
			FResourceHeapStats& StatsLocal = OverallSummary[NumResourceHeaps + PoolSizeIndex];
			FResourceHeapStats& StatsHost = OverallSummary[NumResourceHeaps + NumSmallAllocators + PoolSizeIndex];
			TArray<FVulkanSubresourceAllocator*>& UsedAllocations = UsedBufferAllocations[PoolSizeIndex];
			TArray<FVulkanSubresourceAllocator*>& FreeAllocations = FreeBufferAllocations[PoolSizeIndex];
			if (PoolSizeIndex == (int32)EPoolSizes::SizesCount)
			{
				VULKAN_LOGMEMORY(TEXT("Buffer of large size Allocations: %d Used / %d Free"), UsedAllocations.Num(), FreeAllocations.Num());
			}
			else
			{
				VULKAN_LOGMEMORY(TEXT("Buffer of %d size Allocations: %d Used / %d Free"), PoolSizes[PoolSizeIndex], UsedAllocations.Num(), FreeAllocations.Num());
			}
			//Stats.Pages += UsedAllocations.Num() + FreeAllocations.Num();
			//Stats.BufferPages += UsedAllocations.Num();
			for (int32 Index = 0; Index < FreeAllocations.Num(); ++Index)
			{
				FVulkanSubresourceAllocator* BA = FreeAllocations[Index];
				if (VKHasAnyFlags(BA->MemoryPropertyFlags, VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT))
				{
					StatsLocal.Pages += 1;
					StatsLocal.BufferPages += 1;
					StatsLocal.TotalMemory += BA->MaxSize;
					StatsLocal.MemoryFlags |= BA->MemoryPropertyFlags;
				}
				else
				{
					StatsHost.Pages += 1;
					StatsHost.BufferPages += 1;
					StatsHost.TotalMemory += BA->MaxSize;
					StatsHost.MemoryFlags |= BA->MemoryPropertyFlags;
				}

				const uint32 HeapIndex = MemoryProperties.memoryTypes[BA->MemoryTypeIndex].heapIndex;
				FResourceHeapStats& HeapStats = HeapSummary[HeapIndex];
				HeapStats.Pages += 1;
				HeapStats.BufferPages += 1;
				HeapStats.TotalMemory += BA->MaxSize;
			}

			if (UsedAllocations.Num() > 0)
			{
				uint64 _UsedBinnedTotal = 0;
				uint64 _AllocBinnedTotal = 0;
				uint64 _UsedLargeTotal = 0;
				uint64 _AllocLargeTotal = 0;

				VULKAN_LOGMEMORY(TEXT("Index  BufferHandle       DeviceMemoryHandle MemFlags BufferFlags #Suballocs #FreeChunks UsedSize/MaxSize"));
				for (int32 Index = 0; Index < UsedAllocations.Num(); ++Index)
				{
					FVulkanSubresourceAllocator* BA = UsedAllocations[Index];
					VULKAN_LOGMEMORY(TEXT("%6d 0x%016llx 0x%016llx 0x%06x 0x%08x %6d   %6d        %llu/%u"), Index, (void*)BA->Buffer, (void*)BA->MemoryAllocation->GetHandle(), BA->MemoryPropertyFlags, BA->BufferUsageFlags, BA->NumSubAllocations, BA->FreeList.Num(), BA->UsedSize, BA->MaxSize);

					if (PoolSizeIndex == (int32)EPoolSizes::SizesCount)
					{
						_UsedLargeTotal += BA->UsedSize;
						_AllocLargeTotal += BA->MaxSize;
						UsedLargeTotal += BA->UsedSize;
						AllocLargeTotal += BA->MaxSize;
					}
					else
					{
						_UsedBinnedTotal += BA->UsedSize;
						_AllocBinnedTotal += BA->MaxSize;
						UsedBinnedTotal += BA->UsedSize;
						AllocBinnedTotal += BA->MaxSize;
					}

					if (VKHasAnyFlags(BA->MemoryPropertyFlags, VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT))
					{
						StatsLocal.Pages += 1;
						StatsLocal.BufferPages += 1;
						StatsLocal.UsedBufferMemory += BA->UsedSize;
						StatsLocal.TotalMemory += BA->MaxSize;
						StatsLocal.BufferAllocations += BA->NumSubAllocations;
						StatsLocal.MemoryFlags |= BA->MemoryPropertyFlags;
					}
					else
					{
						StatsHost.Pages += 1;
						StatsHost.BufferPages += 1;
						StatsHost.UsedBufferMemory += BA->UsedSize;
						StatsHost.TotalMemory += BA->MaxSize;
						StatsHost.BufferAllocations += BA->NumSubAllocations;
						StatsHost.MemoryFlags |= BA->MemoryPropertyFlags;
					}
					const uint32 HeapIndex = MemoryProperties.memoryTypes[BA->MemoryTypeIndex].heapIndex;
					FResourceHeapStats& HeapStats = HeapSummary[HeapIndex];
					HeapStats.Pages += 1;
					HeapStats.BufferPages += 1;
					HeapStats.UsedBufferMemory += BA->UsedSize;
					HeapStats.TotalMemory += BA->MaxSize;
					HeapStats.BufferAllocations += BA->NumSubAllocations;
				}

				if (PoolSizeIndex == (int32)EPoolSizes::SizesCount)
				{
					VULKAN_LOGMEMORY(TEXT(" Large Alloc Used/Max %llu/%llu %6.2f%%"), _UsedLargeTotal, _AllocLargeTotal, 100.0f * (float)_UsedLargeTotal / (float)_AllocLargeTotal);
				}
				else
				{
					VULKAN_LOGMEMORY(TEXT(" Binned [%d] Alloc Used/Max %llu/%llu %6.2f%%"), PoolSizes[PoolSizeIndex], _UsedBinnedTotal, _AllocBinnedTotal, 100.0f * (float)_UsedBinnedTotal / (float)_AllocBinnedTotal);
				}
			}
		}

		VULKAN_LOGMEMORY(TEXT("::Totals::"));
		VULKAN_LOGMEMORY(TEXT("Large Alloc Used/Max %llu/%llu %.2f%%"), UsedLargeTotal, AllocLargeTotal, AllocLargeTotal > 0 ? 100.0f * (float)UsedLargeTotal / (float)AllocLargeTotal : 0.0f);
		VULKAN_LOGMEMORY(TEXT("Binned Alloc Used/Max %llu/%llu %.2f%%"), UsedBinnedTotal, AllocBinnedTotal, AllocBinnedTotal > 0 ? 100.0f * (float)UsedBinnedTotal / (float)AllocBinnedTotal : 0.0f);
		{
			FResourceHeapStats& DedicatedStats = OverallSummary[DedicatedAllocatorSummary];

			for (int32 TypeIndex = 0; TypeIndex < ResourceTypeHeaps.Num(); ++TypeIndex)
			{
				FVulkanResourceHeap* ResourceHeap = ResourceTypeHeaps[TypeIndex];
				if (ResourceHeap)
				{
					FResourceHeapStats AccumulatedStats;
					for (FVulkanSubresourceAllocator* Allocator : ResourceHeap->UsedDedicatedImagePages)
					{
						AccumulatedStats.Pages++;
						AccumulatedStats.TotalMemory += Allocator->MaxSize;
						AccumulatedStats.UsedImageMemory += Allocator->UsedSize;
						AccumulatedStats.ImageAllocations += Allocator->NumSubAllocations;
					}

					const uint32 MemoryTypeIndex = ResourceTypeHeaps[TypeIndex]->MemoryTypeIndex;
					const uint32 HeapIndex = MemoryProperties.memoryTypes[MemoryTypeIndex].heapIndex;
					HeapSummary[HeapIndex] += AccumulatedStats;

					DedicatedStats += AccumulatedStats;
				}
			}
		}


		auto WriteLogLine = [](const FString& Name, FResourceHeapStats& Stat)
		{
			const uint64 FreeMemory = Stat.TotalMemory - FMath::Min<uint64>(Stat.TotalMemory, Stat.UsedBufferMemory + Stat.UsedImageMemory);
			FString HostString = (Stat.MemoryFlags != 0) ? VK_FLAGS_TO_STRING(VkMemoryPropertyFlags, Stat.MemoryFlags) : TEXT("");
			VULKAN_LOGMEMORY(TEXT("\t\t%-33s  |%8.2fmb / %8.2fmb / %11.2fmb / %11.2fmb | %10lld %10lld | %6lld %6lld %6lld | %05x | %s"),
				*Name,
				Stat.UsedBufferMemory / (1024.f * 1024.f),
				Stat.UsedImageMemory / (1024.f * 1024.f),
				FreeMemory / (1024.f * 1024.f),
				Stat.TotalMemory / (1024.f * 1024.f),
				Stat.BufferAllocations,
				Stat.ImageAllocations,
				Stat.Pages,
				Stat.BufferPages,
				Stat.ImagePages,
				Stat.MemoryFlags,
				*HostString);
		};



		FResourceHeapStats Staging;
		TArray<FResourceHeapStats> DeviceHeaps;
		Device.GetStagingManager().GetMemoryDump(Staging);
		Device.GetDeviceMemoryManager().GetMemoryDump(DeviceHeaps);

		VULKAN_LOGMEMORY(TEXT("SUMMARY"));
		VULKAN_LOGMEMORY(TEXT("\t\tDevice Heaps                       |    Memory       Reserved      FreeMem       TotalMem   |  Allocs     -         |  Allocs              | Flags | Type   "));
#define VULKAN_LOGMEMORY_PAD TEXT("\t\t---------------------------------------------------------------------------------------------------------------------------------------------------------")
		VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD);
		for (int32 Index = 0; Index < DeviceHeaps.Num(); ++Index)
		{
			FResourceHeapStats& Stat = DeviceHeaps[Index];
			WriteLogLine(FString::Printf(TEXT("Device Heap %d"), Index), Stat);
		}
		VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD);
		VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD);
		VULKAN_LOGMEMORY(TEXT("\t\tAllocators                         |    BufMem       ImgMem        FreeMem       TotalMem   |  BufAllocs  ImgAllocs |  Pages BufPgs ImgPgs | Flags | Type   "));
		VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD);


		FResourceHeapStats Total;
		FResourceHeapStats TotalHost;
		FResourceHeapStats TotalLocal;
		for (int32 Index = 0; Index < OverallSummary.Num(); ++Index)
		{
			FResourceHeapStats& Stat = OverallSummary[Index];
			Total += Stat;
			if (VKHasAnyFlags(Stat.MemoryFlags, VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT))
			{
				TotalLocal += Stat;
				TotalLocal.MemoryFlags |= Stat.MemoryFlags;
			}
			else
			{
				TotalHost += Stat;
				TotalHost.MemoryFlags |= Stat.MemoryFlags;
			}
			if(Index == DedicatedAllocatorSummary)
			{
				VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD);
				WriteLogLine(TEXT("Dedicated Pages"), Stat);
			}
			else if(Index >= (int32)SmallAllocatorsBegin && Index < (int32)SmallAllocatorsEnd)
			{
				int PoolSizeIndex = (Index - NumResourceHeaps) % NumSmallAllocators;
				uint32 PoolSize = PoolSizeIndex >= (int32)EPoolSizes::SizesCount ? -1 : PoolSizes[PoolSizeIndex];
				if(0 == PoolSizeIndex)
				{
					VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD);
				}
				WriteLogLine(FString::Printf(TEXT("Pool %d"), PoolSize), Stat);
			}
			else
			{
				WriteLogLine(FString::Printf(TEXT("ResourceHeap %d"), Index), Stat);
			}
		}
		VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD);
		WriteLogLine(TEXT("TotalHost"), TotalHost);
		WriteLogLine(TEXT("TotalLocal"), TotalLocal);
		WriteLogLine(TEXT("Total"), Total);
		VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD);
		for (int32 HeapIndex = 0; HeapIndex < HeapSummary.Num(); ++HeapIndex)
		{
			FResourceHeapStats& Stat = HeapSummary[HeapIndex];
			//for the heaps, show -actual- max size, not reserved.
			Stat.TotalMemory = MemoryProperties.memoryHeaps[HeapIndex].size;
			WriteLogLine(FString::Printf(TEXT("Allocated Device Heap %d"), HeapIndex), Stat);
		}
		VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD);
		VULKAN_LOGMEMORY(TEXT("\t\tSubsystems                         |    BufMem       ImgMem        FreeMem       TotalMem   |  BufAllocs  ImgAllocs |  Pages BufPgs ImgPgs | Flags | Type   "));
		VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD);
		WriteLogLine(TEXT("Staging"), Staging);
		VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD);

#define VULKAN_LOGMEMORY_PAD2 TEXT("\t\t------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------")

		VULKAN_LOGMEMORY(TEXT("\n\nSubAllocator Dump\n\n"));
		auto WriteLogLineSubAllocator = [](const FString& Name, const FString& MemoryString, FVulkanSubresourceAllocator& Allocator)
		{
			TArrayView<uint32> MemoryUsed = Allocator.GetMemoryUsed();
			uint32 NumAllocations = Allocator.GetNumSubAllocations();
			uint32 TotalMemory = Allocator.GetMaxSize();
			uint32 TotalUsed = 0;
			uint32 FreeCount = Allocator.FreeList.Num();
            uint32 LargestFree = 0;
            for(FRange& R : Allocator.FreeList)
            {
                LargestFree = FMath::Max(LargestFree, R.Size);
            }
			for(uint32 Used : MemoryUsed)
			{
				TotalUsed += Used;
			}
			uint64 Free= TotalMemory - TotalUsed;
			uint8 AllocatorFlags =  Allocator.GetSubresourceAllocatorFlags();
			VULKAN_LOGMEMORY(TEXT("\t\t%-45s  | %4d %8d | %3d%% / %8.2fmb / %8.2fmb / %8.2fmb / %8.2fmb | %8.2fmb / %8.2fmb / %8.2fmb / %8.2fmb | %8.2fmb / %8.2fmb | %8.2fmb / %8.2fmb / %8.2fmb | %d %d | %s"),
				*Name,
				FreeCount,
				NumAllocations,
                int(100 * (TotalMemory-Free) / TotalMemory),
				TotalUsed / (1024.f * 1024.f),
				Free / (1024.f * 1024.f),
				TotalMemory / (1024.f * 1024.f),
                LargestFree / (1024.f * 1024.f),
				MemoryUsed[EVulkanAllocationMetaUnknown] / (1024.f * 1024.f),
				MemoryUsed[EVulkanAllocationMetaUniformBuffer] / (1024.f * 1024.f),
				MemoryUsed[EVulkanAllocationMetaMultiBuffer] / (1024.f * 1024.f),
				MemoryUsed[EVulkanAllocationMetaFrameTempBuffer] / (1024.f * 1024.f),
				MemoryUsed[EVulkanAllocationMetaImageRenderTarget] / (1024.f * 1024.f),
				MemoryUsed[EVulkanAllocationMetaImageOther] / (1024.f * 1024.f),
				MemoryUsed[EVulkanAllocationMetaBufferUAV] / (1024.f * 1024.f),
				MemoryUsed[EVulkanAllocationMetaBufferStaging] / (1024.f * 1024.f),
				MemoryUsed[EVulkanAllocationMetaBufferOther] / (1024.f * 1024.f),
				AllocatorFlags & VulkanAllocationFlagsMapped ? 1 : 0,
				AllocatorFlags & VulkanAllocationFlagsCanEvict ? 1 : 0,
				*MemoryString
			);
		};
		auto DumpAllocatorRange = [&](FString Name, TArray<FVulkanSubresourceAllocator*>& Allocators)
		{
			uint32 Index = 0;
			for (FVulkanSubresourceAllocator* Allocator : Allocators)
			{
				VkMemoryPropertyFlags Flags = Allocator->MemoryPropertyFlags;
				if(!Flags)
				{
					Flags = MemoryProperties.memoryTypes[Allocator->MemoryTypeIndex].propertyFlags;
				}
				FString MemoryString = (Flags != 0) ? VK_FLAGS_TO_STRING(VkMemoryPropertyFlags, Flags) : TEXT("");
				FString NameId = FString::Printf(TEXT("%s [%4d]"), *Name, Allocator->AllocatorIndex);
				WriteLogLineSubAllocator(NameId, MemoryString, *Allocator);
			}
		};
		auto DumpAllocatorRangeContents = [&](FString Name, TArray<FVulkanSubresourceAllocator*>& Allocators)
		{
			uint32 Index = 0;
			for (FVulkanSubresourceAllocator* Allocator : Allocators)
			{
				VkMemoryPropertyFlags Flags = Allocator->MemoryPropertyFlags;
				if (!Flags)
				{
					Flags = MemoryProperties.memoryTypes[Allocator->MemoryTypeIndex].propertyFlags;
				}
				FString MemoryString = (Flags != 0) ? VK_FLAGS_TO_STRING(VkMemoryPropertyFlags, Flags) : TEXT("");
				VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD2);
				VULKAN_LOGMEMORY(TEXT("\t\t%-45s  | %4s %8s | %4s / %10s / %10s / %10s / %10s | %10s / %10s / %10s / %10s | %10s / %10s | %10s / %10s / %10s | Mapped/Evictable |"),
					TEXT(""),
					TEXT("Free"),
					TEXT("Count"),
                    TEXT("Fill"), 
					TEXT("Used"),
					TEXT("Free"),
					TEXT("Total"),
                    TEXT("Largest"),
					VulkanAllocationMetaTypeToString(EVulkanAllocationMetaUnknown),
					VulkanAllocationMetaTypeToString(EVulkanAllocationMetaUniformBuffer),
					VulkanAllocationMetaTypeToString(EVulkanAllocationMetaMultiBuffer),
					VulkanAllocationMetaTypeToString(EVulkanAllocationMetaFrameTempBuffer),
					VulkanAllocationMetaTypeToString(EVulkanAllocationMetaImageRenderTarget),
					VulkanAllocationMetaTypeToString(EVulkanAllocationMetaImageOther),
					VulkanAllocationMetaTypeToString(EVulkanAllocationMetaBufferUAV),
					VulkanAllocationMetaTypeToString(EVulkanAllocationMetaBufferStaging),
					VulkanAllocationMetaTypeToString(EVulkanAllocationMetaBufferOther)
				);			
				VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD2);

				FString NameId = FString::Printf(TEXT("%s %d"), *Name, Index++);
				WriteLogLineSubAllocator(NameId, MemoryString, *Allocator);
				Allocator->DumpFullHeap();
			}


		};




		VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD2);
		VULKAN_LOGMEMORY(TEXT("\t\t%-45s  | %4s %8s | %4s / %10s / %10s / %10s / %10s | %10s / %10s / %10s / %10s | %10s / %10s | %10s / %10s / %10s |"),
			TEXT(""),
			TEXT("Free"),
			TEXT("Count"),
            TEXT("Fill"),
			TEXT("Used"),
			TEXT("Free"),
			TEXT("Total"),
            TEXT("Largest"),
			VulkanAllocationMetaTypeToString(EVulkanAllocationMetaUnknown),
			VulkanAllocationMetaTypeToString(EVulkanAllocationMetaUniformBuffer),
			VulkanAllocationMetaTypeToString(EVulkanAllocationMetaMultiBuffer),
			VulkanAllocationMetaTypeToString(EVulkanAllocationMetaFrameTempBuffer),
			VulkanAllocationMetaTypeToString(EVulkanAllocationMetaImageRenderTarget),
			VulkanAllocationMetaTypeToString(EVulkanAllocationMetaImageOther),
			VulkanAllocationMetaTypeToString(EVulkanAllocationMetaBufferUAV),
			VulkanAllocationMetaTypeToString(EVulkanAllocationMetaBufferStaging),
			VulkanAllocationMetaTypeToString(EVulkanAllocationMetaBufferOther)
		);
        auto PageSuffix = [this](uint32 TypeIndex, uint32 BucketIndex)
        {          
    		auto& PageSizeBuckets = ResourceTypeHeaps[TypeIndex]->PageSizeBuckets;
            uint32 Mask = 0;
		    if (BucketIndex < (uint32)PageSizeBuckets.Num())
			{
				Mask = PageSizeBuckets[BucketIndex].BucketMask;
			}

            switch(Mask)
            {
                case FVulkanPageSizeBucket::BUCKET_MASK_IMAGE: return TEXT("I ");
                case FVulkanPageSizeBucket::BUCKET_MASK_BUFFER: return TEXT("B ");
                case FVulkanPageSizeBucket::BUCKET_MASK_IMAGE|FVulkanPageSizeBucket::BUCKET_MASK_BUFFER: return TEXT("IB");
                default: return TEXT("??");

            }
        };
		VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD2);
		for (int32 TypeIndex = 0; TypeIndex < ResourceTypeHeaps.Num(); ++TypeIndex)
			{
			if (ResourceTypeHeaps[TypeIndex])
				{
				for (uint32 SubIndex = 0; SubIndex < FVulkanResourceHeap::MAX_BUCKETS; ++SubIndex)
					{
					TArray<FVulkanSubresourceAllocator*>& PageArray = ResourceTypeHeaps[TypeIndex]->ActivePages[SubIndex];
					DumpAllocatorRange(FString::Printf(TEXT("Page[%s] - ResTypeHeaps[%d] ActivePages[%d]"), PageSuffix(TypeIndex, SubIndex), TypeIndex, SubIndex), PageArray);
					}
				DumpAllocatorRange(FString::Printf(TEXT("UsedDedicatedImagePages - ResTypeHeaps[%d]"), TypeIndex), ResourceTypeHeaps[TypeIndex]->UsedDedicatedImagePages);
			}
		}

		VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD2);
		for (int32 PoolSizeIndex = 0; PoolSizeIndex < UE_ARRAY_COUNT(UsedBufferAllocations); PoolSizeIndex++)
		{
			FString NameUsed = FString::Printf(TEXT("PoolUsed %d"), PoolSizeIndex);
			FString NameFree = FString::Printf(TEXT("PoolFree %d"), PoolSizeIndex);
			TArray<FVulkanSubresourceAllocator*>& UsedAllocations = UsedBufferAllocations[PoolSizeIndex];
			TArray<FVulkanSubresourceAllocator*>& FreeAllocations = FreeBufferAllocations[PoolSizeIndex];
			DumpAllocatorRange(NameUsed, UsedAllocations);
			DumpAllocatorRange(NameFree, FreeAllocations);
		}
		VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD2);

		if(bFullDump)
		{
			VULKAN_LOGMEMORY(VULKAN_LOGMEMORY_PAD2);
			for (int32 TypeIndex = 0; TypeIndex < ResourceTypeHeaps.Num(); ++TypeIndex)
			{
				if (ResourceTypeHeaps[TypeIndex])
				{
					for (uint32 SubIndex = 0; SubIndex < FVulkanResourceHeap::MAX_BUCKETS; ++SubIndex)
					{
						TArray<FVulkanSubresourceAllocator*>& PageArray = ResourceTypeHeaps[TypeIndex]->ActivePages[SubIndex];
						DumpAllocatorRangeContents(FString::Printf(TEXT("Page[%s] - ResTypeHeaps[%d] ActivePages[%d]"), PageSuffix(TypeIndex, SubIndex), TypeIndex, SubIndex), PageArray);
					}
					DumpAllocatorRangeContents(FString::Printf(TEXT("UsedDedicatedImagePages - ResTypeHeaps[%d]"), TypeIndex), ResourceTypeHeaps[TypeIndex]->UsedDedicatedImagePages);

				}
			}
		}


		GLog->Flush();


#undef VULKAN_LOGMEMORY_PAD
#undef VULKAN_LOGMEMORY_PAD2
	}


	void FMemoryManager::HandleOOM(bool bCanResume, VkResult Result, uint64 AllocationSize, uint32 MemoryTypeIndex)
	{
		if(!bCanResume)
		{
			const TCHAR* MemoryType = TEXT("?");
			switch(Result)
			{
			case VK_ERROR_OUT_OF_HOST_MEMORY: MemoryType = TEXT("Host"); break;
			case VK_ERROR_OUT_OF_DEVICE_MEMORY: MemoryType = TEXT("Local"); break;
			}
			DumpMemory();
			GLog->Panic();
			DumpRenderTargetPoolMemory(*GLog);
			GLog->Flush();

			FPlatformMemory::bIsOOM = true;
			UE_LOGF(LogVulkanRHI, Fatal, "Out of %ls Memory, Requested%.2fKB MemTypeIndex=%d\n", MemoryType, float(AllocationSize), MemoryTypeIndex);
		}
	}

	void FMemoryManager::AllocUniformBuffer(FVulkanAllocation& OutAllocation, uint32 Size)
	{
		const int32 UBAlignment = FMath::Max<uint32>(Device.GetLimits().minUniformBufferOffsetAlignment, 16u);
		if (!AllocateBufferPooled(OutAllocation, nullptr, Size, UBAlignment, VK_BUFFER_USAGE_UNIFORM_BUFFER_BIT | VK_BUFFER_USAGE_TRANSFER_DST_BIT, VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT | VK_MEMORY_PROPERTY_HOST_COHERENT_BIT, EVulkanAllocationMetaUniformBuffer, __FILE__, __LINE__))
		{
			HandleOOM(false);
			checkNoEntry();
		}

		INC_MEMORY_STAT_BY(STAT_UniformBufferMemory, OutAllocation.Size);
	}
	void FMemoryManager::FreeUniformBuffer(FVulkanAllocation& InAllocation)
	{
		if (InAllocation.bTempBlockAllocator)
		{
			Device.GetDeferredDeletionQueue().EnqueueFunc([InAllocation](FVulkanDevice& Device)
			{
				Device.GetTempBlockAllocator().FreeReference(InAllocation);
			});
		}
		else if (InAllocation.HasAllocation())
		{
			DEC_MEMORY_STAT_BY(STAT_UniformBufferMemory, InAllocation.Size);

			FScopeLock ScopeLock(&UBAllocations.CS);
			ProcessPendingUBFreesNoLock(false);
			FUBPendingFree& Pending = UBAllocations.PendingFree.AddDefaulted_GetRef();
			Pending.Frame = GFrameNumberRenderThread;
			Pending.Allocation.Swap(InAllocation);
			UBAllocations.Peak = FMath::Max(UBAllocations.Peak, (uint32)UBAllocations.PendingFree.Num());
		}
	}

	void FMemoryManager::ProcessPendingUBFreesNoLock(bool bForce)
	{
		// this keeps an frame number of the first frame when we can expect to delete things, updated in the loop if any pending allocations are left
		static uint32 GFrameNumberRenderThread_WhenWeCanDelete = 0;

		if (UNLIKELY(bForce))
		{
			int32 NumAlloc = UBAllocations.PendingFree.Num();
			for (int32 Index = 0; Index < NumAlloc; ++Index)
			{
				FUBPendingFree& Alloc = UBAllocations.PendingFree[Index];
				FreeVulkanAllocation(Alloc.Allocation, EVulkanFreeFlag_DontDefer);
			}
			UBAllocations.PendingFree.Empty();

			// invalidate the value
			GFrameNumberRenderThread_WhenWeCanDelete = 0;
		}
		else
		{
			if (LIKELY(GFrameNumberRenderThread < GFrameNumberRenderThread_WhenWeCanDelete))
			{
				// too early
				return;
			}

			// making use of the fact that we always add to the end of the array, so allocations are sorted by frame ascending
			int32 OldestFrameToKeep = GFrameNumberRenderThread - GVulkanNumFramesToWaitForResourceDelete;
			int32 NumAlloc = UBAllocations.PendingFree.Num();
			int32 Index = 0;
			for (; Index < NumAlloc; ++Index)
			{
				FUBPendingFree& Alloc = UBAllocations.PendingFree[Index];
				if (LIKELY(Alloc.Frame < OldestFrameToKeep))
				{
					FreeVulkanAllocation(Alloc.Allocation, EVulkanFreeFlag_DontDefer);
				}
				else
				{
					// calculate when we will be able to delete the oldest allocation
					GFrameNumberRenderThread_WhenWeCanDelete = Alloc.Frame + GVulkanNumFramesToWaitForResourceDelete + 1;
					break;
				}
			}

			int32 ElementsLeft = NumAlloc - Index;
			if (ElementsLeft > 0 && ElementsLeft != NumAlloc)
			{
				// FUBPendingFree is POD because it is stored in a TArray
				FMemory::Memmove(UBAllocations.PendingFree.GetData(), UBAllocations.PendingFree.GetData() + Index, ElementsLeft * sizeof(FUBPendingFree));
				for(int32 EndIndex = ElementsLeft; EndIndex < UBAllocations.PendingFree.Num(); ++EndIndex)
				{
					auto& E = UBAllocations.PendingFree[EndIndex];
					if(E.Allocation.HasAllocation())
					{
						E.Allocation.Disown();
					}
				}
			}
			UBAllocations.PendingFree.SetNum(NumAlloc - Index, EAllowShrinking::No);
		}
	}

	void FMemoryManager::ProcessPendingUBFrees(bool bForce)
	{
		FScopeLock ScopeLock(&UBAllocations.CS);
		ProcessPendingUBFreesNoLock(bForce);
	}

	bool FVulkanSubresourceAllocator::JoinFreeBlocks()
	{
		FScopeLock ScopeLock(&SubresourceAllocatorCS);
#if !UE_VK_MEMORY_JOIN_FREELIST_ON_THE_FLY
		FRange::JoinConsecutiveRanges(FreeList);
#endif

		if (FreeList.Num() == 1)
		{
			if (NumSubAllocations == 0)
			{
				check(UsedSize == 0);
				checkf(FreeList[0].Offset == 0 && FreeList[0].Size == MaxSize, TEXT("Resource Suballocation leak, should have %d free, only have %d; missing %d bytes"), MaxSize, FreeList[0].Size, MaxSize - FreeList[0].Size);
				return true;
			}
		}
		return false;
	}

	FVulkanAllocation::FVulkanAllocation()
		: bHasOwnership(0), bTransient(false), bTempBlockAllocator(false)
	{
		SetType(EVulkanAllocationEmpty);
	}
	FVulkanAllocation::~FVulkanAllocation()
	{
		check(!HasAllocation());
	}


	void FVulkanAllocationInternal::Init(const FVulkanAllocation& Alloc, FVulkanEvictable* InAllocationOwner, uint32 InAllocationOffset, uint32 InAllocationSize, uint32 InAlignment)
	{
		check(State == EUNUSED);
		State = EALLOCATED;
		Type = Alloc.GetType();
		MetaType = Alloc.MetaType;

		Size = Alloc.Size;
		AllocationSize = InAllocationSize;
		AllocationOffset = InAllocationOffset;
		AllocationOwner = InAllocationOwner;
		Alignment = InAlignment;
	}

	void FVulkanAllocation::Init(EVulkanAllocationType InType, EVulkanAllocationMetaType InMetaType, uint64 Handle, uint32 InSize, uint32 InAlignedOffset, uint32 InAllocatorIndex, uint32 InAllocationIndex, uint32 BufferId)
	{
		check(!HasAllocation());
		bHasOwnership = 1;
		SetType(InType);
		MetaType = InMetaType;
		Size = InSize;
		Offset = InAlignedOffset;
		check(InAllocatorIndex < (uint32)MAX_uint16);
		AllocatorIndex = InAllocatorIndex;
		AllocationIndex = InAllocationIndex;
		VulkanHandle = Handle;
		HandleId = BufferId;
		// Make sure all allocations have a valid Id on platforms that use "Descriptor Cache"
		ensure(!UseVulkanDescriptorCache() || HandleId != 0);
	}

	void FVulkanAllocation::Free(FVulkanDevice& Device)
	{
		if (HasAllocation())
		{
			Device.GetMemoryManager().FreeVulkanAllocation(*this);
			check(EVulkanAllocationEmpty != Type);
		}
	}
	void FVulkanAllocation::Swap(FVulkanAllocation& Other)
	{
		::Swap(*this, Other);
	}

	void FVulkanAllocation::Reference(const FVulkanAllocation& Other)
	{
		FMemory::Memcpy(*this, Other);
		bHasOwnership = 0;
	}
	bool FVulkanAllocation::HasAllocation() const
	{
		return Type != EVulkanAllocationEmpty && bHasOwnership;
	}

	void FVulkanAllocation::Disown()
	{
		check(bHasOwnership);
		bHasOwnership = 0;
	}
	void FVulkanAllocation::Own()
	{
		check(!bHasOwnership);
		bHasOwnership = 1;
	}
	bool FVulkanAllocation::IsValid() const
	{
		return Size != 0;
	}
	void* FVulkanAllocation::GetMappedPointer(FVulkanDevice* Device)
	{
		FVulkanSubresourceAllocator* Allocator = GetSubresourceAllocator(Device);
		uint8* pMappedPointer = (uint8*)Allocator->GetMappedPointer();
		check(pMappedPointer);
		return Offset + pMappedPointer;
	}

	void FVulkanAllocation::FlushMappedMemory(FVulkanDevice* Device)
	{
		FVulkanSubresourceAllocator* Allocator = GetSubresourceAllocator(Device);
		Allocator->Flush(Offset, Size);
	}

	void FVulkanAllocation::InvalidateMappedMemory(FVulkanDevice* Device)
	{
		FVulkanSubresourceAllocator* Allocator = GetSubresourceAllocator(Device);
		Allocator->Invalidate(Offset, Size);
	}

	VkBuffer FVulkanAllocation::GetBufferHandle() const
	{
		return (VkBuffer)VulkanHandle;
	}
	uint32 FVulkanAllocation::GetBufferAlignment(FVulkanDevice* Device) const
	{
		FVulkanSubresourceAllocator* Allocator = GetSubresourceAllocator(Device);
		return Allocator->GetAlignment();
	}
	VkDeviceMemory FVulkanAllocation::GetDeviceMemoryHandle(FVulkanDevice* Device) const
	{
		FVulkanSubresourceAllocator* Allocator = GetSubresourceAllocator(Device);
		return Allocator->GetMemoryAllocation()->GetHandle();
	}

	void FVulkanAllocation::BindBuffer(FVulkanDevice* Device, VkBuffer Buffer)
	{
		VkDeviceMemory MemoryHandle = GetDeviceMemoryHandle(Device);
		VkResult Result = VulkanRHI::vkBindBufferMemory(Device->GetHandle(), Buffer, MemoryHandle, Offset);
		if (Result == VK_ERROR_OUT_OF_DEVICE_MEMORY || Result == VK_ERROR_OUT_OF_HOST_MEMORY)
		{
			Device->GetMemoryManager().DumpMemory();
		}
		VERIFYVULKANRESULT(Result);
	}
	void FVulkanAllocation::BindImage(FVulkanDevice* Device, VkImage Image)
	{
		VkDeviceMemory MemoryHandle = GetDeviceMemoryHandle(Device);
		VkResult Result = VulkanRHI::vkBindImageMemory(Device->GetHandle(), Image, MemoryHandle, Offset);
		if (Result == VK_ERROR_OUT_OF_DEVICE_MEMORY || Result == VK_ERROR_OUT_OF_HOST_MEMORY)
		{
			Device->GetMemoryManager().DumpMemory();
		}
		VERIFYVULKANRESULT(Result);
	}
	FVulkanSubresourceAllocator* FVulkanAllocation::GetSubresourceAllocator(FVulkanDevice* Device) const
	{
		switch(Type)
		{
		case EVulkanAllocationEmpty:
			return 0;
		case EVulkanAllocationPooledBuffer:
		case EVulkanAllocationBuffer:
		case EVulkanAllocationImage:
		case EVulkanAllocationImageDedicated:
			return Device->GetMemoryManager().GetSubresourceAllocator(AllocatorIndex);
		default:
			check(0);
		}
		return 0;
	}
	VkMemoryPropertyFlags FVulkanAllocation::GetMemoryPropertyFlags(FVulkanDevice* Device) const
	{
		const VkPhysicalDeviceMemoryProperties& MemoryProperties = Device->GetDeviceMemoryManager().GetMemoryProperties();
		const uint32 MemoryTypeIndex = GetSubresourceAllocator(Device)->GetMemoryTypeIndex();
		check(MemoryTypeIndex < MemoryProperties.memoryTypeCount);
		return MemoryProperties.memoryTypes[MemoryTypeIndex].propertyFlags;
	}

	void FVulkanSubresourceAllocator::FreeInternalData(int32 Index)
	{
		check(InternalData[Index].State == FVulkanAllocationInternal::EUNUSED || InternalData[Index].State ==FVulkanAllocationInternal::EFREED);
		check(InternalData[Index].NextFree == -1);
		InternalData[Index].NextFree = InternalFreeList;
		InternalFreeList = Index;
		InternalData[Index].State  = FVulkanAllocationInternal::EUNUSED;
	}



	int32 FVulkanSubresourceAllocator::AllocateInternalData()
	{
		int32 FreeListHead = InternalFreeList;
		if(FreeListHead < 0)
		{
			int32 Result = InternalData.AddZeroed(1);
			InternalData[Result].NextFree = -1;
			return Result;

		}
		else
		{
			InternalFreeList = InternalData[FreeListHead].NextFree;
			InternalData[FreeListHead].NextFree = -1;
			return FreeListHead;
		}
	}



	bool FVulkanSubresourceAllocator::TryAllocate2(FVulkanAllocation& OutAllocation, FVulkanEvictable* AllocationOwner, uint32 InSize, uint32 InAlignment, EVulkanAllocationMetaType InMetaType, const char* File, uint32 Line)
	{
		FScopeLock ScopeLock(&SubresourceAllocatorCS);
		if (bIsEvicting || bLocked)
		{
			return false;
		}
		InAlignment = FMath::Max(InAlignment, Alignment);
		for (int32 Index = 0; Index < FreeList.Num(); ++Index)
		{
			FRange& Entry = FreeList[Index];
			uint32 AllocatedOffset = Entry.Offset;
			uint32 AlignedOffset = Align(Entry.Offset, InAlignment);
			uint32 AlignmentAdjustment = AlignedOffset - Entry.Offset;
			uint32 AllocatedSize = AlignmentAdjustment + InSize;
			if (AllocatedSize <= Entry.Size)
			{
				FRange::AllocateFromEntry(FreeList, Index, AllocatedSize);

				UsedSize += AllocatedSize;
				int32 ExtraOffset = AllocateInternalData();
				OutAllocation.Init(Type, InMetaType, (uint64)Buffer, InSize, AlignedOffset, GetAllocatorIndex(), ExtraOffset, BufferId);
				MemoryUsed[InMetaType] += AllocatedSize;
				static uint32 UIDCounter = 0;
				UIDCounter++;
				InternalData[ExtraOffset].Init(OutAllocation, AllocationOwner, AllocatedOffset, AllocatedSize, InAlignment);
				VULKAN_FILL_TRACK_INFO(InternalData[ExtraOffset].Track, File, Line);
				AllocCalls++;
				NumSubAllocations++;

				LLM_TRACK_VULKAN_HIGH_LEVEL_ALLOC(InternalData[ExtraOffset], OutAllocation.Size);
				LLM_TRACK_VULKAN_SPARE_MEMORY_GPU(-(int64)OutAllocation.Size);
				bIsDefragging = false;
				return true;
			}
		}
		return false;
	}

	static VkDeviceSize AdjustToNonCoherentAtomSize(const VkDeviceSize RequestedOffset, const VkDeviceSize RequestedSize, const VkDeviceSize WholeAllocationSize, const VkDeviceSize NonCoherentAtomSize)
	{
		if (RequestedSize == VK_WHOLE_SIZE)
		{
			return RequestedSize;
		}
		
		check(RequestedOffset + RequestedSize <= WholeAllocationSize);
		
		return FMath::Min(WholeAllocationSize - RequestedOffset, AlignArbitrary(RequestedSize, NonCoherentAtomSize));
	}

	void FVulkanSubresourceAllocator::Flush(VkDeviceSize Offset, VkDeviceSize AllocationSize)
	{
		if (!MemoryAllocation->IsCoherent() || GForceCoherent != 0)
		{
			const VkDeviceSize NonCoherentAtomSize = Owner->GetDevice().GetLimits().nonCoherentAtomSize;
			MemoryAllocation->FlushMappedMemory(Offset, AdjustToNonCoherentAtomSize(Offset, AllocationSize, MemoryAllocation->GetSize(), NonCoherentAtomSize));
		}
	}
	void FVulkanSubresourceAllocator::Invalidate(VkDeviceSize Offset, VkDeviceSize AllocationSize)
	{
		if (!MemoryAllocation->IsCoherent() || GForceCoherent != 0)
		{
			const VkDeviceSize NonCoherentAtomSize = Owner->GetDevice().GetLimits().nonCoherentAtomSize;
			MemoryAllocation->InvalidateMappedMemory(Offset, AdjustToNonCoherentAtomSize(Offset, AllocationSize, MemoryAllocation->GetSize(), NonCoherentAtomSize));
		}
	}

	TArrayView<uint32> FVulkanSubresourceAllocator::GetMemoryUsed()
	{
		return MemoryUsed;

	}
	uint32 FVulkanSubresourceAllocator::GetNumSubAllocations()
	{
		return NumSubAllocations;

	}


	namespace
	{
		struct SDumpHeapEntry
		{
			uint32 Offset = 0xffffffff;
			uint32 Size = 0;
			int32 FreeListIndex = -1;
			int32 InternalIndex = -1;
		};
	}

	void FVulkanSubresourceAllocator::DumpFullHeap()
	{
		TArray<SDumpHeapEntry> Entries;
		for (FVulkanAllocationInternal& Alloc : InternalData)
		{
			if (Alloc.State != FVulkanAllocationInternal::EUNUSED)
			{
				int32 Index = (int32)(&Alloc - &InternalData[0]);
				SDumpHeapEntry H;
				H.Offset = Alloc.AllocationOffset;
				H.Size = Alloc.AllocationSize;
				H.InternalIndex = Index;
				Entries.Add(H);
			}
		}
		for(FRange& R : FreeList)
		{
			int32 Index = &R - &FreeList[0];
			SDumpHeapEntry H;
			H.Offset = R.Offset;
			H.Size = R.Size;
			H.FreeListIndex = Index;
			Entries.Add(H);
		}
		Entries.Sort([](const SDumpHeapEntry& L, const SDumpHeapEntry& R) -> bool
			{
				return L.Offset < R.Offset;
			}
		);

		for(SDumpHeapEntry& Entry : Entries)
		{
			if(Entry.FreeListIndex >= 0)
			{
				VULKAN_LOGMEMORY(TEXT("\t\t    [%08x - %08x]  | %11.3fMB | **FREE** "), Entry.Offset, Entry.Offset + Entry.Size, Entry.Size / (1024.f * 1024.f));
			}
			else
			{
				check(Entry.InternalIndex >= 0);
				FVulkanAllocationInternal& Alloc = InternalData[Entry.InternalIndex];

				if  (Alloc.State == FVulkanAllocationInternal::EALLOCATED)
				{
				const TCHAR* TypeStr = VulkanAllocationMetaTypeToString(Alloc.MetaType);
				const TCHAR* Name = TEXT("< ? >");
				FString Tmp;

				if(Alloc.MetaType == EVulkanAllocationMetaImageRenderTarget)
				{
					FVulkanEvictable* Evictable = Alloc.AllocationOwner;
					FVulkanTexture* Texture = Evictable->GetEvictableTexture();
					if(Texture)
					{
						Tmp = Texture->GetName().ToString();
						Name = *Tmp;
					}
				}

				VULKAN_LOGMEMORY(TEXT("\t\t    [%08x - %08x]  | %11.3fMB | %20s   %s"), Entry.Offset, Entry.Offset + Entry.Size, Entry.Size / (1024.f * 1024.f), TypeStr, Name);
			}
				else
				{
					const TCHAR* StateStr =
						(Alloc.State == FVulkanAllocationInternal::EFREED) ? TEXT("**Freed**") :
						(Alloc.State == FVulkanAllocationInternal::EFREEPENDING) ? TEXT("**FreePending**") :
						(Alloc.State == FVulkanAllocationInternal::EFREEDISCARDED) ? TEXT("**FreeDiscarded**") :
						TEXT("< ? >");

					VULKAN_LOGMEMORY(TEXT("\t\t    [%08x - %08x]  | %11.3fMB | %s"), Entry.Offset, Entry.Offset + Entry.Size, Entry.Size / (1024.f * 1024.f), StateStr);
				}
			}
		}
	}

	bool FVulkanSubresourceAllocator::CanDefrag()
	{
		for (FVulkanAllocationInternal& Alloc : InternalData)
		{
			if (Alloc.State == FVulkanAllocationInternal::EALLOCATED)
			{
				FVulkanEvictable* EvictableOwner = Alloc.AllocationOwner;
				if((EvictableOwner == nullptr) || !EvictableOwner->CanMove())
				{
					return false;
				}
			}
		}
		return true;
	}
	
	// One tick of incremental defrag. This will move max Count_ allocations from this suballocator to other SubAllocators on the same Heap
	// This will eventually free up the page, which can then be released
	// Returns # of allocations moved.
	int32 FVulkanSubresourceAllocator::DefragTick(FVulkanDevice& Device, const FVulkanContextArray& Contexts, FVulkanResourceHeap* Heap, uint32 Count_)
	{
		LastDefragFrame = GFrameNumberRenderThread;
		int32 DefragCount = 0;
		int32 Count = (int32)Count_;
		bLocked = true;

		auto PrintFreeList = [&](int32 id)
		{
			uint32 mmin = 0xffffffff;
			uint32 mmax = 0x0;
			for (FRange& R : FreeList)
			{
				mmin = FMath::Min(R.Size, mmin);
				mmax = FMath::Max(R.Size, mmax);
			}

			VULKAN_LOGMEMORY(TEXT("**** %d   %02d  %6.2fMB   %6.2fMB     : "), id, FreeList.Num(), mmax / (1024.f * 1024.f), mmin / (1024.f * 1024.f));
			for (FRange& R : FreeList)
			{
				VULKAN_LOGMEMORY(TEXT("[%08x / %6.2fMB / %08x]"), R.Offset, R.Size / (1024.f * 1024.f), R.Offset + R.Size);
			}
			VULKAN_LOGMEMORY(TEXT("\n"));

		};

		FScopeLock ScopeLock(&SubresourceAllocatorCS);

		// Search for allocations to move to different pages.
		for (FVulkanAllocationInternal& Alloc : InternalData)
		{
			if (Alloc.State == FVulkanAllocationInternal::EALLOCATED)
			{
				FVulkanEvictable* EvictableOwner = Alloc.AllocationOwner;
				switch (Alloc.MetaType)
				{
					case EVulkanAllocationMetaImageRenderTarget: //only rendertargets can be defragged
					{
						FVulkanTexture* Texture = EvictableOwner->GetEvictableTexture();

						// Only work with straightforward targets that are in a single layout
						if (Texture)
						{
							FVulkanAllocation Allocation;
							// The current SubAllocator is tagged as locked, this will never allocate in the current SubAllocator.
							if (Heap->TryRealloc(Allocation, EvictableOwner, EType::Image, Alloc.Size, Alloc.Alignment, Alloc.MetaType))
							{
								check(Allocation.HasAllocation());

								if (GVulkanLogDefrag)
								{
									VULKAN_LOGMEMORY(TEXT("Moving %6.2fMB : %d:%08x -> %d/%08x\n"), Alloc.Size / (1024.f * 1024.f),
										AllocatorIndex,
										Alloc.AllocationOffset,
										Allocation.AllocatorIndex,
										Allocation.Offset);
								}

								//Move the Rendertarget to the new allocation
								//Function swaps the old allocation into the Allocation object
								Texture->Move(Device, Contexts, Allocation);
								DefragCount++;
								Device.GetMemoryManager().FreeVulkanAllocation(Allocation);

								check(Alloc.State != FVulkanAllocationInternal::EALLOCATED);
								check(!Allocation.HasAllocation()); //must be consumed by Move
							}
							else
							{
								check(!Allocation.HasAllocation());
								bLocked = false;
								return DefragCount;
							}
						}
					}
					break;
					default:
						checkNoEntry(); //not implemented.
				}
				if (0 >= --Count)
				{
					break;
				}
			}
		}
		bLocked = false;
		return DefragCount;
	}

	uint64 FVulkanSubresourceAllocator::EvictToHost(FVulkanDevice& Device, const FVulkanContextArray& Contexts)
	{
		FScopeLock ScopeLock(&SubresourceAllocatorCS);
		bIsEvicting = true;
		for (FVulkanAllocationInternal& Alloc : InternalData)
		{
			if (Alloc.State == FVulkanAllocationInternal::EALLOCATED)
			{
				switch(Alloc.MetaType)
				{
				case EVulkanAllocationMetaImageOther:
				{
					FVulkanEvictable* Texture = Alloc.AllocationOwner;
					if (Texture->CanEvict())
					{
						Texture->Evict(Device, Contexts);
					}
				}
				break;
				default:
					//right now only there is only support for evicting non-rt images
					checkNoEntry();

				}
			}

		}
		return MemoryAllocation->GetSize();
	}

	FStagingBuffer::FStagingBuffer(FVulkanDevice* InDevice)
		: Device(InDevice)
		, Buffer(VK_NULL_HANDLE)
		, MemoryReadFlags(VK_MEMORY_PROPERTY_HOST_COHERENT_BIT)
		, BufferSize(0)
	{
	}
	VkBuffer FStagingBuffer::GetHandle() const
	{
		return Buffer;
	}

	FStagingBuffer::~FStagingBuffer()
	{
		Device->GetMemoryManager().FreeVulkanAllocation(Allocation);
	}


	void* FStagingBuffer::GetMappedPointer()
	{
		return Allocation.GetMappedPointer(Device);
	}

	uint32 FStagingBuffer::GetSize() const
	{
		return BufferSize;
	}

	VkDeviceMemory FStagingBuffer::GetDeviceMemoryHandle() const
	{
		return Allocation.GetDeviceMemoryHandle(Device);
	}

	void FStagingBuffer::FlushMappedMemory()
	{
		Allocation.FlushMappedMemory(Device);
	}

	void FStagingBuffer::InvalidateMappedMemory()
	{
		Allocation.InvalidateMappedMemory(Device);
	}


	void FStagingBuffer::Destroy()
	{
		//// Does not need to go in the deferred deletion queue
		VulkanRHI::vkDestroyBuffer(Device->GetHandle(), Buffer, VULKAN_CPU_ALLOCATOR);
		Buffer = VK_NULL_HANDLE;
		Device->GetMemoryManager().FreeVulkanAllocation(Allocation);
	}

	FStagingManager::FStagingManager()
	{
	}

	FStagingManager::~FStagingManager()
	{
	}

	void FStagingManager::Deinit()
	{
		ProcessPendingFree(true, true);

		if ((UsedStagingBuffers.Num() != 0) || (PendingFreeStagingBuffers.Num() != 0) || (FreeStagingBuffers.Num() != 0))
		{
			UE_LOGF(LogVulkanRHI, Warning,
				"Some resources in the FStagingManager were not freed!  (UsedBuffer=%d, PendingFree=%d, FreeBuffer=%d)",
				UsedStagingBuffers.Num(), PendingFreeStagingBuffers.Num(), FreeStagingBuffers.Num());
		}
	}

	FStagingBuffer* FStagingManager::AcquireBuffer(uint32 Size, VkBufferUsageFlags InUsageFlags, VkMemoryPropertyFlagBits InMemoryReadFlags)
	{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
		SCOPE_CYCLE_COUNTER(STAT_VulkanStagingBuffer);
#endif
		LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanStagingBuffers);

		const bool IsHostCached = (InMemoryReadFlags == VK_MEMORY_PROPERTY_HOST_CACHED_BIT);
		if (IsHostCached)
		{
			Size = AlignArbitrary(Size, (uint32)Device->GetLimits().nonCoherentAtomSize);
		}

		// Add both source and dest flags
		if ((InUsageFlags & (VK_BUFFER_USAGE_TRANSFER_SRC_BIT | VK_BUFFER_USAGE_TRANSFER_DST_BIT)) != 0)
		{
			InUsageFlags |= (VK_BUFFER_USAGE_TRANSFER_SRC_BIT | VK_BUFFER_USAGE_TRANSFER_DST_BIT);
		}

		// For descriptors buffers
		if (Device->GetOptionalExtensions().HasBufferDeviceAddress)
		{
			InUsageFlags |= VK_BUFFER_USAGE_SHADER_DEVICE_ADDRESS_BIT;
		}

		//#todo-rco: Better locking!
		{
			FScopeLock Lock(&StagingLock);
			for (int32 Index = 0; Index < FreeStagingBuffers.Num(); ++Index)
			{
				FFreeEntry& FreeBuffer = FreeStagingBuffers[Index];
				if (FreeBuffer.StagingBuffer->GetSize() == Size && FreeBuffer.StagingBuffer->MemoryReadFlags == InMemoryReadFlags)
				{
					FStagingBuffer* Buffer = FreeBuffer.StagingBuffer;
					FreeStagingBuffers.RemoveAtSwap(Index, EAllowShrinking::No);
					UsedStagingBuffers.Add(Buffer);
					VULKAN_FILL_TRACK_INFO(Buffer->Track, __FILE__, __LINE__);
					check(Buffer->GetHandle());
					return Buffer;
				}
			}
		}

		FStagingBuffer* StagingBuffer = new FStagingBuffer(Device);
		StagingBuffer->MemoryReadFlags = InMemoryReadFlags;
		StagingBuffer->BufferSize = Size;
		StagingBuffer->Buffer = Device->CreateBuffer(Size, InUsageFlags);

		// Set minimum alignment to 16 bytes, as some buffers are used with CPU SIMD instructions
		uint32 ForcedMinAlignment = 16u;
		static const bool bIsAmd = (Device->GetDeviceProperties().vendorID == (uint32)EGpuVendorId::Amd);
		if (IsHostCached || bIsAmd)
		{
			ForcedMinAlignment = AlignArbitrary(ForcedMinAlignment, (uint32)Device->GetLimits().nonCoherentAtomSize);
		}

		const EVulkanAllocationFlags AllocFlags = EVulkanAllocationFlags::AutoBind |
			(IsHostCached ? EVulkanAllocationFlags::HostCached : EVulkanAllocationFlags::HostVisible);

		Device->GetMemoryManager().AllocateBufferMemory(StagingBuffer->Allocation, StagingBuffer->Buffer, AllocFlags, TEXT("StagingBuffer"), ForcedMinAlignment);

		{
			FScopeLock Lock(&StagingLock);
			UsedStagingBuffers.Add(StagingBuffer);
			UsedMemory += StagingBuffer->GetSize();
			PeakUsedMemory = FMath::Max(UsedMemory, PeakUsedMemory);
		}

		VULKAN_FILL_TRACK_INFO(StagingBuffer->Track, __FILE__, __LINE__);
		return StagingBuffer;
	}

	void FStagingManager::ReleaseBuffer(FVulkanContextCommon* Context, FStagingBuffer*& StagingBuffer)
	{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
		SCOPE_CYCLE_COUNTER(STAT_VulkanStagingBuffer);
#endif

		FScopeLock Lock(&StagingLock);
		UsedStagingBuffers.RemoveSingleSwap(StagingBuffer, EAllowShrinking::No);

		if (Context)
		{
			TArray<FStagingBuffer*>& BufferArray = PendingFreeStagingBuffers.FindOrAdd(Context->GetContextSyncPoint());
			BufferArray.Add(StagingBuffer);
		}
		else
		{
			FreeStagingBuffers.Add({StagingBuffer, GFrameNumberRenderThread});
		}
		StagingBuffer = nullptr;
	}

	void FStagingManager::GetMemoryDump(FResourceHeapStats& Stats)
	{
		for (int32 Index = 0; Index < UsedStagingBuffers.Num(); ++Index)
		{
			FStagingBuffer* Buffer = UsedStagingBuffers[Index];
			Stats.BufferAllocations += 1;
			Stats.UsedBufferMemory += Buffer->BufferSize;
			Stats.TotalMemory += Buffer->BufferSize;
			Stats.MemoryFlags |= Buffer->MemoryReadFlags|VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT;
		}

		for (auto& Pair : PendingFreeStagingBuffers)
		{
			for (FStagingBuffer* StagingBuffer : Pair.Value)
			{
				Stats.BufferAllocations += 1;
				Stats.UsedBufferMemory += StagingBuffer->BufferSize;
				Stats.TotalMemory += StagingBuffer->BufferSize;
				Stats.MemoryFlags |= StagingBuffer->MemoryReadFlags | VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT;
			}
		}

		for (int32 Index = 0; Index < FreeStagingBuffers.Num(); ++Index)
		{
			FFreeEntry& Entry = FreeStagingBuffers[Index];
			Stats.BufferAllocations += 1;
			Stats.TotalMemory += Entry.StagingBuffer->BufferSize;
			Stats.MemoryFlags |= Entry.StagingBuffer->MemoryReadFlags|VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT;
		}
	}

	void FStagingManager::DumpMemory()
	{
		VULKAN_LOGMEMORY(TEXT("/******************************************* STAGING *******************************************\\"));
		VULKAN_LOGMEMORY(TEXT("StagingManager %d Used %d Pending Free %d Free"), UsedStagingBuffers.Num(), PendingFreeStagingBuffers.Num(), FreeStagingBuffers.Num());
		VULKAN_LOGMEMORY(TEXT("Used   BufferHandle       ResourceAllocation Size"));
		for (int32 Index = 0; Index < UsedStagingBuffers.Num(); ++Index)
		{
			FStagingBuffer* Buffer = UsedStagingBuffers[Index];
			VULKAN_LOGMEMORY(TEXT("%6d 0x%016llx 0x%016llx %6d"), Index, (void*)Buffer->GetHandle(), (void*)Buffer->Allocation.GetBufferHandle(), Buffer->BufferSize);
		}

		VULKAN_LOGMEMORY(TEXT("Pending Fence    BufferHandle       ResourceAllocation Size"));
		int32 PendingFreeIndex = 0;
		for (auto& Pair : PendingFreeStagingBuffers)
		{
			for (FStagingBuffer* StagingBuffer : Pair.Value)
			{
				VULKAN_LOGMEMORY(TEXT("%6d  %p 0x%016llx 0x%016llx %6d"), PendingFreeIndex++, Pair.Key.GetReference(),
					(void*)StagingBuffer->GetHandle(), (void*)StagingBuffer->Allocation.GetBufferHandle(), StagingBuffer->BufferSize);
			}
		}

		VULKAN_LOGMEMORY(TEXT("Free   BufferHandle     ResourceAllocation Size"));
		for (int32 Index = 0; Index < FreeStagingBuffers.Num(); ++Index)
		{
			FFreeEntry& Entry = FreeStagingBuffers[Index];
			VULKAN_LOGMEMORY(TEXT("%6d 0x%016llx 0x%016llx %6d"), Index, (void*)Entry.StagingBuffer->GetHandle(), (void*)Entry.StagingBuffer->Allocation.GetBufferHandle(), Entry.StagingBuffer->BufferSize);
		}
	}


	void FStagingManager::ProcessPendingFreeNoLock(bool bImmediately, bool bFreeToOS)
	{
		const int32 NumOriginalFreeBuffers = FreeStagingBuffers.Num();

		TArray<FVulkanSyncPoint*> Completed;
		for (auto& Pair : PendingFreeStagingBuffers)
		{
			if (bImmediately || Pair.Key->IsComplete())
			{
				for (FStagingBuffer* StagingBuffer : Pair.Value)
				{
					FreeStagingBuffers.Add({ StagingBuffer, GFrameNumberRenderThread });
				}
				Completed.Add(Pair.Key);
			}
		}
		for (FVulkanSyncPoint* SyncPoint : Completed)
		{
			PendingFreeStagingBuffers.Remove(SyncPoint);
		}

		if (bFreeToOS)
		{
			int32 NumFreeBuffers = bImmediately ? FreeStagingBuffers.Num() : NumOriginalFreeBuffers;
			for (int32 Index = NumFreeBuffers - 1; Index >= 0; --Index)
			{
				FFreeEntry& Entry = FreeStagingBuffers[Index];
				if (bImmediately || Entry.FrameNumber + GVulkanNumFramesToWaitForResourceDelete < GFrameNumberRenderThread)
				{
					UsedMemory -= Entry.StagingBuffer->GetSize();
					Entry.StagingBuffer->Destroy();
					delete Entry.StagingBuffer;
					FreeStagingBuffers.RemoveAtSwap(Index, EAllowShrinking::No);
				}
			}
		}
	}

	void FStagingManager::ProcessPendingFree(bool bImmediately, bool bFreeToOS)
	{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
		SCOPE_CYCLE_COUNTER(STAT_VulkanStagingBuffer);
#endif

		FScopeLock Lock(&StagingLock);
		ProcessPendingFreeNoLock(bImmediately, bFreeToOS);
	}

	FDeferredDeletionQueue2::FDeferredDeletionQueue2(FVulkanDevice& InDevice)
		: Device(InDevice)
	{
	}

	FDeferredDeletionQueue2::~FDeferredDeletionQueue2()
	{
		checkf(Entries.Num() == 0, TEXT("There are %d entries remaining in the Deferred Deletion Queue!"), Entries.Num());
	}

	void FDeferredDeletionQueue2::EnqueueGenericResource(EType Type, uint64 Handle)
	{
		FEntry Entry;
		Entry.StructureType = Type;
		Entry.FrameNumber = GVulkanRHIDeletionFrameNumber;
		Entry.Handle = Handle;
		{
			FScopeLock ScopeLock(&CS);

#if (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)
			FEntry* ExistingEntry = Entries.FindByPredicate([&](const FEntry& InEntry)
				{
					return (InEntry.Handle == Entry.Handle) && (InEntry.StructureType == Entry.StructureType);
				});
			checkf(ExistingEntry == nullptr, TEXT("Attempt to double-delete resource, FDeferredDeletionQueue2::EType: %d, Handle: %llu"), (int32)Type, Handle);
#endif

			Entries.Add(Entry);
		}
	}

	void FDeferredDeletionQueue2::EnqueueFunc(TUniqueFunction<void(FVulkanDevice&)>&& Func)
	{
		FEntry Entry;
		Entry.StructureType = EType::Func;
		Entry.FrameNumber = GVulkanRHIDeletionFrameNumber;
		Entry.Func = new TUniqueFunction<void(FVulkanDevice&)>(MoveTemp(Func));
		{
			FScopeLock ScopeLock(&CS);
			Entries.Add(Entry);
		}
	}

	void FDeferredDeletionQueue2::EnqueueDeviceAllocation(FDeviceMemoryAllocation* DeviceMemoryAllocation)
	{
		check(DeviceMemoryAllocation);

		FEntry Entry;
		Entry.StructureType = EType::DeviceMemoryAllocation;
		Entry.FrameNumber = GVulkanRHIDeletionFrameNumber;
		Entry.DeviceMemoryAllocation = DeviceMemoryAllocation;
		{
			FScopeLock ScopeLock(&CS);
			Entries.Add(Entry);
		}
	}

	void FDeferredDeletionQueue2::EnqueueResourceAllocation(FVulkanAllocation& Allocation)
	{
		if (!Allocation.HasAllocation())
		{
			return;
		}
		Allocation.Disown();

		FEntry Entry;
		Entry.StructureType = EType::ResourceAllocation;
		Entry.FrameNumber = GVulkanRHIDeletionFrameNumber;
		Entry.Allocation = Allocation;
		{
			FScopeLock ScopeLock(&CS);
			Entries.Add(Entry);
		}
		check(!Allocation.HasAllocation());
	}


	void FDeferredDeletionQueue2::ReleaseResources(bool bDeleteImmediately)
	{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
		SCOPE_CYCLE_COUNTER(STAT_VulkanDeletionQueue);
#endif

		if (bDeleteImmediately)
		{
			FScopeLock ScopeLock(&CS);
			ReleaseResourcesImmediately(Entries);
			Entries.Reset();
		}
		else
		{
			TArray<FEntry> TempEntries;
			{
				FScopeLock ScopeLock(&CS);
				if (GVulkanNumFramesToWaitForResourceDelete)
				{
					TempEntries.Reserve(Entries.Num()/2);
					for (int32 Index = Entries.Num() - 1; Index >= 0; --Index)
					{
						FEntry& Entry = Entries[Index];
						if (GVulkanRHIDeletionFrameNumber > Entry.FrameNumber + GVulkanNumFramesToWaitForResourceDelete)
						{
							TempEntries.Add(Entry);
							Entries.RemoveAtSwap(Index, EAllowShrinking::No);
						}
					}
				}
				else
				{
					TempEntries = MoveTemp(Entries);
				}
			}

			if (TempEntries.Num())
			{
				FVulkanDynamicRHI::Get().EnqueueEndOfPipeTask([&Device = Device, Array = MoveTemp(TempEntries)]()
				{
					Device.GetDeferredDeletionQueue().ReleaseResourcesImmediately(Array);
				});
			}
		}
	}

	void FDeferredDeletionQueue2::ReleaseResourcesImmediately(const TArray<FEntry>& InEntries)
	{
		VkDevice DeviceHandle = Device.GetHandle();

		// Traverse list backwards so the swap switches to elements already tested
		for (const FEntry& Entry : InEntries)
		{
			switch (Entry.StructureType)
			{
#define VKSWITCH(Type, ...)	case EType::Type: __VA_ARGS__; VulkanRHI::vkDestroy##Type(DeviceHandle, (Vk##Type)Entry.Handle, VULKAN_CPU_ALLOCATOR); break
				VKSWITCH(RenderPass);
				VKSWITCH(Buffer);
				VKSWITCH(BufferView);
				VKSWITCH(Image);
				VKSWITCH(ImageView);
				VKSWITCH(Pipeline, DEC_DWORD_STAT(STAT_VulkanNumPSOs));
				VKSWITCH(PipelineLayout);
				VKSWITCH(Framebuffer);
				VKSWITCH(DescriptorSetLayout);
				VKSWITCH(Sampler);
				VKSWITCH(Semaphore);
				VKSWITCH(ShaderModule);
				VKSWITCH(Event);
#undef VKSWITCH
			case EType::ResourceAllocation:
			{
				FVulkanAllocation Allocation = Entry.Allocation;
				Allocation.Own();
				Device.GetMemoryManager().FreeVulkanAllocation(Allocation, EVulkanFreeFlag_DontDefer);
				break;
			}
			case EType::DeviceMemoryAllocation:
			{
				check(Entry.DeviceMemoryAllocation);
				FDeviceMemoryAllocation* DeviceMemoryAllocation = Entry.DeviceMemoryAllocation;
				Device.GetDeviceMemoryManager().Free(DeviceMemoryAllocation);
				break;
			}
			case EType::AccelerationStructure:
			{
				VulkanDynamicAPI::vkDestroyAccelerationStructureKHR(DeviceHandle, (VkAccelerationStructureKHR)Entry.Handle, VULKAN_CPU_ALLOCATOR);
				break;
			}
			case EType::BindlessHandle:
			{
				check(Device.SupportsBindless());
				const FRHIDescriptorHandle& DescriptorHandle = reinterpret_cast<const FRHIDescriptorHandle&>(Entry.Handle);
				Device.GetBindlessDescriptorManager()->FreeDescriptor(DescriptorHandle);
				break;
			}
			case EType::Func:
			{
				(*Entry.Func)(Device);
				delete Entry.Func;
				break;
			}

			default:
				check(0);
				break;
			}
		}
	}



	FTempBlockAllocator::FTempBlockAllocator(FVulkanDevice& InDevice, uint32 InBlockSize, uint32 InBlockAlignment, VkBufferUsageFlags InBufferUsage)
		: Device(InDevice)
		, BlockSize(InBlockSize)
		, BlockAlignment(InBlockAlignment)
		, BufferUsage(InBufferUsage)
	{
		CurrentBlock = AllocBlock();
	}

	FTempBlockAllocator::~FTempBlockAllocator()
	{
		auto FreeBlock = [](FVulkanDevice& InDevice, FTempMemoryBlock* Block)
		{
			VulkanRHI::vkDestroyBuffer(InDevice.GetHandle(), Block->Buffer, VULKAN_CPU_ALLOCATOR);
			InDevice.GetMemoryManager().FreeVulkanAllocation(Block->Allocation, VulkanRHI::EVulkanFreeFlag_DontDefer);
			delete Block;
		};

		for (FTempMemoryBlock* Block : AvailableBlocks)
		{
			FreeBlock(Device, Block);
		}
		AvailableBlocks.Empty();

		for (FTempMemoryBlock* Block : BusyBlocks)
		{
			FreeBlock(Device, Block);
		}
		BusyBlocks.Empty();

		FreeBlock(Device, CurrentBlock);
		CurrentBlock = nullptr;
	}

	FTempBlockAllocator::FTempMemoryBlock* FTempBlockAllocator::AllocBlock()
	{
		FTempMemoryBlock* NewBlock = new FTempMemoryBlock;

		// Create buffer
		{
			NewBlock->Buffer = Device.CreateBuffer(BlockSize, BufferUsage);
		}

		// Allocate memory
		{
			const VulkanRHI::EVulkanAllocationFlags AllocFlags =
				VulkanRHI::EVulkanAllocationFlags::HostVisible |
				VulkanRHI::EVulkanAllocationFlags::PreferBAR |
				VulkanRHI::EVulkanAllocationFlags::AutoBind |
				VulkanRHI::EVulkanAllocationFlags::Dedicated;
			Device.GetMemoryManager().AllocateBufferMemory(NewBlock->Allocation, NewBlock->Buffer, AllocFlags, TEXT("FTempBlockAllocator"), BlockAlignment);

			// Pull the stat of the generic "buffer other" to put it in the temp block bucket (will get cleaned up soon)
			DEC_DWORD_STAT_BY(STAT_VulkanAllocation_BufferOther, NewBlock->Allocation.Size);
			INC_DWORD_STAT_BY(STAT_VulkanAllocation_TempBlocks, NewBlock->Allocation.Size);

			NewBlock->MappedPointer = (uint8*)NewBlock->Allocation.GetMappedPointer(&Device);
		}

		// Get the device addr if needed
		if (VKHasAllFlags(BufferUsage, VK_BUFFER_USAGE_SHADER_DEVICE_ADDRESS_BIT) && Device.GetOptionalExtensions().HasBufferDeviceAddress)
		{
			VkBufferDeviceAddressInfo BufferInfo;
			ZeroVulkanStruct(BufferInfo, VK_STRUCTURE_TYPE_BUFFER_DEVICE_ADDRESS_INFO);
			BufferInfo.buffer = NewBlock->Buffer;
			NewBlock->BufferAddress = VulkanRHI::vkGetBufferDeviceAddressKHR(Device.GetHandle(), &BufferInfo);
		}

		return NewBlock;
	}

	FTempBlockAllocator::FInternalAlloc FTempBlockAllocator::InternalAlloc(uint32 InSize, FVulkanContextCommon* Context)
	{
		const uint32 AlignedSize = Align(InSize, BlockAlignment);
		checkfSlow(BlockAlignment < BlockSize, TEXT("Requested size of %d (%d aligned) is too large for block size of %d"), InSize, AlignedSize, BlockSize);

		// Parallel scope: allocate in existing block from existing command buffer
		{
			FReadScopeLock ScopeLock(RWLock);

			if (!Context || CurrentBlock->SyncPoints.Contains(Context->GetContextSyncPoint()))
			{
				const uint32 AllocOffset = CurrentBlock->CurrentOffset.fetch_add(AlignedSize);
				if (AllocOffset + InSize < BlockSize)
				{
					if (!Context)
					{
						CurrentBlock->References++;
					}

					return FInternalAlloc{ CurrentBlock, AllocOffset };
				}
			}
		}

		// Locked path (allocate a new block or from a new command buffer)
		{
			FWriteScopeLock ScopeLock(RWLock);

			// Make sure someone else didn't swap the block before it was our turn
			uint32 AllocOffset = CurrentBlock->CurrentOffset.fetch_add(AlignedSize);
			if (AllocOffset + InSize < BlockSize)
			{
				if (Context)
				{
					CurrentBlock->SyncPoints.AddUnique(Context->GetContextSyncPoint());
				}
				else
				{
					CurrentBlock->References++;
				}
				return FInternalAlloc{ CurrentBlock, AllocOffset };
			}

			// Get a new block
			BusyBlocks.Add(CurrentBlock);
			if (AvailableBlocks.Num())
			{
				CurrentBlock = AvailableBlocks.Pop(EAllowShrinking::No);
			}
			else
			{
				CurrentBlock = AllocBlock();
			}

			if (Context)
			{
				CurrentBlock->SyncPoints.AddUnique(Context->GetContextSyncPoint());
			}
			else
			{
				CurrentBlock->References++;
			}

			AllocOffset = CurrentBlock->CurrentOffset.fetch_add(AlignedSize);
			checkSlow(AllocOffset == 0);
			checkSlow(AllocOffset + InSize < BlockSize);
			return FInternalAlloc{ CurrentBlock, AllocOffset };
		}
	}

	uint8* FTempBlockAllocator::Alloc(uint32 InSize, FVulkanContextCommon& Context, VkDescriptorBufferBindingInfoEXT& OutBindingInfo, VkDeviceSize& OutOffset)
	{
		FInternalAlloc Alloc = InternalAlloc(InSize, &Context);

		ZeroVulkanStruct(OutBindingInfo, VK_STRUCTURE_TYPE_DESCRIPTOR_BUFFER_BINDING_INFO_EXT);
		OutBindingInfo.address = Alloc.Block->BufferAddress;
		OutBindingInfo.usage = BufferUsage;
		OutOffset = Alloc.Offset;
		return Alloc.Block->MappedPointer + Alloc.Offset;
	}

	uint8* FTempBlockAllocator::Alloc(uint32 InSize, uint32 InAlignment, FVulkanContextCommon* Context, FVulkanAllocation& OutAllocation, VkDescriptorAddressInfoEXT* OutDescriptorAddressInfo)
	{
		const uint32 AlignedSize = Align(InSize, BlockAlignment);
		checkfSlow(AlignedSize < BlockSize, TEXT("Requested size of %d (%d aligned) is too large for block size of %d"), InSize, AlignedSize, BlockSize);

		FInternalAlloc Alloc = InternalAlloc(InSize, Context);

		OutAllocation.Reference(Alloc.Block->Allocation);
		OutAllocation.VulkanHandle = (uint64)Alloc.Block->Buffer;
		OutAllocation.Size = InSize;
		OutAllocation.Offset += Alloc.Offset;
		OutAllocation.bTempBlockAllocator = (Context == nullptr);

		if (OutDescriptorAddressInfo)
		{
			ZeroVulkanStruct(*OutDescriptorAddressInfo, VK_STRUCTURE_TYPE_DESCRIPTOR_ADDRESS_INFO_EXT);
			OutDescriptorAddressInfo->address = Alloc.Block->BufferAddress + Alloc.Offset;
			OutDescriptorAddressInfo->range = InSize;
		}

		return Alloc.Block->MappedPointer + Alloc.Offset;
	}

	uint8* FTempBlockAllocator::Alloc(uint32 InSize, FVulkanContextCommon& Context, VkStridedDeviceAddressRegionKHR* OutStridedDeviceAddressRegion)
	{
		const uint32 AlignedSize = Align(InSize, BlockAlignment);
		checkfSlow(AlignedSize < BlockSize, TEXT("Requested size of %d (%d aligned) is too large for block size of %d"), InSize, AlignedSize, BlockSize);
		checkSlow(OutStridedDeviceAddressRegion);

		FInternalAlloc Alloc = InternalAlloc(InSize, &Context);
		OutStridedDeviceAddressRegion->deviceAddress = Alloc.Block->BufferAddress + Alloc.Offset;
		OutStridedDeviceAddressRegion->size = InSize;
		OutStridedDeviceAddressRegion->stride = OutStridedDeviceAddressRegion->size;
		return Alloc.Block->MappedPointer + Alloc.Offset;
	}

	void FTempBlockAllocator::FreeReference(const FVulkanAllocation& InAllocation)
	{
		FReadScopeLock ScopeLock(RWLock);

		if ((InAllocation.VulkanHandle == (uint64)CurrentBlock->Buffer) &&
			(InAllocation.AllocatorIndex == CurrentBlock->Allocation.AllocatorIndex) &&
			(InAllocation.AllocationIndex == CurrentBlock->Allocation.AllocationIndex))
		{
			CurrentBlock->References--;
			return;
		}

		for (int32 Index = 0; Index < BusyBlocks.Num(); ++Index)
		{
			FTempMemoryBlock* Block = BusyBlocks[Index];
			FVulkanAllocation& BlockAllocation = Block->Allocation;
			if ((InAllocation.VulkanHandle == (uint64)Block->Buffer) &&
				(InAllocation.AllocatorIndex == BlockAllocation.AllocatorIndex) &&
				(InAllocation.AllocationIndex == BlockAllocation.AllocationIndex))
			{
				check(Block->References > 0);
				Block->References--;
				return;
			}
		}

		checkf(false, TEXT("Freed allocation's block was not found in the temp allocator!"));
	}

	void FTempBlockAllocator::UpdateBlocks()
	{
		FWriteScopeLock ScopeLock(RWLock);

		for (int32 Index = BusyBlocks.Num() - 1; Index >= 0; --Index)
		{
			FTempMemoryBlock* Block = BusyBlocks[Index];

			bool BlockReady = (Block->References == 0);

			if (BlockReady)
			{
				for (FVulkanSyncPointRef& SyncPoint : Block->SyncPoints)
				{
					if (!SyncPoint->IsComplete())
					{
						BlockReady = false;
						break;
					}
				}
			}

			if (BlockReady)
			{
				Block->SyncPoints.Empty();
				Block->CurrentOffset = 0;
				BusyBlocks.RemoveAtSwap(Index, EAllowShrinking::No);
				AvailableBlocks.Add(Block);
			}
		}
	}
}



#if VULKAN_CUSTOM_MEMORY_MANAGER_ENABLED
namespace VulkanRHI
{
	VkAllocationCallbacks GAllocationCallbacks;
}
static FCriticalSection GMemMgrCS;
static FVulkanCustomMemManager GVulkanInstrumentedMemMgr;
//VkAllocationCallbacks GDescriptorAllocationCallbacks;


FVulkanCustomMemManager::FVulkanCustomMemManager()
{
	VulkanRHI::GAllocationCallbacks.pUserData = nullptr;
	VulkanRHI::GAllocationCallbacks.pfnAllocation = (PFN_vkAllocationFunction)&FVulkanCustomMemManager::Alloc;
	VulkanRHI::GAllocationCallbacks.pfnReallocation = (PFN_vkReallocationFunction)&FVulkanCustomMemManager::Realloc;
	VulkanRHI::GAllocationCallbacks.pfnFree = (PFN_vkFreeFunction)&FVulkanCustomMemManager::Free;
	VulkanRHI::GAllocationCallbacks.pfnInternalAllocation = (PFN_vkInternalAllocationNotification)&FVulkanCustomMemManager::InternalAllocationNotification;
	VulkanRHI::GAllocationCallbacks.pfnInternalFree = (PFN_vkInternalFreeNotification)&FVulkanCustomMemManager::InternalFreeNotification;
}

inline FVulkanCustomMemManager::FType& FVulkanCustomMemManager::GetType(void* UserData, VkSystemAllocationScope AllocScope)
{
	check(AllocScope < VK_SYSTEM_ALLOCATION_SCOPE_RANGE_SIZE);
	return GVulkanInstrumentedMemMgr.Types[AllocScope];
}

void* FVulkanCustomMemManager::Alloc(void* UserData, size_t Size, size_t Alignment, VkSystemAllocationScope AllocScope)
{
	check(AllocScope < VK_SYSTEM_ALLOCATION_SCOPE_RANGE_SIZE);
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanDriverMemoryCPU);
	LLM_PLATFORM_SCOPE(ELLMTag::GraphicsPlatform);

	FScopeLock Lock(&GMemMgrCS);
	void* Data = FMemory::Malloc(Size, Alignment);
	FType& Type = GetType(UserData, AllocScope);
	Type.MaxAllocSize = FMath::Max(Type.MaxAllocSize, Size);
	Type.UsedMemory += Size;
	Type.Allocs.Add(Data, Size);
	return Data;
}

void FVulkanCustomMemManager::Free(void* UserData, void* Mem)
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanDriverMemoryCPU);
	LLM_PLATFORM_SCOPE(ELLMTag::GraphicsPlatform);

	FScopeLock Lock(&GMemMgrCS);
	FMemory::Free(Mem);
	for (int32 Index = 0; Index < GVulkanInstrumentedMemMgr.Types.Num(); ++Index)
	{
		FType& Type = GVulkanInstrumentedMemMgr.Types[Index];
		size_t* Found = Type.Allocs.Find(Mem);
		if (Found)
		{
			Type.UsedMemory -= *Found;
			break;
		}
	}
}

void* FVulkanCustomMemManager::Realloc(void* UserData, void* Original, size_t Size, size_t Alignment, VkSystemAllocationScope AllocScope)
{
	check(AllocScope < VK_SYSTEM_ALLOCATION_SCOPE_RANGE_SIZE);
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanDriverMemoryCPU);
	LLM_PLATFORM_SCOPE(ELLMTag::GraphicsPlatform);

	FScopeLock Lock(&GMemMgrCS);
	void* Data = FMemory::Realloc(Original, Size, Alignment);
	FType& Type = GetType(UserData, AllocScope);
	size_t OldSize = Original ? Type.Allocs.FindAndRemoveChecked(Original) : 0;
	Type.UsedMemory -= OldSize;
	Type.Allocs.Add(Data, Size);
	Type.UsedMemory += Size;
	Type.MaxAllocSize = FMath::Max(Type.MaxAllocSize, Size);
	return Data;
}

void FVulkanCustomMemManager::InternalAllocationNotification(void* UserData, size_t Size, VkInternalAllocationType AllocationType, VkSystemAllocationScope AllocScope)
{
	check(AllocScope < VK_SYSTEM_ALLOCATION_SCOPE_RANGE_SIZE);
}

void FVulkanCustomMemManager::InternalFreeNotification(void* UserData, size_t Size, VkInternalAllocationType AllocationType, VkSystemAllocationScope AllocScope)
{
	check(AllocScope < VK_SYSTEM_ALLOCATION_SCOPE_RANGE_SIZE);
}
#endif


VkResult VulkanRHI::FDeviceMemoryManager::GetMemoryTypeFromProperties(uint32 TypeBits, VkMemoryPropertyFlags Properties, uint32* OutTypeIndex)
{
	//#todo-rco: Might need to revisit based on https://gitlab.khronos.org/vulkan/vulkan/merge_requests/1165
	// Search memtypes to find first index with those properties
	for (uint32 i = 0; i < MemoryProperties.memoryTypeCount && TypeBits; i++)
	{
		if ((TypeBits & 1) == 1)
		{
			// Type is available, does it match user properties?
			if ((MemoryProperties.memoryTypes[i].propertyFlags & Properties) == Properties)
			{
				*OutTypeIndex = i;
				return VK_SUCCESS;
			}
		}
		TypeBits >>= 1;
	}

	// No memory types matched, return failure
	return VK_ERROR_FEATURE_NOT_PRESENT;
}

VkResult VulkanRHI::FDeviceMemoryManager::GetMemoryTypeFromPropertiesExcluding(uint32 TypeBits, VkMemoryPropertyFlags Properties, uint32 ExcludeTypeIndex, uint32* OutTypeIndex)
{
	// Search memtypes to find first index with those properties
	for (uint32 i = 0; i < MemoryProperties.memoryTypeCount && TypeBits; i++)
	{
		if ((TypeBits & 1) == 1)
		{
			// Type is available, does it match user properties?
			if ((MemoryProperties.memoryTypes[i].propertyFlags & Properties) == Properties && ExcludeTypeIndex != i)
			{
				*OutTypeIndex = i;
				return VK_SUCCESS;
			}
		}
		TypeBits >>= 1;
	}

	// No memory types matched, return failure
	return VK_ERROR_FEATURE_NOT_PRESENT;
}

const VkPhysicalDeviceMemoryProperties& VulkanRHI::FDeviceMemoryManager::GetMemoryProperties() const
{
	return MemoryProperties;
}

// Patch external allocations into the engine's allocators
class FVulkanMemoryAllocator : public IVulkanMemoryAllocator
{
public:
	FVulkanMemoryAllocator(FVulkanDevice& InDevice) : Device(InDevice), MemoryManager(InDevice.GetMemoryManager()) {}
	virtual ~FVulkanMemoryAllocator() {}

	virtual VkResult AllocateImageMemory(VkImage Image, VkMemoryPropertyFlags Flags, bool bForceDedicatedAllocation, AllocHandle* OutAllocHandle) override final
	{
		const VulkanRHI::EVulkanAllocationMetaType MetaType = VulkanRHI::EVulkanAllocationMetaImageOther;

		// AllocateDedicatedImageMemory will let the driver decide if allocation should be dedicated
		VulkanRHI::FVulkanAllocation* Allocation = new VulkanRHI::FVulkanAllocation();
		if (Device.GetMemoryManager().AllocateDedicatedImageMemory(*Allocation, nullptr, Image, Flags, MetaType, false, bForceDedicatedAllocation, __FILE__, __LINE__))
		{
			(*OutAllocHandle) = Allocation;
			return VK_SUCCESS;
		}
		else
		{
			delete Allocation;
			(*OutAllocHandle) = nullptr;

			const bool bIsHostMemory = VKHasAnyFlags(Flags, VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT);
			return bIsHostMemory ? VK_ERROR_OUT_OF_HOST_MEMORY : VK_ERROR_OUT_OF_DEVICE_MEMORY;
		}
	}

	virtual VkResult AllocateBufferMemory(VkBuffer Buffer, VkMemoryPropertyFlags Flags, bool bForceDedicatedAllocation, AllocHandle* OutAllocHandle) override final
	{
		VulkanRHI::EVulkanAllocationFlags UEAllocFlags = VulkanRHI::EVulkanAllocationFlags::None;

		const bool bIsHostMemory = VKHasAnyFlags(Flags, VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT);
		if (bIsHostMemory)
		{
			UEAllocFlags |= VulkanRHI::EVulkanAllocationFlags::HostVisible;
		}
		if (VKHasAnyFlags(Flags, VK_MEMORY_PROPERTY_HOST_CACHED_BIT))
		{
			UEAllocFlags |= VulkanRHI::EVulkanAllocationFlags::HostCached;
		}
		if (VKHasAnyFlags(Flags, VK_MEMORY_PROPERTY_LAZILY_ALLOCATED_BIT))
		{
			UEAllocFlags |= VulkanRHI::EVulkanAllocationFlags::Memoryless;
		}
		if (VKHasAnyFlags(Flags, VK_MEMORY_PROPERTY_PROTECTED_BIT))
		{
			UEAllocFlags |= VulkanRHI::EVulkanAllocationFlags::Protected;
		}
		if (bForceDedicatedAllocation)
		{
			UEAllocFlags |= VulkanRHI::EVulkanAllocationFlags::Dedicated;
		}

		VulkanRHI::FVulkanAllocation* Allocation = new VulkanRHI::FVulkanAllocation();
		if (MemoryManager.AllocateBufferMemory(*Allocation, Buffer, UEAllocFlags, TEXT("External")))
		{
			(*OutAllocHandle) = Allocation;
			return VK_SUCCESS;
		}
		else
		{
			delete Allocation;
			(*OutAllocHandle) = nullptr;
			return bIsHostMemory ? VK_ERROR_OUT_OF_HOST_MEMORY : VK_ERROR_OUT_OF_DEVICE_MEMORY;
		}
	}

	virtual void FreeMemory(const AllocHandle& AllocHandle) override final
	{
		VulkanRHI::FVulkanAllocation* Allocation = reinterpret_cast<VulkanRHI::FVulkanAllocation*>(AllocHandle);
		Allocation->Free(Device);
		delete Allocation;
	}

	virtual FVulkanRHIAllocationInfo GetAllocInfo(const AllocHandle& AllocHandle) override final
	{
		VulkanRHI::FVulkanAllocation* Allocation = reinterpret_cast<VulkanRHI::FVulkanAllocation*>(AllocHandle);
		FVulkanRHIAllocationInfo Info;
		Info.Handle = Allocation->GetDeviceMemoryHandle(&Device);
		Info.Offset = Allocation->Offset;
		Info.Size = Allocation->Size;
		Info.Flags = Allocation->GetMemoryPropertyFlags(&Device);
		return Info;
	}

	virtual void* MapMemory(const AllocHandle& AllocHandle) override final
	{
		VulkanRHI::FVulkanAllocation* Allocation = reinterpret_cast<VulkanRHI::FVulkanAllocation*>(AllocHandle);
		return Allocation->GetMappedPointer(&Device);
	}

	virtual void UnmapMemory(const AllocHandle& AllocHandle) override final
	{
		// Not needed for current allocator since all host visible blocks are always mapped
		// Keep the hooks around in case future allocators don't behave the same way
	}

	virtual void FlushMemory(const AllocHandle& AllocHandle, VkDeviceSize Offset, VkDeviceSize Size) override final
	{
		VulkanRHI::FVulkanAllocation* Allocation = reinterpret_cast<VulkanRHI::FVulkanAllocation*>(AllocHandle);
		if (Size == VK_WHOLE_SIZE)
		{
			check(Offset < Allocation->Size);
			Size = Allocation->Size - Offset;
		}
		else
		{
			checkf(Offset + Size <= Allocation->Size, TEXT("Flushing memory beyond the allocation size."));
		}
		VulkanRHI::FVulkanSubresourceAllocator* Allocator = Allocation->GetSubresourceAllocator(&Device);
		Offset += Allocation->Offset;
		Allocator->Flush(Offset, Size);
	}

	virtual void InvalidateMemory(const AllocHandle& AllocHandle, VkDeviceSize Offset, VkDeviceSize Size) override final
	{
		VulkanRHI::FVulkanAllocation* Allocation = reinterpret_cast<VulkanRHI::FVulkanAllocation*>(AllocHandle);
		if (Size == VK_WHOLE_SIZE)
		{
			check(Offset < Allocation->Size);
			Size = Allocation->Size - Offset;
		}
		else
		{
			checkf(Offset + Size <= Allocation->Size, TEXT("Invalidating memory beyond the allocation size."));
		}
		VulkanRHI::FVulkanSubresourceAllocator* Allocator = Allocation->GetSubresourceAllocator(&Device);
		Offset += Allocation->Offset;
		Allocator->Invalidate(Offset, Size);
	}

	virtual FRHIMemoryStats GetMemoryStats() override final
	{
		return Device.GetDeviceMemoryManager().GetRHIMemoryStats();
	}

protected:

	FVulkanDevice& Device;
	VulkanRHI::FMemoryManager& MemoryManager;
};

IVulkanMemoryAllocator* FVulkanDynamicRHI::RHIGetVulkanMemoryAllocator()
{
	return new FVulkanMemoryAllocator(*Device);
}
