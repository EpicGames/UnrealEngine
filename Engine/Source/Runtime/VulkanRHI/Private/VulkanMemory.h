// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanMemory.h: Vulkan Memory RHI definitions.
=============================================================================*/

#pragma once 

#include "Containers/Map.h"
#include "Misc/ScopeRWLock.h"
#include "Templates/RefCounting.h"
#include "VulkanConfiguration.h"
#include "VulkanThirdParty.h"

#define UE_API VULKANRHI_API

//enable to store FILE/LINE, and optionally a stacktrace via r.vulkan.backtrace
#if !UE_BUILD_SHIPPING
#define VULKAN_MEMORY_TRACK 1
#else
#define VULKAN_MEMORY_TRACK 0
#endif


#define VULKAN_MEMORY_LOW_PRIORITY 0.f
#define VULKAN_MEMORY_MEDIUM_PRIORITY 0.5f
#define VULKAN_MEMORY_HIGHER_PRIORITY 0.75f
#define VULKAN_MEMORY_HIGHEST_PRIORITY 1.f

class FVulkanContextCommon;
class FVulkanCommandListContext;
class FVulkanDevice;
class FVulkanQueue;
class FVulkanCommandBuffer;
class FVulkanTexture;

struct FVulkanContextArray;

class FVulkanSyncPoint;
using FVulkanSyncPointRef = TRefCountPtr<FVulkanSyncPoint>;

#if !VULKAN_OBJECT_TRACKING
#define VULKAN_TRACK_OBJECT_CREATE(Type, Ptr) do{}while(0)
#define VULKAN_TRACK_OBJECT_DELETE(Type, Ptr) do{}while(0)
#else
#define VULKAN_TRACK_OBJECT_CREATE(Type, Ptr) do{TVulkanTrackBase<Type>::Add(Ptr);}while(0)
#define VULKAN_TRACK_OBJECT_DELETE(Type, Ptr) do{TVulkanTrackBase<Type>::Remove(Ptr);}while(0)

template<typename Type>
class TVulkanTrackBase
{
public:

	static FCriticalSection Lock;
	static TSet<Type*> Objects;
	template<typename Callback>
	static uint32 CollectAll(Callback CB)
	{
		uint32 Count = 0;
		FScopeLock L(&Lock);
		TArray<Type*> Temp;
		for(Type* Object : Objects)
		{
			Object->DumpMemory(CB);
			Count++;
		}
		return Count;
	}
	static void Add(Type* Object)
	{
		FScopeLock L(&Lock);
		Objects.Add(Object);

	}
	static void Remove(Type* Object)
	{
		FScopeLock L(&Lock);
		Objects.Remove(Object);
	}
};

template<typename Type>
FCriticalSection TVulkanTrackBase<Type>::Lock;
template<typename Type>
TSet<Type*> TVulkanTrackBase<Type>::Objects;
#endif

namespace VulkanRHI
{
	class FFenceManager;
	class FDeviceMemoryManager;					// Manager of low level vulkan memory allocations
	class FDeviceMemoryAllocation;				// A single low level allocation
	class FVulkanSubresourceAllocator;			// handles suballocation of one allocation into many
	class FVulkanResourceHeap;					// has multiple suballocators. On resource heap per memory type
	class FMemoryManager;						// entry point for everything. Contains all heaps. has a small allocator for small stuff.
	struct FRange;								// Helper class for suballocation (this is effectively the allocator implementation)
	class FVulkanAllocation;					// External handle to any allocation handled by FMemoryManager
	struct FVulkanAllocationInternal;			// Internal representation mirroring vulkan allocation
	struct FResourceHeapStats;
}


class FVulkanEvictable
{
public:
	virtual bool CanMove() const { return false; }
	virtual bool CanEvict() const { return false; }
	virtual void Evict(FVulkanDevice& Device, const FVulkanContextArray& Contexts) { checkNoEntry(); }
	virtual void Move(FVulkanDevice& Device, const FVulkanContextArray& Contexts, VulkanRHI::FVulkanAllocation& Allocation) { checkNoEntry(); }
	virtual FVulkanTexture* GetEvictableTexture() { return nullptr; }
};


struct FVulkanTrackInfo
{
	FVulkanTrackInfo();
	void* Data;
	int32 SizeOrLine; //negative indicates a stack trace, Positive line number
};

enum class EDelayAcquireImageType
{
	None,			// acquire next image on frame start
	DelayAcquire,	// acquire next image just before presenting, rendering is done to intermediate image which is copied to real backbuffer
};

extern EDelayAcquireImageType GVulkanDelayAcquireImage;

extern int32 GVulkanUseBufferBinning;

namespace VulkanRHI
{
	enum EVulkanAllocationType : uint8 {
		EVulkanAllocationEmpty,
		EVulkanAllocationPooledBuffer,
		EVulkanAllocationBuffer,
		EVulkanAllocationImage,
		EVulkanAllocationImageDedicated,
		EVulkanAllocationSize, // keep last
	};

	enum EVulkanAllocationMetaType : uint8
	{
		EVulkanAllocationMetaUnknown,
		EVulkanAllocationMetaUniformBuffer,
		EVulkanAllocationMetaMultiBuffer,
		EVulkanAllocationMetaRingBuffer,
		EVulkanAllocationMetaFrameTempBuffer,
		EVulkanAllocationMetaImageRenderTarget,
		EVulkanAllocationMetaImageOther,
		EVulkanAllocationMetaBufferUAV,
		EVulkanAllocationMetaBufferStaging,
		EVulkanAllocationMetaBufferOther,
		EVulkanAllocationMetaSize, // keep last
	};
	enum ELegacyVulkanAllocationFlags
	{
		VulkanAllocationFlagsMapped = 0x1,
		VulkanAllocationFlagsCanEvict= 0x2,
	};

	enum class EType
	{
		Image,
		Buffer,
	};


	// An Allocation off a Device Heap. Lowest level of allocations and bounded by VkPhysicalDeviceLimits::maxMemoryAllocationCount.
	class FDeviceMemoryAllocation
	{
	public:
		FDeviceMemoryAllocation()
			: Size(0)
			, DeviceHandle(VK_NULL_HANDLE)
			, Handle(VK_NULL_HANDLE)
			, MappedPointer(nullptr)
			, MemoryTypeIndex(0)
			, bCanBeMapped(0)
			, bIsCoherent(0)
			, bIsCached(0)
			, bFreedBySystem(false)
			, bDedicatedMemory(0)
		{
		}

		void* Map(VkDeviceSize Size, VkDeviceSize Offset);
		void Unmap();

		inline bool CanBeMapped() const
		{
			return bCanBeMapped != 0;
		}

		inline bool IsMapped() const
		{
			return !!MappedPointer;
		}

		inline void* GetMappedPointer()
		{
			check(IsMapped());
			return MappedPointer;
		}

		inline bool IsCoherent() const
		{
			return bIsCoherent != 0;
		}

		void FlushMappedMemory(VkDeviceSize InOffset, VkDeviceSize InSize);
		void InvalidateMappedMemory(VkDeviceSize InOffset, VkDeviceSize InSize);

		inline VkDeviceMemory GetHandle() const
		{
			return Handle;
		}

		inline VkDeviceSize GetSize() const
		{
			return Size;
		}

		inline uint32 GetMemoryTypeIndex() const
		{
			return MemoryTypeIndex;
		}

	protected:
		VkDeviceSize Size;
		VkDevice DeviceHandle;
		VkDeviceMemory Handle;
		void* MappedPointer;
		uint32 MemoryTypeIndex : 8;
		uint32 bCanBeMapped : 1;
		uint32 bIsCoherent : 1;
		uint32 bIsCached : 1;
		uint32 bFreedBySystem : 1;
		uint32 bDedicatedMemory : 1;
		uint32 : 0;


#if VULKAN_MEMORY_TRACK
		FVulkanTrackInfo Track;
#endif
		~FDeviceMemoryAllocation();

		friend class FDeviceMemoryManager;
	};

	struct FDeviceMemoryBlockKey
	{
		uint32 MemoryTypeIndex;
		VkDeviceSize BlockSize;

		bool operator==(const FDeviceMemoryBlockKey& Other) const
		{
			return MemoryTypeIndex == Other.MemoryTypeIndex &&  BlockSize == Other.BlockSize;
		}
	};

	FORCEINLINE uint32 GetTypeHash(const FDeviceMemoryBlockKey& BlockKey)
	{
		return HashCombine(FCrc::TypeCrc32(BlockKey.MemoryTypeIndex), FCrc::TypeCrc32(BlockKey.BlockSize));
	}

	struct FDeviceMemoryBlock
	{
		FDeviceMemoryBlockKey Key;
		struct FFreeBlock
		{
			FDeviceMemoryAllocation* Allocation;
			uint32 FrameFreed;
		};
		TArray<FFreeBlock> Allocations;

	};



	// Manager of Device Heap Allocations. Calling Alloc/Free is expensive!
	class FDeviceMemoryManager
	{
		TMap<FDeviceMemoryBlockKey, FDeviceMemoryBlock> Allocations;
		FRHIMemoryStats MemoryStats;
	public:
		FDeviceMemoryManager();
		~FDeviceMemoryManager();

		void Init(FVulkanDevice* InDevice);

		void Deinit();

		uint32 GetEvictedMemoryProperties();

		inline bool HasUnifiedMemory() const
		{
			return bHasUnifiedMemory;
		}

		inline bool SupportsMemoryless() const
		{
			return bSupportsMemoryless;
		}

		inline uint32 GetNumMemoryTypes() const
		{
			return MemoryProperties.memoryTypeCount;
		}

		bool SupportsMemoryType(VkMemoryPropertyFlags Properties) const;
		void GetPrimaryHeapStatus(uint64& OutAllocated, uint64& OutLimit);

		VkResult GetMemoryTypeFromProperties(uint32 TypeBits, VkMemoryPropertyFlags Properties, uint32* OutTypeIndex);
		VkResult GetMemoryTypeFromPropertiesExcluding(uint32 TypeBits, VkMemoryPropertyFlags Properties, uint32 ExcludeTypeIndex, uint32* OutTypeIndex);
		const VkPhysicalDeviceMemoryProperties& GetMemoryProperties() const;


		// bCanFail means an allocation failing is not a fatal error, just returns nullptr
		FDeviceMemoryAllocation* Alloc(bool bCanFail, VkDeviceSize AllocationSize, uint32 MemoryTypeIndex, void* DedicatedAllocateInfo, float Priority, bool bExternal, const char* File, uint32 Line);
		FDeviceMemoryAllocation* Alloc(bool bCanFail, VkDeviceSize AllocationSize, uint32 MemoryTypeBits, VkMemoryPropertyFlags MemoryPropertyFlags, void* DedicatedAllocateInfo, float Priority, bool bExternal, const char* File, uint32 Line);

		// Sets the Allocation to nullptr
		void Free(FDeviceMemoryAllocation*& Allocation);

		uint64 GetTotalMemory(bool bGPU) const;
		VkDeviceSize GetBaseHeapSize(uint32 HeapIndex) const;
		uint32 GetHeapIndex(uint32 MemoryTypeIndex);

		void UpdateMemoryProperties();
		FRHIMemoryStats GetRHIMemoryStats() const { return MemoryStats; }

	protected:
		void FreeInternal(FDeviceMemoryAllocation* Allocation);
		void TrimMemory(bool bFullTrim);
		friend class FMemoryManager;
		void GetMemoryDump(TArray<FResourceHeapStats>& OutDeviceHeapsStats);
		void DumpMemory();
		void PrintMemInfo() const;

		double MemoryUpdateTime;
		VkPhysicalDeviceMemoryBudgetPropertiesEXT MemoryBudget;
		VkPhysicalDeviceMemoryProperties MemoryProperties;
		VkDevice DeviceHandle;
		FVulkanDevice* Device;
		uint32 NumAllocations;
		uint32 PeakNumAllocations;
		bool bHasUnifiedMemory;
		bool bSupportsMemoryless;

		struct FHeapInfo
		{
			VkDeviceSize UsedSize;
			VkDeviceSize PeakSize;
			TArray<FDeviceMemoryAllocation*> Allocations;

			FHeapInfo() :
				UsedSize(0),
				PeakSize(0)
			{
			}
		};

		TArray<FHeapInfo> HeapInfos;

		int32 PrimaryHeapIndex; // memory usage of this heap will decide when to evict.

		FCriticalSection DeviceMemLock;
	};

	struct FRange
	{
		uint32 Offset;
		uint32 Size;

		inline bool operator<(const FRange& In) const
		{
			return Offset < In.Offset;
		}

		static void JoinConsecutiveRanges(TArray<FRange>& Ranges);

		/** Tries to insert the item so it has index ProposedIndex, but may end up merging it with neighbors */
		static int32 InsertAndTryToMerge(TArray<FRange>& Ranges, const FRange& Item, int32 ProposedIndex);

		/** Tries to append the item to the end but may end up merging it with the neighbor */
		static int32 AppendAndTryToMerge(TArray<FRange>& Ranges, const FRange& Item);

		/** Attempts to allocate from an entry - can remove it if it was used up*/
		static void AllocateFromEntry(TArray<FRange>& Ranges, int32 Index, uint32 SizeToAllocate);

		/** Sanity checks an array of ranges */
		static void SanityCheck(TArray<FRange>& Ranges);

		/** Adds to the array while maintaing the sort. */
		static int32 Add(TArray<FRange>& Ranges, const FRange& Item);
	};

	struct FVulkanPageSizeBucket
	{
		uint64 AllocationMax;
		uint32 PageSize;
		enum
		{
			BUCKET_MASK_IMAGE=0x1,
			BUCKET_MASK_BUFFER=0x2,
		};
		uint32 BucketMask;
	};


	// Holds a reference to -any- vulkan gpu allocation
	// !Intentionally not reference counted
	// !User must call Free exactly once
	class FVulkanAllocation
	{
	public:
		FVulkanAllocation();
		~FVulkanAllocation();
		void Init(EVulkanAllocationType Type, EVulkanAllocationMetaType MetaType, uint64 Handle, uint32 InSize, uint32 AlignedOffset, uint32 AllocatorIndex, uint32 AllocationIndex, uint32 BufferId);
		void Free(FVulkanDevice& Device);
		void Swap(FVulkanAllocation& Other);
		void Reference(const FVulkanAllocation& Other); //point to other, but don't take ownership
		bool HasAllocation() const;

		void Disown(); //disown & own should be used if ownership is transferred.
		void Own();
		bool IsValid() const;
		
		uint64 VulkanHandle = 0;
		uint32 HandleId = 0;
		uint32 Size = 0;
		uint32 Offset = 0;
		uint32 AllocationIndex = 0;
		uint16 AllocatorIndex = 0;
		EVulkanAllocationMetaType MetaType = EVulkanAllocationMetaUnknown;
		uint8 Type : 5;
		uint8 bHasOwnership : 1;
		uint8 bTransient : 1;
		uint8 bTempBlockAllocator : 1;

		static_assert(EVulkanAllocationSize < 32, "Not enough bits to hold EVulkanAllocationType");
		inline EVulkanAllocationType GetType() const
		{
			return (EVulkanAllocationType)Type;
		}
		inline void SetType(EVulkanAllocationType InType)
		{
			Type = (uint8)InType;
		}

		//helper functions
		void* GetMappedPointer(FVulkanDevice* Device);
		void FlushMappedMemory(FVulkanDevice* Device);
		void InvalidateMappedMemory(FVulkanDevice* Device);
		VkBuffer GetBufferHandle() const;
		uint32 GetBufferAlignment(FVulkanDevice* Device) const;
		VkDeviceMemory GetDeviceMemoryHandle(FVulkanDevice* Device) const;
		FVulkanSubresourceAllocator* GetSubresourceAllocator(FVulkanDevice* Device) const;
		VkMemoryPropertyFlags GetMemoryPropertyFlags(FVulkanDevice* Device) const;
		void BindBuffer(FVulkanDevice* Device, VkBuffer Buffer);
		void BindImage(FVulkanDevice* Device, VkImage Image);
	};



	struct FVulkanAllocationInternal
	{
		enum
		{
			EUNUSED,
			EALLOCATED,
			EFREED,
			EFREEPENDING,
			EFREEDISCARDED, // if a defrag happens with a pending free, its put into this state, so we can ignore the deferred delete once it happens.
		};

		int State = EUNUSED;
		void Init(const FVulkanAllocation& Alloc, FVulkanEvictable* InAllocationOwner, uint32 AllocationOffset, uint32 AllocationSize, uint32 Alignment);

		EVulkanAllocationType Type = EVulkanAllocationEmpty;
		EVulkanAllocationMetaType MetaType = EVulkanAllocationMetaUnknown;

		uint32 Size = 0;
		uint32 AllocationSize = 0;
		uint32 AllocationOffset = 0;
		FVulkanEvictable* AllocationOwner = 0;
		uint32 Alignment = 0;

		int32 NextFree = -1;

#if VULKAN_MEMORY_TRACK
		FVulkanTrackInfo Track;
#endif
#if VULKAN_USE_LLM
		uint64 LLMTrackerID;
		void SetLLMTrackerID(uint64 InTrackerID) { LLMTrackerID = InTrackerID; }
		uint64 GetLLMTrackerID() { return LLMTrackerID; }
#endif
	};



	class FVulkanSubresourceAllocator
	{
		friend class FMemoryManager;
		friend class FVulkanResourceHeap;
	public:
		FVulkanSubresourceAllocator(EVulkanAllocationType InType, FMemoryManager* InOwner, uint8 InSubResourceAllocatorFlags, FDeviceMemoryAllocation* InDeviceMemoryAllocation,
			uint32 InMemoryTypeIndex, VkMemoryPropertyFlags InMemoryPropertyFlags,
			uint32 InAlignment, VkBuffer InBuffer, uint32 InBufferSize, uint32 InBufferId, VkBufferUsageFlags InBufferUsageFlags, int32 InPoolSizeIndex);


		FVulkanSubresourceAllocator(EVulkanAllocationType InType, FMemoryManager* InOwner, uint8 InSubResourceAllocatorFlags, FDeviceMemoryAllocation* InDeviceMemoryAllocation,
			uint32 InMemoryTypeIndex, uint32 BufferId = 0xffffffff);

		~FVulkanSubresourceAllocator();

		void Destroy(FVulkanDevice* Device);
		bool TryAllocate2(FVulkanAllocation& OutAllocation, FVulkanEvictable* Owner, uint32 InSize, uint32 InAlignment, EVulkanAllocationMetaType InMetaType, const char* File, uint32 Line);
		void Free(FVulkanAllocation& Allocation);

		uint32 GetAlignment() const
		{
			return Alignment;
		}

		void* GetMappedPointer()
		{
			return MemoryAllocation->GetMappedPointer();
		}

		void Flush(VkDeviceSize Offset, VkDeviceSize AllocationSize);
		void Invalidate(VkDeviceSize Offset, VkDeviceSize AllocationSize);
		uint32 GetMaxSize() const { return MaxSize; }
		uint32 GetUsedSize() const { return UsedSize; }

		uint32 GetMemoryTypeIndex() const
		{
			return MemoryTypeIndex;
		}

		uint32 GetHandleId() const
		{
			return BufferId;
		}
		FDeviceMemoryAllocation* GetMemoryAllocation()
		{
			return MemoryAllocation;
		}
		EVulkanAllocationType GetType(){ return Type; }

		TArrayView<uint32> GetMemoryUsed();
		uint32 GetNumSubAllocations();

		uint8 GetSubresourceAllocatorFlags() const
		{
			return SubresourceAllocatorFlags;
		}
		uint32 GetId() const
		{
			return BufferId;
		}
		bool GetIsDefragging() const
		{
			return bIsDefragging;
		}

	protected:
		void SetIsDefragging(bool bInIsDefragging){ bIsDefragging = bInIsDefragging; }
		void DumpFullHeap();
		int32 DefragTick(FVulkanDevice& Device, const FVulkanContextArray& Contexts, FVulkanResourceHeap* Heap, uint32 Count);
		bool CanDefrag();
		uint64 EvictToHost(FVulkanDevice& Device, const FVulkanContextArray& Contexts);
		void SetFreePending(FVulkanAllocation& Allocation);
		void FreeInternalData(int32 Index);
		int32 AllocateInternalData();

		uint32 MemoryUsed[EVulkanAllocationMetaSize];

		EVulkanAllocationType Type;
		FMemoryManager* Owner;
		const uint32 MemoryTypeIndex;
		VkMemoryPropertyFlags MemoryPropertyFlags;
		FDeviceMemoryAllocation* MemoryAllocation;
		uint32 MaxSize;
		uint32 Alignment;
		uint32 FrameFreed;
		uint32 LastDefragFrame = 0;
		int64 UsedSize;
		VkBufferUsageFlags BufferUsageFlags;
		VkBuffer Buffer;
		uint32 BufferId;
		int32 PoolSizeIndex;
		uint32 AllocatorIndex;
		uint8 SubresourceAllocatorFlags;
		uint8 BucketId;
		bool bIsEvicting = false;
		bool bLocked = false;
		bool bIsDefragging = false;

		uint32 NumSubAllocations = 0;
		uint32 AllocCalls = 0;
		uint32 FreeCalls = 0;

		// List of free ranges
		TArray<FRange> FreeList;
		bool JoinFreeBlocks();
		uint32 GetAllocatorIndex()
		{
			return AllocatorIndex;
		}


#if VULKAN_MEMORY_TRACK
		FVulkanTrackInfo Track;
#endif
		TArray<FVulkanAllocationInternal> InternalData;
		int32 InternalFreeList= -1;
		FCriticalSection SubresourceAllocatorCS;
	};


	// A set of Device Allocations (Heap Pages) for a specific memory type. This handles pooling allocations inside memory pages to avoid
	// doing allocations directly off the device's heaps
	class FVulkanResourceHeap
	{
	public:
		FVulkanResourceHeap(FMemoryManager* InOwner, uint32 InMemoryTypeIndex, uint32 InOverridePageSize = 0);
		~FVulkanResourceHeap();

		void FreePage(FVulkanSubresourceAllocator* InPage);
		void ReleasePage(FVulkanSubresourceAllocator* InPage);

		//void ReleaseFreedPages(bool bImmediately);

		inline FMemoryManager* GetOwner()
		{
			return Owner;
		}

		inline bool IsHostCachedSupported() const
		{
			return bIsHostCachedSupported;
		}

		inline bool IsLazilyAllocatedSupported() const
		{
			return bIsLazilyAllocatedSupported;
		}

		inline uint32 GetMemoryTypeIndex() const
		{
			return MemoryTypeIndex;
		}

		uint64 EvictOne(FVulkanDevice& Device, const FVulkanContextArray& Contexts);
		void DefragTick(FVulkanDevice& Device, const FVulkanContextArray& Contexts, uint32 Count);
		void DumpMemory(FResourceHeapStats& Stats);

		void SetDefragging(FVulkanSubresourceAllocator* Allocator);
		bool GetIsDefragging(FVulkanSubresourceAllocator* Allocator);
		uint32 GetPageSizeBucket(FVulkanPageSizeBucket& BucketOut, EType Type, uint32 AllocationSize, bool bForceSingleAllocation);

	protected:
		enum {			
			MAX_BUCKETS = 5,
		};
		TArray<FVulkanPageSizeBucket, TFixedAllocator<MAX_BUCKETS> > PageSizeBuckets;
		uint32 GetPageSize();
		FMemoryManager* Owner;
		const uint16 MemoryTypeIndex;
		const uint16 HeapIndex;

		bool bIsHostCachedSupported;
		bool bIsLazilyAllocatedSupported;
		uint8 DefragCountDown = 0;

		uint32 OverridePageSize;

		uint32 PeakPageSize;
		uint64 UsedMemory;
		uint32 PageIDCounter;

		FCriticalSection PagesLock;
		TArray<FVulkanSubresourceAllocator*> ActivePages[MAX_BUCKETS];
		TArray<FVulkanSubresourceAllocator*> UsedDedicatedImagePages;

		bool TryRealloc(FVulkanAllocation& OutAllocation, FVulkanEvictable* AllocationOwner, EType Type, uint32 Size, uint32 Alignment, EVulkanAllocationMetaType MetaType);
		bool AllocateResource(FVulkanAllocation& OutAllocation, FVulkanEvictable* AllocationOwner, EType Type, uint32 Size, uint32 Alignment, bool bMapAllocation, bool bForceSeparateAllocation, EVulkanAllocationMetaType MetaType, bool bExternal, const char* File, uint32 Line);
		bool AllocateDedicatedImage(FVulkanAllocation& OutAllocation, FVulkanEvictable* AllocationOwner, VkImage Image, uint32 Size, uint32 Alignment, EVulkanAllocationMetaType MetaType, bool bExternal, const char* File, uint32 Line);

		friend class FMemoryManager;
		friend class FVulkanSubresourceAllocator;
	};

	enum EVulkanFreeFlags
	{
		EVulkanFreeFlag_None = 0x0,
		EVulkanFreeFlag_DontDefer = 0x1,
	};


	// Allocation flags to account for conditions not covered by vkGet*MemoryRequirements or resource usage flags
	enum class EVulkanAllocationFlags : uint16
	{
		None         = 0x0000,

		HostVisible  = 0x0001,    // Will be written from CPU, will likely contain the HOST_VISIBLE + HOST_COHERENT flags
		HostCached   = 0x0002,    // Will be used for readback, will likely contain the HOST_CACHED flag.  Implies HostVisible.
		PreferBAR    = 0x0004,    // Will be allocated from a HOST_VISIBLE + DEVICE_LOCAL heap if available and possible (HOST_VISIBLE if not)

		Dedicated    = 0x0008,	  // Will not share a memory block with other resources
		External     = 0x0010,    // To be used with VK_KHR_external_memory
		Memoryless   = 0x0020,	  // Will use LAZILY_ALLOCATED
		Protected    = 0x0040,	  // Will use PROTECTED

		NoError      = 0x0080,    // OOM is not fatal, return an invalid allocation
		AutoBind     = 0x0100,	  // Will automatically bind the allocation to the supplied VkBuffer/VkImage, avoids an extra lock to bind separately
	};
	ENUM_CLASS_FLAGS(EVulkanAllocationFlags);

	// Manages heaps and their interactions
	class FMemoryManager
	{
		friend class FVulkanAllocation;
	public:

		FMemoryManager(FVulkanDevice& InDevice);
		~FMemoryManager();

		void Init();
		void Deinit();

		FVulkanDevice& GetDevice()
		{
			return Device;
		}

		static uint32 CalculateBufferAlignment(FVulkanDevice& InDevice, EBufferUsageFlags InUEUsage, bool bZeroSize);
		static float CalculateBufferPriority(const VkBufferUsageFlags BufferUsageFlags);

		void FreeVulkanAllocation(FVulkanAllocation& Allocation, EVulkanFreeFlags FreeFlags = EVulkanFreeFlag_None);
		void FreeVulkanAllocationPooledBuffer(FVulkanAllocation& Allocation);
		void FreeVulkanAllocationBuffer(FVulkanAllocation& Allocation);
		void FreeVulkanAllocationImage(FVulkanAllocation& Allocation);
		void FreeVulkanAllocationImageDedicated(FVulkanAllocation& Allocation);

		// Legacy calls
		bool AllocateBufferPooled(FVulkanAllocation& Allocation, FVulkanEvictable* AllocationOwner, uint32 Size, uint32 MinAlignment, VkBufferUsageFlags BufferUsageFlags, VkMemoryPropertyFlags MemoryPropertyFlags, EVulkanAllocationMetaType MetaType, const char* File, uint32 Line);
		bool AllocateImageMemory(FVulkanAllocation& Allocation, FVulkanEvictable* AllocationOwner, const VkMemoryRequirements& MemoryReqs, VkMemoryPropertyFlags MemoryPropertyFlags, EVulkanAllocationMetaType MetaType, bool bExternal, const char* File, uint32 Line);
		bool AllocateDedicatedImageMemory(FVulkanAllocation& Allocation, FVulkanEvictable* AllocationOwner, VkImage Image, VkMemoryPropertyFlags MemoryPropertyFlags, EVulkanAllocationMetaType MetaType, bool bExternal, bool bForceDedicatedAllocation, const char* File, uint32 Line);
	private:
		bool AllocateBufferMemory(FVulkanAllocation& Allocation, FVulkanEvictable* AllocationOwner, const VkMemoryRequirements& MemoryReqs, VkMemoryPropertyFlags MemoryPropertyFlags, EVulkanAllocationMetaType MetaType, bool bExternal, bool bForceSeparateAllocation, const char* File, uint32 Line);

		// New calls
	public:
		bool AllocateBufferMemory(FVulkanAllocation& OutAllocation, VkBuffer InBuffer, EVulkanAllocationFlags InAllocFlags, const TCHAR* InDebugName, uint32 InForceMinAlignment = 1);

		void RegisterSubresourceAllocator(FVulkanSubresourceAllocator* SubresourceAllocator);
		void UnregisterSubresourceAllocator(FVulkanSubresourceAllocator* SubresourceAllocator);
		bool ReleaseSubresourceAllocator(FVulkanSubresourceAllocator* SubresourceAllocator);


		void ReleaseFreedPages(const FVulkanContextArray& Contexts);
		void DumpMemory(bool bFullDump = true);


		void AllocUniformBuffer(FVulkanAllocation& OutAllocation, uint32 Size);
		void FreeUniformBuffer(FVulkanAllocation& InAllocation);

		void HandleOOM(bool bCanResume = false, VkResult Result = VK_SUCCESS, uint64 AllocationSize = 0, uint32 MemoryTypeIndex = 0);
		bool UpdateEvictThreshold(bool bLog);

	protected:
		FVulkanDevice& Device;
		FDeviceMemoryManager* DeviceMemoryManager;
		TArray<FVulkanResourceHeap*> ResourceTypeHeaps;

		enum
		{
			BufferAllocationSize = 1 * 1024 * 1024,
			UniformBufferAllocationSize = 2 * 1024 * 1024,
		};


		// pool sizes that we support
		enum class EPoolSizes : uint8
		{
// 			E32,
// 			E64,
			E128,
			E256,
			E512,
			E1k,
			E2k,
			E8k,
			E16k,
			SizesCount,
		};

		constexpr static uint32 PoolSizes[(int32)EPoolSizes::SizesCount] =
		{
// 			32,
// 			64,
			128,
			256,
			512,
			1024,
			2048,
			8192,
// 			16 * 1024,
		};

		constexpr static uint32 BufferSizes[(int32)EPoolSizes::SizesCount + 1] =
		{
// 			64 * 1024,
// 			64 * 1024,
			128 * 1024,
			128 * 1024,
			256 * 1024,
			256 * 1024,
			512 * 1024,
			512 * 1024,
			1024 * 1024,
			1 * 1024 * 1024,
		};

		EPoolSizes GetPoolTypeForAlloc(uint32 Size, uint32 Alignment)
		{
			EPoolSizes PoolSize = EPoolSizes::SizesCount;
			if (GVulkanUseBufferBinning != 0)
			{
				for (int32 i = 0; i < (int32)EPoolSizes::SizesCount; ++i)
				{
					if (PoolSizes[i] >= Size)
					{
						PoolSize = (EPoolSizes)i;
						break;
					}
				}
			}
			return PoolSize;
		}

		FCriticalSection UsedFreeBufferAllocationsLock;
		TArray<FVulkanSubresourceAllocator*> UsedBufferAllocations[(int32)EPoolSizes::SizesCount + 1];
		TArray<FVulkanSubresourceAllocator*> FreeBufferAllocations[(int32)EPoolSizes::SizesCount + 1];

		FRWLock AllBufferAllocationsLock;  // protects against resizing of array (RenderThread<->RHIThread)
		TArray<FVulkanSubresourceAllocator*> AllBufferAllocations;
		PTRINT AllBufferAllocationsFreeListHead = (PTRINT)-1;
		inline FVulkanSubresourceAllocator* GetSubresourceAllocator(const uint32 AllocatorIndex)
		{
			FRWScopeLock ScopedReadLock(AllBufferAllocationsLock, SLT_ReadOnly);
			return AllBufferAllocations[AllocatorIndex];
		}

		uint64 PendingEvictBytes = 0;
		bool bIsEvicting = false;
		bool bWantEviction = false;

		friend class FVulkanResourceHeap;

		void ReleaseFreedResources(bool bImmediately);
		void DestroyResourceAllocations();
		struct FUBPendingFree
		{
			FVulkanAllocation Allocation;
			int64 Frame = 0;
		};

		struct
		{
			FCriticalSection CS;
			TArray<FUBPendingFree> PendingFree;
			uint32 Peak = 0;
		} UBAllocations;

		void ProcessPendingUBFreesNoLock(bool bForce);
		void ProcessPendingUBFrees(bool bForce);
	};

	class FStagingBuffer : public FRefCountedObject
	{
	public:
		FStagingBuffer(FVulkanDevice* InDevice);

		VkBuffer GetHandle() const;
		void* GetMappedPointer();
		uint32 GetSize() const;
		VkDeviceMemory GetDeviceMemoryHandle() const;
		void FlushMappedMemory();
		void InvalidateMappedMemory();

#if VULKAN_MEMORY_TRACK
		FVulkanTrackInfo Track;
#endif

	protected:
		FVulkanDevice* Device;
		FVulkanAllocation Allocation;

		VkBuffer Buffer;
		VkMemoryPropertyFlagBits MemoryReadFlags;
		uint32 BufferSize;

		// Owner maintains lifetime
		virtual ~FStagingBuffer();

		void Destroy();

		friend class FStagingManager;
	};

	class FStagingManager
	{
	public:
		FStagingManager();
		~FStagingManager();

		void Init(FVulkanDevice* InDevice)
		{
			Device = InDevice;
		}

		void Deinit();

		FStagingBuffer* AcquireBuffer(uint32 Size, VkBufferUsageFlags InUsageFlags = VK_BUFFER_USAGE_TRANSFER_SRC_BIT, VkMemoryPropertyFlagBits InMemoryReadFlags = VK_MEMORY_PROPERTY_HOST_COHERENT_BIT);

		// Sets pointer to nullptr
		void ReleaseBuffer(FVulkanContextCommon* Context, FStagingBuffer*& StagingBuffer);

		void ProcessPendingFree(bool bImmediately, bool bFreeToOS);

		void GetMemoryDump(FResourceHeapStats& Stats);
		void DumpMemory();

	protected:
		friend class FMemoryManager;

		FCriticalSection StagingLock;

		TArray<FStagingBuffer*> UsedStagingBuffers;
		TMap<FVulkanSyncPointRef, TArray<FStagingBuffer*>> PendingFreeStagingBuffers;
		struct FFreeEntry
		{
			FStagingBuffer* StagingBuffer;
			uint32 FrameNumber;
		};
		TArray<FFreeEntry> FreeStagingBuffers;

		uint64 PeakUsedMemory = 0;
		uint64 UsedMemory = 0;

		void ProcessPendingFreeNoLock(bool bImmediately, bool bFreeToOS);

		FVulkanDevice* Device = nullptr;
	};

	// Simple temp allocation blocks used for volatile allocations
	class FTempBlockAllocator
	{
	public:
		struct FTempMemoryBlock
		{
			FVulkanAllocation Allocation;
			VkBuffer Buffer = 0;
			uint8* MappedPointer = 0;
			VkDeviceAddress BufferAddress = 0;

			// Used by owners to suballocate
			TArray<FVulkanSyncPointRef> SyncPoints;
			std::atomic<uint32> References = 0;
			std::atomic<uint32> CurrentOffset = 0;
		};

		struct FInternalAlloc
		{
			FTempMemoryBlock* Block = nullptr;
			uint32 Offset = 0;
		};

		FTempBlockAllocator(FVulkanDevice& InDevice, uint32 InBlockSize, uint32 InBlockAlignment, VkBufferUsageFlags InBufferUsage);
		virtual ~FTempBlockAllocator();

		uint8* Alloc(uint32 InSize, FVulkanContextCommon& Context, VkDescriptorBufferBindingInfoEXT& OutBindingInfo, VkDeviceSize& OutOffset);
		uint8* Alloc(uint32 InSize, uint32 InAlignment, FVulkanContextCommon* Context, FVulkanAllocation& OutAllocation, VkDescriptorAddressInfoEXT* OutDescriptorAddressInfo = nullptr);
		uint8* Alloc(uint32 InSize, FVulkanContextCommon& Context, VkStridedDeviceAddressRegionKHR* OutStridedDeviceAddressRegion);

		uint8* Alloc(uint32 InSize, uint32 InAlignment, FVulkanContextCommon& Context, FVulkanAllocation& OutAllocation, VkDescriptorAddressInfoEXT* OutDescriptorAddressInfo = nullptr)
		{
			return Alloc(InSize, InAlignment, &Context, OutAllocation, OutDescriptorAddressInfo);
		}

		void FreeReference(const FVulkanAllocation& InAllocation);

		void UpdateBlocks();

	protected:
		FTempMemoryBlock* AllocBlock();
		FInternalAlloc InternalAlloc(uint32 InSize, FVulkanContextCommon* Context);

		FVulkanDevice& Device;

		const uint32 BlockSize;
		const uint32 BlockAlignment;
		const VkBufferUsageFlags BufferUsage;

		FTempMemoryBlock* CurrentBlock = nullptr;
		TArray<FTempMemoryBlock*> BusyBlocks;
		TArray<FTempMemoryBlock*> AvailableBlocks;
		FRWLock RWLock;
	};


}

#if VULKAN_CUSTOM_MEMORY_MANAGER_ENABLED
struct FVulkanCustomMemManager
{
	static VKAPI_ATTR void* Alloc(void* UserData, size_t Size, size_t Alignment, VkSystemAllocationScope AllocScope);
	static VKAPI_ATTR void Free(void* pUserData, void* pMem);
	static VKAPI_ATTR void* Realloc(void* pUserData, void* pOriginal, size_t size, size_t alignment, VkSystemAllocationScope allocScope);
	static VKAPI_ATTR void InternalAllocationNotification(void* pUserData, size_t size, VkInternalAllocationType allocationType, VkSystemAllocationScope allocationScope);
	static VKAPI_ATTR void InternalFreeNotification(void* pUserData, size_t size, VkInternalAllocationType allocationType, VkSystemAllocationScope allocationScope);

	FVulkanCustomMemManager();

	enum
	{
		VK_SYSTEM_ALLOCATION_SCOPE_RANGE_SIZE = 5,
	};

	struct FType
	{
		size_t UsedMemory = 0;
		size_t MaxAllocSize = 0;
		TMap<void*, size_t> Allocs;
	};
	TStaticArray<FType, VK_SYSTEM_ALLOCATION_SCOPE_RANGE_SIZE> Types;

	static FType& GetType(void* pUserData, VkSystemAllocationScope AllocScope);
};
#endif

#undef UE_API
