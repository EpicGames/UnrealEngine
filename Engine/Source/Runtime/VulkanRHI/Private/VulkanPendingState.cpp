// Copyright Epic Games, Inc. All Rights Reserved..

/*=============================================================================
	VulkanPendingState.cpp: Private VulkanPendingState function definitions.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanCommandWrappers.h"
#include "VulkanFramebuffer.h"
#include "VulkanPendingState.h"
#include "VulkanPipeline.h"
#include "VulkanContext.h"

extern int32 GVulkanNumFramesToWaitForResourceDelete;

FVulkanDescriptorPool::FVulkanDescriptorPool(FVulkanDevice* InDevice, const FVulkanDescriptorSetsLayout& InLayout, uint32 MaxSetsAllocations)
	: Device(InDevice)
	, MaxDescriptorSets(0)
	, NumAllocatedDescriptorSets(0)
	, PeakAllocatedDescriptorSets(0)
	, Layout(InLayout)
	, DescriptorPool(VK_NULL_HANDLE)
{
	INC_DWORD_STAT(STAT_VulkanNumDescPools);

	// Descriptor sets number required to allocate the max number of descriptor sets layout.
	// When we're hashing pools with types usage ID the descriptor pool can be used for different layouts so the initial layout does not make much sense.
	// In the latter case we'll be probably overallocating the descriptor types but given the relatively small number of max allocations this should not have
	// a serious impact.
	MaxDescriptorSets = MaxSetsAllocations*(VULKAN_HASH_POOLS_WITH_TYPES_USAGE_ID ? 1 : Layout.GetLayouts().Num());
	TArray<VkDescriptorPoolSize, TFixedAllocator<VK_DESCRIPTOR_TYPE_RANGE_SIZE>> Types;
	for (uint32 TypeIndex = VK_DESCRIPTOR_TYPE_BEGIN_RANGE; TypeIndex <= VK_DESCRIPTOR_TYPE_END_RANGE; ++TypeIndex)
	{
		VkDescriptorType DescriptorType =(VkDescriptorType)TypeIndex;
		uint32 NumTypesUsed = Layout.GetTypesUsed(DescriptorType);
		if (NumTypesUsed > 0)
		{
			VkDescriptorPoolSize& Type = Types.AddDefaulted_GetRef();
			FMemory::Memzero(Type);
			Type.type = DescriptorType;
			Type.descriptorCount = NumTypesUsed * MaxSetsAllocations;
		}
	}

	{
		uint32 NumTypesUsed = Layout.GetTypesUsed(VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR);
		if (NumTypesUsed > 0)
		{
			VkDescriptorPoolSize& Type = Types.AddDefaulted_GetRef();
			FMemory::Memzero(Type);
			Type.type = VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR;
			Type.descriptorCount = NumTypesUsed * MaxSetsAllocations;
		}
	}

	VkDescriptorPoolCreateInfo PoolInfo;
	ZeroVulkanStruct(PoolInfo, VK_STRUCTURE_TYPE_DESCRIPTOR_POOL_CREATE_INFO);
	// you don't need this flag because pool reset feature. Also this flag increase pool size in memory and vkResetDescriptorPool time.
	//PoolInfo.flags = VK_DESCRIPTOR_POOL_CREATE_FREE_DESCRIPTOR_SET_BIT;
	PoolInfo.poolSizeCount = Types.Num();
	PoolInfo.pPoolSizes = Types.GetData();
	PoolInfo.maxSets = MaxDescriptorSets;

#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanVkCreateDescriptorPool);
#endif
	VERIFYVULKANRESULT(VulkanRHI::vkCreateDescriptorPool(Device->GetHandle(), &PoolInfo, VULKAN_CPU_ALLOCATOR, &DescriptorPool));

	INC_DWORD_STAT_BY(STAT_VulkanNumDescSetsTotal, MaxDescriptorSets);
}

FVulkanDescriptorPool::~FVulkanDescriptorPool()
{
	DEC_DWORD_STAT_BY(STAT_VulkanNumDescSetsTotal, MaxDescriptorSets);
	DEC_DWORD_STAT(STAT_VulkanNumDescPools);

	if (DescriptorPool != VK_NULL_HANDLE)
	{
		VulkanRHI::vkDestroyDescriptorPool(Device->GetHandle(), DescriptorPool, VULKAN_CPU_ALLOCATOR);
		DescriptorPool = VK_NULL_HANDLE;
	}
}

void FVulkanDescriptorPool::TrackAddUsage(const FVulkanDescriptorSetsLayout& InLayout)
{
	// Check and increment our current type usage
	for (uint32 TypeIndex = VK_DESCRIPTOR_TYPE_BEGIN_RANGE; TypeIndex <= VK_DESCRIPTOR_TYPE_END_RANGE; ++TypeIndex)
	{
		ensure(Layout.GetTypesUsed((VkDescriptorType)TypeIndex) == InLayout.GetTypesUsed((VkDescriptorType)TypeIndex));
	}

	{
		ensure(Layout.GetTypesUsed(VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR) == InLayout.GetTypesUsed(VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR));
	}

	NumAllocatedDescriptorSets += InLayout.GetLayouts().Num();
	PeakAllocatedDescriptorSets = FMath::Max(NumAllocatedDescriptorSets, PeakAllocatedDescriptorSets);
}

void FVulkanDescriptorPool::TrackRemoveUsage(const FVulkanDescriptorSetsLayout& InLayout)
{
	for (uint32 TypeIndex = VK_DESCRIPTOR_TYPE_BEGIN_RANGE; TypeIndex <= VK_DESCRIPTOR_TYPE_END_RANGE; ++TypeIndex)
	{
		check(Layout.GetTypesUsed((VkDescriptorType)TypeIndex) == InLayout.GetTypesUsed((VkDescriptorType)TypeIndex));
	}

	{
		check(Layout.GetTypesUsed(VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR) == InLayout.GetTypesUsed(VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR));
	}

	NumAllocatedDescriptorSets -= InLayout.GetLayouts().Num();
}

void FVulkanDescriptorPool::Reset()
{
	if (DescriptorPool != VK_NULL_HANDLE)
	{
		VERIFYVULKANRESULT(VulkanRHI::vkResetDescriptorPool(Device->GetHandle(), DescriptorPool, 0));
	}

	NumAllocatedDescriptorSets = 0;
}

bool FVulkanDescriptorPool::AllocateDescriptorSets(const VkDescriptorSetAllocateInfo& InDescriptorSetAllocateInfo, VkDescriptorSet* OutSets)
{
	VkDescriptorSetAllocateInfo DescriptorSetAllocateInfo = InDescriptorSetAllocateInfo;
	DescriptorSetAllocateInfo.descriptorPool = DescriptorPool;

	return VK_SUCCESS == VulkanRHI::vkAllocateDescriptorSets(Device->GetHandle(), &DescriptorSetAllocateInfo, OutSets);
}

FVulkanTypedDescriptorPoolSet::~FVulkanTypedDescriptorPoolSet()
{
	for (FPoolList* Pool = PoolListHead; Pool;)
	{
		FPoolList* Next = Pool->Next;

		delete Pool->Element;
		delete Pool;

		Pool = Next;
	}
	PoolsCount = 0;
}

FVulkanDescriptorPool* FVulkanTypedDescriptorPoolSet::PushNewPool()
{
	// Max number of descriptor sets layout allocations
	const uint32 MaxSetsAllocationsBase = 32;
	// Allow max 128 setS per pool (32 << 2)
	const uint32 MaxSetsAllocations = MaxSetsAllocationsBase << FMath::Min(PoolsCount, 2u);

	auto* NewPool = new FVulkanDescriptorPool(Device, Layout, MaxSetsAllocations);

	if (PoolListCurrent)
	{
		PoolListCurrent->Next = new FPoolList(NewPool);
		PoolListCurrent = PoolListCurrent->Next;
	}
	else
	{
		PoolListCurrent = PoolListHead = new FPoolList(NewPool);
	}
	++PoolsCount;

	return NewPool;
}

FVulkanDescriptorPool* FVulkanTypedDescriptorPoolSet::GetFreePool(bool bForceNewPool)
{
	// Likely this
	if (!bForceNewPool)
	{
		return PoolListCurrent->Element;
	}

	if (PoolListCurrent->Next)
	{
		PoolListCurrent = PoolListCurrent->Next;
		return PoolListCurrent->Element;
	}

	return PushNewPool();
}

bool FVulkanTypedDescriptorPoolSet::AllocateDescriptorSets(const FVulkanDescriptorSetsLayout& InLayout, VkDescriptorSet* OutSets)
{
	const TArray<VkDescriptorSetLayout>& LayoutHandles = InLayout.GetHandles();

	if (LayoutHandles.Num() > 0)
	{
		auto* Pool = PoolListCurrent->Element;
		while (!Pool->AllocateDescriptorSets(InLayout.GetAllocateInfo(), OutSets))
		{
			Pool = GetFreePool(true);
		}

#if VULKAN_ENABLE_AGGRESSIVE_STATS
		//INC_DWORD_STAT_BY(STAT_VulkanNumDescSetsTotal, LayoutHandles.Num());
		Pool->TrackAddUsage(InLayout);
#endif

		return true;
	}

	return true;
}

void FVulkanTypedDescriptorPoolSet::Reset()
{
	for (FPoolList* Pool = PoolListHead; Pool; Pool = Pool->Next)
	{
		Pool->Element->Reset();
	}

	PoolListCurrent = PoolListHead;
}

FVulkanDescriptorPoolSetContainer::~FVulkanDescriptorPoolSetContainer()
{
	for (auto& Pair : TypedDescriptorPools)
	{
		FVulkanTypedDescriptorPoolSet* TypedPool = Pair.Value;
		delete TypedPool;
	}

	TypedDescriptorPools.Reset();
}

FVulkanTypedDescriptorPoolSet* FVulkanDescriptorPoolSetContainer::AcquireTypedPoolSet(const FVulkanDescriptorSetsLayout& Layout)
{
	const uint32 Hash = VULKAN_HASH_POOLS_WITH_TYPES_USAGE_ID ? Layout.GetTypesUsageID() : GetTypeHash(Layout);

	FVulkanTypedDescriptorPoolSet* TypedPool = TypedDescriptorPools.FindRef(Hash);

	if (!TypedPool)
	{
		TypedPool = new FVulkanTypedDescriptorPoolSet(Device, Layout);
		TypedDescriptorPools.Add(Hash, TypedPool);
	}

	return TypedPool;
}

void FVulkanDescriptorPoolSetContainer::Reset()
{
	for (auto& Pair : TypedDescriptorPools)
	{
		FVulkanTypedDescriptorPoolSet* TypedPool = Pair.Value;
		TypedPool->Reset();
	}
}

FVulkanDescriptorPoolsManager::~FVulkanDescriptorPoolsManager()
{
	for (auto* PoolSet : PoolSets)
	{
		delete PoolSet;
	}

	PoolSets.Reset();
}

FVulkanDescriptorPoolSetContainer& FVulkanDescriptorPoolsManager::AcquirePoolSetContainer()
{
	FScopeLock ScopeLock(&CS);

	for (auto* PoolSet : PoolSets)
	{
		if (PoolSet->IsUnused())
		{
			PoolSet->SetUsed(true);
			return *PoolSet;
		}
	}

	FVulkanDescriptorPoolSetContainer* PoolSet = new FVulkanDescriptorPoolSetContainer(Device);
	PoolSets.Add(PoolSet);

	return *PoolSet;
}

void FVulkanDescriptorPoolsManager::ReleasePoolSet(FVulkanDescriptorPoolSetContainer& PoolSet)
{
	PoolSet.Reset();
	PoolSet.SetUsed(false);
}

void FVulkanDescriptorPoolsManager::GC()
{
	FScopeLock ScopeLock(&CS);

	// Pool sets are forward allocated - iterate from the back to increase the chance of finding an unused one
	for (int32 Index = PoolSets.Num() - 1; Index >= 0; Index--)
	{
		auto* PoolSet = PoolSets[Index];
		if (PoolSet->IsUnused() && ((int32)(GFrameNumberRenderThread - PoolSet->GetLastFrameUsed()) > GVulkanNumFramesToWaitForResourceDelete))
		{
			PoolSets.RemoveAtSwap(Index, EAllowShrinking::Yes);

			if (AsyncDeletionTask)
			{
				if (!AsyncDeletionTask->IsDone())
				{
					AsyncDeletionTask->EnsureCompletion();
				}

				AsyncDeletionTask->GetTask().SetPoolSet(PoolSet);
			}
			else
			{
				AsyncDeletionTask = new FAsyncTask<FVulkanAsyncPoolSetDeletionWorker>(PoolSet);
			}

			AsyncDeletionTask->StartBackgroundTask();

			break;
		}
	}
}


FVulkanPendingComputeState::~FVulkanPendingComputeState()
{
	for (auto It = PipelineStates.CreateIterator(); It; ++It)
	{
		FVulkanCommonPipelineDescriptorState* State = It->Value;
		delete State;
	}
}

void FVulkanPendingComputeState::PrepareForDispatch(FVulkanCommandListContext& Context)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanDispatchCallPrepareTime);
#endif

	check(CurrentState);

	if (CurrentPipeline->UsesBindless())
	{
		CurrentState->UpdateBindlessDescriptors(Context);
		CurrentPipeline->Bind(Context.GetCommandBuffer());
	}
	else
	{
		const bool bHasDescriptorSets = CurrentState->UpdateDescriptorSets(Context);

		//#todo-rco: Move this to SetComputePipeline()
#if VULKAN_ENABLE_AGGRESSIVE_STATS
		SCOPE_CYCLE_COUNTER(STAT_VulkanPipelineBind);
#endif

		CurrentPipeline->Bind(Context.GetCommandBuffer());
		if (bHasDescriptorSets)
		{
			CurrentState->BindDescriptorSets(Context.GetCommandBuffer().GetHandle());
		}
	}
}

FVulkanPendingGfxState::~FVulkanPendingGfxState()
{
	TMap<FVulkanGraphicsPipelineState*, FVulkanGraphicsPipelineDescriptorState*> Temp;
	Swap(Temp, PipelineStates);
	for (auto& Pair : Temp)
	{
		FVulkanGraphicsPipelineDescriptorState* State = Pair.Value;
		delete State;
	}
}

void FVulkanPendingGfxState::PrepareForDraw(FVulkanCommandListContext& Context)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanDrawCallPrepareTime);
#endif

	check(Context.GetCommandBuffer().bHasPipeline);

	if (CurrentPipeline->UsesBindless())
	{
		check(!CurrentPipeline->bHasInputAttachments); // todo-jn: bindless + InputAttachments
		UpdateDynamicStates(Context.GetCommandBuffer());
		CurrentState->UpdateBindlessDescriptors(Context);
	}
	else
	{
		// TODO: Add 'dirty' flag? Need to rebind only on PSO change
		if (CurrentPipeline->bHasInputAttachments)
		{
			UpdateInputAttachments(Context);
		}

		const bool bHasDescriptorSets = CurrentState->UpdateDescriptorSets(Context);

		UpdateDynamicStates(Context.GetCommandBuffer());

		if (bHasDescriptorSets)
		{
			CurrentState->BindDescriptorSets(Context.GetCommandBuffer().GetHandle());
		}
	}

	if (bDirtyVertexStreams)
	{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
		SCOPE_CYCLE_COUNTER(STAT_VulkanBindVertexStreamsTime);
#endif
		// Its possible to have no vertex buffers
		const FVulkanVertexInputStateInfo& VertexInputStateInfo = CurrentPipeline->GetVertexInputState();
		if (VertexInputStateInfo.AttributesNum == 0)
		{
			// However, we need to verify that there are also no bindings
			check(VertexInputStateInfo.BindingsNum == 0);
			return;
		}

		struct FTemporaryIA
		{
			VkBuffer VertexBuffers[MaxVertexElementCount];
			VkDeviceSize VertexOffsets[MaxVertexElementCount];
			VkDeviceSize VertexSizes[MaxVertexElementCount];
			VkDeviceSize VertexStrides[MaxVertexElementCount];
			int32 NumUsed = 0;

			void Add(VkBuffer InBuffer, VkDeviceSize InOffset, VkDeviceSize InStride)
			{
				check(NumUsed < MaxVertexElementCount);
				VertexBuffers[NumUsed] = InBuffer;
				VertexOffsets[NumUsed] = InOffset;
				VertexSizes[NumUsed] = VK_WHOLE_SIZE;
				VertexStrides[NumUsed] = InStride;
				++NumUsed;
			}
		} TemporaryIA;

		const VkVertexInputAttributeDescription* CurrAttribute = nullptr;
		for (uint32 BindingIndex = 0; BindingIndex < VertexInputStateInfo.BindingsNum; BindingIndex++)
		{
			const VkVertexInputBindingDescription& CurrBinding = VertexInputStateInfo.Bindings[BindingIndex];

			uint32 StreamIndex = VertexInputStateInfo.BindingToStream.FindChecked(BindingIndex);
			FVulkanPendingGfxState::FVertexStream& CurrStream = PendingStreams[StreamIndex];

			// Verify the vertex buffer is set
			if (CurrStream.Stream == VK_NULL_HANDLE)
			{
				// The attribute in stream index is probably compiled out
#if UE_BUILD_DEBUG
				// Lets verify
				for (uint32 AttributeIndex = 0; AttributeIndex < VertexInputStateInfo.AttributesNum; AttributeIndex++)
				{
					if (VertexInputStateInfo.Attributes[AttributeIndex].binding == CurrBinding.binding)
					{
						uint64 VertexShaderKey = GetCurrentShaderKey(ShaderStage::Vertex);
						FVulkanVertexShader* VertexShader = Device.GetShaderFactory().LookupShader<FVulkanVertexShader>(VertexShaderKey);
						UE_LOGF(LogVulkanRHI, Warning, "Missing input assembly binding on location %d in Vertex shader '%ls'",
							CurrBinding.binding,
							VertexShader ? *VertexShader->GetDebugName() : TEXT("Null"));
						ensure(0);
					}
				}
#endif
				continue;
			}

			TemporaryIA.Add(CurrStream.Stream, CurrStream.BufferOffset, CurrBinding.stride);
		}

		if (TemporaryIA.NumUsed > 0)
		{
			// Bindings are expected to be in ascending order with no index gaps in between:
			// Correct:		0, 1, 2, 3
			// Incorrect:	1, 0, 2, 3
			// Incorrect:	0, 2, 3, 5
			// Reordering and creation of stream binding index is done in "GenerateVertexInputStateInfo()"
			VulkanRHI::vkCmdBindVertexBuffers(Context.GetCommandBuffer().GetHandle(), 0, TemporaryIA.NumUsed, TemporaryIA.VertexBuffers, TemporaryIA.VertexOffsets);
		}

		bDirtyVertexStreams = false;
	}
}

void FVulkanPendingGfxState::UpdateDynamicStates(FVulkanCommandBuffer& CommandBuffer)
{
	const bool bHasEXTExtendedDynamicState1 = Device.GetOptionalExtensions().HasEXTExtendedDynamicState1 || Device.SupportsShaderObjects();
	const bool bHasEXTExtendedDynamicState2 = Device.GetOptionalExtensions().HasEXTExtendedDynamicState2 || Device.SupportsShaderObjects();
	const bool bHasEXTExtendedDynamicState3 = Device.GetOptionalExtensions().HasEXTExtendedDynamicState3 || Device.SupportsShaderObjects();
	const bool bHasEXTVertexInputDynamicState = Device.GetOptionalExtensions().HasEXTVertexInputDynamicState || Device.SupportsShaderObjects();

	const bool bNeedsUpdateViewport = !CommandBuffer.bHasViewport || Viewports.Num() != CommandBuffer.CurrentViewports.Num() ||
		(FMemory::Memcmp((const void*)CommandBuffer.CurrentViewports.GetData(), (const void*)Viewports.GetData(), Viewports.Num() * sizeof(VkViewport)) != 0);
	// Validate and update Viewport
	if (bNeedsUpdateViewport)
	{
		// it is legal to pass a zero-area viewport, and the higher level expectation (see e.g. FProjectedShadowInfo::SetupProjectionStencilMask()) is that
		// such viewport is going to be essentially disabled.

		// Flip viewport on Y-axis to be uniform between DXC generated SPIR-V shaders (requires VK_KHR_maintenance1 extension)
		TArray<VkViewport, TInlineAllocator<2>> FlippedViewports = Viewports;
		for (VkViewport& FlippedViewport : FlippedViewports)
		{
			FlippedViewport.y += FlippedViewport.height;
			FlippedViewport.height = -FlippedViewport.height;
		}

		if (bHasEXTExtendedDynamicState1)
		{
			VulkanRHI::vkCmdSetViewportWithCountEXT(CommandBuffer.GetHandle(), FlippedViewports.Num(), FlippedViewports.GetData());
		}
		else
		{
			VulkanRHI::vkCmdSetViewport(CommandBuffer.GetHandle(), 0, FlippedViewports.Num(), FlippedViewports.GetData());
		}

		CommandBuffer.CurrentViewports = Viewports;
		CommandBuffer.bHasViewport = true;
	}

	const bool bNeedsUpdateScissor = !CommandBuffer.bHasScissor || Scissors.Num() != CommandBuffer.CurrentScissors.Num() || 
		(FMemory::Memcmp((const void*)CommandBuffer.CurrentScissors.GetData(), (const void*)Scissors.GetData(), Scissors.Num() * sizeof(VkRect2D)) != 0);
	if (bNeedsUpdateScissor)
	{
		if (bHasEXTExtendedDynamicState1)
		{
			VulkanRHI::vkCmdSetScissorWithCountEXT(CommandBuffer.GetHandle(), Scissors.Num(), Scissors.GetData());
		}
		else
		{
			VulkanRHI::vkCmdSetScissor(CommandBuffer.GetHandle(), 0, Scissors.Num(), Scissors.GetData());
		}

		CommandBuffer.CurrentScissors = Scissors;
		CommandBuffer.bHasScissor = true;
	}

	const bool bNeedsUpdateStencil = !CommandBuffer.bHasStencilRef || (CommandBuffer.CurrentStencilRef != StencilRef);
	if (bNeedsUpdateStencil)
	{
		VulkanRHI::vkCmdSetStencilReference(CommandBuffer.GetHandle(), VK_STENCIL_FRONT_AND_BACK, StencilRef);
		CommandBuffer.CurrentStencilRef = StencilRef;
		CommandBuffer.bHasStencilRef = true;
	}

	const bool bForceUpdate = CommandBuffer.bNeedsFullDynamicStateUpdate;
	FGfxPipelineDesc* LastDynamicStateUpdate = CommandBuffer.LastDynamicStateUpdate;
	const FGfxPipelineDesc& NewDynamicState = CurrentPipeline->Desc;
	const bool bHasMeshShader = (CurrentPipeline->GetVulkanShader(SF_Mesh) != nullptr);
	const bool bHasGeometryShader = (CurrentPipeline->GetVulkanShader(SF_Geometry) != nullptr);

	// Set the unused graphic states at least once when we use a command buffer for graphics the first time
	if (bForceUpdate)
	{
		CommandBuffer.SetUnusedGraphicsDynamicStates();
	}

	// Bunch of macros to make dynamic state comparions below easier to read
#define VULKAN_DYNAMIC_STATE_COMPARE(Name) (LastDynamicStateUpdate->Name != NewDynamicState.Name)
#define VULKAN_DYNAMIC_STATE_COMPARE1(Name, ...) VULKAN_DYNAMIC_STATE_COMPARE(Name)
#define VULKAN_DYNAMIC_STATE_COMPARE2(Name, ...) VULKAN_DYNAMIC_STATE_COMPARE(Name) __VA_OPT__( || VULKAN_DYNAMIC_STATE_COMPARE1(__VA_ARGS__))
#define VULKAN_DYNAMIC_STATE_COMPARE3(Name, ...) VULKAN_DYNAMIC_STATE_COMPARE(Name) __VA_OPT__( || VULKAN_DYNAMIC_STATE_COMPARE2(__VA_ARGS__))

#define VULKAN_DYNAMIC_STATE_UPDATE(Name) LastDynamicStateUpdate->Name = NewDynamicState.Name; 
#define VULKAN_DYNAMIC_STATE_UPDATE1(Name, ...) VULKAN_DYNAMIC_STATE_UPDATE(Name)
#define VULKAN_DYNAMIC_STATE_UPDATE2(Name, ...) VULKAN_DYNAMIC_STATE_UPDATE(Name) __VA_OPT__( VULKAN_DYNAMIC_STATE_UPDATE1(__VA_ARGS__))
#define VULKAN_DYNAMIC_STATE_UPDATE3(Name, ...) VULKAN_DYNAMIC_STATE_UPDATE(Name) __VA_OPT__( VULKAN_DYNAMIC_STATE_UPDATE2(__VA_ARGS__))

#define VULKAN_DYNAMIC_STATE_BLOCK_START(...) if (bForceUpdate || VULKAN_DYNAMIC_STATE_COMPARE3(__VA_ARGS__)) { VULKAN_DYNAMIC_STATE_UPDATE3(__VA_ARGS__)
#define VULKAN_DYNAMIC_STATE_BLOCK_END()      }

	// NOTE: All of the different VK_DYNAMIC_STATE_* values being set can be found in VulkanPipeline.cpp

	if (bHasEXTExtendedDynamicState1)
	{
		check(LastDynamicStateUpdate);

		if (!bHasMeshShader)
		{
			// VK_DYNAMIC_STATE_PRIMITIVE_TOPOLOGY
			VULKAN_DYNAMIC_STATE_BLOCK_START(Topology);
			VulkanRHI::vkCmdSetPrimitiveTopologyEXT(CommandBuffer.GetHandle(), (VkPrimitiveTopology)NewDynamicState.Topology);
			VULKAN_DYNAMIC_STATE_BLOCK_END();
		}
		else
		{
			// The state will be invalidated, make sure we set it again
			LastDynamicStateUpdate->Topology = VK_PRIMITIVE_TOPOLOGY_MAX_ENUM;
		}

		// VK_DYNAMIC_STATE_CULL_MODE
		VULKAN_DYNAMIC_STATE_BLOCK_START(Rasterizer.CullMode);
		VulkanRHI::vkCmdSetCullModeEXT(CommandBuffer.GetHandle(), (VkCullModeFlags)NewDynamicState.Rasterizer.CullMode);
		VULKAN_DYNAMIC_STATE_BLOCK_END();

		// VK_DYNAMIC_STATE_STENCIL_TEST_ENABLE
		VULKAN_DYNAMIC_STATE_BLOCK_START(DepthStencil.bStencilTestEnable);
		VulkanRHI::vkCmdSetStencilTestEnableEXT(CommandBuffer.GetHandle(), NewDynamicState.DepthStencil.bStencilTestEnable ? VK_TRUE : VK_FALSE);
		VULKAN_DYNAMIC_STATE_BLOCK_END();

		// VK_DYNAMIC_STATE_STENCIL_OP (front)
		VULKAN_DYNAMIC_STATE_BLOCK_START(DepthStencil.FrontFailOp, DepthStencil.FrontPassOp, DepthStencil.FrontDepthFailOp, DepthStencil.FrontCompareOp);
		VulkanRHI::vkCmdSetStencilOpEXT(
			CommandBuffer.GetHandle(),
			VK_STENCIL_FACE_FRONT_BIT,
			(VkStencilOp)NewDynamicState.DepthStencil.FrontFailOp,
			(VkStencilOp)NewDynamicState.DepthStencil.FrontPassOp,
			(VkStencilOp)NewDynamicState.DepthStencil.FrontDepthFailOp,
			(VkCompareOp)NewDynamicState.DepthStencil.FrontCompareOp
		);
		VULKAN_DYNAMIC_STATE_BLOCK_END();

		// VK_DYNAMIC_STATE_STENCIL_OP (back)
		VULKAN_DYNAMIC_STATE_BLOCK_START(DepthStencil.BackFailOp, DepthStencil.BackPassOp, DepthStencil.BackDepthFailOp, DepthStencil.BackCompareOp);
		VulkanRHI::vkCmdSetStencilOpEXT(
			CommandBuffer.GetHandle(),
			VK_STENCIL_FACE_BACK_BIT,
			(VkStencilOp)NewDynamicState.DepthStencil.BackFailOp,
			(VkStencilOp)NewDynamicState.DepthStencil.BackPassOp,
			(VkStencilOp)NewDynamicState.DepthStencil.BackDepthFailOp,
			(VkCompareOp)NewDynamicState.DepthStencil.BackCompareOp
		);
		VULKAN_DYNAMIC_STATE_BLOCK_END();

		// VK_DYNAMIC_STATE_DEPTH_COMPARE_OP
		VULKAN_DYNAMIC_STATE_BLOCK_START(DepthStencil.DepthCompareOp);
		VulkanRHI::vkCmdSetDepthCompareOpEXT(CommandBuffer.GetHandle(), (VkCompareOp)NewDynamicState.DepthStencil.DepthCompareOp);
		VULKAN_DYNAMIC_STATE_BLOCK_END();

		// VK_DYNAMIC_STATE_DEPTH_TEST_ENABLE
		VULKAN_DYNAMIC_STATE_BLOCK_START(DepthStencil.bDepthTestEnable);
		VulkanRHI::vkCmdSetDepthTestEnableEXT(CommandBuffer.GetHandle(), NewDynamicState.DepthStencil.bDepthTestEnable ? VK_TRUE : VK_FALSE);
		VULKAN_DYNAMIC_STATE_BLOCK_END();

		// VK_DYNAMIC_STATE_DEPTH_BOUNDS_TEST_ENABLE
		VULKAN_DYNAMIC_STATE_BLOCK_START(DepthStencil.bDepthBoundsTestEnable);
		VulkanRHI::vkCmdSetDepthBoundsTestEnableEXT(CommandBuffer.GetHandle(), NewDynamicState.DepthStencil.bDepthBoundsTestEnable ? VK_TRUE : VK_FALSE);
		VULKAN_DYNAMIC_STATE_BLOCK_END();

		// VK_DYNAMIC_STATE_DEPTH_WRITE_ENABLE
		VULKAN_DYNAMIC_STATE_BLOCK_START(DepthStencil.bDepthWriteEnable);
		VulkanRHI::vkCmdSetDepthWriteEnableEXT(CommandBuffer.GetHandle(), NewDynamicState.DepthStencil.bDepthWriteEnable ? VK_TRUE : VK_FALSE);
		VULKAN_DYNAMIC_STATE_BLOCK_END();
	}

	if (bHasEXTExtendedDynamicState2)
	{
		// VK_DYNAMIC_STATE_DEPTH_BIAS_ENABLE
		// VK_DYNAMIC_STATE_DEPTH_BIAS
		VULKAN_DYNAMIC_STATE_BLOCK_START(Rasterizer.DepthBiasConstantFactor, Rasterizer.DepthBiasSlopeScale);
		constexpr float depthBiasClamp = 0.0f;
		VulkanRHI::vkCmdSetDepthBiasEnableEXT(CommandBuffer.GetHandle(), (NewDynamicState.Rasterizer.DepthBiasConstantFactor != 0.0f) ? VK_TRUE : VK_FALSE);
		VulkanRHI::vkCmdSetDepthBias(CommandBuffer.GetHandle(), NewDynamicState.Rasterizer.DepthBiasConstantFactor, depthBiasClamp, NewDynamicState.Rasterizer.DepthBiasSlopeScale);
		VULKAN_DYNAMIC_STATE_BLOCK_END();
	}

	if (bHasEXTExtendedDynamicState3)
	{
		// VK_DYNAMIC_STATE_POLYGON_MODE_EXT
		VULKAN_DYNAMIC_STATE_BLOCK_START(Rasterizer.PolygonMode);
		VulkanRHI::vkCmdSetPolygonModeEXT(CommandBuffer.GetHandle(), (VkPolygonMode)NewDynamicState.Rasterizer.PolygonMode);
		VULKAN_DYNAMIC_STATE_BLOCK_END();

		// VK_DYNAMIC_STATE_RASTERIZATION_SAMPLES_EXT
		VULKAN_DYNAMIC_STATE_BLOCK_START(RasterizationSamples);
		const VkSampleCountFlagBits RasterizationSamples = (VkSampleCountFlagBits)FMath::Max<uint16>(1u, NewDynamicState.RasterizationSamples);
		VulkanRHI::vkCmdSetRasterizationSamplesEXT(CommandBuffer.GetHandle(), RasterizationSamples);
		VULKAN_DYNAMIC_STATE_BLOCK_END();

		//VK_DYNAMIC_STATE_ALPHA_TO_COVERAGE_ENABLE_EXT
		VULKAN_DYNAMIC_STATE_BLOCK_START(UseAlphaToCoverage);
		VulkanRHI::vkCmdSetAlphaToCoverageEnableEXT(CommandBuffer.GetHandle(), NewDynamicState.UseAlphaToCoverage ? VK_TRUE : VK_FALSE);
		VULKAN_DYNAMIC_STATE_BLOCK_END();

		// VK_DYNAMIC_STATE_COLOR_BLEND_ENABLE_EXT
		// VK_DYNAMIC_STATE_COLOR_BLEND_EQUATION_EXT
		// VK_DYNAMIC_STATE_COLOR_WRITE_MASK_EXT
		if (bForceUpdate || 
			(LastDynamicStateUpdate->ShaderKeyShared != NewDynamicState.ShaderKeyShared) ||
			(LastDynamicStateUpdate->ColorAttachmentStates != NewDynamicState.ColorAttachmentStates))
		{
			if (NewDynamicState.ColorAttachmentStates.Num())
			{
				check(NewDynamicState.ColorAttachmentStates.Num() <= MaxSimultaneousRenderTargets);

				VkBool32 BlendEnabled[MaxSimultaneousRenderTargets];
				VkColorBlendEquationEXT ColorBlendEquation[MaxSimultaneousRenderTargets];
				VkColorComponentFlags ColorWriteMask[MaxSimultaneousRenderTargets];
				uint32 TargetWriteMask = 0xffffffff;
				if (const FVulkanShader* PixelShader = CurrentPipeline->GetVulkanShader(SF_Pixel))
				{
					TargetWriteMask = PixelShader->GetCodeHeader().InOutMask;
				}
				for (int32 Index = 0; Index < NewDynamicState.ColorAttachmentStates.Num(); ++Index)
				{
					const FGfxPipelineDesc::FBlendAttachment& BlendAttachment = NewDynamicState.ColorAttachmentStates[Index];

					BlendEnabled[Index] = BlendAttachment.bBlend ? VK_TRUE : VK_FALSE;

					ColorBlendEquation[Index].colorBlendOp = (VkBlendOp)BlendAttachment.ColorBlendOp;
					ColorBlendEquation[Index].srcColorBlendFactor = (VkBlendFactor)BlendAttachment.SrcColorBlendFactor;
					ColorBlendEquation[Index].dstColorBlendFactor = (VkBlendFactor)BlendAttachment.DstColorBlendFactor;
					ColorBlendEquation[Index].alphaBlendOp = (VkBlendOp)BlendAttachment.AlphaBlendOp;
					ColorBlendEquation[Index].srcAlphaBlendFactor = (VkBlendFactor)BlendAttachment.SrcAlphaBlendFactor;
					ColorBlendEquation[Index].dstAlphaBlendFactor = (VkBlendFactor)BlendAttachment.DstAlphaBlendFactor;

					ColorWriteMask[Index] = (VkColorComponentFlags)BlendAttachment.ColorWriteMask;
					if (0 == (TargetWriteMask & 1)) //clear write mask of rendertargets not written by pixelshader.
					{
						ColorWriteMask[Index] = 0;
					}
					TargetWriteMask >>= 1;
				}

				VulkanRHI::vkCmdSetColorBlendEnableEXT(
					CommandBuffer.GetHandle(),
					0, NewDynamicState.ColorAttachmentStates.Num(),
					BlendEnabled);

				VulkanRHI::vkCmdSetColorBlendEquationEXT(
					CommandBuffer.GetHandle(),
					0, NewDynamicState.ColorAttachmentStates.Num(),
					ColorBlendEquation);

				VulkanRHI::vkCmdSetColorWriteMaskEXT(
					CommandBuffer.GetHandle(),
					0, NewDynamicState.ColorAttachmentStates.Num(),
					ColorWriteMask);
			}

			LastDynamicStateUpdate->ShaderKeyShared = NewDynamicState.ShaderKeyShared;
			LastDynamicStateUpdate->ColorAttachmentStates = NewDynamicState.ColorAttachmentStates;
		}
	}

	if (bHasEXTVertexInputDynamicState)
	{
		bDirtyDynamicVertexInput = bDirtyDynamicVertexInput || bForceUpdate;

		if (bDirtyDynamicVertexInput ||
			(LastDynamicStateUpdate->VertexAttributes != NewDynamicState.VertexAttributes) ||
			(LastDynamicStateUpdate->VertexBindings != NewDynamicState.VertexBindings))
		{
			if (!bHasMeshShader)
			{
				TArray<VkVertexInputBindingDescription2EXT, TInlineAllocator<32>> BindingDescription;
				for (const FGfxPipelineDesc::FVertexBinding& SourceBinding : NewDynamicState.VertexBindings)
				{
					VkVertexInputBindingDescription2EXT& Binding = BindingDescription.AddZeroed_GetRef();
					Binding.sType = VK_STRUCTURE_TYPE_VERTEX_INPUT_BINDING_DESCRIPTION_2_EXT;
					SourceBinding.WriteInto(Binding);
				}

				TArray<VkVertexInputAttributeDescription2EXT, TInlineAllocator<32>> AttributeDescription;
				for (const FGfxPipelineDesc::FVertexAttribute& SourceAttr : NewDynamicState.VertexAttributes)
				{
					VkVertexInputAttributeDescription2EXT& Attr = AttributeDescription.AddZeroed_GetRef();
					Attr.sType = VK_STRUCTURE_TYPE_VERTEX_INPUT_ATTRIBUTE_DESCRIPTION_2_EXT;
					SourceAttr.WriteInto(Attr);
				}

				VulkanRHI::vkCmdSetVertexInputEXT(CommandBuffer.GetHandle(),
					BindingDescription.Num(), BindingDescription.GetData(),
					AttributeDescription.Num(), AttributeDescription.GetData());

				LastDynamicStateUpdate->VertexAttributes = NewDynamicState.VertexAttributes;
				LastDynamicStateUpdate->VertexBindings = NewDynamicState.VertexBindings;
				bDirtyDynamicVertexInput = false;
			}
		}

		// Pipelines with mesh or geometry shaders invalidate the dynamic state
		if (bHasMeshShader || bHasGeometryShader)
		{
			bDirtyDynamicVertexInput = true;
		}
	}

	if (Device.SupportsShaderObjects())
	{
		VULKAN_DYNAMIC_STATE_BLOCK_START(DepthStencil.FrontCompareMask);
		VulkanRHI::vkCmdSetStencilCompareMask(CommandBuffer.GetHandle(), VK_STENCIL_FACE_FRONT_BIT, NewDynamicState.DepthStencil.FrontCompareMask);
		VULKAN_DYNAMIC_STATE_BLOCK_END();

		VULKAN_DYNAMIC_STATE_BLOCK_START(DepthStencil.BackCompareMask);
		VulkanRHI::vkCmdSetStencilCompareMask(CommandBuffer.GetHandle(), VK_STENCIL_FACE_BACK_BIT, NewDynamicState.DepthStencil.BackCompareMask);
		VULKAN_DYNAMIC_STATE_BLOCK_END();

		VULKAN_DYNAMIC_STATE_BLOCK_START(DepthStencil.FrontWriteMask);
		VulkanRHI::vkCmdSetStencilWriteMask(CommandBuffer.GetHandle(), VK_STENCIL_FACE_FRONT_BIT, NewDynamicState.DepthStencil.FrontWriteMask);
		VULKAN_DYNAMIC_STATE_BLOCK_END();

		VULKAN_DYNAMIC_STATE_BLOCK_START(DepthStencil.BackWriteMask);
		VulkanRHI::vkCmdSetStencilWriteMask(CommandBuffer.GetHandle(), VK_STENCIL_FACE_BACK_BIT, NewDynamicState.DepthStencil.BackWriteMask);
		VULKAN_DYNAMIC_STATE_BLOCK_END();

		VULKAN_DYNAMIC_STATE_BLOCK_START(ShadingRate, RenderTargets.bHasFragmentDensityAttachment, Combiner);
		{
			if (GRHISupportsPipelineVariableRateShading && (GRHIVariableRateShadingImageDataType == VRSImage_Palette))
			{
				const bool bNewUsingVariableRateShading = (NewDynamicState.ShadingRate != EVRSShadingRate::VRSSR_1x1) ||
					(NewDynamicState.RenderTargets.bHasFragmentDensityAttachment && NewDynamicState.Combiner != EVRSRateCombiner::VRSRB_Passthrough);

				if (bNewUsingVariableRateShading)
				{
					extern VkFragmentShadingRateCombinerOpKHR GetFragmentCombinerOp(EVRSRateCombiner RateCombiner);

					const VkExtent2D FragmentSize = Device.GetBestMatchedFragmentSize(NewDynamicState.ShadingRate);
					VkFragmentShadingRateCombinerOpKHR CombinerOps[2] =
					{
						GetFragmentCombinerOp(NewDynamicState.Combiner),
						VK_FRAGMENT_SHADING_RATE_COMBINER_OP_MAX_KHR // @todo: This needs to be specified too.
					};
					VulkanRHI::vkCmdSetFragmentShadingRateKHR(CommandBuffer.GetHandle(), &FragmentSize, CombinerOps);
				}
				else
				{
					const VkExtent2D FragmentSize = { 1, 1 };
					const VkFragmentShadingRateCombinerOpKHR CombinerOps[] = { VK_FRAGMENT_SHADING_RATE_COMBINER_OP_KEEP_KHR , VK_FRAGMENT_SHADING_RATE_COMBINER_OP_KEEP_KHR };
					VulkanRHI::vkCmdSetFragmentShadingRateKHR(CommandBuffer.GetHandle(), &FragmentSize, CombinerOps);
				}
			}
			else if (bForceUpdate && Device.GetOptionalExtensionProperties().FragmentShadingRateFeatures.pipelineFragmentShadingRate)
			{
				const VkExtent2D FragmentSize = { 1, 1 };
				const VkFragmentShadingRateCombinerOpKHR CombinerOps[] = { VK_FRAGMENT_SHADING_RATE_COMBINER_OP_KEEP_KHR , VK_FRAGMENT_SHADING_RATE_COMBINER_OP_KEEP_KHR };
				VulkanRHI::vkCmdSetFragmentShadingRateKHR(CommandBuffer.GetHandle(), &FragmentSize, CombinerOps);
			}
		}
		VULKAN_DYNAMIC_STATE_BLOCK_END();
	}

#undef VULKAN_DYNAMIC_STATE_COMPARE
#undef VULKAN_DYNAMIC_STATE_UPDATE
#undef VULKAN_DYNAMIC_STATE_BLOCK_START
#undef VULKAN_DYNAMIC_STATE_BLOCK_END

	CommandBuffer.bNeedsFullDynamicStateUpdate = false;
}

void FVulkanPendingGfxState::UpdateInputAttachments(FVulkanCommandListContext& Context)
{
	const FVulkanShader* PixelShader = CurrentPipeline->GetVulkanShader(SF_Pixel);
	if (PixelShader)
	{
		const bool bHasDynamicRendering = Device.GetOptionalExtensions().HasKHRDynamicRendering;
		const FVulkanShaderHeader& Header = PixelShader->GetCodeHeader();
		const TArray<FVulkanShaderHeader::FInputAttachmentInfo>& InputAttachmentData = Header.InputAttachmentInfos;
	
		auto GetDynamicRenderingInfo = [&Context]() {

			FVulkanDynamicRenderingInfo* DynamicRenderingInfo = Context.CurrentDynamicRenderingInfo;
			if (!DynamicRenderingInfo && Context.GetParallelRenderPassInfo())
			{
				DynamicRenderingInfo = Context.GetParallelRenderPassInfo()->DynamicRenderingInfo;
			}
			checkf(DynamicRenderingInfo, TEXT("Missing DynamicRenderingInfo fo input attachments!"));
			return DynamicRenderingInfo;
		};

		for (int32 Index = 0; Index < InputAttachmentData.Num(); ++Index)
		{
			const FVulkanShaderHeader::FInputAttachmentInfo& AttachmentData = InputAttachmentData[Index];

			switch (AttachmentData.Type)
			{
			case FVulkanShaderHeader::EAttachmentType::Color0:
			case FVulkanShaderHeader::EAttachmentType::Color1:
			case FVulkanShaderHeader::EAttachmentType::Color2:
			case FVulkanShaderHeader::EAttachmentType::Color3:
			case FVulkanShaderHeader::EAttachmentType::Color4:
			case FVulkanShaderHeader::EAttachmentType::Color5:
			case FVulkanShaderHeader::EAttachmentType::Color6:
			case FVulkanShaderHeader::EAttachmentType::Color7:
			{
				const int32 ColorIndex = static_cast<int32>(AttachmentData.Type) - (int32)FVulkanShaderHeader::EAttachmentType::Color0;
				const FVulkanView* ColorView = nullptr;
				if (bHasDynamicRendering)
				{
					ColorView = GetDynamicRenderingInfo()->ColorAttachmentViews[ColorIndex];
				}
				else
				{
					check((ColorIndex >= 0) && (ColorIndex < (int32)Context.GetCurrentFramebuffer()->GetNumColorAttachments()));
					ColorView = Context.GetCurrentFramebuffer()->AttachmentTextureViews[ColorIndex];
				}

				checkf(ColorView, TEXT("Missing view pointer for color attachment %d."), ColorIndex);
				CurrentState->SetInputAttachment(ShaderStage::Pixel, AttachmentData.BindingIndex, ColorView->GetTextureView(), VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL);
			}
			break;
			case FVulkanShaderHeader::EAttachmentType::Depth:
			{
				const FVulkanView::FTextureView* PartialDepthView = nullptr;
				if (bHasDynamicRendering)
				{
					FVulkanDynamicRenderingInfo* DynamicRenderingInfo = GetDynamicRenderingInfo();
					checkf(DynamicRenderingInfo->PartialDepthAttachmentView, TEXT("Missing PartialDepthAttachmentView for input attachments!"))
					PartialDepthView = &DynamicRenderingInfo->PartialDepthAttachmentView->GetTextureView();
				}
				else
				{
					PartialDepthView = &Context.GetCurrentFramebuffer()->GetPartialDepthTextureView();
				}

				CurrentState->SetInputAttachment(ShaderStage::Pixel, AttachmentData.BindingIndex, *PartialDepthView, VK_IMAGE_LAYOUT_DEPTH_STENCIL_READ_ONLY_OPTIMAL);
			}
			break;
			default:
				checkf(0, TEXT("Unknown attachment type %d."), (int32)AttachmentData.Type);
			}
		}
	}
}


int32 GDSetCacheTargetSetsPerPool = 4096;
FAutoConsoleVariableRef CVarDSetCacheTargetSetsPerPool(
	TEXT("r.Vulkan.DSetCacheTargetSetsPerPool"),
	GDSetCacheTargetSetsPerPool,
	TEXT("Target number of descriptor set allocations per single pool.\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

int32 GDSetCacheMaxPoolLookups = 2;
FAutoConsoleVariableRef CVarDSetCacheMaxPoolLookups(
	TEXT("r.Vulkan.DSetCacheMaxPoolLookups"),
	GDSetCacheMaxPoolLookups,
	TEXT("Maximum count of pool's caches to lookup before allocating new descriptor.\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

const float DefaultPoolSizes[VK_DESCRIPTOR_TYPE_RANGE_SIZE] = 
{
	2,		// VK_DESCRIPTOR_TYPE_SAMPLER
	2,		// VK_DESCRIPTOR_TYPE_COMBINED_IMAGE_SAMPLER
	2,		// VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE
	1/8.0,	// VK_DESCRIPTOR_TYPE_STORAGE_IMAGE
	1/2.0,	// VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER
	1/8.0,	// VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER
	1/4.0,	// VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER
	1/8.0,	// VK_DESCRIPTOR_TYPE_STORAGE_BUFFER
	4,		// VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC
	1/8.0,	// VK_DESCRIPTOR_TYPE_STORAGE_BUFFER_DYNAMIC
	1/8.0	// VK_DESCRIPTOR_TYPE_INPUT_ATTACHMENT
};

FVulkanGenericDescriptorPool::FVulkanGenericDescriptorPool(FVulkanDevice* InDevice, uint32 InMaxDescriptorSets, const float PoolSizesRatio[VK_DESCRIPTOR_TYPE_RANGE_SIZE])
	: Device(InDevice)
	, MaxDescriptorSets(InMaxDescriptorSets)
	, DescriptorPool(VK_NULL_HANDLE)
{
	VkDescriptorPoolSize Types[VK_DESCRIPTOR_TYPE_RANGE_SIZE];
	FMemory::Memzero(Types);

	for (uint32 i = 0; i < VK_DESCRIPTOR_TYPE_RANGE_SIZE; ++i)
	{
		VkDescriptorType DescriptorType = static_cast<VkDescriptorType>(VK_DESCRIPTOR_TYPE_BEGIN_RANGE + i);
		
		float MinSize = FMath::Max(DefaultPoolSizes[i]*InMaxDescriptorSets, 4.0f);
		PoolSizes[i] = (uint32)FMath::Max(PoolSizesRatio[i]*InMaxDescriptorSets, MinSize);
		
		Types[i].type = DescriptorType;
		Types[i].descriptorCount = PoolSizes[i];
	}

	VkDescriptorPoolCreateInfo PoolInfo;
	ZeroVulkanStruct(PoolInfo, VK_STRUCTURE_TYPE_DESCRIPTOR_POOL_CREATE_INFO);
	PoolInfo.poolSizeCount = VK_DESCRIPTOR_TYPE_RANGE_SIZE;
	PoolInfo.pPoolSizes = Types;
	PoolInfo.maxSets = MaxDescriptorSets;

#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanVkCreateDescriptorPool);
#endif
	VERIFYVULKANRESULT(VulkanRHI::vkCreateDescriptorPool(Device->GetHandle(), &PoolInfo, VULKAN_CPU_ALLOCATOR, &DescriptorPool));

	INC_DWORD_STAT_BY(STAT_VulkanNumDescSetsTotal, MaxDescriptorSets);
	INC_DWORD_STAT(STAT_VulkanNumDescPools);
}

FVulkanGenericDescriptorPool::~FVulkanGenericDescriptorPool()
{
	if (DescriptorPool != VK_NULL_HANDLE)
	{
		DEC_DWORD_STAT_BY(STAT_VulkanNumDescSetsTotal, MaxDescriptorSets);
		DEC_DWORD_STAT(STAT_VulkanNumDescPools);

		VulkanRHI::vkDestroyDescriptorPool(Device->GetHandle(), DescriptorPool, VULKAN_CPU_ALLOCATOR);
	}
}

void FVulkanGenericDescriptorPool::Reset()
{
	check(DescriptorPool != VK_NULL_HANDLE);
	VERIFYVULKANRESULT(VulkanRHI::vkResetDescriptorPool(Device->GetHandle(), DescriptorPool, 0));
}

bool FVulkanGenericDescriptorPool::AllocateDescriptorSet(VkDescriptorSetLayout Layout, VkDescriptorSet& OutSet)
{
	check(DescriptorPool != VK_NULL_HANDLE);

	VkDescriptorSetAllocateInfo DescriptorSetAllocateInfo;
	ZeroVulkanStruct(DescriptorSetAllocateInfo, VK_STRUCTURE_TYPE_DESCRIPTOR_SET_ALLOCATE_INFO);
	DescriptorSetAllocateInfo.descriptorPool = DescriptorPool;
	DescriptorSetAllocateInfo.descriptorSetCount = 1;
	DescriptorSetAllocateInfo.pSetLayouts = &Layout;

	return (VK_SUCCESS == VulkanRHI::vkAllocateDescriptorSets(Device->GetHandle(), &DescriptorSetAllocateInfo, &OutSet));
}

FVulkanDescriptorSetCache::FVulkanDescriptorSetCache(FVulkanDevice* InDevice)
	: Device(InDevice)
	, PoolAllocRatio(0.0f)
{
	constexpr uint32 ProbePoolMaxNumSets = 128; // Used for initial estimation of the Allocation Ratio
	CachedPools.Add(MakeUnique<FCachedPool>(Device, ProbePoolMaxNumSets, DefaultPoolSizes));
}

FVulkanDescriptorSetCache::~FVulkanDescriptorSetCache()
{
}

void FVulkanDescriptorSetCache::UpdateAllocRatio()
{
	const float FilterParam = ((PoolAllocRatio > 0.0f) ? 2.0f : 0.0f);
	PoolAllocRatio = (PoolAllocRatio * FilterParam + CachedPools[0]->CalcAllocRatio()) / (FilterParam + 1.0f);
}

void FVulkanDescriptorSetCache::AddCachedPool()
{
	check(PoolAllocRatio > 0.0f);
	const uint32 MaxDescriptorSets = FMath::RoundFromZero(GDSetCacheTargetSetsPerPool / PoolAllocRatio);
	if (FreePool)
	{
		constexpr float MinErrorTolerance = -0.10f;
		constexpr float MaxErrorTolerance = 0.50f;
		const float Error = ((static_cast<float>(FreePool->GetMaxDescriptorSets()) - static_cast<float>(MaxDescriptorSets)) / static_cast<float>(MaxDescriptorSets));
		if ((Error >= MinErrorTolerance) && (Error <= MaxErrorTolerance))
		{
			FreePool->Reset();
			CachedPools.EmplaceAt(0, MoveTemp(FreePool));
			return;
		}

		// Don't write 'error' as it confuses reporting; it's a perf warning more than an actual error
		UE_LOGF(LogVulkanRHI, Display, "FVulkanDescriptorSetCache::AddCachedPool() MaxDescriptorSets Delta/Err: %f. Tolerance: [%f..%f].",	Error, MinErrorTolerance, MaxErrorTolerance);
		FreePool.Reset();
	}
	
	// use current pool sizes statistic for a new pool
	float PoolSizesRatio[VK_DESCRIPTOR_TYPE_RANGE_SIZE];
	CachedPools[0]->CalcPoolSizesRatio(PoolSizesRatio);

	CachedPools.EmplaceAt(0, MakeUnique<FCachedPool>(Device, MaxDescriptorSets, PoolSizesRatio));
}

void FVulkanDescriptorSetCache::GetDescriptorSets(const FVulkanDSetsKey& DSetsKey, const FVulkanDescriptorSetsLayout& SetsLayout,
	TArray<FVulkanDescriptorSetWriter>& DSWriters, VkDescriptorSet* OutSets)
{
	check(CachedPools.Num() > 0);

	for (int32 Index = 0; (Index < GDSetCacheMaxPoolLookups) && (Index < CachedPools.Num()); ++Index)
	{
		if (CachedPools[Index]->FindDescriptorSets(DSetsKey, OutSets))
		{
			return;
		}
	}

	bool bFirstTime = true;
	while (!CachedPools[0]->CreateDescriptorSets(DSetsKey, SetsLayout, DSWriters, OutSets))
	{
		checkf(bFirstTime, TEXT("FATAL! Failed to create descriptor sets from new pool!"));
		bFirstTime = false;
		UpdateAllocRatio();
		AddCachedPool();
	}
}

void FVulkanDescriptorSetCache::GC()
{
	// Loop is for OOM safety. Normally there would be at most 1 loop.
	while ((CachedPools.Num() > GDSetCacheMaxPoolLookups) && CachedPools.Last()->CanGC())
	{
		const uint32 RemoveIndex = (CachedPools.Num() - 1);
		if (FreePool)
		{
			UE_LOGF(LogVulkanRHI, Warning, "FVulkanDescriptorSetCache::GC() Free Pool is not empty! Too small r.Vulkan.DSetCacheTargetSetsPerPool?");
		}
		FreePool = MoveTemp(CachedPools[RemoveIndex]);
		CachedPools.RemoveAt(RemoveIndex, EAllowShrinking::No);
	}
}

const float FVulkanDescriptorSetCache::FCachedPool::MinAllocRatio = 0.5f;
const float FVulkanDescriptorSetCache::FCachedPool::MaxAllocRatio = 16.0f;

FVulkanDescriptorSetCache::FCachedPool::FCachedPool(FVulkanDevice* InDevice, uint32 InMaxDescriptorSets, const float PoolSizes[VK_DESCRIPTOR_TYPE_RANGE_SIZE])
	: SetCapacity(FMath::RoundToZero(InMaxDescriptorSets * MaxAllocRatio))
	, Pool(InDevice, InMaxDescriptorSets, PoolSizes)
	, RecentFrame(0)
{
	FMemory::Memzero(PoolSizesStatistic, sizeof(PoolSizesStatistic));
}

bool FVulkanDescriptorSetCache::FCachedPool::FindDescriptorSets(const FVulkanDSetsKey& DSetsKey, VkDescriptorSet* OutSets)
{
	FSetsEntry* SetsEntry = SetsCache.Find(DSetsKey);
	if (!SetsEntry)
	{
		return false;
	}

	FMemory::Memcpy(OutSets, &SetsEntry->Sets, sizeof(VkDescriptorSet) * SetsEntry->NumSets);
	RecentFrame = GFrameNumberRenderThread;

	return true;
}

bool FVulkanDescriptorSetCache::FCachedPool::CreateDescriptorSets(
	const FVulkanDSetsKey& DSetsKey, 
	const FVulkanDescriptorSetsLayout& SetsLayout,
	TArray<FVulkanDescriptorSetWriter>& DSWriters, 
	VkDescriptorSet* OutSets)
{
	FSetsEntry NewSetEntry{};

	NewSetEntry.NumSets = DSWriters.Num();
	check(NewSetEntry.NumSets <= NewSetEntry.Sets.Num());
	check(SetsLayout.GetHandles().Num() <= NewSetEntry.NumSets);

	for (int32 Index = 0; Index < NewSetEntry.NumSets; ++Index)
	{
		FVulkanDescriptorSetWriter& DSWriter = DSWriters[Index];
		if (DSWriter.GetNumWrites() == 0)
		{
			NewSetEntry.Sets[Index] = VK_NULL_HANDLE;
			continue;
		}
		if (VkDescriptorSet* FoundSet = SetCache.Find(DSWriter.GetKey()))
		{
			NewSetEntry.Sets[Index] = *FoundSet;
			continue;
		}

		if ((SetCache.Num() == SetCapacity) ||
			!Pool.AllocateDescriptorSet(SetsLayout.GetHandles()[Index], NewSetEntry.Sets[Index]))
		{
			return false;
		}
		SetCache.Emplace(DSWriter.GetKey().CopyDeep(), NewSetEntry.Sets[Index]);

		DSWriter.SetDescriptorSet(NewSetEntry.Sets[Index]);
		DSWriter.CheckAllWritten();

		for (int32 i = 0; i < VK_DESCRIPTOR_TYPE_RANGE_SIZE; ++i)
		{
			VkDescriptorType DescriptorType = static_cast<VkDescriptorType>(VK_DESCRIPTOR_TYPE_BEGIN_RANGE + i);
			PoolSizesStatistic[i] += SetsLayout.GetTypesUsed(DescriptorType);
		}

#if VULKAN_ENABLE_AGGRESSIVE_STATS
		INC_DWORD_STAT_BY(STAT_VulkanNumUpdateDescriptors, DSWriter.GetNumWrites());
		INC_DWORD_STAT(STAT_VulkanNumDescSets);
		SCOPE_CYCLE_COUNTER(STAT_VulkanVkUpdateDS);
#endif
		VulkanRHI::vkUpdateDescriptorSets(Pool.GetDevice()->GetHandle(),
			DSWriter.GetNumWrites(), DSWriter.GetWriteDescriptors(), 0, nullptr);
	}

	SetsCache.Emplace(DSetsKey.CopyDeep(), MoveTemp(NewSetEntry));

	FMemory::Memcpy(OutSets, &NewSetEntry.Sets, sizeof(VkDescriptorSet) * NewSetEntry.NumSets);
	RecentFrame = GFrameNumberRenderThread;

	return true;
}

void FVulkanDescriptorSetCache::FCachedPool::Reset()
{
	Pool.Reset();
	SetsCache.Reset();
	SetCache.Reset();
	FMemory::Memzero(PoolSizesStatistic, sizeof(PoolSizesStatistic));
}

bool FVulkanDescriptorSetCache::FCachedPool::CanGC() const
{
	return ((int32)(GFrameNumberRenderThread - RecentFrame) > GVulkanNumFramesToWaitForResourceDelete);
}

float FVulkanDescriptorSetCache::FCachedPool::CalcAllocRatio() const
{
	float AllocRatio = (static_cast<float>(SetCache.Num()) / static_cast<float>(Pool.GetMaxDescriptorSets()));
	if (AllocRatio < MinAllocRatio)
	{
		UE_LOGF(LogVulkanRHI, Warning, "FVulkanDescriptorSetCache::FCachedPool::CalcAllocRatio() Pool Allocation Ratio is too low: %f. Using: %f.", AllocRatio, MinAllocRatio);
		AllocRatio = MinAllocRatio;
	}
	return AllocRatio;
}

void FVulkanDescriptorSetCache::FCachedPool::CalcPoolSizesRatio(float PoolSizesRatio[VK_DESCRIPTOR_TYPE_RANGE_SIZE])
{
	int32 NumSets = FMath::Max(SetCache.Num(), 1);
	for (uint32 i = 0; i < VK_DESCRIPTOR_TYPE_RANGE_SIZE; ++i)
	{
		PoolSizesRatio[i] = PoolSizesStatistic[i]/(float)NumSets;
	}
}

bool FVulkanPendingGfxState::SetGfxPipeline(FVulkanGraphicsPipelineState* InGfxPipeline, bool bForceReset)
{
	bool bChanged = bForceReset;

	if (InGfxPipeline != CurrentPipeline)
	{
		CurrentPipeline = InGfxPipeline;
		FVulkanGraphicsPipelineDescriptorState** Found = PipelineStates.Find(InGfxPipeline);
		if (Found)
		{
			CurrentState = *Found;
			check(CurrentState->GfxPipeline == InGfxPipeline);
		}
		else
		{
			CurrentState = new FVulkanGraphicsPipelineDescriptorState(Device, InGfxPipeline);
			PipelineStates.Add(CurrentPipeline, CurrentState);
		}

		PrimitiveType = InGfxPipeline->PrimitiveType;
		bChanged = true;
	}

	if (bChanged || bForceReset)
	{
		CurrentState->Reset();
	}

	return bChanged;
}
