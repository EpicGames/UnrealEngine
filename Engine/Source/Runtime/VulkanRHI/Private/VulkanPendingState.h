// Copyright Epic Games, Inc. All Rights Reserved..

/*=============================================================================
	VulkanPendingState.h: Private VulkanPendingState definitions.
=============================================================================*/

#pragma once

// Dependencies
#include "VulkanConfiguration.h"
#include "VulkanState.h"
#include "VulkanResources.h"
#include "VulkanUtil.h"
#include "VulkanViewport.h"
#include "VulkanDynamicRHI.h"
#include "VulkanPipeline.h"
#include "VulkanPipelineState.h"

// All the current compute pipeline states in use
class FVulkanPendingComputeState
{
public:
	FVulkanPendingComputeState(FVulkanDevice& InDevice)
		: Device(InDevice)
	{
	}

	~FVulkanPendingComputeState();

	void Reset()
	{
		CurrentPipeline = nullptr;
		CurrentState = nullptr;
	}

	void SetComputePipeline(FVulkanComputePipeline* InComputePipeline)
	{
		if (InComputePipeline != CurrentPipeline)
		{
			CurrentPipeline = InComputePipeline;
			FVulkanComputePipelineDescriptorState** Found = PipelineStates.Find(InComputePipeline);
			if (Found)
			{
				CurrentState = *Found;
				check(CurrentState->ComputePipeline == InComputePipeline);
			}
			else
			{
				CurrentState = new FVulkanComputePipelineDescriptorState(Device, InComputePipeline);
				PipelineStates.Add(CurrentPipeline, CurrentState);
			}

			CurrentState->Reset();
		}
	}

	void PrepareForDispatch(FVulkanCommandListContext& Context);

	inline const FVulkanComputeShader* GetCurrentShader() const
	{
		return CurrentPipeline ? ResourceCast(CurrentPipeline->GetComputeShader()) : nullptr;
	}

	FVulkanPackedUniformBufferState& GetUniformBuffers(uint8 Stage)
	{
		return CurrentState->GetUniformBuffers(Stage);
	}

	inline void SetSamplerStateForUBResource(uint32 DescriptorSet, uint32 BindingIndex, FVulkanSamplerState* Sampler)
	{
		CurrentState->SetSamplerState(DescriptorSet, BindingIndex, Sampler);
	}

	void NotifyDeletedPipeline(FVulkanComputePipeline* Pipeline)
	{
		PipelineStates.Remove(Pipeline);
	}

	FVulkanComputePipelineDescriptorState& GetCurrentState()
	{
		return *CurrentState;
	}

protected:
	FVulkanComputePipeline* CurrentPipeline = nullptr;
	FVulkanComputePipelineDescriptorState* CurrentState = nullptr;

	TMap<FVulkanComputePipeline*, FVulkanComputePipelineDescriptorState*> PipelineStates;

	FVulkanDevice& Device;

	friend class FVulkanCommandListContext;
};

// All the current gfx pipeline states in use
class FVulkanPendingGfxState
{
public:
	FVulkanPendingGfxState(FVulkanDevice& InDevice)
		: Device(InDevice)
	{
		Reset();
	}

	~FVulkanPendingGfxState();

	void Reset()
	{
		Viewports.SetNumZeroed(1);
		Scissors.SetNumZeroed(1);
		StencilRef = 0;
		bScissorEnable = false;

		CurrentPipeline = nullptr;
		CurrentState = nullptr;
		bDirtyVertexStreams = true;
		bDirtyDynamicVertexInput = true;

		PrimitiveType = PT_Num;

		//#todo-rco: Would this cause issues?
		//FMemory::Memzero(PendingStreams);
	}

	const uint64 GetCurrentShaderKey(EShaderFrequency Frequency) const
	{
		return (CurrentPipeline ? CurrentPipeline->GetShaderKey(Frequency) : 0);
	}

	const uint64 GetCurrentShaderKey(ShaderStage::EStage Stage) const
	{
		return GetCurrentShaderKey(ShaderStage::GetFrequencyForGfxStage(Stage));
	}

	const FVulkanShader* GetCurrentShader(EShaderFrequency Frequency) const
	{
		return (CurrentPipeline ? CurrentPipeline->GetVulkanShader(Frequency) : nullptr);
	}

	void SetViewport(float MinX, float MinY, float MinZ, float MaxX, float MaxY, float MaxZ)
	{
		Viewports.SetNumZeroed(1);

		Viewports[0].x = MinX;
		Viewports[0].y = MinY;
		Viewports[0].width = MaxX - MinX;
		Viewports[0].height = MaxY - MinY;
		Viewports[0].minDepth = MinZ;
		if (MinZ == MaxZ)
		{
			// Engine pases in some cases MaxZ as 0.0
			Viewports[0].maxDepth = MinZ + 1.0f;
		}
		else
		{
			Viewports[0].maxDepth = MaxZ;
		}

		SetScissorRect((uint32)MinX, (uint32)MinY, (uint32)(MaxX - MinX), (uint32)(MaxY - MinY));
		bScissorEnable = false;
	}

	void SetMultiViewport(const TArrayView<VkViewport>& InViewports)
	{
		Viewports = InViewports;

		// Set the scissor rects appropriately.
		Scissors.SetNumZeroed(Viewports.Num());
		for (int32 Idx = 0; Idx < Scissors.Num(); ++Idx)
		{
			Scissors[Idx].offset.x = Viewports[Idx].x;
			Scissors[Idx].offset.y = Viewports[Idx].y;
			Scissors[Idx].extent.width = Viewports[Idx].width;
			Scissors[Idx].extent.height = Viewports[Idx].height;
		}
		bScissorEnable = true;
	}

	void SetScissor(bool bInEnable, uint32 MinX, uint32 MinY, uint32 MaxX, uint32 MaxY)
	{
		if (bInEnable)
		{
			SetScissorRect(MinX, MinY, MaxX - MinX, MaxY - MinY);
		}
		else
		{
			checkf(Viewports.Num() > 0, TEXT("At least one Viewport is expected to be configured."));
			SetScissorRect(Viewports[0].x, Viewports[0].y, Viewports[0].width, Viewports[0].height);
		}

		bScissorEnable = bInEnable;
	}

	void SetScissorRect(uint32 MinX, uint32 MinY, uint32 Width, uint32 Height)
	{
		Scissors.SetNumZeroed(1);

		Scissors[0].offset.x = MinX;
		Scissors[0].offset.y = MinY;
		Scissors[0].extent.width = Width;
		Scissors[0].extent.height = Height;
	}

	void SetStreamSource(uint32 StreamIndex, VkBuffer VertexBuffer, uint32 Offset)
	{
		PendingStreams[StreamIndex].Stream = VertexBuffer;
		PendingStreams[StreamIndex].BufferOffset = Offset;
		bDirtyVertexStreams = true;
	}

	void Bind(FVulkanCommandBuffer& CommandBuffer)
	{
		CurrentPipeline->Bind(CommandBuffer);
	}

	FVulkanPackedUniformBufferState& GetUniformBuffers(uint8 Stage)
	{
		return CurrentState->GetUniformBuffers(Stage);
	}

	void PrepareForDraw(FVulkanCommandListContext& Context);

	bool SetGfxPipeline(FVulkanGraphicsPipelineState* InGfxPipeline, bool bForceReset);

	void UpdateDynamicStates(FVulkanCommandBuffer& CommandBuffer);

	void SetStencilRef(uint32 InStencilRef)
	{
		if (InStencilRef != StencilRef)
		{
			StencilRef = InStencilRef;
		}
	}

	void NotifyDeletedPipeline(FVulkanGraphicsPipelineState* Pipeline)
	{
		PipelineStates.Remove(Pipeline);
	}

	FVulkanGraphicsPipelineDescriptorState& GetCurrentState()
	{
		return *CurrentState;
	}

protected:
	TArray<VkViewport, TInlineAllocator<2>> Viewports;
	TArray<VkRect2D, TInlineAllocator<2>> Scissors;

	EPrimitiveType PrimitiveType = PT_Num;
	uint32 StencilRef;
	bool bScissorEnable;

	bool bNeedToClear;

	FVulkanGraphicsPipelineState* CurrentPipeline;
	FVulkanGraphicsPipelineDescriptorState* CurrentState;

	TMap<FVulkanGraphicsPipelineState*, FVulkanGraphicsPipelineDescriptorState*> PipelineStates;

	struct FVertexStream
	{
		FVertexStream() :
			Stream(VK_NULL_HANDLE),
			BufferOffset(0)
		{
		}

		VkBuffer Stream;
		uint32 BufferOffset;
	};
	FVertexStream PendingStreams[MaxVertexElementCount];
	bool bDirtyVertexStreams;
	bool bDirtyDynamicVertexInput;

	void UpdateInputAttachments(FVulkanCommandListContext& Context);

	FVulkanDevice& Device;

	friend class FVulkanCommandListContext;
};
