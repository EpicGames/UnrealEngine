// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanPipeline.cpp: Vulkan device RHI implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanPipeline.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "RenderingThread.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "VulkanPendingState.h"
#include "VulkanContext.h"
#include "GlobalShader.h"
#include "VulkanLLM.h"
#include "Misc/ScopeRWLock.h"
#include "VulkanChunkedPipelineCache.h"
#include "VulkanRenderTargetLayout.h"
#include "VulkanRenderpass.h"
#include "VulkanBindlessDescriptorManager.h"
#include "VulkanShaderObjectManager.h"

#define LRU_DEBUG 0
#if !UE_BUILD_SHIPPING
#define LRUPRINT(...) FPlatformMisc::LowLevelOutputDebugStringf(__VA_ARGS__)
#if LRU_DEBUG
#define LRUPRINT_DEBUG(...) FPlatformMisc::LowLevelOutputDebugStringf(__VA_ARGS__)
#endif
#else
#define LRUPRINT(...) do{}while(0)
#endif

#ifndef LRUPRINT_DEBUG
#define LRUPRINT_DEBUG(...) do{}while(0)
#endif



#if PLATFORM_ANDROID
#define LRU_MAX_PIPELINE_SIZE 10
#define LRU_PIPELINE_CAPACITY 2048
#else
#define LRU_MAX_PIPELINE_SIZE 512 //needs to be super high to work on pc.
#define LRU_PIPELINE_CAPACITY 8192
#endif



#if !UE_BUILD_SHIPPING
static TAtomic<uint64> SGraphicsRHICount;
static TAtomic<uint64> SPipelineCount;
static TAtomic<uint64> SPipelineGfxCount;
#endif

static const double HitchTime = 1.0 / 1000.0;

static TAutoConsoleVariable<int32> CVarPipelineDebugForceEvictImmediately(
	TEXT("r.Vulkan.PipelineDebugForceEvictImmediately"),
	0,
	TEXT("1: Force all created PSOs to be evicted immediately. Only for debugging"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);



static TAutoConsoleVariable<int32> CVarPipelineLRUCacheEvictBinaryPreloadScreen(
	TEXT("r.Vulkan.PipelineLRUCacheEvictBinaryPreloadScreen"),
	0,
	TEXT("1: Use a preload screen while loading preevicted PSOs ala r.Vulkan.PipelineLRUCacheEvictBinary"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarEnableLRU(
	TEXT("r.Vulkan.EnablePipelineLRUCache"),
	0,
	TEXT("Pipeline LRU cache.\n")
	TEXT("0: disable LRU\n")
	TEXT("1: Enable LRU"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarPipelineLRUCacheEvictBinary(
	TEXT("r.Vulkan.PipelineLRUCacheEvictBinary"),
	0,
	TEXT("0: create pipelines in from the binary PSO cache and binary shader cache and evict them only as it fills up.\n")
	TEXT("1: don't create pipelines....just immediately evict them"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);


static TAutoConsoleVariable<int32> CVarLRUMaxPipelineSize(
	TEXT("r.Vulkan.PipelineLRUSize"),
	LRU_MAX_PIPELINE_SIZE * 1024 * 1024,
	TEXT("Maximum size of shader memory ."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLRUPipelineCapacity(
	TEXT("r.Vulkan.PipelineLRUCapactiy"),
	LRU_PIPELINE_CAPACITY,
	TEXT("Maximum no. of PSOs in LRU."),
	ECVF_RenderThreadSafe| ECVF_ReadOnly);

static TAutoConsoleVariable<int32> GEnablePipelineCacheLoadCvar(
	TEXT("r.Vulkan.PipelineCacheLoad"),
	1,
	TEXT("0 to disable loading the pipeline cache.\n")
	TEXT("1 to enable using pipeline cache.\n")
);

static int32 GEnablePipelineCacheCompression = 1;
static FAutoConsoleVariableRef GEnablePipelineCacheCompressionCvar(
	TEXT("r.Vulkan.PipelineCacheCompression"),
	GEnablePipelineCacheCompression,
	TEXT("Enable/disable compression on the Vulkan pipeline cache disk file\n"),
	ECVF_Default | ECVF_RenderThreadSafe
);

static bool GEnableGfxPipelineCrashContext = !PLATFORM_ANDROID;
static FAutoConsoleVariableRef GEnableGfxPipelineCrashContextCvar(
	TEXT("r.Vulkan.EnableGfxPipelineCrashContext"),
	GEnableGfxPipelineCrashContext,
	TEXT("Whether to include additional crash context information on crashes coming from GFX pipeline creation. (default: mobile=disabled desktop=enabled)"),
	ECVF_Default | ECVF_RenderThreadSafe
);

static bool GEnableComputePipelineCrashContext = !PLATFORM_ANDROID;
static FAutoConsoleVariableRef GEnableComputePipelineCrashContextCvar(
	TEXT("r.Vulkan.EnableComputePipelineCrashContext"),
	GEnableComputePipelineCrashContext,
	TEXT("Whether to include additional crash context information on crashes coming from GFX pipeline creation. (default: mobile=disabled desktop=enabled)"),
	ECVF_Default | ECVF_RenderThreadSafe
);

enum class ESingleThreadedPSOCreateMode
{
	None = 0,
	All = 1,
	Precompile = 2,
	NonPrecompiled = 3,
};

static int32 GVulkanPSOForceSingleThreaded = (int32)ESingleThreadedPSOCreateMode::None;
static FAutoConsoleVariableRef GVulkanPSOForceSingleThreadedCVar(
	TEXT("r.Vulkan.ForcePSOSingleThreaded"),
	GVulkanPSOForceSingleThreaded,
	TEXT("Enable to force singlethreaded creation of PSOs. Only intended as a workaround for buggy drivers\n")
	TEXT("0: (default) Allow Async precompile PSO creation.\n")
	TEXT("1: force singlethreaded creation of all PSOs.\n")
	TEXT("2: force singlethreaded creation of precompile PSOs only.\n")
	TEXT("3: force singlethreaded creation of non-precompile PSOs only."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

static int32 GVulkanPSOLRUEvictAfterUnusedFrames = 0;
static FAutoConsoleVariableRef GVulkanPSOLRUEvictAfterUnusedFramesCVar(
	TEXT("r.Vulkan.PSOLRUEvictAfterUnusedFrames"),
	GVulkanPSOLRUEvictAfterUnusedFrames,
	TEXT("0: unused PSOs are not removed from the PSO LRU cache. (default)\n")
	TEXT(">0: The number of frames an unused PSO can remain in the PSO LRU cache. When this is exceeded the PSO is destroyed and memory returned to the system. This can save memory with the risk of increased hitching.")
	, ECVF_RenderThreadSafe
);


static int32 GVulkanReleaseShaderModuleWhenEvictingPSO = 0;
static FAutoConsoleVariableRef GVulkanReleaseShaderModuleWhenEvictingPSOCVar(
	TEXT("r.Vulkan.ReleaseShaderModuleWhenEvictingPSO"),
	GVulkanReleaseShaderModuleWhenEvictingPSO,
	TEXT("0: shader modules remain when a PSO is removed from the PSO LRU cache. (default)\n")
	TEXT("1: shader modules are destroyed when a PSO is removed from the PSO LRU cache. This can save memory at the risk of increased hitching and cpu cost.")
	,ECVF_RenderThreadSafe
);


int32 GVulkanGraphicPipelineLibraryLinkingMode = 0;
static TAutoConsoleVariable<int32> GVulkanGraphicPipelineLibraryLinkingModeCvar(
	TEXT("r.Vulkan.GraphicPipelineLibraryLinkingMode"),
	0,
	TEXT("0 to disable link time optimization (faster on CPU, slower on GPU).\n")
	TEXT("1 to enable link time optimization (slower on CPU, faster on GPU).\n")
);  // :todo-jn: Add "2": create unoptimized pipeline on first draw, but replace it eventually with optimized pipeline.




template <typename TRHIType, typename TVulkanType>
static inline FShaderHash GetShaderHash(TRHIType* RHIShader)
{
	if (RHIShader)
	{
		const TVulkanType* VulkanShader = ResourceCast<TRHIType>(RHIShader);
		const FVulkanShader* Shader = static_cast<const FVulkanShader*>(VulkanShader);
		check(Shader);
		return Shader->GetCodeHeader().CodeHash;
	}

	FShaderHash Dummy;
	return Dummy;
}

static inline FShaderHash GetShaderHashForStage(const FGraphicsPipelineStateInitializer& PSOInitializer, ShaderStage::EStage Stage)
{
	switch (Stage)
	{
	case ShaderStage::Vertex:		return GetShaderHash<FRHIVertexShader, FVulkanVertexShader>(PSOInitializer.BoundShaderState.VertexShaderRHI);
	case ShaderStage::Pixel:		return GetShaderHash<FRHIPixelShader, FVulkanPixelShader>(PSOInitializer.BoundShaderState.PixelShaderRHI);
#if PLATFORM_SUPPORTS_MESH_SHADERS
	case ShaderStage::Mesh:			return GetShaderHash<FRHIMeshShader, FVulkanMeshShader>(PSOInitializer.BoundShaderState.GetMeshShader());
	case ShaderStage::Task:			return GetShaderHash<FRHIAmplificationShader, FVulkanTaskShader>(PSOInitializer.BoundShaderState.GetAmplificationShader());
#endif
#if VULKAN_SUPPORTS_GEOMETRY_SHADERS
	case ShaderStage::Geometry:		return GetShaderHash<FRHIGeometryShader, FVulkanGeometryShader>(PSOInitializer.BoundShaderState.GetGeometryShader());
#endif
	default:			check(0);	break;
	}

	FShaderHash Dummy;
	return Dummy;
}

FVulkanPipeline::FVulkanPipeline(FVulkanDevice* InDevice)
	: Device(InDevice)
	, Pipeline(VK_NULL_HANDLE)
	, Layout(nullptr)
{
#if !UE_BUILD_SHIPPING
	SPipelineCount++;
#endif
}

FVulkanPipeline::~FVulkanPipeline()
{
#if !UE_BUILD_SHIPPING
	SPipelineCount--;
#endif	
	if (Pipeline != VK_NULL_HANDLE)
	{
		Device->GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::Pipeline, Pipeline);
		Pipeline = VK_NULL_HANDLE;
	}
	/* we do NOT own Layout !*/
}

FVulkanComputePipeline::FVulkanComputePipeline(FVulkanDevice* InDevice, const FComputePipelineStateInitializer& Initializer)
	: FVulkanPipeline(InDevice)
	, FRHIComputePipelineState(Initializer.ComputeShader)
	, bUsesBindless(ResourceCast(Initializer.ComputeShader)->UsesBindless())
	, SpecializationConstants(Initializer.SpecializationConstants)
{
	INC_DWORD_STAT(STAT_VulkanNumComputePSOs);
}

FVulkanComputePipeline::~FVulkanComputePipeline()
{
	Device->NotifyDeletedComputePipeline(this);
	DEC_DWORD_STAT(STAT_VulkanNumComputePSOs);
}

FVulkanGraphicsPipelineState::~FVulkanGraphicsPipelineState()
{
#if !UE_BUILD_SHIPPING
	SGraphicsRHICount--;
#endif	
	DEC_DWORD_STAT(STAT_VulkanNumGraphicsPSOs);

	Device->PipelineStateCache->NotifyDeletedGraphicsPSO(this);

	for (int ShaderStageIndex = 0; ShaderStageIndex < ShaderStage::NumGraphicsStages; ShaderStageIndex++)
	{
		if (VulkanShaders[ShaderStageIndex] != nullptr)
		{
			VulkanShaders[ShaderStageIndex]->Release();
		}
	}
}

void FVulkanGraphicsPipelineState::GetOrCreateShaderModules(TRefCountPtr<FVulkanShaderModule> (&ShaderModulesOUT)[ShaderStage::NumGraphicsStages], FVulkanShader*const* Shaders)
{
	for (int32 Index = 0; Index < ShaderStage::NumGraphicsStages; ++Index)
	{
		check(!ShaderModulesOUT[Index].IsValid());
		FVulkanShader* Shader = Shaders[Index];
		if (Shader)
		{
			ShaderModulesOUT[Index] = Shader->GetOrCreateHandle(Desc, Layout, Layout->GetDescriptorSetLayoutHash());
		}
	}
}

FVulkanShader::FSpirvCode FVulkanGraphicsPipelineState::GetPatchedSpirvCode(FVulkanShader* Shader)
{
	check(Shader);
	return Shader->GetPatchedSpirvCode(Desc, Layout);
}

void FVulkanGraphicsPipelineState::PurgeShaderModules(FVulkanShader*const* Shaders)
{
	for (int32 Index = 0; Index < ShaderStage::NumGraphicsStages; ++Index)
	{
		FVulkanShader* Shader = Shaders[Index];
		if (Shader)
		{
			Shader->PurgeShaderModules();
		}
	}
}

FVulkanPipelineStateCacheManager::FVulkanPipelineStateCacheManager(FVulkanDevice* InDevice)
	: Device(InDevice)
	, bEvictImmediately(false)
	, bPrecompilingCacheLoadedFromFile(false)
{
	bUseLRU = (int32)CVarEnableLRU.GetValueOnAnyThread() != 0;
	LRUUsedPipelineMax = CVarLRUPipelineCapacity.GetValueOnAnyThread();

	if (InDevice->SupportsGraphicPipelineLibraries())
	{
		CreateVertexInputLibraries();
		CreateEmptyFragmentStageLibrary();
	}

	constexpr uint32 SpecializationConstantSize = sizeof(uint32);
	for (uint32 MapEntryIndex = 0; MapEntryIndex < MaxNumSpecializationConstants; ++MapEntryIndex)
	{
		SpecializationMapEntries[MapEntryIndex] = {
			.constantID = MapEntryIndex,
			.offset = MapEntryIndex * SpecializationConstantSize,
			.size = SpecializationConstantSize
		};
	}
}


FVulkanPipelineStateCacheManager::~FVulkanPipelineStateCacheManager()
{
	if (OnShaderPipelineCacheOpenedDelegate.IsValid())
	{
		FShaderPipelineCache::GetCacheOpenedDelegate().Remove(OnShaderPipelineCacheOpenedDelegate);
	}

	if (OnShaderPipelineCachePrecompilationCompleteDelegate.IsValid())
	{
		FShaderPipelineCache::GetPrecompilationCompleteDelegate().Remove(OnShaderPipelineCachePrecompilationCompleteDelegate);
	}
	DestroyCache();


	if (VertexInputPipelineLibraryVS)
	{
		VulkanRHI::vkDestroyPipeline(Device->GetHandle(), VertexInputPipelineLibraryVS, VULKAN_CPU_ALLOCATOR);
		VertexInputPipelineLibraryVS = VK_NULL_HANDLE;
	}
	if (VertexInputPipelineLibraryMS)
	{
		VulkanRHI::vkDestroyPipeline(Device->GetHandle(), VertexInputPipelineLibraryMS, VULKAN_CPU_ALLOCATOR);
		VertexInputPipelineLibraryMS = VK_NULL_HANDLE;
	}
	if (EmptyFragmentStageLibrary)
	{
		VulkanRHI::vkDestroyPipeline(Device->GetHandle(), EmptyFragmentStageLibrary, VULKAN_CPU_ALLOCATOR);
		EmptyFragmentStageLibrary = VK_NULL_HANDLE;
	}
	for (auto& Pair : FragmentOutputStatePipelineLibraries)
	{
		VulkanRHI::vkDestroyPipeline(Device->GetHandle(), Pair.Value, VULKAN_CPU_ALLOCATOR);
	}
	FragmentOutputStatePipelineLibraries.Empty();

	// Only destroy layouts when quitting
	for (auto& Pair : LayoutMap)
	{
		delete Pair.Value;
	}
	for (auto& Pair : DSetLayoutMap)
	{
		VulkanRHI::vkDestroyDescriptorSetLayout(Device->GetHandle(), Pair.Value.Handle, VULKAN_CPU_ALLOCATOR);
	}

	{
		//TODO: Save PSOCache here?!
		FScopedPipelineCache PipelineCacheExclusive = GlobalPSOCache.Get(EPipelineCacheAccess::Exclusive);
		VulkanRHI::vkDestroyPipelineCache(Device->GetHandle(), PipelineCacheExclusive.Get(), VULKAN_CPU_ALLOCATOR);
	}

	{
		FScopedPipelineCache PipelineCacheExclusive = CurrentPrecompilingPSOCache.Get(EPipelineCacheAccess::Exclusive);
		if (PipelineCacheExclusive.Get() != VK_NULL_HANDLE)
		{
			//If CurrentOpenedPSOCache is still valid then it has never received OnShaderPipelineCachePrecompilationComplete callback, so we are not going to save it's content to disk as it's most likely is incomplete at this point
			VulkanRHI::vkDestroyPipelineCache(Device->GetHandle(), PipelineCacheExclusive.Get(), VULKAN_CPU_ALLOCATOR);
		}
	}
}

bool FVulkanPipelineStateCacheManager::Load(const TArray<FString>& CacheFilenames, FPipelineCache& Cache)
{
	FScopedPipelineCache PipelineCacheExclusive = Cache.Get(EPipelineCacheAccess::Exclusive);

	bool bResult = false;
	// Try to load device cache first
	for (const FString& CacheFilename : CacheFilenames)
	{
		double BeginTime = FPlatformTime::Seconds();
		FString BinaryCacheFilename = FVulkanPlatform::CreatePSOBinaryCacheFilename(Device, CacheFilename);
		FString BinaryCacheLoadingFilename = BinaryCacheFilename + TEXT(".temp");

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		if (PlatformFile.FileExists(*BinaryCacheLoadingFilename))
		{
			PlatformFile.DeleteFile(*BinaryCacheLoadingFilename);
		}

		if (!PlatformFile.FileExists(*BinaryCacheFilename))
		{
			UE_LOGF(LogVulkanRHI, Display, "FVulkanPipelineStateCacheManager: Binary pipeline cache '%ls' not found.", *BinaryCacheFilename);
			continue;
		}
		if (!PlatformFile.MoveFile(*BinaryCacheLoadingFilename, *BinaryCacheFilename))
		{
			UE_LOGF(LogVulkanRHI, Warning, "FVulkanPipelineStateCacheManager: Failed to rename '%ls' to '%ls' before load. Skipping.", *BinaryCacheFilename, *BinaryCacheLoadingFilename);
			continue;
		}

		TArray<uint8> DeviceCache;
		if (FFileHelper::LoadFileToArray(DeviceCache, *BinaryCacheLoadingFilename, FILEREAD_Silent))
		{
			if (FVulkanPlatform::PSOBinaryCacheMatches(Device, DeviceCache))
			{
				VkPipelineCacheCreateInfo PipelineCacheInfo;
				ZeroVulkanStruct(PipelineCacheInfo, VK_STRUCTURE_TYPE_PIPELINE_CACHE_CREATE_INFO);
				PipelineCacheInfo.initialDataSize = DeviceCache.Num();
				PipelineCacheInfo.pInitialData = DeviceCache.GetData();

				VkResult LoadResult = VK_SUCCESS;
				if (PipelineCacheExclusive.Get() == VK_NULL_HANDLE)
				{
					LoadResult = VulkanRHI::vkCreatePipelineCache(Device->GetHandle(), &PipelineCacheInfo, VULKAN_CPU_ALLOCATOR, &PipelineCacheExclusive.Get());
					// Overkill - spec says the out handle is VK_NULL_HANDLE on failure, but if anything is left make sure its gone.
					if (LoadResult != VK_SUCCESS && !ensure(PipelineCacheExclusive.Get() == VK_NULL_HANDLE))
					{
						UE_LOGF(LogVulkanRHI, Warning, "FVulkanPipelineStateCacheManager: received cache after error.");
						VulkanRHI::vkDestroyPipelineCache(Device->GetHandle(), PipelineCacheExclusive.Get(), VULKAN_CPU_ALLOCATOR);
						PipelineCacheExclusive.Get() = VK_NULL_HANDLE;
					}
				}
				else
				{
					//TODO: assert on reopening the same cache twice?!
					// if we have one already, create a temp one and merge it
					VkPipelineCache TempPipelineCache = VK_NULL_HANDLE;
					LoadResult = VulkanRHI::vkCreatePipelineCache(Device->GetHandle(), &PipelineCacheInfo, VULKAN_CPU_ALLOCATOR, &TempPipelineCache);
					if (LoadResult == VK_SUCCESS)
					{
						// if the merge fails then we discard the file. take no risks.
						LoadResult = VulkanRHI::vkMergePipelineCaches(Device->GetHandle(), PipelineCacheExclusive.Get(), 1, &TempPipelineCache);
					}
					if (TempPipelineCache != VK_NULL_HANDLE)
					{
						VulkanRHI::vkDestroyPipelineCache(Device->GetHandle(), TempPipelineCache, VULKAN_CPU_ALLOCATOR);
					}
				}

				if (LoadResult == VK_SUCCESS)
				{
					// Restore the original name so the next run reuses the cache.
					if (!PlatformFile.MoveFile(*BinaryCacheFilename, *BinaryCacheLoadingFilename))
					{
						// This shouldn't happen. nothing should occupy BinaryCacheFilename at this point.
						UE_LOGF(LogVulkanRHI, Warning, "FVulkanPipelineStateCacheManager: Failed to rename back to '%ls' after successful load. '%ls' will be removed.", *BinaryCacheFilename, *BinaryCacheLoadingFilename);
					}
					double EndTime = FPlatformTime::Seconds();
					UE_LOGF(LogVulkanRHI, Display, "FVulkanPipelineStateCacheManager: Loaded binary pipeline cache %ls in %.3f seconds", *BinaryCacheFilename, (float)(EndTime - BeginTime));
					bResult = true;
				}
				else
				{
					UE_LOGF(LogVulkanRHI, Warning, "FVulkanPipelineStateCacheManager: failed to load pipeline cache %ls (VkResult=%d).", *BinaryCacheFilename, (int32)LoadResult);
				}
			}
			else
			{
				UE_LOGF(LogVulkanRHI, Error, "FVulkanPipelineStateCacheManager: Mismatched binary pipeline cache %ls.", *BinaryCacheFilename);
			}
		}
		else
		{
			UE_LOGF(LogVulkanRHI, Display, "FVulkanPipelineStateCacheManager: Binary pipeline cache '%ls' could not be loaded.", *BinaryCacheLoadingFilename);
		}

		// delete any renamed file that still remains, it cant be used.
		PlatformFile.DeleteFile(*BinaryCacheLoadingFilename);
	}

	//TODO: how to load LRU cache as it will have info about PSOs from multiple caches
	if(CVarEnableLRU.GetValueOnAnyThread() != 0)
	{
		for (const FString& CacheFilename : CacheFilenames)
		{
			double BeginTime = FPlatformTime::Seconds();
			FString LruCacheFilename = FVulkanPlatform::CreatePSOBinaryCacheFilename(Device, CacheFilename);
			LruCacheFilename += TEXT(".lru");
			LruCacheFilename.ReplaceInline(TEXT("TempScanVulkanPSO_"), TEXT("VulkanPSO_"));  //lru files do not use the rename trick...but are still protected against corruption indirectly

			TArray<uint8> MemFile;
			if (FFileHelper::LoadFileToArray(MemFile, *LruCacheFilename, FILEREAD_Silent))
			{
				FMemoryReader Ar(MemFile);

				FVulkanLRUCacheFile File;
				bool Valid = File.Load(Ar);
				if (!Valid)
				{
					UE_LOGF(LogVulkanRHI, Warning, "Unable to load lru pipeline cache '%ls'", *LruCacheFilename);
					bResult = false;
				}

				for (int32 Index = 0; Index < File.PipelineSizes.Num(); ++Index)
				{
					LRU2SizeList.Add(File.PipelineSizes[Index].ShaderHash, File.PipelineSizes[Index]);
				}
				UE_LOGF(LogVulkanRHI, Display, "Loaded %d LRU size entries for '%ls'", File.PipelineSizes.Num(), *LruCacheFilename);
			}
			else
			{
				UE_LOGF(LogVulkanRHI, Warning, "Unable to load lru pipeline cache '%ls'", *LruCacheFilename);
				bResult = false;
			}
		}
	}

	// Lazily create the cache in case the load failed
	if (PipelineCacheExclusive.Get() == VK_NULL_HANDLE)
	{
		VkPipelineCacheCreateInfo PipelineCacheInfo;
		ZeroVulkanStruct(PipelineCacheInfo, VK_STRUCTURE_TYPE_PIPELINE_CACHE_CREATE_INFO);
		VERIFYVULKANRESULT(VulkanRHI::vkCreatePipelineCache(Device->GetHandle(), &PipelineCacheInfo, VULKAN_CPU_ALLOCATOR, &PipelineCacheExclusive.Get()));
	}

	return bResult;
}

void FVulkanPipelineStateCacheManager::InitAndLoad(const TArray<FString>& CacheFilenames)
{
	// Where PSOFC cache is stored
	CompiledPSOCacheTopFolderPath = FVulkanPlatform::GetCompiledPSOCacheTopFolderPath();

	// Remove pipeline cache files on disk if either -ClearVulkanBinaryProgramCache or clearPSODriverCache are specified on command line
	if (FParse::Param(FCommandLine::Get(), TEXT("ClearVulkanBinaryProgramCache")) || 
		FParse::Param(FCommandLine::Get(), TEXT("clearPSODriverCache")))
	{
		UE_LOGF(LogVulkanRHI, Log, "FVulkanPipelineStateCacheManager: Clearing pipeline cache on disk");
		for (const FString& CacheFilename : CacheFilenames)
		{
			FString BinaryCacheFilename = FVulkanPlatform::CreatePSOBinaryCacheFilename(Device, CacheFilename);
			FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*BinaryCacheFilename);
		}
		FPlatformFileManager::Get().GetPlatformFile().DeleteDirectoryRecursively(*CompiledPSOCacheTopFolderPath);
	}
	
	if (GEnablePipelineCacheLoadCvar.GetValueOnAnyThread() == 0)
	{
		UE_LOGF(LogVulkanRHI, Display, "Not loading pipeline cache per r.Vulkan.PipelineCacheLoad=0");
	}
	else
	{
		if (!FPipelineFileCacheManager::IsPipelineFileCacheEnabled())
		{
			// Load the default PSO cache from disk.
			// Stores all compute PSOs, and graphics PSOs when the PSOFC and chunked PSO cache are disabled
			Load(CacheFilenames, GlobalPSOCache);

			//TODO: Since CreateComputePipelineFromShader uses only GlobalPSOCache, PSOFC will always miss CS PSOs
		}

		OnShaderPipelineCacheOpenedDelegate = FShaderPipelineCache::GetCacheOpenedDelegate().AddRaw(this, &FVulkanPipelineStateCacheManager::OnShaderPipelineCacheOpened);
		OnShaderPipelineCachePrecompilationCompleteDelegate = FShaderPipelineCache::GetPrecompilationCompleteDelegate().AddRaw(this, &FVulkanPipelineStateCacheManager::OnShaderPipelineCachePrecompilationComplete);
	}

	FScopedPipelineCache PipelineCacheExclusive = GlobalPSOCache.Get(EPipelineCacheAccess::Exclusive);
	// Lazily create the cache in case the load failed
	if (PipelineCacheExclusive.Get() == VK_NULL_HANDLE)
	{
		VkPipelineCacheCreateInfo PipelineCacheInfo;
		ZeroVulkanStruct(PipelineCacheInfo, VK_STRUCTURE_TYPE_PIPELINE_CACHE_CREATE_INFO);
		VERIFYVULKANRESULT(VulkanRHI::vkCreatePipelineCache(Device->GetHandle(), &PipelineCacheInfo, VULKAN_CPU_ALLOCATOR, &PipelineCacheExclusive.Get()));
	}
}

void FVulkanPipelineStateCacheManager::Save(const FString& CacheFilename)
{
	if (!FPipelineFileCacheManager::IsPipelineFileCacheEnabled())
	{
		SavePSOCache(CacheFilename, GlobalPSOCache);
	}

	//TODO: Save LRU cache here
}


#if PLATFORM_ANDROID
static int32 GNumRemoteProgramCompileServices = 6;
static FAutoConsoleVariableRef CVarNumRemoteProgramCompileServices(
	TEXT("Android.Vulkan.NumRemoteProgramCompileServices"),
	GNumRemoteProgramCompileServices,
	TEXT("The number of separate processes to make available to compile Vulkan PSOs.\n")
	TEXT("0 to disable use of separate processes to precompile PSOs\n")
	TEXT("valid range is 1-8 (4 default).")
	,
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);
#endif

void FVulkanPipelineStateCacheManager::OnShaderPipelineCacheOpened(FString const& Name, EShaderPlatform Platform, uint32 Count, const FGuid& VersionGuid, FShaderPipelineCache::FShaderCachePrecompileContext& ShaderCachePrecompileContext)
{
	//TODO: support reloading the same cache
	if (CompiledPSOCaches.Contains(VersionGuid))
	{
		UE_LOGF(LogVulkanRHI, Log, "FVulkanPipelineStateCacheManager::OnShaderPipelineCacheOpened attempts to load a cache that was already loaded before %ls %ls", *Name, *VersionGuid.ToString());
		return;
	}

	CurrentPrecompilingPSOCacheGuid = VersionGuid;

	UE_LOGF(LogVulkanRHI, Log, "FVulkanPipelineStateCacheManager::OnShaderPipelineCacheOpened %ls %d %ls", *Name, Count, *VersionGuid.ToString());

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	const VkPhysicalDeviceProperties& DeviceProperties = Device->GetDeviceProperties();
	FString BinaryCacheAppendage = FString::Printf(TEXT(".%x.%x"), DeviceProperties.vendorID, DeviceProperties.deviceID);

	CompiledPSOCacheFolderName = CompiledPSOCacheTopFolderPath / TEXT("VulkanPSO_") + VersionGuid.ToString() + BinaryCacheAppendage;
	FString TempName = CompiledPSOCacheTopFolderPath / TEXT("TempScanVulkanPSO_") + VersionGuid.ToString() + BinaryCacheAppendage;

	{
		FScopedPipelineCache PipelineCacheExclusive = CurrentPrecompilingPSOCache.Get(EPipelineCacheAccess::Exclusive);
		checkf(PipelineCacheExclusive.Get() == VK_NULL_HANDLE, TEXT("Trying to open more than one shader pipeline cache"));

		VkPipelineCacheCreateInfo PipelineCacheInfo;
		ZeroVulkanStruct(PipelineCacheInfo, VK_STRUCTURE_TYPE_PIPELINE_CACHE_CREATE_INFO);
		VERIFYVULKANRESULT(VulkanRHI::vkCreatePipelineCache(Device->GetHandle(), &PipelineCacheInfo, VULKAN_CPU_ALLOCATOR, &PipelineCacheExclusive.Get()));
	}

	if (PlatformFile.FileExists(*CompiledPSOCacheFolderName))
	{
		// Try to move the file to a temporary filename before the scan, so we won't try to read it again if it's corrupted
		PlatformFile.DeleteFile(*TempName);
		PlatformFile.MoveFile(*TempName, *CompiledPSOCacheFolderName);

		TArray<FString> CacheFilenames;
		CacheFilenames.Add(TempName);

		// Rename the file back after a successful scan.
		if (Load(CacheFilenames, CurrentPrecompilingPSOCache))
		{
			bPrecompilingCacheLoadedFromFile = true;
			PlatformFile.MoveFile(*CompiledPSOCacheFolderName, *TempName);

			if (CVarPipelineLRUCacheEvictBinary.GetValueOnAnyThread())
			{
				bEvictImmediately = true;
			}
		}
		else
		{
			UE_LOGF(LogVulkanRHI, Log, "FVulkanPipelineStateCacheManager: PSO cache failed to load, deleting file: %ls", *CompiledPSOCacheFolderName);
			FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*CompiledPSOCacheFolderName);
		}
	}
	else
	{
		UE_LOGF(LogVulkanRHI, Log, "FVulkanPipelineStateCacheManager: %ls does not exist.", *CompiledPSOCacheFolderName);
	}



	if (!bPrecompilingCacheLoadedFromFile || (bEvictImmediately && CVarPipelineLRUCacheEvictBinaryPreloadScreen.GetValueOnAnyThread()))
	{
		ShaderCachePrecompileContext.SetPrecompilationIsSlowTask();
#if PLATFORM_ANDROID
		if (GNumRemoteProgramCompileServices)
		{
			FVulkanAndroidPlatform::StartRemoteCompileServices(GNumRemoteProgramCompileServices);
		}
#endif
	}
}

void FVulkanPipelineStateCacheManager::OnShaderPipelineCachePrecompilationComplete(uint32 Count, double Seconds, const FShaderPipelineCache::FShaderCachePrecompileContext& ShaderCachePrecompileContext)
{
	UE_LOGF(LogVulkanRHI, Log, "FVulkanPipelineStateCacheManager::OnShaderPipelineCachePrecompilationComplete");

#if PLATFORM_ANDROID
	if (FVulkanAndroidPlatform::AreRemoteCompileServicesActive())
	{
		FVulkanAndroidPlatform::StopRemoteCompileServices();
	}
#endif

	bEvictImmediately = false;
	if (!bPrecompilingCacheLoadedFromFile)
	{
		//Save PSO cache only if it failed to load
		SavePSOCache(CompiledPSOCacheFolderName, CurrentPrecompilingPSOCache);
	}

	if (!CompiledPSOCaches.Contains(CurrentPrecompilingPSOCacheGuid))
	{
		CompiledPSOCaches.Add(CurrentPrecompilingPSOCacheGuid);

		//Merge this CurrentOpenedPSOCache with global PSOCache
		QUICK_SCOPE_CYCLE_COUNTER(STAT_VulkanPSOCacheMerge);
		FScopedPipelineCache GlobalPipelineCacheExclusive = GlobalPSOCache.Get(EPipelineCacheAccess::Exclusive);
		FScopedPipelineCache CurrentPipelineCacheExclusive = CurrentPrecompilingPSOCache.Get(EPipelineCacheAccess::Exclusive);
		VERIFYVULKANRESULT(VulkanRHI::vkMergePipelineCaches(Device->GetHandle(), GlobalPipelineCacheExclusive.Get(), 1, &CurrentPipelineCacheExclusive.Get()));
		VulkanRHI::vkDestroyPipelineCache(Device->GetHandle(), CurrentPipelineCacheExclusive.Get(), VULKAN_CPU_ALLOCATOR);

		CurrentPipelineCacheExclusive.Get() = VK_NULL_HANDLE;
	}

	bPrecompilingCacheLoadedFromFile = false;
	CurrentPrecompilingPSOCacheGuid = FGuid();
}

void FVulkanPipelineStateCacheManager::SavePSOCache(const FString& CacheFilename, FPipelineCache& Cache)
{
	FScopedPipelineCache PipelineCacheExclusive = Cache.Get(EPipelineCacheAccess::Exclusive);
	FScopeLock Lock1(&GraphicsPSOLockedCS);		//TODO: Do we really need this here?!
	FScopeLock Lock2(&LRUCS);


	// First save Device Cache
	size_t Size = 0;
	VERIFYVULKANRESULT(VulkanRHI::vkGetPipelineCacheData(Device->GetHandle(), PipelineCacheExclusive.Get(), &Size, nullptr));
	// 16 is HeaderSize + HeaderVersion
	if (Size >= 16 + VK_UUID_SIZE)
	{
		TArray<uint8> DeviceCache;
		DeviceCache.AddUninitialized(Size);
		VkResult Result = VulkanRHI::vkGetPipelineCacheData(Device->GetHandle(), PipelineCacheExclusive.Get(), &Size, DeviceCache.GetData());
		if (Result == VK_SUCCESS)
		{
			FString BinaryCacheFilename = FVulkanPlatform::CreatePSOBinaryCacheFilename(Device, CacheFilename);

			if (FFileHelper::SaveArrayToFile(DeviceCache, *BinaryCacheFilename))
			{
				UE_LOGF(LogVulkanRHI, Display, "FVulkanPipelineStateCacheManager: Saved device pipeline cache file '%ls', %d bytes", *BinaryCacheFilename, DeviceCache.Num());
			}
			else
			{
				UE_LOGF(LogVulkanRHI, Error, "FVulkanPipelineStateCacheManager: Failed to save device pipeline cache file '%ls', %d bytes", *BinaryCacheFilename, DeviceCache.Num());
			}
		}
		else if (Result == VK_INCOMPLETE || Result == VK_ERROR_OUT_OF_HOST_MEMORY)
		{
			UE_LOGF(LogVulkanRHI, Warning, "Failed to get Vulkan pipeline cache data. Error %d, %zu bytes", Result, Size);

			//TODO: Resave it when we shutdown the manager?!
			VulkanRHI::vkDestroyPipelineCache(Device->GetHandle(), PipelineCacheExclusive.Get(), VULKAN_CPU_ALLOCATOR);
			VkPipelineCacheCreateInfo PipelineCacheInfo;
			ZeroVulkanStruct(PipelineCacheInfo, VK_STRUCTURE_TYPE_PIPELINE_CACHE_CREATE_INFO);
			VERIFYVULKANRESULT(VulkanRHI::vkCreatePipelineCache(Device->GetHandle(), &PipelineCacheInfo, VULKAN_CPU_ALLOCATOR, &PipelineCacheExclusive.Get()));
		}
		else
		{
			VERIFYVULKANRESULT(Result);
		}
	}

	if (CVarEnableLRU.GetValueOnAnyThread() != 0)
	{
		// LRU cache file
		TArray<uint8> MemFile;
		FMemoryWriter Ar(MemFile);
		FVulkanLRUCacheFile File;
		File.Header.Version = FVulkanLRUCacheFile::LRU_CACHE_VERSION;
		File.Header.SizeOfPipelineSizes = (int32)sizeof(FVulkanPipelineSize);
		LRU2SizeList.GenerateValueArray(File.PipelineSizes);
		File.Save(Ar);

		FString LruCacheFilename = FVulkanPlatform::CreatePSOBinaryCacheFilename(Device, CacheFilename);
		LruCacheFilename += TEXT(".lru");

		if (FFileHelper::SaveArrayToFile(MemFile, *LruCacheFilename))
		{
			UE_LOGF(LogVulkanRHI, Display, "FVulkanPipelineStateCacheManager: Saved pipeline lru pipeline cache file '%ls', %d hashes, %d bytes", *LruCacheFilename, LRU2SizeList.Num(), MemFile.Num());
		}
		else
		{
			UE_LOGF(LogVulkanRHI, Error, "FVulkanPipelineStateCacheManager: Failed to save pipeline lru pipeline cache file '%ls', %d hashes, %d bytes", *LruCacheFilename, LRU2SizeList.Num(), MemFile.Num());
		}
	}
}

FArchive& operator << (FArchive& Ar, FGfxPipelineDesc::FBlendAttachment& Attachment)
{
	// Modify VERSION if serialization changes
	Ar << Attachment.bBlend;
	Ar << Attachment.ColorBlendOp;
	Ar << Attachment.SrcColorBlendFactor;
	Ar << Attachment.DstColorBlendFactor;
	Ar << Attachment.AlphaBlendOp;
	Ar << Attachment.SrcAlphaBlendFactor;
	Ar << Attachment.DstAlphaBlendFactor;
	Ar << Attachment.ColorWriteMask;
	return Ar;
}

void FGfxPipelineDesc::FBlendAttachment::ReadFrom(const VkPipelineColorBlendAttachmentState& InState)
{
	bBlend =				InState.blendEnable != VK_FALSE;
	ColorBlendOp =			(uint8)InState.colorBlendOp;
	SrcColorBlendFactor =	(uint8)InState.srcColorBlendFactor;
	DstColorBlendFactor =	(uint8)InState.dstColorBlendFactor;
	AlphaBlendOp =			(uint8)InState.alphaBlendOp;
	SrcAlphaBlendFactor =	(uint8)InState.srcAlphaBlendFactor;
	DstAlphaBlendFactor =	(uint8)InState.dstAlphaBlendFactor;
	ColorWriteMask =		(uint8)InState.colorWriteMask;
}

void FGfxPipelineDesc::FBlendAttachment::WriteInto(VkPipelineColorBlendAttachmentState& Out) const
{
	Out.blendEnable =			bBlend ? VK_TRUE : VK_FALSE;
	Out.colorBlendOp =			(VkBlendOp)ColorBlendOp;
	Out.srcColorBlendFactor =	(VkBlendFactor)SrcColorBlendFactor;
	Out.dstColorBlendFactor =	(VkBlendFactor)DstColorBlendFactor;
	Out.alphaBlendOp =			(VkBlendOp)AlphaBlendOp;
	Out.srcAlphaBlendFactor =	(VkBlendFactor)SrcAlphaBlendFactor;
	Out.dstAlphaBlendFactor =	(VkBlendFactor)DstAlphaBlendFactor;
	Out.colorWriteMask =		(VkColorComponentFlags)ColorWriteMask;
}


void FDescriptorSetLayoutBinding::ReadFrom(const VkDescriptorSetLayoutBinding& InState)
{
	Binding =			InState.binding;
	ensure(InState.descriptorCount == 1);
	//DescriptorCount =	InState.descriptorCount;
	DescriptorType =	InState.descriptorType;
	StageFlags =		InState.stageFlags;
}

void FDescriptorSetLayoutBinding::WriteInto(VkDescriptorSetLayoutBinding& Out) const
{
	Out.binding = Binding;
	//Out.descriptorCount = DescriptorCount;
	Out.descriptorType = (VkDescriptorType)DescriptorType;
	Out.stageFlags = StageFlags;
}

FArchive& operator << (FArchive& Ar, FDescriptorSetLayoutBinding& Binding)
{
	// Modify VERSION if serialization changes
	Ar << Binding.Binding;
	//Ar << Binding.DescriptorCount;
	Ar << Binding.DescriptorType;
	Ar << Binding.StageFlags;
	return Ar;
}

void FGfxPipelineDesc::FVertexBinding::ReadFrom(const VkVertexInputBindingDescription& InState)
{
	Binding =	InState.binding;
	InputRate =	(uint16)InState.inputRate;
	Stride =	InState.stride;
}

void FGfxPipelineDesc::FVertexBinding::WriteInto(VkVertexInputBindingDescription& Out) const
{
	Out.binding =	Binding;
	Out.inputRate =	(VkVertexInputRate)InputRate;
	Out.stride =	Stride;
}

void FGfxPipelineDesc::FVertexBinding::WriteInto(VkVertexInputBindingDescription2EXT& Out) const
{
	Out.binding =	Binding;
	Out.inputRate =	(VkVertexInputRate)InputRate;
	Out.stride =	Stride;
	Out.divisor =	1;
}

FArchive& operator << (FArchive& Ar, FGfxPipelineDesc::FVertexBinding& Binding)
{
	// Modify VERSION if serialization changes
	Ar << Binding.Stride;
	Ar << Binding.Binding;
	Ar << Binding.InputRate;
	return Ar;
}

void FGfxPipelineDesc::FVertexAttribute::ReadFrom(const VkVertexInputAttributeDescription& InState)
{
	Binding =	InState.binding;
	Format =	(uint32)InState.format;
	Location =	InState.location;
	Offset =	InState.offset;
}

void FGfxPipelineDesc::FVertexAttribute::WriteInto(VkVertexInputAttributeDescription& Out) const
{
	Out.binding =	Binding;
	Out.format =	(VkFormat)Format;
	Out.location =	Location;
	Out.offset =	Offset;
}

void FGfxPipelineDesc::FVertexAttribute::WriteInto(VkVertexInputAttributeDescription2EXT& Out) const
{
	Out.binding = Binding;
	Out.format = (VkFormat)Format;
	Out.location = Location;
	Out.offset = Offset;
}

FArchive& operator << (FArchive& Ar, FGfxPipelineDesc::FVertexAttribute& Attribute)
{
	// Modify VERSION if serialization changes
	Ar << Attribute.Location;
	Ar << Attribute.Binding;
	Ar << Attribute.Format;
	Ar << Attribute.Offset;
	return Ar;
}

void FGfxPipelineDesc::FRasterizer::ReadFrom(const VkPipelineRasterizationStateCreateInfo& InState)
{
	PolygonMode =				InState.polygonMode;
	CullMode =					InState.cullMode;
	DepthBiasSlopeScale =		InState.depthBiasSlopeFactor;
	DepthBiasConstantFactor =	InState.depthBiasConstantFactor;
}

void FGfxPipelineDesc::FRasterizer::WriteInto(VkPipelineRasterizationStateCreateInfo& Out) const
{
	Out.polygonMode =				(VkPolygonMode)PolygonMode;
	Out.cullMode =					(VkCullModeFlags)CullMode;
	Out.frontFace =					VK_FRONT_FACE_CLOCKWISE;
	Out.depthClampEnable =			VK_FALSE;
	Out.depthBiasEnable =			DepthBiasConstantFactor != 0.0f ? VK_TRUE : VK_FALSE;
	Out.rasterizerDiscardEnable =	VK_FALSE;
	Out.depthBiasSlopeFactor =		DepthBiasSlopeScale;
	Out.depthBiasConstantFactor =	DepthBiasConstantFactor;
}

FArchive& operator << (FArchive& Ar, FGfxPipelineDesc::FRasterizer& Rasterizer)
{
	// Modify VERSION if serialization changes
	Ar << Rasterizer.PolygonMode;
	Ar << Rasterizer.CullMode;
	Ar << Rasterizer.DepthBiasSlopeScale;
	Ar << Rasterizer.DepthBiasConstantFactor;
	return Ar;
}

void FGfxPipelineDesc::FDepthStencil::ReadFrom(const VkPipelineDepthStencilStateCreateInfo& InState)
{
	DepthCompareOp =			(uint8)InState.depthCompareOp;
	bDepthTestEnable =			InState.depthTestEnable != VK_FALSE;
	bDepthWriteEnable =			InState.depthWriteEnable != VK_FALSE;
	bDepthBoundsTestEnable =	InState.depthBoundsTestEnable != VK_FALSE;
	bStencilTestEnable =		InState.stencilTestEnable != VK_FALSE;
	FrontFailOp =				(uint8)InState.front.failOp;
	FrontPassOp =				(uint8)InState.front.passOp;
	FrontDepthFailOp =			(uint8)InState.front.depthFailOp;
	FrontCompareOp =			(uint8)InState.front.compareOp;
	FrontCompareMask =			(uint8)InState.front.compareMask;
	FrontWriteMask =			InState.front.writeMask;
	FrontReference =			InState.front.reference;
	BackFailOp =				(uint8)InState.back.failOp;
	BackPassOp =				(uint8)InState.back.passOp;
	BackDepthFailOp =			(uint8)InState.back.depthFailOp;
	BackCompareOp =				(uint8)InState.back.compareOp;
	BackCompareMask =			(uint8)InState.back.compareMask;
	BackWriteMask =				InState.back.writeMask;
	BackReference =				InState.back.reference;
}

void FGfxPipelineDesc::FDepthStencil::WriteInto(VkPipelineDepthStencilStateCreateInfo& Out) const
{
	Out.depthCompareOp =		(VkCompareOp)DepthCompareOp;
	Out.depthTestEnable =		bDepthTestEnable;
	Out.depthWriteEnable =		bDepthWriteEnable;
	Out.depthBoundsTestEnable =	bDepthBoundsTestEnable;
	Out.stencilTestEnable =		bStencilTestEnable;
	Out.front.failOp =			(VkStencilOp)FrontFailOp;
	Out.front.passOp =			(VkStencilOp)FrontPassOp;
	Out.front.depthFailOp =		(VkStencilOp)FrontDepthFailOp;
	Out.front.compareOp =		(VkCompareOp)FrontCompareOp;
	Out.front.compareMask =		FrontCompareMask;
	Out.front.writeMask =		FrontWriteMask;
	Out.front.reference =		FrontReference;
	Out.back.failOp =			(VkStencilOp)BackFailOp;
	Out.back.passOp =			(VkStencilOp)BackPassOp;
	Out.back.depthFailOp =		(VkStencilOp)BackDepthFailOp;
	Out.back.compareOp =		(VkCompareOp)BackCompareOp;
	Out.back.writeMask =		BackWriteMask;
	Out.back.compareMask =		BackCompareMask;
	Out.back.reference =		BackReference;
}

FArchive& operator << (FArchive& Ar, FGfxPipelineDesc::FDepthStencil& DepthStencil)
{
	// Modify VERSION if serialization changes
	Ar << DepthStencil.DepthCompareOp;
	Ar << DepthStencil.bDepthTestEnable;
	Ar << DepthStencil.bDepthWriteEnable;
	Ar << DepthStencil.bDepthBoundsTestEnable;
	Ar << DepthStencil.bStencilTestEnable;
	Ar << DepthStencil.FrontFailOp;
	Ar << DepthStencil.FrontPassOp;
	Ar << DepthStencil.FrontDepthFailOp;
	Ar << DepthStencil.FrontCompareOp;
	Ar << DepthStencil.FrontCompareMask;
	Ar << DepthStencil.FrontWriteMask;
	Ar << DepthStencil.FrontReference;
	Ar << DepthStencil.BackFailOp;
	Ar << DepthStencil.BackPassOp;
	Ar << DepthStencil.BackDepthFailOp;
	Ar << DepthStencil.BackCompareOp;
	Ar << DepthStencil.BackCompareMask;
	Ar << DepthStencil.BackWriteMask;
	Ar << DepthStencil.BackReference;
	return Ar;
}

void FGfxPipelineDesc::FRenderTargets::FAttachmentRef::ReadFrom(const VkAttachmentReference& InState)
{
	Attachment =	InState.attachment;
	Layout =		(uint64)InState.layout;
}

void FGfxPipelineDesc::FRenderTargets::FAttachmentRef::WriteInto(VkAttachmentReference& Out) const
{
	Out.attachment =	Attachment;
	Out.layout =		(VkImageLayout)Layout;
}

FArchive& operator << (FArchive& Ar, FGfxPipelineDesc::FRenderTargets::FAttachmentRef& AttachmentRef)
{
	// Modify VERSION if serialization changes
	Ar << AttachmentRef.Attachment;
	Ar << AttachmentRef.Layout;
	return Ar;
}

void FGfxPipelineDesc::FRenderTargets::FStencilAttachmentRef::ReadFrom(const VkAttachmentReferenceStencilLayout& InState)
{
	Layout = (uint64)InState.stencilLayout;
}

void FGfxPipelineDesc::FRenderTargets::FStencilAttachmentRef::WriteInto(VkAttachmentReferenceStencilLayout& Out) const
{
	Out.stencilLayout = (VkImageLayout)Layout;
}

FArchive& operator << (FArchive& Ar, FGfxPipelineDesc::FRenderTargets::FStencilAttachmentRef& AttachmentRef)
{
	// Modify VERSION if serialization changes
	Ar << AttachmentRef.Layout;
	return Ar;
}

void FGfxPipelineDesc::FRenderTargets::FAttachmentDesc::ReadFrom(const VkAttachmentDescription &InState)
{
	Format =			(uint32)InState.format;
	Flags =				(uint8)InState.flags;
	Samples =			(uint8)InState.samples;
	LoadOp =			(uint8)InState.loadOp;
	StoreOp =			(uint8)InState.storeOp;
	StencilLoadOp =		(uint8)InState.stencilLoadOp;
	StencilStoreOp =	(uint8)InState.stencilStoreOp;
	InitialLayout =		(uint64)InState.initialLayout;
	FinalLayout =		(uint64)InState.finalLayout;
}

void FGfxPipelineDesc::FRenderTargets::FAttachmentDesc::WriteInto(VkAttachmentDescription& Out) const
{
	Out.format =			(VkFormat)Format;
	Out.flags =				Flags;
	Out.samples =			(VkSampleCountFlagBits)Samples;
	Out.loadOp =			(VkAttachmentLoadOp)LoadOp;
	Out.storeOp =			(VkAttachmentStoreOp)StoreOp;
	Out.stencilLoadOp =		(VkAttachmentLoadOp)StencilLoadOp;
	Out.stencilStoreOp =	(VkAttachmentStoreOp)StencilStoreOp;
	Out.initialLayout =		(VkImageLayout)InitialLayout;
	Out.finalLayout =		(VkImageLayout)FinalLayout;
}

FArchive& operator << (FArchive& Ar, FGfxPipelineDesc::FRenderTargets::FAttachmentDesc& AttachmentDesc)
{
	// Modify VERSION if serialization changes
	Ar << AttachmentDesc.Format;
	Ar << AttachmentDesc.Flags;
	Ar << AttachmentDesc.Samples;
	Ar << AttachmentDesc.LoadOp;
	Ar << AttachmentDesc.StoreOp;
	Ar << AttachmentDesc.StencilLoadOp;
	Ar << AttachmentDesc.StencilStoreOp;
	Ar << AttachmentDesc.InitialLayout;
	Ar << AttachmentDesc.FinalLayout;

	return Ar;
}

void FGfxPipelineDesc::FRenderTargets::FStencilAttachmentDesc::ReadFrom(const VkAttachmentDescriptionStencilLayout& InState)
{
	InitialLayout = (uint64)InState.stencilInitialLayout;
	FinalLayout = (uint64)InState.stencilFinalLayout;
}

void FGfxPipelineDesc::FRenderTargets::FStencilAttachmentDesc::WriteInto(VkAttachmentDescriptionStencilLayout& Out) const
{
	Out.stencilInitialLayout = (VkImageLayout)InitialLayout;
	Out.stencilFinalLayout = (VkImageLayout)FinalLayout;
}

FArchive& operator << (FArchive& Ar, FGfxPipelineDesc::FRenderTargets::FStencilAttachmentDesc& StencilAttachmentDesc)
{
	// Modify VERSION if serialization changes
	Ar << StencilAttachmentDesc.InitialLayout;
	Ar << StencilAttachmentDesc.FinalLayout;

	return Ar;
}

void FGfxPipelineDesc::FRenderTargets::ReadFrom(const FVulkanRenderTargetLayout& RTLayout)
{
	NumAttachments =			RTLayout.NumAttachmentDescriptions;
	NumColorAttachments =		RTLayout.NumColorAttachments;

	bHasDepthStencil =			RTLayout.bHasDepthStencil != 0;
	bHasResolveAttachments =	RTLayout.bHasResolveAttachments != 0;
	bHasDepthStencilResolve =	RTLayout.bHasDepthStencilResolve != 0;
	bHasFragmentDensityAttachment =	RTLayout.bHasFragmentDensityAttachment != 0;
	NumUsedClearValues =		RTLayout.NumUsedClearValues;

	RenderPassCompatibleHash =	RTLayout.GetRenderPassCompatibleHash();

	Extent3D.X = RTLayout.Extent.Extent3D.width;
	Extent3D.Y = RTLayout.Extent.Extent3D.height;
	Extent3D.Z = RTLayout.Extent.Extent3D.depth;

	auto CopyAttachmentRefs = [&](TArray<FGfxPipelineDesc::FRenderTargets::FAttachmentRef>& Dest, const VkAttachmentReference* Source, uint32 Count)
	{
		for (uint32 Index = 0; Index < Count; ++Index)
		{
			FGfxPipelineDesc::FRenderTargets::FAttachmentRef& New = Dest.AddDefaulted_GetRef();
			New.ReadFrom(Source[Index]);
		}
	};
	CopyAttachmentRefs(ColorAttachments, RTLayout.ColorReferences, UE_ARRAY_COUNT(RTLayout.ColorReferences));
	CopyAttachmentRefs(ResolveAttachments, RTLayout.ResolveReferences, UE_ARRAY_COUNT(RTLayout.ResolveReferences));
	Depth.ReadFrom(RTLayout.DepthReference);
	Stencil.ReadFrom(RTLayout.StencilReference);
	FragmentDensity.ReadFrom(RTLayout.FragmentDensityReference);

	Descriptions.AddZeroed(UE_ARRAY_COUNT(RTLayout.Desc));
	for (int32 Index = 0; Index < UE_ARRAY_COUNT(RTLayout.Desc); ++Index)
	{
		Descriptions[Index].ReadFrom(RTLayout.Desc[Index]);
	}
	StencilDescription.ReadFrom(RTLayout.StencilDesc);
}

void FGfxPipelineDesc::FRenderTargets::WriteInto(FVulkanRenderTargetLayout& Out) const
{
	Out.NumAttachmentDescriptions =	NumAttachments;
	Out.NumColorAttachments =		NumColorAttachments;

	Out.bHasDepthStencil =			bHasDepthStencil;
	Out.bHasResolveAttachments =	bHasResolveAttachments;
	Out.bHasDepthStencilResolve =	bHasDepthStencilResolve;
	Out.bHasFragmentDensityAttachment =	bHasFragmentDensityAttachment;
	Out.NumUsedClearValues =		NumUsedClearValues;

	ensure(0);
	Out.RenderPassCompatibleHash =	RenderPassCompatibleHash;

	Out.Extent.Extent3D.width =		Extent3D.X;
	Out.Extent.Extent3D.height =	Extent3D.Y;
	Out.Extent.Extent3D.depth =		Extent3D.Z;

	auto CopyAttachmentRefs = [&](const TArray<FGfxPipelineDesc::FRenderTargets::FAttachmentRef>& Source, VkAttachmentReference* Dest, uint32 Count)
	{
		for (uint32 Index = 0; Index < Count; ++Index, ++Dest)
		{
			Source[Index].WriteInto(*Dest);
		}
	};
	CopyAttachmentRefs(ColorAttachments, Out.ColorReferences, UE_ARRAY_COUNT(Out.ColorReferences));
	CopyAttachmentRefs(ResolveAttachments, Out.ResolveReferences, UE_ARRAY_COUNT(Out.ResolveReferences));
	Depth.WriteInto(Out.DepthReference);
	Stencil.WriteInto(Out.StencilReference);
	FragmentDensity.WriteInto(Out.FragmentDensityReference);


	for (int32 Index = 0; Index < UE_ARRAY_COUNT(Out.Desc); ++Index)
	{
		Descriptions[Index].WriteInto(Out.Desc[Index]);
	}
	StencilDescription.WriteInto(Out.StencilDesc);
}

FArchive& operator << (FArchive& Ar, FGfxPipelineDesc::FRenderTargets& RTs)
{
	// Modify VERSION if serialization changes
	Ar << RTs.NumAttachments;
	Ar << RTs.NumColorAttachments;
	Ar << RTs.NumUsedClearValues;
	Ar << RTs.ColorAttachments;
	Ar << RTs.ResolveAttachments;
	Ar << RTs.Depth;
	Ar << RTs.Stencil;
	Ar << RTs.FragmentDensity;

	Ar << RTs.Descriptions;
	Ar << RTs.StencilDescription;

	Ar << RTs.bHasDepthStencil;
	Ar << RTs.bHasResolveAttachments;
	Ar << RTs.bHasDepthStencilResolve;
	Ar << RTs.RenderPassCompatibleHash;
	Ar << RTs.Extent3D;

	return Ar;
}

FArchive& operator << (FArchive& Ar, FGfxPipelineDesc& Entry)
{
	// Modify VERSION if serialization changes
	Ar << Entry.VertexInputKey;
	Ar << Entry.RasterizationSamples;
	Ar << Entry.Topology;

	Ar << Entry.ColorAttachmentStates;

	Ar << Entry.DescriptorSetLayoutBindings;

	Ar << Entry.VertexBindings;
	Ar << Entry.VertexAttributes;
	Ar << Entry.Rasterizer;

	Ar << Entry.DepthStencil;

#if VULKAN_USE_SHADERKEYS
	for (uint64& ShaderKey : Entry.ShaderKeys)
	{
		Ar << ShaderKey;
	}
#else
	for (int32 Index = 0; Index < UE_ARRAY_COUNT(Entry.ShaderHashes.Stages); ++Index)
	{
		Ar << Entry.ShaderHashes.Stages[Index];
	}
#endif
	Ar << Entry.SpecializationKey;

	Ar << Entry.RenderTargets;

	uint8 ShadingRate = static_cast<uint8>(Entry.ShadingRate);
	uint8 Combiner = static_cast<uint8>(Entry.Combiner);
	
	Ar << ShadingRate;
	Ar << Combiner;

	Ar << Entry.UseAlphaToCoverage;

	return Ar;
}

FArchive& operator << (FArchive& Ar, FGfxPipelineDesc* Entry)
{
	return Ar << (*Entry);
}

FArchive& operator << (FArchive& Ar, FVulkanPipelineSize& PS)
{
	Ar << PS.ShaderHash;
	Ar << PS.PipelineSize;

	return Ar;
}


FVulkanPSOKey FGfxPipelineDesc::CreateKey2() const
{
	FVulkanPSOKey Result;
	Result.GenerateFromArchive([this](FArchive& Ar)
	{
		Ar << const_cast<FGfxPipelineDesc&>(*this);
	});
	return Result;
}

// Map Unreal VRS combiner operation enums to Vulkan enums.
static const TMap<uint8, VkFragmentShadingRateCombinerOpKHR> FragmentCombinerOpMap
{
	{ VRSRB_Passthrough,	VK_FRAGMENT_SHADING_RATE_COMBINER_OP_KEEP_KHR },
	{ VRSRB_Override,	VK_FRAGMENT_SHADING_RATE_COMBINER_OP_REPLACE_KHR },
	{ VRSRB_Min,			VK_FRAGMENT_SHADING_RATE_COMBINER_OP_MIN_KHR },
	{ VRSRB_Max,			VK_FRAGMENT_SHADING_RATE_COMBINER_OP_MAX_KHR },
	{ VRSRB_Sum,			VK_FRAGMENT_SHADING_RATE_COMBINER_OP_MAX_KHR },		// No concept of Sum in Vulkan - fall back to max.
	// @todo: Add "VK_FRAGMENT_SHADING_RATE_COMBINER_OP_MUL_KHR"?
};

VkFragmentShadingRateCombinerOpKHR GetFragmentCombinerOp(EVRSRateCombiner RateCombiner)
{
	return FragmentCombinerOpMap[(uint8)RateCombiner];
}

static FString GfxShaderHashesToString(FVulkanShader* Shaders[ShaderStage::NumGraphicsStages])
{
	FString ShaderHashes;
	if (Shaders[ShaderStage::Vertex])
	{
		ShaderHashes += TEXT("VS: ") + static_cast<FVulkanVertexShader*>(Shaders[ShaderStage::Vertex])->GetHash().ToString() + TEXT("\n");
	}
	if (Shaders[ShaderStage::Pixel] )
	{
		ShaderHashes += TEXT("PS: ") + static_cast<FVulkanPixelShader*>(Shaders[ShaderStage::Pixel])->GetHash().ToString() + TEXT("\n");
	}
#if PLATFORM_SUPPORTS_MESH_SHADERS
	if (Shaders[ShaderStage::Mesh])
	{
		ShaderHashes += TEXT("MS: ") + static_cast<FVulkanMeshShader*>(Shaders[ShaderStage::Mesh])->GetHash().ToString() + TEXT("\n");
	}
	if (Shaders[ShaderStage::Task])
	{
		ShaderHashes += TEXT("AS: ") + static_cast<FVulkanTaskShader*>(Shaders[ShaderStage::Task])->GetHash().ToString() + TEXT("\n");
	}
#endif
#if VULKAN_SUPPORTS_GEOMETRY_SHADERS
	if (Shaders[ShaderStage::Geometry])
	{
		ShaderHashes += TEXT("GS: ") + static_cast<FVulkanGeometryShader*>(Shaders[ShaderStage::Geometry])->GetHash().ToString() + TEXT("\n");
	}
#endif
	return ShaderHashes;
}


static TArray<VkDynamicState> FillDynamicStates(const FVulkanDevice& Device, const FGfxPipelineDesc& PipelineDesc)
{
	TArray<VkDynamicState> DynamicStates;

	const bool bHasEXTExtendedDynamicState1 = Device.GetOptionalExtensions().HasEXTExtendedDynamicState1;
	const bool bHasEXTExtendedDynamicState2 = Device.GetOptionalExtensions().HasEXTExtendedDynamicState2;
	const bool bHasEXTExtendedDynamicState3 = Device.GetOptionalExtensions().HasEXTExtendedDynamicState3;
	const bool bHasEXTVertexInputDynamicState = Device.GetOptionalExtensions().HasEXTVertexInputDynamicState;

#if PLATFORM_SUPPORTS_MESH_SHADERS
#if VULKAN_USE_SHADERKEYS
	const bool bHasMeshShader = (PipelineDesc.ShaderKeys[ShaderStage::Mesh] != 0);
#else
	const bool bHasMeshShader = (PipelineDesc.ShaderHashes.Stages[ShaderStage::Mesh] != 0);
#endif
#else
	constexpr bool bHasMeshShader = false;
#endif

	// Preallocate the array
	const int32 DynamicStateCount =
		4 +
		(bHasEXTExtendedDynamicState1 ? 8 : 0) +
		(bHasEXTExtendedDynamicState2 ? 1 : 0) +
		(bHasEXTExtendedDynamicState3 ? 6 : 0) +
		(bHasEXTVertexInputDynamicState ? 1 : 0);
	DynamicStates.Reserve(DynamicStateCount);

	// Basic states that are always enabled
	DynamicStates.Add(VK_DYNAMIC_STATE_STENCIL_REFERENCE);
	DynamicStates.Add(VK_DYNAMIC_STATE_DEPTH_BOUNDS);

	if (bHasEXTExtendedDynamicState1)
	{
		if (!bHasMeshShader)
		{
			DynamicStates.Add(VK_DYNAMIC_STATE_PRIMITIVE_TOPOLOGY);
		}

		DynamicStates.Add(VK_DYNAMIC_STATE_CULL_MODE);
		DynamicStates.Add(VK_DYNAMIC_STATE_STENCIL_TEST_ENABLE);
		DynamicStates.Add(VK_DYNAMIC_STATE_STENCIL_OP);
		DynamicStates.Add(VK_DYNAMIC_STATE_DEPTH_COMPARE_OP);
		DynamicStates.Add(VK_DYNAMIC_STATE_DEPTH_TEST_ENABLE);
		DynamicStates.Add(VK_DYNAMIC_STATE_DEPTH_BOUNDS_TEST_ENABLE);
		DynamicStates.Add(VK_DYNAMIC_STATE_DEPTH_WRITE_ENABLE);

		// VK_DYNAMIC_STATE_FRONT_FACE unused
		// VK_DYNAMIC_STATE_VERTEX_INPUT_BINDING_STRIDE not used, using VK_DYNAMIC_STATE_VERTEX_INPUT_EXT instead

		DynamicStates.Add(VK_DYNAMIC_STATE_VIEWPORT_WITH_COUNT);
		DynamicStates.Add(VK_DYNAMIC_STATE_SCISSOR_WITH_COUNT);
	}
	else
	{
		DynamicStates.Add(VK_DYNAMIC_STATE_VIEWPORT);
		DynamicStates.Add(VK_DYNAMIC_STATE_SCISSOR);
	}

	if (bHasEXTExtendedDynamicState2)
	{
		DynamicStates.Add(VK_DYNAMIC_STATE_DEPTH_BIAS_ENABLE);
		DynamicStates.Add(VK_DYNAMIC_STATE_DEPTH_BIAS);  // from Vulkan 1.0

		// VK_DYNAMIC_STATE_LOGIC_OP_EXT not used
		// VK_DYNAMIC_STATE_PATCH_CONTROL_POINTS_EXT not used
		// VK_DYNAMIC_STATE_PRIMITIVE_RESTART_ENABLE not used
		// VK_DYNAMIC_STATE_RASTERIZER_DISCARD_ENABLE not used
	}

	if (bHasEXTExtendedDynamicState3)
	{
		if (PipelineDesc.ColorAttachmentStates.Num() > 0)
		{
			DynamicStates.Add(VK_DYNAMIC_STATE_COLOR_BLEND_ENABLE_EXT);
			DynamicStates.Add(VK_DYNAMIC_STATE_COLOR_WRITE_MASK_EXT);
			DynamicStates.Add(VK_DYNAMIC_STATE_COLOR_BLEND_EQUATION_EXT);
		}
		DynamicStates.Add(VK_DYNAMIC_STATE_POLYGON_MODE_EXT);
		DynamicStates.Add(VK_DYNAMIC_STATE_RASTERIZATION_SAMPLES_EXT);
		DynamicStates.Add(VK_DYNAMIC_STATE_ALPHA_TO_COVERAGE_ENABLE_EXT);

		// VK_DYNAMIC_STATE_ALPHA_TO_ONE_ENABLE_EXT not used
		// VK_DYNAMIC_STATE_DEPTH_CLAMP_ENABLE_EXT not used (:todo-jn: but should be?)
		// VK_DYNAMIC_STATE_LOGIC_OP_ENABLE_EXT not used
		// VK_DYNAMIC_STATE_SAMPLE_MASK_EXT not used
	}

	if (bHasEXTVertexInputDynamicState)
	{
		if (!bHasMeshShader)
		{
			DynamicStates.Add(VK_DYNAMIC_STATE_VERTEX_INPUT_EXT);
		}
	}

	return DynamicStates;
}


void FVulkanPipelineStateCacheManager::CreateVertexInputLibraries()
{
	INC_DWORD_STAT(STAT_VulkanNumVertexInputLibs);
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	TRACE_CPUPROFILER_EVENT_SCOPE(VulkanVertexInputLibTime);
#endif

	VkGraphicsPipelineLibraryCreateInfoEXT LibraryInfo;
	ZeroVulkanStruct(LibraryInfo, VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_LIBRARY_CREATE_INFO_EXT);
	LibraryInfo.flags = VK_GRAPHICS_PIPELINE_LIBRARY_VERTEX_INPUT_INTERFACE_BIT_EXT;

	// Set default values for unused states
	VkPipelineVertexInputStateCreateInfo VertexInputStateCreateInfo;
	ZeroVulkanStruct(VertexInputStateCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_VERTEX_INPUT_STATE_CREATE_INFO);
	VkPipelineInputAssemblyStateCreateInfo InputAssemblyStateCreateInfo;
	ZeroVulkanStruct(InputAssemblyStateCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_INPUT_ASSEMBLY_STATE_CREATE_INFO);

	// See table for vertex input: https://docs.vulkan.org/guide/latest/dynamic_state_map.html
	const VkDynamicState VertexInputDynamicStates[] = {
		VK_DYNAMIC_STATE_PRIMITIVE_TOPOLOGY,
		VK_DYNAMIC_STATE_VERTEX_INPUT_EXT
	};

	VkPipelineDynamicStateCreateInfo DynamicStateCreateInfo;
	ZeroVulkanStruct(DynamicStateCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_DYNAMIC_STATE_CREATE_INFO);
	DynamicStateCreateInfo.dynamicStateCount = UE_ARRAY_COUNT(VertexInputDynamicStates);
	DynamicStateCreateInfo.pDynamicStates = VertexInputDynamicStates;

	const VkPipelineCreateFlags2CreateInfo Flags2CreateInfo = {
		.sType = VK_STRUCTURE_TYPE_PIPELINE_CREATE_FLAGS_2_CREATE_INFO,
		.pNext = &LibraryInfo,
		.flags = VK_PIPELINE_CREATE_2_LIBRARY_BIT_KHR |
				 Device->GetBindlessDescriptorManager()->GetPipelineCreateFlag() |
				 (GVulkanGraphicPipelineLibraryLinkingMode == 1 ? VK_PIPELINE_CREATE_2_RETAIN_LINK_TIME_OPTIMIZATION_INFO_BIT_EXT : 0ull)
	};

	VkGraphicsPipelineCreateInfo GraphicsPipelineCreateInfo;
	ZeroVulkanStruct(GraphicsPipelineCreateInfo, VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_CREATE_INFO);
	GraphicsPipelineCreateInfo.sType = VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_CREATE_INFO;
	GraphicsPipelineCreateInfo.pNext = &Flags2CreateInfo;
	GraphicsPipelineCreateInfo.flags = 0; // using VkPipelineCreateFlags2CreateInfo
	GraphicsPipelineCreateInfo.layout = Device->GetBindlessDescriptorManager()->GetPipelineLayout();
	GraphicsPipelineCreateInfo.pVertexInputState = &VertexInputStateCreateInfo;
	GraphicsPipelineCreateInfo.pInputAssemblyState = &InputAssemblyStateCreateInfo;
	GraphicsPipelineCreateInfo.pDynamicState = &DynamicStateCreateInfo;

	VERIFYVULKANRESULT(VulkanRHI::vkCreateGraphicsPipelines(Device->GetHandle(), VK_NULL_HANDLE, 1, &GraphicsPipelineCreateInfo, VULKAN_CPU_ALLOCATOR, &VertexInputPipelineLibraryVS));

	if (Device->GetOptionalExtensions().HasEXTMeshShader)
	{
		// Mesh shader is the same, but without the dynamic states (no topology, no vertex input)
		DynamicStateCreateInfo.dynamicStateCount = 0;
		DynamicStateCreateInfo.pDynamicStates = nullptr;

		VERIFYVULKANRESULT(VulkanRHI::vkCreateGraphicsPipelines(Device->GetHandle(), VK_NULL_HANDLE, 1, &GraphicsPipelineCreateInfo, VULKAN_CPU_ALLOCATOR, &VertexInputPipelineLibraryMS));
	}
}

void FVulkanPipelineStateCacheManager::CreateEmptyFragmentStageLibrary()
{
	INC_DWORD_STAT(STAT_VulkanNumPSLibs);
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	TRACE_CPUPROFILER_EVENT_SCOPE(VulkanPSLibTime);
#endif

	VkGraphicsPipelineLibraryCreateInfoEXT LibraryInfo;
	ZeroVulkanStruct(LibraryInfo, VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_LIBRARY_CREATE_INFO_EXT);
	LibraryInfo.flags = VK_GRAPHICS_PIPELINE_LIBRARY_FRAGMENT_SHADER_BIT_EXT;

	VkPipelineRenderingCreateInfo RenderingCreateInfo;
	ZeroVulkanStruct(RenderingCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_RENDERING_CREATE_INFO);
	RenderingCreateInfo.viewMask = 0;  // :todo-jn: fill viewMask for multiview
	LibraryInfo.pNext = &RenderingCreateInfo;

	// See table for fragment shader: https://docs.vulkan.org/guide/latest/dynamic_state_map.html
	const VkDynamicState FragmentShaderDynamicStates[] = {
		VK_DYNAMIC_STATE_STENCIL_REFERENCE,
		VK_DYNAMIC_STATE_DEPTH_BOUNDS,
		VK_DYNAMIC_STATE_STENCIL_TEST_ENABLE, // ext1
		VK_DYNAMIC_STATE_STENCIL_OP, // ext1
		VK_DYNAMIC_STATE_DEPTH_COMPARE_OP, // ext1
		VK_DYNAMIC_STATE_DEPTH_TEST_ENABLE, // ext1
		VK_DYNAMIC_STATE_DEPTH_BOUNDS_TEST_ENABLE, // ext1
		VK_DYNAMIC_STATE_DEPTH_WRITE_ENABLE, // ext1
		VK_DYNAMIC_STATE_RASTERIZATION_SAMPLES_EXT, // ext3
		VK_DYNAMIC_STATE_ALPHA_TO_COVERAGE_ENABLE_EXT // ext3
	};

	VkPipelineDynamicStateCreateInfo DynamicStateCreateInfo;
	ZeroVulkanStruct(DynamicStateCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_DYNAMIC_STATE_CREATE_INFO);
	DynamicStateCreateInfo.dynamicStateCount = UE_ARRAY_COUNT(FragmentShaderDynamicStates);
	DynamicStateCreateInfo.pDynamicStates = FragmentShaderDynamicStates;

	// These struct's important fields are dynamic, set the rest to the default
	VkPipelineMultisampleStateCreateInfo MultisampleStateCreateInfo;
	ZeroVulkanStruct(MultisampleStateCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_MULTISAMPLE_STATE_CREATE_INFO);
	VkPipelineDepthStencilStateCreateInfo DepthStencilStateCreateInfo;
	ZeroVulkanStruct(DepthStencilStateCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_DEPTH_STENCIL_STATE_CREATE_INFO);

	const VkPipelineCreateFlags2CreateInfo Flags2CreateInfo = {
		.sType = VK_STRUCTURE_TYPE_PIPELINE_CREATE_FLAGS_2_CREATE_INFO,
		.pNext = &LibraryInfo,
		.flags = VK_PIPELINE_CREATE_2_LIBRARY_BIT_KHR | 
				 Device->GetBindlessDescriptorManager()->GetPipelineCreateFlag() |
				 (GVulkanGraphicPipelineLibraryLinkingMode == 1 ? VK_PIPELINE_CREATE_2_RETAIN_LINK_TIME_OPTIMIZATION_INFO_BIT_EXT : 0ull)
	};

	VkGraphicsPipelineCreateInfo GraphicsPipelineCreateInfo;
	ZeroVulkanStruct(GraphicsPipelineCreateInfo, VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_CREATE_INFO);
	GraphicsPipelineCreateInfo.sType = VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_CREATE_INFO;
	GraphicsPipelineCreateInfo.pNext = &Flags2CreateInfo;
	GraphicsPipelineCreateInfo.flags = 0; // using VkPipelineCreateFlags2CreateInfo
	GraphicsPipelineCreateInfo.pMultisampleState = &MultisampleStateCreateInfo;
	GraphicsPipelineCreateInfo.pDepthStencilState = &DepthStencilStateCreateInfo;
	GraphicsPipelineCreateInfo.layout = Device->GetBindlessDescriptorManager()->GetPipelineLayout();
	GraphicsPipelineCreateInfo.pDynamicState = &DynamicStateCreateInfo;

	// Add the NullPS for platforms that don't allow a null pixel stage
	VkShaderModuleCreateInfo ModuleCreateInfo;
	VkPipelineShaderStageCreateInfo ShaderStageCreateInfo;
	TArrayView<uint32> SpirvCode;
	ANSICHAR EntryPoint[24];
	VkShaderDescriptorSetAndBindingMappingInfoEXT BindingMappingInfo;
	if (!FVulkanPlatform::SupportsNullPixelShader())
	{
		ZeroVulkanStruct(ModuleCreateInfo, VK_STRUCTURE_TYPE_SHADER_MODULE_CREATE_INFO);
		ZeroVulkanStruct(ShaderStageCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_SHADER_STAGE_CREATE_INFO);

		FVulkanPixelShader* PixelShader = ResourceCast(TShaderMapRef<FNULLPS>(GetGlobalShaderMap(GMaxRHIFeatureLevel)).GetPixelShader());
		SpirvCode = PixelShader->GetSpirvCode().GetCodeView();
		PixelShader->GetEntryPoint(EntryPoint, 24);

		ModuleCreateInfo = {
			.sType = VK_STRUCTURE_TYPE_SHADER_MODULE_CREATE_INFO,
			.pNext = nullptr,
			.flags = 0,
			.codeSize = SpirvCode.Num() * sizeof(uint32),
			.pCode = SpirvCode.GetData()
		};

		ShaderStageCreateInfo = {
			.sType = VK_STRUCTURE_TYPE_PIPELINE_SHADER_STAGE_CREATE_INFO,
			.pNext = &ModuleCreateInfo,
			.flags = 0,
			.stage = VK_SHADER_STAGE_FRAGMENT_BIT,
			.module = VK_NULL_HANDLE,
			.pName = EntryPoint,
			.pSpecializationInfo = nullptr // :todo:
		};

		GraphicsPipelineCreateInfo.pStages = &ShaderStageCreateInfo;

		if (Device->GetBindlessDescriptorManager()->UseDescriptorHeaps())
		{
			TConstArrayView<VkDescriptorSetAndBindingMappingEXT> BindingMappings = Device->GetBindlessDescriptorManager()->GetBindingMappings(SF_Pixel, false);
			BindingMappingInfo = {
				.sType = VK_STRUCTURE_TYPE_SHADER_DESCRIPTOR_SET_AND_BINDING_MAPPING_INFO_EXT,
				.pNext = nullptr,
				.mappingCount = (uint32)BindingMappings.Num(),
				.pMappings = BindingMappings.GetData()
			};

			AddToPNext(ShaderStageCreateInfo, BindingMappingInfo);
		}
	}

	VERIFYVULKANRESULT(VulkanRHI::vkCreateGraphicsPipelines(Device->GetHandle(), VK_NULL_HANDLE, 1, &GraphicsPipelineCreateInfo, VULKAN_CPU_ALLOCATOR, &EmptyFragmentStageLibrary));
}

void FillDynamicRenderingCreateInfo(
	const FGfxPipelineDesc::FRenderTargets& InRenderTargets, 
	VkPipelineRenderingCreateInfo& OutPipelineRenderingCreateInfo,
	TStaticArray<VkFormat, MaxSimultaneousRenderTargets>& OutColorAttachmentFormats)
{
	if (InRenderTargets.NumColorAttachments > 0)
	{
		for (uint8 ColorIndex = 0; ColorIndex < InRenderTargets.NumColorAttachments; ++ColorIndex)
		{
			const uint32 Attachment = InRenderTargets.ColorAttachments[ColorIndex].Attachment;
			OutColorAttachmentFormats[ColorIndex] = (VkFormat)InRenderTargets.Descriptions[Attachment].Format;
		}

		OutPipelineRenderingCreateInfo.pColorAttachmentFormats = OutColorAttachmentFormats.GetData();
		OutPipelineRenderingCreateInfo.colorAttachmentCount = InRenderTargets.NumColorAttachments;
	}

	if (InRenderTargets.bHasDepthStencil)
	{
		// Depth
		const uint32 Attachment = InRenderTargets.Depth.Attachment;
		OutPipelineRenderingCreateInfo.depthAttachmentFormat = (VkFormat)InRenderTargets.Descriptions[Attachment].Format;

		// Stencil
		if (VulkanRHI::VulkanFormatHasStencil(OutPipelineRenderingCreateInfo.depthAttachmentFormat))
		{
			// :todo-jn: leave unused attachements to VK_FORMAT_UNDEFINED
			OutPipelineRenderingCreateInfo.stencilAttachmentFormat = OutPipelineRenderingCreateInfo.depthAttachmentFormat;
		}
	}
}

VkPipeline FVulkanPipelineStateCacheManager::FindOrCreateFragmentOutputStateLibrary(const FVulkanGraphicsPipelineState& GraphicsPipelineState)
{
	TStaticArray<VkFormat, MaxSimultaneousRenderTargets> ColorAttachmentFormats;
	VkPipelineRenderingCreateInfo PipelineRenderingCreateInfo;
	ZeroVulkanStruct(PipelineRenderingCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_RENDERING_CREATE_INFO);
	FillDynamicRenderingCreateInfo(GraphicsPipelineState.Desc.RenderTargets, PipelineRenderingCreateInfo, ColorAttachmentFormats);

	uint32 LibraryCRC = FCrc::MemCrc32(PipelineRenderingCreateInfo.pColorAttachmentFormats, PipelineRenderingCreateInfo.colorAttachmentCount * sizeof(VkFormat));
	LibraryCRC = FCrc::MemCrc32(&PipelineRenderingCreateInfo.depthAttachmentFormat, sizeof(VkFormat), LibraryCRC);
	LibraryCRC = FCrc::MemCrc32(&PipelineRenderingCreateInfo.stencilAttachmentFormat, sizeof(VkFormat), LibraryCRC);

	FScopeLock Lock(&FragmentOutputStateCS);
	if (VkPipeline* ExistingPipelineLibrary = FragmentOutputStatePipelineLibraries.Find(LibraryCRC))
	{
		return *ExistingPipelineLibrary;
	}

	INC_DWORD_STAT(STAT_VulkanNumFragOutputStateLibs);
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	TRACE_CPUPROFILER_EVENT_SCOPE(VulkanFragOutputStateLibTime);
#endif

	VkGraphicsPipelineLibraryCreateInfoEXT LibraryInfo;
	ZeroVulkanStruct(LibraryInfo, VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_LIBRARY_CREATE_INFO_EXT);
	LibraryInfo.flags = VK_GRAPHICS_PIPELINE_LIBRARY_FRAGMENT_OUTPUT_INTERFACE_BIT_EXT;
	LibraryInfo.pNext = &PipelineRenderingCreateInfo;

	// Set default values for unused states
	VkPipelineColorBlendStateCreateInfo ColorBlendStateCreateInfo;
	ZeroVulkanStruct(ColorBlendStateCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_COLOR_BLEND_STATE_CREATE_INFO);
	ColorBlendStateCreateInfo.blendConstants[0] = 1.0f;
	ColorBlendStateCreateInfo.blendConstants[1] = 1.0f;
	ColorBlendStateCreateInfo.blendConstants[2] = 1.0f;
	ColorBlendStateCreateInfo.blendConstants[3] = 1.0f;

	VkPipelineMultisampleStateCreateInfo MultisampleStateCreateInfo;
	ZeroVulkanStruct(MultisampleStateCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_MULTISAMPLE_STATE_CREATE_INFO);

	// See table for fragment output state: https://docs.vulkan.org/guide/latest/dynamic_state_map.html
	const VkDynamicState FragmentOutputStateDynamicStates[] = {
		VK_DYNAMIC_STATE_RASTERIZATION_SAMPLES_EXT,
		VK_DYNAMIC_STATE_ALPHA_TO_COVERAGE_ENABLE_EXT,
		VK_DYNAMIC_STATE_COLOR_BLEND_ENABLE_EXT,
		VK_DYNAMIC_STATE_COLOR_WRITE_MASK_EXT,
		VK_DYNAMIC_STATE_COLOR_BLEND_EQUATION_EXT
	};

	VkPipelineDynamicStateCreateInfo DynamicStateCreateInfo;
	ZeroVulkanStruct(DynamicStateCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_DYNAMIC_STATE_CREATE_INFO);
	DynamicStateCreateInfo.dynamicStateCount = UE_ARRAY_COUNT(FragmentOutputStateDynamicStates);
	DynamicStateCreateInfo.pDynamicStates = FragmentOutputStateDynamicStates;

	const VkPipelineCreateFlags2CreateInfo Flags2CreateInfo = {
		.sType = VK_STRUCTURE_TYPE_PIPELINE_CREATE_FLAGS_2_CREATE_INFO,
		.pNext = &LibraryInfo,
		.flags = VK_PIPELINE_CREATE_2_LIBRARY_BIT_KHR | 
				 Device->GetBindlessDescriptorManager()->GetPipelineCreateFlag() |
				 (GVulkanGraphicPipelineLibraryLinkingMode == 1 ? VK_PIPELINE_CREATE_2_RETAIN_LINK_TIME_OPTIMIZATION_INFO_BIT_EXT : 0ull)
	};

	VkGraphicsPipelineCreateInfo GraphicsPipelineCreateInfo;
	ZeroVulkanStruct(GraphicsPipelineCreateInfo, VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_CREATE_INFO);
	GraphicsPipelineCreateInfo.sType = VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_CREATE_INFO;
	GraphicsPipelineCreateInfo.pNext = &Flags2CreateInfo;
	GraphicsPipelineCreateInfo.flags = 0; // using VkPipelineCreateFlags2CreateInfo
	GraphicsPipelineCreateInfo.layout = Device->GetBindlessDescriptorManager()->GetPipelineLayout();
	GraphicsPipelineCreateInfo.pColorBlendState = &ColorBlendStateCreateInfo;
	GraphicsPipelineCreateInfo.pMultisampleState = &MultisampleStateCreateInfo;
	GraphicsPipelineCreateInfo.pDynamicState = &DynamicStateCreateInfo;

	VkPipeline FragmentOutputStatePipelineLibrary = VK_NULL_HANDLE;
	VERIFYVULKANRESULT(VulkanRHI::vkCreateGraphicsPipelines(Device->GetHandle(), VK_NULL_HANDLE, 1, &GraphicsPipelineCreateInfo, VULKAN_CPU_ALLOCATOR, &FragmentOutputStatePipelineLibrary));
	FragmentOutputStatePipelineLibraries.Add(LibraryCRC, FragmentOutputStatePipelineLibrary);
	return FragmentOutputStatePipelineLibrary;
}


bool FVulkanPipelineStateCacheManager::CreateGfxPipelineFromLibrary(FVulkanGraphicsPipelineState& GraphicsPipelineState, FVulkanShader* Shaders[ShaderStage::NumGraphicsStages], FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType PSOCompileType)
{
	// Only tackle the most common case for now, use regular pipelines for the rest
	if (Shaders[ShaderStage::Task] || Shaders[ShaderStage::Geometry])
	{
		return false;
	}

#if VULKAN_ENABLE_AGGRESSIVE_STATS
	TRACE_CPUPROFILER_EVENT_SCOPE(VulkanLibPSOCompileTime);
#endif

	FVulkanShader* VertexShader = Shaders[ShaderStage::Vertex];
	FVulkanShader* PixelShader = Shaders[ShaderStage::Pixel];
	FVulkanShader* MeshShader = Shaders[ShaderStage::Mesh];

	TStaticArray<VkPipeline, 4> Libraries;
	Libraries[0] = MeshShader ? VertexInputPipelineLibraryMS : VertexInputPipelineLibraryVS;
	Libraries[1] = MeshShader ? MeshShader->PipelineLibrary : VertexShader->PipelineLibrary;
	Libraries[2] = PixelShader ? PixelShader->PipelineLibrary : EmptyFragmentStageLibrary;
	Libraries[3] = FindOrCreateFragmentOutputStateLibrary(GraphicsPipelineState);

	const VkPipelineLibraryCreateInfoKHR LibraryCreateInfo = {
		.sType = VK_STRUCTURE_TYPE_PIPELINE_LIBRARY_CREATE_INFO_KHR,
		.pNext = nullptr,
		.libraryCount = Libraries.Num(),
		.pLibraries = Libraries.GetData()
	};

	const VkPipelineCreateFlags2CreateInfo Flags2CreateInfo = {
		.sType = VK_STRUCTURE_TYPE_PIPELINE_CREATE_FLAGS_2_CREATE_INFO,
		.pNext = &LibraryCreateInfo,
		.flags = Device->GetBindlessDescriptorManager()->GetPipelineCreateFlag() |
				 (GVulkanGraphicPipelineLibraryLinkingMode == 1 ? VK_PIPELINE_CREATE_2_LINK_TIME_OPTIMIZATION_BIT_EXT : 0)
	};

	VkGraphicsPipelineCreateInfo ExecutablePipelineCreateInfo;
	ZeroVulkanStruct(ExecutablePipelineCreateInfo, VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_CREATE_INFO);
	ExecutablePipelineCreateInfo.pNext = &Flags2CreateInfo;
	ExecutablePipelineCreateInfo.flags = 0;  // using VkPipelineCreateFlags2CreateInfo
	ExecutablePipelineCreateInfo.layout = Device->GetBindlessDescriptorManager()->GetPipelineLayout();

	const VkResult Result = VulkanRHI::vkCreateGraphicsPipelines(Device->GetHandle(), nullptr, 1, &ExecutablePipelineCreateInfo, VULKAN_CPU_ALLOCATOR, &GraphicsPipelineState.VulkanPipeline);
	VERIFYVULKANRESULT(Result);
	return (Result == VK_SUCCESS);
}

bool FVulkanPipelineStateCacheManager::CreateGfxPipelineFromEntry(FVulkanGraphicsPipelineState* PSO, FVulkanShader* Shaders[ShaderStage::NumGraphicsStages], FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType PSOCompileType)
{
	// When shader objects are used, create linked shader objects instead of pipelines
	if (Device->SupportsShaderObjects())
	{
		return Device->GetShaderObjectManager()->CreateLinkedShaderObjects(PSO, Shaders, PSOCompileType);
	}

	// See if it can be created from a pipeline library first
	if (Device->SupportsGraphicPipelineLibraries() && CreateGfxPipelineFromLibrary(*PSO, Shaders, PSOCompileType))
	{
		INC_DWORD_STAT(STAT_VulkanLibPSOCompileCount);
		return true;
	}

	INC_DWORD_STAT(STAT_VulkanRegPSOCompileCount);
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	TRACE_CPUPROFILER_EVENT_SCOPE(VulkanRegPSOCompileTime);
#endif

	VkPipeline* Pipeline = &PSO->VulkanPipeline;
	const FGfxPipelineDesc* GfxEntry = &PSO->Desc;
	if (Shaders[ShaderStage::Pixel] == nullptr && !FVulkanPlatform::SupportsNullPixelShader())
	{
		Shaders[ShaderStage::Pixel] = ResourceCast(TShaderMapRef<FNULLPS>(GetGlobalShaderMap(GMaxRHIFeatureLevel)).GetPixelShader());
	}

	TRefCountPtr<FVulkanShaderModule> ShaderModules[ShaderStage::NumGraphicsStages];
	PSO->GetOrCreateShaderModules(ShaderModules, Shaders);

	// Pipeline
	VkGraphicsPipelineCreateInfo PipelineInfo;
	ZeroVulkanStruct(PipelineInfo, VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_CREATE_INFO);
	PipelineInfo.layout = PSO->Layout->GetPipelineLayout();

	// Color Blend
	VkPipelineColorBlendStateCreateInfo CBInfo;
	ZeroVulkanStruct(CBInfo, VK_STRUCTURE_TYPE_PIPELINE_COLOR_BLEND_STATE_CREATE_INFO);
	CBInfo.attachmentCount = GfxEntry->ColorAttachmentStates.Num();
	VkPipelineColorBlendAttachmentState BlendStates[MaxSimultaneousRenderTargets];
	FMemory::Memzero(BlendStates);
	uint32 ColorWriteMask = 0xffffffff;
	if(Shaders[ShaderStage::Pixel])
	{
		ColorWriteMask = Shaders[ShaderStage::Pixel]->CodeHeader.InOutMask;
	}
	for (int32 Index = 0; Index < GfxEntry->ColorAttachmentStates.Num(); ++Index)
	{
		GfxEntry->ColorAttachmentStates[Index].WriteInto(BlendStates[Index]);
		
		if(0 == (ColorWriteMask & 1)) //clear write mask of rendertargets not written by pixelshader.
		{
			BlendStates[Index].colorWriteMask = 0;
		}
		ColorWriteMask >>= 1;		
	}
	CBInfo.pAttachments = BlendStates;
	CBInfo.blendConstants[0] = 1.0f;
	CBInfo.blendConstants[1] = 1.0f;
	CBInfo.blendConstants[2] = 1.0f;
	CBInfo.blendConstants[3] = 1.0f;

	// Viewport
	VkPipelineViewportStateCreateInfo VPInfo;
	ZeroVulkanStruct(VPInfo, VK_STRUCTURE_TYPE_PIPELINE_VIEWPORT_STATE_CREATE_INFO);
	VPInfo.viewportCount = Device->GetOptionalExtensions().HasEXTExtendedDynamicState1 ? 0: 1;
	VPInfo.scissorCount = Device->GetOptionalExtensions().HasEXTExtendedDynamicState1 ? 0 : 1;

	// Multisample
	VkPipelineMultisampleStateCreateInfo MSInfo;
	ZeroVulkanStruct(MSInfo, VK_STRUCTURE_TYPE_PIPELINE_MULTISAMPLE_STATE_CREATE_INFO);
	MSInfo.rasterizationSamples = (VkSampleCountFlagBits)FMath::Max<uint16>(1u, GfxEntry->RasterizationSamples);
	MSInfo.alphaToCoverageEnable = GfxEntry->UseAlphaToCoverage;

	VkPipelineShaderStageCreateInfo ShaderStages[ShaderStage::NumGraphicsStages];
	FMemory::Memzero(ShaderStages);
	PipelineInfo.stageCount = 0;
	PipelineInfo.pStages = ShaderStages;
	// main_00000000_00000000
	ANSICHAR EntryPoints[ShaderStage::NumGraphicsStages][24];
	VkPipelineShaderStageRequiredSubgroupSizeCreateInfo RequiredSubgroupSizeCreateInfo[ShaderStage::NumGraphicsStages];
	VkSpecializationInfo SpecializationInfo[ShaderStage::NumGraphicsStages];
	VkShaderDescriptorSetAndBindingMappingInfoEXT BindingMappingInfo[ShaderStage::NumGraphicsStages];
	for (int32 ShaderStage = 0; ShaderStage < ShaderStage::NumGraphicsStages; ++ShaderStage)
	{
		if (!ShaderModules[ShaderStage].IsValid() || (Shaders[ShaderStage] == nullptr))
		{
			continue;
		}
		const ShaderStage::EStage CurrStage = (ShaderStage::EStage)ShaderStage;

		ShaderStages[PipelineInfo.stageCount].sType = VK_STRUCTURE_TYPE_PIPELINE_SHADER_STAGE_CREATE_INFO;
		const VkShaderStageFlagBits Stage = UEFrequencyToVKStageBit(ShaderStage::GetFrequencyForGfxStage(CurrStage));
		ShaderStages[PipelineInfo.stageCount].stage = Stage;
		ShaderStages[PipelineInfo.stageCount].module = ShaderModules[CurrStage]->GetVkShaderModule();
		Shaders[ShaderStage]->GetEntryPoint(EntryPoints[PipelineInfo.stageCount], 24);
		ShaderStages[PipelineInfo.stageCount].pName = EntryPoints[PipelineInfo.stageCount];

		const FVulkanShaderHeader& ShaderHeader = Shaders[ShaderStage]->GetCodeHeader();
		if (Device->GetOptionalExtensions().HasEXTSubgroupSizeControl)
		{
			if (ShaderHeader.WaveSize > 0)
			{
				// Check if supported by this stage and Check if requested size is supported
				const VkPhysicalDeviceSubgroupSizeControlPropertiesEXT& SubgroupSizeControlProperties = Device->GetOptionalExtensionProperties().SubgroupSizeControlProperties;
				const bool bSupportedStage = (VKHasAllFlags(SubgroupSizeControlProperties.requiredSubgroupSizeStages, Stage));
				const bool bSupportedSize = ((ShaderHeader.WaveSize >= SubgroupSizeControlProperties.minSubgroupSize) && (ShaderHeader.WaveSize <= SubgroupSizeControlProperties.maxSubgroupSize));
				if (bSupportedStage && bSupportedSize)
				{
					RequiredSubgroupSizeCreateInfo[PipelineInfo.stageCount] = { 
						.sType = VK_STRUCTURE_TYPE_PIPELINE_SHADER_STAGE_REQUIRED_SUBGROUP_SIZE_CREATE_INFO,
						.pNext = nullptr,
						.requiredSubgroupSize = ShaderHeader.WaveSize
					};
					AddToPNext(ShaderStages[PipelineInfo.stageCount], RequiredSubgroupSizeCreateInfo[PipelineInfo.stageCount]);
				}
			}
		}

		const uint32 NumSpecConsts = PSO->SpecializationConstants[ShaderStage].Num();
		checkf(NumSpecConsts < MaxNumSpecializationConstants, TEXT("Exceeded maximum number of specialization constants!  Raise FVulkanPipelineStateCacheManager::MaxNumSpecializationConstants."));
		if (NumSpecConsts)
		{
			SpecializationInfo[ShaderStage] = {
				.mapEntryCount = NumSpecConsts,
				.pMapEntries = SpecializationMapEntries.GetData(),
				.dataSize = PSO->SpecializationConstants[ShaderStage].NumBytes(),
				.pData = PSO->SpecializationConstants[ShaderStage].GetData()
			};

			ShaderStages[PipelineInfo.stageCount].pSpecializationInfo = &SpecializationInfo[ShaderStage];
		}

		if (Device->GetBindlessDescriptorManager()->UseDescriptorHeaps())
		{
			TConstArrayView<VkDescriptorSetAndBindingMappingEXT> BindingMappings = Device->GetBindlessDescriptorManager()->GetBindingMappings(ShaderStage::GetFrequencyForGfxStage(CurrStage), (ShaderHeader.PackedGlobalsSize > 0));
			BindingMappingInfo[PipelineInfo.stageCount] = {
				.sType = VK_STRUCTURE_TYPE_SHADER_DESCRIPTOR_SET_AND_BINDING_MAPPING_INFO_EXT,
				.pNext = nullptr,
				.mappingCount = (uint32)BindingMappings.Num(),
				.pMappings = BindingMappings.GetData()
			};

			AddToPNext(ShaderStages[PipelineInfo.stageCount], BindingMappingInfo[PipelineInfo.stageCount]);
		}

		PipelineInfo.stageCount++;
	}

	check(PipelineInfo.stageCount != 0);

	// Vertex Input. The structure is mandatory even without vertex attributes.
	VkPipelineVertexInputStateCreateInfo VBInfo;
	ZeroVulkanStruct(VBInfo, VK_STRUCTURE_TYPE_PIPELINE_VERTEX_INPUT_STATE_CREATE_INFO);
	TArray<VkVertexInputBindingDescription, TInlineAllocator<32>> VBBindings;
	for (const FGfxPipelineDesc::FVertexBinding& SourceBinding : GfxEntry->VertexBindings)
	{
		VkVertexInputBindingDescription& Binding = VBBindings.AddDefaulted_GetRef();
		SourceBinding.WriteInto(Binding);
	}
	VBInfo.vertexBindingDescriptionCount = VBBindings.Num();
	VBInfo.pVertexBindingDescriptions = VBBindings.GetData();
	TArray<VkVertexInputAttributeDescription, TInlineAllocator<32>> VBAttributes;
	for (const FGfxPipelineDesc::FVertexAttribute& SourceAttr : GfxEntry->VertexAttributes)
	{
		VkVertexInputAttributeDescription& Attr = VBAttributes.AddDefaulted_GetRef();
		SourceAttr.WriteInto(Attr);
	}
	VBInfo.vertexAttributeDescriptionCount = VBAttributes.Num();
	VBInfo.pVertexAttributeDescriptions = VBAttributes.GetData();
	PipelineInfo.pVertexInputState = &VBInfo;

	PipelineInfo.pColorBlendState = &CBInfo;
	PipelineInfo.pMultisampleState = &MSInfo;
	PipelineInfo.pViewportState = &VPInfo;

	VkPipelineRenderingCreateInfo PipelineRenderingCreateInfo;
	TStaticArray<VkFormat, MaxSimultaneousRenderTargets> ColorAttachmentFormats;
	VkRenderingInputAttachmentIndexInfo InputAttachmentIndexInfo;
	if (Device->GetOptionalExtensions().HasKHRDynamicRendering)
	{
		ZeroVulkanStruct(PipelineRenderingCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_RENDERING_CREATE_INFO);
		FillDynamicRenderingCreateInfo(PSO->Desc.RenderTargets, PipelineRenderingCreateInfo, ColorAttachmentFormats);
		AddToPNext(PipelineInfo, PipelineRenderingCreateInfo);

		InputAttachmentIndexInfo = GetDefaultRenderingInputAttachmentIndexInfo(PSO->Desc.RenderTargets.NumColorAttachments);
		AddToPNext(PipelineInfo, InputAttachmentIndexInfo);
	}
	else
	{
		PipelineInfo.renderPass = PSO->RenderPass->GetHandle();
	}

	PipelineInfo.subpass = GfxEntry->SubpassIndex;

	VkPipelineInputAssemblyStateCreateInfo InputAssembly;
	ZeroVulkanStruct(InputAssembly, VK_STRUCTURE_TYPE_PIPELINE_INPUT_ASSEMBLY_STATE_CREATE_INFO);
	InputAssembly.topology = (VkPrimitiveTopology)GfxEntry->Topology;

	PipelineInfo.pInputAssemblyState = &InputAssembly;

	VkPipelineRasterizationStateCreateInfo RasterizerState;
	FVulkanRasterizerState::ResetCreateInfo(RasterizerState);
	GfxEntry->Rasterizer.WriteInto(RasterizerState);

	VkPipelineDepthStencilStateCreateInfo DepthStencilState;
	ZeroVulkanStruct(DepthStencilState, VK_STRUCTURE_TYPE_PIPELINE_DEPTH_STENCIL_STATE_CREATE_INFO);
	GfxEntry->DepthStencil.WriteInto(DepthStencilState);

	PipelineInfo.pRasterizationState = &RasterizerState;
	PipelineInfo.pDepthStencilState = &DepthStencilState;

	VkPipelineDynamicStateCreateInfo DynamicState;
	ZeroVulkanStruct(DynamicState, VK_STRUCTURE_TYPE_PIPELINE_DYNAMIC_STATE_CREATE_INFO);
	TArray<VkDynamicState> DynamicStatesEnabled = FillDynamicStates(*PSO->Device, *GfxEntry);
	DynamicState.pDynamicStates = DynamicStatesEnabled.GetData();
	DynamicState.dynamicStateCount = DynamicStatesEnabled.Num();
	PipelineInfo.pDynamicState = &DynamicState;

	const bool bUsingVariableRateShading = (PSO->Desc.ShadingRate != EVRSShadingRate::VRSSR_1x1) || 
		(PSO->Desc.RenderTargets.bHasFragmentDensityAttachment && PSO->Desc.Combiner != EVRSRateCombiner::VRSRB_Passthrough);

	VkPipelineFragmentShadingRateStateCreateInfoKHR PipelineFragmentShadingRate;
	if (GRHISupportsPipelineVariableRateShading && GRHIVariableRateShadingImageDataType == VRSImage_Palette && bUsingVariableRateShading)
	{
		const VkExtent2D FragmentSize = Device->GetBestMatchedFragmentSize(PSO->Desc.ShadingRate);
		VkFragmentShadingRateCombinerOpKHR PipelineToPrimitiveCombinerOperation = FragmentCombinerOpMap[(uint8)PSO->Desc.Combiner];
		
		ZeroVulkanStruct(PipelineFragmentShadingRate, VK_STRUCTURE_TYPE_PIPELINE_FRAGMENT_SHADING_RATE_STATE_CREATE_INFO_KHR);
		PipelineFragmentShadingRate.fragmentSize = FragmentSize;
		PipelineFragmentShadingRate.combinerOps[0] = PipelineToPrimitiveCombinerOperation;
		PipelineFragmentShadingRate.combinerOps[1] = VK_FRAGMENT_SHADING_RATE_COMBINER_OP_MAX_KHR;		// @todo: This needs to be specified too.

		AddToPNext(PipelineInfo, PipelineFragmentShadingRate);
	}

	VkPipelineCreateFlags2CreateInfo Flags2CreateInfo;
	if (PSO->UsesBindless())
	{
		Flags2CreateInfo = {
			.sType = VK_STRUCTURE_TYPE_PIPELINE_CREATE_FLAGS_2_CREATE_INFO,
			.pNext = nullptr,
			.flags = PipelineInfo.flags | Device->GetBindlessDescriptorManager()->GetPipelineCreateFlag()
		};

		PipelineInfo.flags = 0;

		AddToPNext(PipelineInfo, Flags2CreateInfo);
	}

	VkResult Result = VK_ERROR_INITIALIZATION_FAILED;

	double BeginTime = FPlatformTime::Seconds();

	Result = CreateVKPipeline(PSO, Shaders, PipelineInfo, PSOCompileType);

	if (Result != VK_SUCCESS)
	{
		FString ShaderHashes = GfxShaderHashesToString(Shaders);

		UE_LOGF(LogVulkanRHI, Error, "Failed to create graphics pipeline.\nShaders in pipeline: %ls", *ShaderHashes);
		return false;
	}

	double EndTime = FPlatformTime::Seconds();
	double Delta = EndTime - BeginTime;
	if (Delta > HitchTime)
	{
		UE_LOGF(LogVulkanRHI, Verbose, "Hitchy gfx pipeline key CS (%.3f ms)", (float)(Delta * 1000.0));
	}

	INC_DWORD_STAT(STAT_VulkanNumPSOs);
	return true;
}

#if PLATFORM_ANDROID

static VkResult CreatePSOWithExternalService(FVulkanDevice* Device, FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType PSOCompileType, FVulkanGraphicsPipelineState* PSO, FVulkanShader* Shaders[ShaderStage::NumGraphicsStages], const VkGraphicsPipelineCreateInfo& PipelineInfo, VkPipelineCache DestPipelineCache, FRWLock& PipelineLock)
{
	VkResult Result = VK_ERROR_INITIALIZATION_FAILED;
	FVulkanShader::FSpirvCode VS = PSO->GetPatchedSpirvCode(Shaders[ShaderStage::Vertex]);
	FVulkanShader::FSpirvCode PS = PSO->GetPatchedSpirvCode(Shaders[ShaderStage::Pixel]);
	TArrayView<uint32_t> VSCode = VS.GetCodeView();
	TArrayView<uint32_t> PSCode = PS.GetCodeView();
	size_t AfterSize = 0;

	VkPipelineCache LocalPipelineCache = VK_NULL_HANDLE;
	const FGfxPipelineDesc* GfxEntry = &PSO->Desc;

	TArray<uint8> InitialCacheData;
	bool bSupplyDestPSOCacheData = false; // this can be an optimization, but only if the PSO compile is able to use content from an existing cache.
	if( bSupplyDestPSOCacheData )
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_ExternalInitialCache);
		FRWScopeLock Lock(PipelineLock, SLT_Write);
		size_t InitialCacheSize = 0;
		VulkanRHI::vkGetPipelineCacheData(Device->GetHandle(), DestPipelineCache, &InitialCacheSize, nullptr);
		InitialCacheData.SetNumUninitialized(InitialCacheSize);
		VulkanRHI::vkGetPipelineCacheData(Device->GetHandle(), DestPipelineCache, &InitialCacheSize, InitialCacheData.GetData());
	}

	TArray<uint8> HashBytes;
	HashBytes.SetNumZeroed(sizeof(FShaderHash::Hash) * 2);
	if(Shaders[ShaderStage::Vertex])
	{
		FMemory::Memcpy(HashBytes.GetData(), &static_cast<FVulkanVertexShader*>(Shaders[ShaderStage::Vertex])->GetHash().Hash, sizeof(FShaderHash::Hash));
	}
	if (Shaders[ShaderStage::Pixel])
	{
		FMemory::Memcpy(HashBytes.GetData()+sizeof(FShaderHash::Hash), &static_cast<FVulkanPixelShader*>(Shaders[ShaderStage::Pixel])->GetHash().Hash, sizeof(FShaderHash::Hash));
	}

	FString FailLog;
	LocalPipelineCache = FVulkanPlatform::PrecompilePSO(Device, HashBytes, InitialCacheData, PSOCompileType, &PipelineInfo, GfxEntry, &PSO->RenderPass->GetLayout(), VSCode, PSCode, AfterSize,&FailLog);

	if (ensure(LocalPipelineCache != VK_NULL_HANDLE))
	{
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_ExternalMergeResult);
			FRWScopeLock Lock(PipelineLock, SLT_Write);
			Result = VK_SUCCESS;
			VERIFYVULKANRESULT(VulkanRHI::vkMergePipelineCaches(Device->GetHandle(), DestPipelineCache, 1, &LocalPipelineCache));
		}
		VulkanRHI::vkDestroyPipelineCache(Device->GetHandle(), LocalPipelineCache, VULKAN_CPU_ALLOCATOR);
	}
	else
	{
  		UE_LOGF(LogVulkanRHI, Error, "Android RemoteCompileServices Failed to create graphics pipeline (%ls).\nShaders in pipeline %ls",*FailLog, *GfxShaderHashesToString(Shaders));
	}
	return Result;
}
#endif

VkResult FVulkanPipelineStateCacheManager::CreateVKPipeline(FVulkanGraphicsPipelineState* PSO, FVulkanShader* Shaders[ShaderStage::NumGraphicsStages], const VkGraphicsPipelineCreateInfo& PipelineInfo, FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType PSOCompileType)
{
#if WITH_ADDITIONAL_CRASH_CONTEXTS
	// during a crash write out the shader hashes and give the platform's pipeline crash handler an opportunity.
	const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
	auto AddToCrash = [Shaders, ThreadId, PSOCompileType](FCrashContextExtendedWriter& Writer)
		{
			const bool bIsPrecompileJob = PSOCompileType != FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType::NotSet;
			TArrayView<FVulkanShader*> ShaderView(Shaders, ShaderStage::NumGraphicsStages);

			TCHAR ExtraString[64] = { 0 };
			FCString::Snprintf(ExtraString, UE_ARRAY_COUNT(ExtraString), TEXT("(tid:%u, pt %d)"), ThreadId, (int32)PSOCompileType);

			FVulkanPlatform::OnCreatePipelineCrash(Writer, ShaderView, ExtraString, bIsPrecompileJob);
		};

	UE_ADD_CRASH_CONTEXT_SCOPE((GEnableGfxPipelineCrashContext ? MoveTemp(AddToCrash) : TUniqueFunction<void(FCrashContextExtendedWriter&)>([](FCrashContextExtendedWriter&) {})));
#endif

	bool bIsPrecompileJob = PSOCompileType != FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType::NotSet;
	if(FVulkanChunkedPipelineCacheManager::IsEnabled())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_CREATE_VKPIPELINE);

		// Use chunk caching and bypass FVulkanPipelineStateCacheManager's PSO caching
		// Placeholder PSO size - TODO: remove pipeline cache size stuff.	
		PSO->PipelineCacheSize = 20 * 1024; // This is only required bUseLRU == true.
		return FVulkanChunkedPipelineCacheManager::Get().CreatePSO(PSO, bIsPrecompileJob, FVulkanChunkedPipelineCacheManager::FPSOCreateCallbackFunc<FVulkanGraphicsPipelineState>(
				[&](FVulkanChunkedPipelineCacheManager::FPSOCreateFuncParams<FVulkanGraphicsPipelineState>& Params)
			{
				FVulkanGraphicsPipelineState* PSO = Params.PSO;
				VkPipelineCache& PipelineCache = Params.DestPipelineCache;
				FVulkanChunkedPipelineCacheManager::EPSOOperation PSOOperation = Params.PSOOperation;
				
				check(PSO->VulkanPipeline == 0);
				check(PSOOperation == FVulkanChunkedPipelineCacheManager::EPSOOperation::CreateAndStorePSO || PSOOperation == FVulkanChunkedPipelineCacheManager::EPSOOperation::CreateIfPresent);
				VkResult Result = VK_ERROR_UNKNOWN;

				if(PSOOperation == FVulkanChunkedPipelineCacheManager::EPSOOperation::CreateIfPresent)
				{
					const bool bCanTestForExistence = Device->GetOptionalExtensions().HasEXTPipelineCreationCacheControl;
					if (bCanTestForExistence)
					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_TestCreate);
						FRWScopeLock Lock(Params.DestPipelineCacheLock, SLT_ReadOnly);
						VkGraphicsPipelineCreateInfo TestPipelineInfo = PipelineInfo;
						TestPipelineInfo.flags |= VK_PIPELINE_CREATE_FAIL_ON_PIPELINE_COMPILE_REQUIRED_BIT_EXT;
						Result = VulkanRHI::vkCreateGraphicsPipelines(Device->GetHandle(), PipelineCache, 1, &TestPipelineInfo, VULKAN_CPU_ALLOCATOR, &PSO->VulkanPipeline);
					}
					else
					{
						// if we cant test we must create.
						Result = VK_PIPELINE_COMPILE_REQUIRED_EXT;
					}
					return Result;
				}

				QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_vkCreateGraphicsPipeline);
#if PLATFORM_ANDROID
				if (FVulkanAndroidPlatform::AreRemoteCompileServicesActive() && bIsPrecompileJob)
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_ExternalCreate);
					Result = CreatePSOWithExternalService(Device, PSOCompileType, PSO, Shaders, PipelineInfo, PipelineCache, Params.DestPipelineCacheLock);
				}
				// if the external service did not produce a result the following will create it in-process as a best effort fallback.
#endif
				if (Result != VK_SUCCESS) //-V547
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_vkCreate);
					FRWScopeLock Lock(Params.DestPipelineCacheLock, SLT_ReadOnly);
					Result = VulkanRHI::vkCreateGraphicsPipelines(Device->GetHandle(), PipelineCache, 1, &PipelineInfo, VULKAN_CPU_ALLOCATOR, &PSO->VulkanPipeline);
				}
				return Result;
			}));
	}

	VkPipeline* Pipeline = &PSO->VulkanPipeline;

	FPipelineCache& Cache = bIsPrecompileJob ? CurrentPrecompilingPSOCache : GlobalPSOCache;

	VkPipelineCache LocalPipelineCache = VK_NULL_HANDLE;
	VkResult Result = VK_ERROR_INITIALIZATION_FAILED;
	uint32 PSOSize = 0;
	bool bWantPSOSize = false;
	const FGfxPipelineDesc* GfxEntry = &PSO->Desc;
	uint64 ShaderHash = 0;
	bool bValidateServicePSO = false;
	if (bUseLRU)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_VulkanPSOLRUSizeLookup);

#if VULKAN_USE_SHADERKEYS
		ShaderHash = GfxEntry->ShaderKeyShared;
#else
		ShaderHash = GfxEntry->ShaderHashes.Hash;
#endif
		{
			FScopeLock Lock(&LRUCS);
			FVulkanPipelineSize* Found = LRU2SizeList.Find(ShaderHash);
			if (Found)
			{
				PSOSize = Found->PipelineSize;
			}
			else
			{
				bWantPSOSize = true;
			}
		}
	}

#if PLATFORM_ANDROID
	if (bIsPrecompileJob && FVulkanAndroidPlatform::AreRemoteCompileServicesActive() /*&&
		CVarPipelineLRUCacheEvictBinary.GetValueOnAnyThread()*/)
	{
		FVulkanShader::FSpirvCode VS = PSO->GetPatchedSpirvCode(Shaders[ShaderStage::Vertex]);
		FVulkanShader::FSpirvCode PS = PSO->GetPatchedSpirvCode(Shaders[ShaderStage::Pixel]);
		TArrayView<uint32_t> VSCode = VS.GetCodeView();
		TArrayView<uint32_t> PSCode = PS.GetCodeView();
		size_t AfterSize = 0;
		FString FailLog;
		LocalPipelineCache = FVulkanPlatform::PrecompilePSO(Device, MakeArrayView<uint8>(nullptr,0), MakeArrayView<uint8>(nullptr, 0), PSOCompileType, &PipelineInfo, GfxEntry, &PSO->RenderPass->GetLayout(), VSCode, PSCode, AfterSize,&FailLog);

		if (ensure(LocalPipelineCache != VK_NULL_HANDLE))
		{
			Pipeline[0] = VK_NULL_HANDLE;
			Result = VK_SUCCESS;

			// enable bValidateServicePSO to compare PSOService result against engine's result.
			//bValidateServicePSO = true;
			if(bValidateServicePSO)
			{
				Result = VK_ERROR_INITIALIZATION_FAILED;
				bWantPSOSize = true;
				VulkanRHI::vkDestroyPipelineCache(Device->GetHandle(), LocalPipelineCache, VULKAN_CPU_ALLOCATOR);
				LocalPipelineCache = VK_NULL_HANDLE;
			}

			if(bWantPSOSize)
			{
				PSOSize = AfterSize;
			}
		}
		else
		{
			FString ShaderHashes = GfxShaderHashesToString(Shaders);
			UE_LOGF(LogVulkanRHI, Error, "Android RemoteCompileServices Failed to create graphics pipeline (%ls).\nShaders in pipeline: %ls", *FailLog, *ShaderHashes);
		}
	}
#endif
	// Disabling 'V547: Expression is always true'
	// The precompile code above can set Result to success on Android platforms.
	if(Result != VK_SUCCESS)  //-V547
	{
		if (bWantPSOSize)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_VulkanPSOCreationTimeLRU);

			// We create a single pipeline cache for this create so we can observe the size for LRU cache's accounting.
			// measuring deltas from the global PipelineCache is not thread safe.
			VkPipelineCacheCreateInfo PipelineCacheInfo;
			ZeroVulkanStruct(PipelineCacheInfo, VK_STRUCTURE_TYPE_PIPELINE_CACHE_CREATE_INFO);
			VERIFYVULKANRESULT(VulkanRHI::vkCreatePipelineCache(Device->GetHandle(), &PipelineCacheInfo, VULKAN_CPU_ALLOCATOR, &LocalPipelineCache));
			Result = VulkanRHI::vkCreateGraphicsPipelines(Device->GetHandle(), LocalPipelineCache, 1, &PipelineInfo, VULKAN_CPU_ALLOCATOR, Pipeline);
			if (bValidateServicePSO)
			{
				size_t Diff = 0;
				if (ensure(LocalPipelineCache != VK_NULL_HANDLE))
				{
					VulkanRHI::vkGetPipelineCacheData(Device->GetHandle(), LocalPipelineCache, &Diff, nullptr);
				}
				UE_CLOGF(Diff != PSOSize, LogVulkanRHI, Warning, "PSO service size mismatches engine size! [PSOService = %d, Game Process = %zu]", PSOSize, Diff);
			}
		}
		else
		{
			SCOPE_CYCLE_COUNTER(STAT_VulkanPSOVulkanCreationTime);
			FScopedPipelineCache PipelineCacheShared = Cache.Get(EPipelineCacheAccess::Shared);
			Result = VulkanRHI::vkCreateGraphicsPipelines(Device->GetHandle(), PipelineCacheShared.Get(), 1, &PipelineInfo, VULKAN_CPU_ALLOCATOR, Pipeline);
		}
	}

	if (LocalPipelineCache != VK_NULL_HANDLE)
	{
		if (bWantPSOSize)
		{
			FScopeLock Lock(&LRUCS);
			FVulkanPipelineSize* Found = LRU2SizeList.Find(ShaderHash);
			if (Found == nullptr) // Check we're not beaten to it..
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_Calc_LRU_Size);
				size_t Diff = 0;

				if (LocalPipelineCache != VK_NULL_HANDLE)
				{
					VulkanRHI::vkGetPipelineCacheData(Device->GetHandle(), LocalPipelineCache, &Diff, nullptr);
				}

				if (!Diff)
				{
					UE_LOGF(LogVulkanRHI, Warning, "Shader size was computed as zero, using 20k instead.");
					Diff = 20 * 1024;
				}
				FVulkanPipelineSize PipelineSize;
				PipelineSize.ShaderHash = ShaderHash;
				PipelineSize.PipelineSize = (uint32)Diff;
				LRU2SizeList.Add(ShaderHash, PipelineSize);
				PSOSize = Diff;
			}
			else
			{
				PSOSize = Found->PipelineSize;
			}
		}
				
		FScopedPipelineCache PipelineCacheExclusive = Cache.Get(EPipelineCacheAccess::Exclusive);
		if (PipelineCacheExclusive.Get() != VK_NULL_HANDLE)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_VulkanPSOCacheMerge);
			VERIFYVULKANRESULT(VulkanRHI::vkMergePipelineCaches(Device->GetHandle(), PipelineCacheExclusive.Get(), 1, &LocalPipelineCache));
		}
		VulkanRHI::vkDestroyPipelineCache(Device->GetHandle(), LocalPipelineCache, VULKAN_CPU_ALLOCATOR);
	}

	VERIFYVULKANRESULT(Result);

	PSO->PipelineCacheSize = PSOSize;

 	return Result;
}

VkResult FVulkanPipelineStateCacheManager::CreateRayTracingPipeline(const VkRayTracingPipelineCreateInfoKHR& CreateInfo, bool bIsPartial, VkPipeline& OutPipeline)
{
	if (bIsPartial)
	{
		FScopedPipelineCache PipelineCacheShared = CurrentPrecompilingPSOCache.Get(EPipelineCacheAccess::Shared);
		if (PipelineCacheShared.Get() != VK_NULL_HANDLE)
		{
			return VulkanDynamicAPI::vkCreateRayTracingPipelinesKHR(
				Device->GetHandle(),
				VK_NULL_HANDLE, // Deferred Operation
				PipelineCacheShared.Get(), // Pipeline Cache 
				1,
				&CreateInfo,
				VULKAN_CPU_ALLOCATOR,
				&OutPipeline);
		}
	}

	FScopedPipelineCache PipelineCacheShared = GlobalPSOCache.Get(EPipelineCacheAccess::Shared);
	return VulkanDynamicAPI::vkCreateRayTracingPipelinesKHR(
		Device->GetHandle(),
		VK_NULL_HANDLE, // Deferred Operation
		PipelineCacheShared.Get(), // Pipeline Cache 
		1,
		&CreateInfo,
		VULKAN_CPU_ALLOCATOR,
		&OutPipeline);
}

void FVulkanPipelineStateCacheManager::DestroyCache()
{
	VkDevice DeviceHandle = Device->GetHandle();

	QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_DestroyCache_PSOLock);
	FScopeLock Lock1(&GraphicsPSOLockedCS);
	int idx = 0;
	for (auto& Pair : GraphicsPSOLockedMap)
	{
		FVulkanGraphicsPipelineState* Pipeline = Pair.Value;
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Leaked PSO %05d: RefCount=%d Handle=0x%p\n"), idx++, Pipeline->GetRefCount(), Pipeline);
	}
	LRU2SizeList.Reset();

#if LRU_DEBUG
	LRUDump();
#endif

	// Compute pipelines already deleted...
	ComputePipelineEntries.Reset();
}

void FVulkanPipelineStateCacheManager::RebuildCache()
{
	if (IsInGameThread())
	{
		FlushRenderingCommands();
	}
	DestroyCache();
}

FVulkanShaderHashes::FVulkanShaderHashes(const FGraphicsPipelineStateInitializer& PSOInitializer)
{
	Stages[ShaderStage::Vertex] = GetShaderHash<FRHIVertexShader, FVulkanVertexShader>(PSOInitializer.BoundShaderState.VertexShaderRHI);
	Stages[ShaderStage::Pixel] = GetShaderHash<FRHIPixelShader, FVulkanPixelShader>(PSOInitializer.BoundShaderState.PixelShaderRHI);
#if PLATFORM_SUPPORTS_MESH_SHADERS
	Stages[ShaderStage::Mesh] = GetShaderHash<FRHIMeshShader, FVulkanMeshShader>(PSOInitializer.BoundShaderState.GetMeshShader());
	Stages[ShaderStage::Task] = GetShaderHash<FRHIAmplificationShader, FVulkanTaskShader>(PSOInitializer.BoundShaderState.GetAmplificationShader());
#endif
#if VULKAN_SUPPORTS_GEOMETRY_SHADERS
	Stages[ShaderStage::Geometry] = GetShaderHash<FRHIGeometryShader, FVulkanGeometryShader>(PSOInitializer.BoundShaderState.GetGeometryShader());
#endif
	Finalize();
}

FVulkanShaderHashes::FVulkanShaderHashes()
{
	FMemory::Memzero(Stages);
	Hash = 0;
}

FVulkanLayout* FVulkanPipelineStateCacheManager::FindOrAddLayout(const FVulkanDescriptorSetsLayoutInfo& DescriptorSetLayoutInfo, bool bGfxLayout, bool bUsesBindless)
{
	FScopeLock Lock(&LayoutMapCS);
	if (FVulkanLayout** FoundLayout = LayoutMap.Find(DescriptorSetLayoutInfo))
	{
		check(bGfxLayout == (*FoundLayout)->IsGfxLayout());
		return *FoundLayout;
	}

	FVulkanLayout* Layout = new FVulkanLayout(*Device, bGfxLayout, bUsesBindless);
	Layout->DescriptorSetLayout.CopyFrom(DescriptorSetLayoutInfo);
	Layout->Compile(DSetLayoutMap);

	LayoutMap.Add(DescriptorSetLayoutInfo, Layout);
	return Layout;
}

static inline VkPrimitiveTopology UEToVulkanTopologyType(EPrimitiveType PrimitiveType)
{
	switch (PrimitiveType)
	{
	case PT_PointList:
		return VK_PRIMITIVE_TOPOLOGY_POINT_LIST;
	case PT_LineList:
		return VK_PRIMITIVE_TOPOLOGY_LINE_LIST;
	case PT_TriangleList:
		return VK_PRIMITIVE_TOPOLOGY_TRIANGLE_LIST;
	case PT_TriangleStrip:
		return VK_PRIMITIVE_TOPOLOGY_TRIANGLE_STRIP;
	default:
		checkf(false, TEXT("Unsupported EPrimitiveType %d"), (uint32)PrimitiveType);
		break;
	}

	return VK_PRIMITIVE_TOPOLOGY_MAX_ENUM;
}

void FVulkanPipelineStateCacheManager::CreateGfxEntry(const FGraphicsPipelineStateInitializer& PSOInitializer, FVulkanDescriptorSetsLayoutInfo& DescriptorSetLayoutInfo, FGfxPipelineDesc* Desc)
{
	FGfxPipelineDesc* OutGfxEntry = Desc;

	FVulkanShader* Shaders[ShaderStage::NumGraphicsStages];
	GetVulkanGfxShaders(PSOInitializer.BoundShaderState, Shaders);

	FVulkanVertexInputStateInfo VertexInputState;
	
	{
		const FBoundShaderStateInput& BSI = PSOInitializer.BoundShaderState;

		FUniformBufferGatherInfo UBGatherInfo;
		uint32 NumActiveShaders = 0;
		uint32 NumBindlessShaders = 0;

		auto ProcessShaderStage = [&DescriptorSetLayoutInfo, &UBGatherInfo, &NumActiveShaders, &NumBindlessShaders](VkShaderStageFlagBits StageFlag, ShaderStage::EStage Stage, FVulkanShader* Shader)
		{
			if (Shader)
			{
				const FVulkanShaderHeader& Header = Shader->GetCodeHeader();
				UBGatherInfo.CodeHeaders[Stage] = &Header;
				NumActiveShaders++;
				if (Shader->UsesBindless())
				{
					NumBindlessShaders++;
				}
			}
		};

		if (Shaders[ShaderStage::Vertex])
		{
			const FVulkanShaderHeader& VSHeader = Shaders[ShaderStage::Vertex]->GetCodeHeader();
			VertexInputState.Generate(ResourceCast(PSOInitializer.BoundShaderState.VertexDeclarationRHI), VSHeader.InOutMask);
		}

		if (Shaders[ShaderStage::Pixel] && Shaders[ShaderStage::Pixel]->GetCodeHeader().InputAttachmentInfos.Num())
		{
			// input attachements can't exist in a first sub-pass
			check(PSOInitializer.SubpassHint != ESubpassHint::None);
			check(PSOInitializer.SubpassIndex != 0);
		}

		ProcessShaderStage(VK_SHADER_STAGE_VERTEX_BIT, ShaderStage::Vertex, Shaders[ShaderStage::Vertex]);
		ProcessShaderStage(VK_SHADER_STAGE_FRAGMENT_BIT, ShaderStage::Pixel, Shaders[ShaderStage::Pixel]);

#if PLATFORM_SUPPORTS_MESH_SHADERS
		ProcessShaderStage(VK_SHADER_STAGE_MESH_BIT_EXT, ShaderStage::Mesh, Shaders[ShaderStage::Mesh]);
		ProcessShaderStage(VK_SHADER_STAGE_TASK_BIT_EXT, ShaderStage::Task, Shaders[ShaderStage::Task]);
#endif

#if VULKAN_SUPPORTS_GEOMETRY_SHADERS
		ProcessShaderStage(VK_SHADER_STAGE_GEOMETRY_BIT, ShaderStage::Geometry, Shaders[ShaderStage::Geometry]);
#endif

		checkf((NumBindlessShaders == 0) || (NumBindlessShaders == NumActiveShaders), 
			TEXT("All shaders must be bindless or non-bindless (NumBindlessShaders=%u, NumActiveShaders=%u)."),
			NumBindlessShaders, NumActiveShaders);

		// Second pass
		DescriptorSetLayoutInfo.FinalizeBindings<false>(*Device, UBGatherInfo, (NumBindlessShaders != 0));
	}

	checkf(PSOInitializer.SubpassIndex == 0 || PSOInitializer.SubpassHint != ESubpassHint::None, TEXT("Subpass hint must be set if subpass index > 0."));

	OutGfxEntry->SubpassIndex = PSOInitializer.SubpassIndex;

	FVulkanBlendState* BlendState = ResourceCast(PSOInitializer.BlendState);

	OutGfxEntry->UseAlphaToCoverage = PSOInitializer.NumSamples > 1 && BlendState->Initializer.bUseAlphaToCoverage ? 1 : 0;

	OutGfxEntry->RasterizationSamples = PSOInitializer.NumSamples;
	OutGfxEntry->Topology = (uint32)UEToVulkanTopologyType(PSOInitializer.PrimitiveType);
	uint32 NumRenderTargets = PSOInitializer.ComputeNumValidRenderTargets();
	
	if (PSOInitializer.SubpassHint == ESubpassHint::DeferredShadingSubpass && PSOInitializer.SubpassIndex >= 2)
	{
		// GBuffer attachements are not used as output in a shading sub-pass
		// Only SceneColor is used as a color attachment
		NumRenderTargets = 1;
	}

	if (PSOInitializer.SubpassHint == ESubpassHint::DepthReadSubpass && PSOInitializer.SubpassIndex >= 1)
	{
		// Only SceneColor is used as a color attachment after the first subpass (not SceneDepthAux)
		NumRenderTargets = 1;
	}

	if (PSOInitializer.SubpassHint == ESubpassHint::CustomResolveSubpass)
	{
		NumRenderTargets = 1; // This applies to base and depth passes as well. One render target for base and depth, another one for custom resolve.
		if (PSOInitializer.SubpassIndex >= 2)
		{ 
			// the resolve subpass renders to a non MSAA surface
			OutGfxEntry->RasterizationSamples = 1;
		}
	}

	OutGfxEntry->ColorAttachmentStates.AddUninitialized(NumRenderTargets);
	for (int32 Index = 0; Index < OutGfxEntry->ColorAttachmentStates.Num(); ++Index)
	{
		OutGfxEntry->ColorAttachmentStates[Index].ReadFrom(BlendState->BlendStates[Index]);
	}

	{
		const VkPipelineVertexInputStateCreateInfo& VBInfo = VertexInputState.GetInfo();
		OutGfxEntry->VertexBindings.AddUninitialized(VBInfo.vertexBindingDescriptionCount);
		for (uint32 Index = 0; Index < VBInfo.vertexBindingDescriptionCount; ++Index)
		{
			OutGfxEntry->VertexBindings[Index].ReadFrom(VBInfo.pVertexBindingDescriptions[Index]);
		}

		OutGfxEntry->VertexAttributes.AddUninitialized(VBInfo.vertexAttributeDescriptionCount);
		for (uint32 Index = 0; Index < VBInfo.vertexAttributeDescriptionCount; ++Index)
		{
			OutGfxEntry->VertexAttributes[Index].ReadFrom(VBInfo.pVertexAttributeDescriptions[Index]);
		}
	}

	const TArray<FVulkanDescriptorSetsLayout::FSetLayout>& Layouts = DescriptorSetLayoutInfo.GetLayouts();
	OutGfxEntry->DescriptorSetLayoutBindings.AddDefaulted(Layouts.Num());
	for (int32 Index = 0; Index < Layouts.Num(); ++Index)
	{
		for (int32 SubIndex = 0; SubIndex < Layouts[Index].LayoutBindings.Num(); ++SubIndex)
		{
			FDescriptorSetLayoutBinding& Binding = OutGfxEntry->DescriptorSetLayoutBindings[Index].AddDefaulted_GetRef();
			Binding.ReadFrom(Layouts[Index].LayoutBindings[SubIndex]);
		}
	}

	OutGfxEntry->Rasterizer.ReadFrom(ResourceCast(PSOInitializer.RasterizerState)->RasterizerState);
	{
		VkPipelineDepthStencilStateCreateInfo DSInfo;
		ResourceCast(PSOInitializer.DepthStencilState)->SetupCreateInfo(PSOInitializer, DSInfo);
		OutGfxEntry->DepthStencil.ReadFrom(DSInfo);
	}

	int32 NumShaders = 0;
#if VULKAN_USE_SHADERKEYS
	uint64 SharedKey = 0;
	uint64 Primes[] = {
		6843488303525203279llu,
		3095754086865563867llu,
		8242695776924673527llu,
		7556751872809527943llu,
		8278265491465149053llu,
		1263027877466626099llu,
		2698115308251696101llu,
	};
	static_assert(sizeof(Primes) / sizeof(Primes[0]) >= ShaderStage::NumGraphicsStages);
	for (int32 Index = 0; Index < ShaderStage::NumGraphicsStages; ++Index)
	{
		FVulkanShader* Shader = Shaders[Index];
		uint64 Key = 0;
		if (Shader)
		{
			Key = Shader->GetShaderKey();
			++NumShaders;
		}
		OutGfxEntry->ShaderKeys[Index] = Key;
		SharedKey += Key * Primes[Index];
	}
	OutGfxEntry->ShaderKeyShared = SharedKey;
#else
	for (int32 Index = 0; Index < ShaderStage::NumGraphicsStages; ++Index)
	{
		FVulkanShader* Shader = Shaders[Index];
		if (Shader)
		{
			check(Shader->Spirv.Num() != 0);

			FShaderHash Hash = GetShaderHashForStage(PSOInitializer, (ShaderStage::EStage)Index);
			OutGfxEntry->ShaderHashes.Stages[Index] = Hash;

			++NumShaders;
		}
	}
	OutGfxEntry->ShaderHashes.Finalize();
#endif
	check(NumShaders > 0);

	// Include a hash of specialization constants used with PSO
	{
		const FBoundShaderStateInput& BSI = PSOInitializer.BoundShaderState;
		TMemoryHasher<FShaderHashBuilder, FShaderHash> HashAr;

#define VK_HASH_SPECIALIZATION_ARRAY(Array)	if (!Array.IsEmpty()) HashAr.Serialize((void*)Array.GetData(), Array.NumBytes())

		VK_HASH_SPECIALIZATION_ARRAY(BSI.VertexShaderSpecializationConstants);
		VK_HASH_SPECIALIZATION_ARRAY(BSI.PixelShaderSpecializationConstants);
#if VULKAN_SUPPORTS_GEOMETRY_SHADERS
		VK_HASH_SPECIALIZATION_ARRAY(BSI.GeometryShaderSpecializationConstants);
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
		VK_HASH_SPECIALIZATION_ARRAY(BSI.MeshShaderSpecializationConstants);
		VK_HASH_SPECIALIZATION_ARRAY(BSI.AmplificationShaderSpecializationConstants);
#endif

#undef VK_HASH_SPECIALIZATION_ARRAY

		OutGfxEntry->SpecializationKey = HashAr.Finalize().Hash;
	}

	FVulkanRenderTargetLayout RTLayout(PSOInitializer);
	OutGfxEntry->RenderTargets.ReadFrom(RTLayout);

	// Shading rate:
	OutGfxEntry->ShadingRate = PSOInitializer.bAllowVariableRateShading ? PSOInitializer.ShadingRate : EVRSShadingRate::VRSSR_1x1;
	OutGfxEntry->Combiner = PSOInitializer.bAllowVariableRateShading ? EVRSRateCombiner::VRSRB_Max : EVRSRateCombiner::VRSRB_Passthrough; // Forces using the 1x1 rate over any fragment density attachment when VRS is disallowed in material settings
}





FVulkanGraphicsPipelineState::FVulkanGraphicsPipelineState(FVulkanDevice* InDevice, const FGraphicsPipelineStateInitializer& PSOInitializer_, const FGfxPipelineDesc& InDesc, FVulkanPSOKey* VulkanKey)
	: bIsRegistered(false)
	, PrimitiveType(PSOInitializer_.PrimitiveType)
	, VulkanPipeline(0)
	, Device(InDevice)
	, Desc(InDesc)
	, VulkanKey(VulkanKey->CopyDeep())
{
#if !UE_BUILD_SHIPPING
	SGraphicsRHICount++;
#endif

	FMemory::Memset(VulkanShaders, 0, sizeof(VulkanShaders));
	VulkanShaders[ShaderStage::Vertex] = static_cast<FVulkanVertexShader*>(PSOInitializer_.BoundShaderState.VertexShaderRHI);
	SpecializationConstants[ShaderStage::Vertex] = PSOInitializer_.BoundShaderState.VertexShaderSpecializationConstants;
#if PLATFORM_SUPPORTS_MESH_SHADERS
	VulkanShaders[ShaderStage::Mesh] = static_cast<FVulkanMeshShader*>(PSOInitializer_.BoundShaderState.GetMeshShader());
	SpecializationConstants[ShaderStage::Mesh] = PSOInitializer_.BoundShaderState.MeshShaderSpecializationConstants;
	VulkanShaders[ShaderStage::Task] = static_cast<FVulkanTaskShader*>(PSOInitializer_.BoundShaderState.GetAmplificationShader());
	SpecializationConstants[ShaderStage::Task] = PSOInitializer_.BoundShaderState.AmplificationShaderSpecializationConstants;
#endif
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	VulkanShaders[ShaderStage::Geometry] = static_cast<FVulkanGeometryShader*>(PSOInitializer_.BoundShaderState.GetGeometryShader());
	SpecializationConstants[ShaderStage::Geometry] = PSOInitializer_.BoundShaderState.GeometryShaderSpecializationConstants;
#endif
	VulkanShaders[ShaderStage::Pixel] = static_cast<FVulkanPixelShader*>(PSOInitializer_.BoundShaderState.PixelShaderRHI);
	SpecializationConstants[ShaderStage::Pixel] = PSOInitializer_.BoundShaderState.PixelShaderSpecializationConstants;

	uint32 ActiveShaderCount = 0;
	uint32 BindlessShaderCount = 0;
	for (int32 ShaderStageIndex = 0; ShaderStageIndex < ShaderStage::NumGraphicsStages; ShaderStageIndex++)
	{
		if (VulkanShaders[ShaderStageIndex] != nullptr)
		{
			VulkanShaders[ShaderStageIndex]->AddRef();

			ActiveShaderCount++;
			if (VulkanShaders[ShaderStageIndex]->UsesBindless())
			{
				BindlessShaderCount++;
			}
		}
	}
	checkf((BindlessShaderCount == 0) || (ActiveShaderCount == BindlessShaderCount), TEXT("Pipelines can't be created with mix of bindless and non-bindless shaders."));
	bUsesBindless = (BindlessShaderCount != 0);

#if VULKAN_PSO_CACHE_DEBUG
	PixelShaderRHI = PSOInitializer_.BoundShaderState.PixelShaderRHI;
	VertexShaderRHI = PSOInitializer_.BoundShaderState.VertexShaderRHI;
	VertexDeclarationRHI = PSOInitializer_.BoundShaderState.VertexDeclarationRHI;

#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	GeometryShaderRHI = PSOInitializer_.BoundShaderState.GeometryShaderRHI;
#endif

	PSOInitializer = PSOInitializer_;
#endif
	PrecacheKey = RHIComputePrecachePSOHash(PSOInitializer_);
	INC_DWORD_STAT(STAT_VulkanNumGraphicsPSOs);
	INC_DWORD_STAT_BY(STAT_VulkanPSOKeyMemory, this->VulkanKey.GetDataRef().Num());
}

void FVulkanPipelineStateCacheManager::NotifyDeletedGraphicsPSO(FRHIGraphicsPipelineState* PSO)
{
	FVulkanGraphicsPipelineState* VkPSO = (FVulkanGraphicsPipelineState*)PSO;
	Device->NotifyDeletedGfxPipeline(VkPSO);
	FVulkanPSOKey& Key = VkPSO->VulkanKey;
	DEC_DWORD_STAT_BY(STAT_VulkanPSOKeyMemory, Key.GetDataRef().Num());
	if(VkPSO->bIsRegistered)
	{
		FScopeLock Lock(&GraphicsPSOLockedCS);
		FVulkanGraphicsPipelineState** Contained = GraphicsPSOLockedMap.Find(Key);
		check(Contained && *Contained == PSO);
		VkPSO->bIsRegistered = false;
		if(bUseLRU)
		{
			LRURemove(*Contained);
			check((*Contained)->LRUNode == 0);
		}
		else
		{
			(*Contained)->DeleteVkPipeline(false);
			check(VkPSO->GetVulkanPipeline() == 0 );
		}
		GraphicsPSOLockedMap.Remove(Key);
	}
	else
	{
		FScopeLock Lock(&GraphicsPSOLockedCS);
		FVulkanGraphicsPipelineState** Contained = GraphicsPSOLockedMap.Find(Key);
		if (Contained && *Contained == VkPSO)
		{
			check(0);
		}
		VkPSO->DeleteVkPipeline(false);
	}
}


static FCriticalSection CreateGraphicsPSOMutex;

FGraphicsPipelineStateRHIRef FVulkanPipelineStateCacheManager::RHICreateGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_NEW);

	// Optional lock for PSO creation, GVulkanPSOForceSingleThreaded is used to work around driver bugs.
	// GVulkanPSOForceSingleThreaded == Precompile can be used when the driver internally serializes PSO creation, this option reduces the driver queue size.
	// We stall precompile PSOs which increases the likelihood for non-precompile PSO to jump the queue.
	// Not using GraphicsPSOLockedCS as the create could take a long time on some platforms, holding GraphicsPSOLockedCS the whole time could cause hitching.
	const ESingleThreadedPSOCreateMode ThreadingMode = (ESingleThreadedPSOCreateMode)GVulkanPSOForceSingleThreaded;
	const bool bIsPrecache = Initializer.bFromPSOFileCache || Initializer.bPSOPrecache;
	bool bShouldLock = ThreadingMode == ESingleThreadedPSOCreateMode::All
		|| (ThreadingMode == ESingleThreadedPSOCreateMode::Precompile && bIsPrecache)
		|| (ThreadingMode == ESingleThreadedPSOCreateMode::NonPrecompiled && !bIsPrecache);

	UE::TConditionalScopeLock PSOSingleThreadedLock(CreateGraphicsPSOMutex, bShouldLock);

	FVulkanPSOKey Key;
	FGfxPipelineDesc Desc;
	FVulkanDescriptorSetsLayoutInfo DescriptorSetLayoutInfo;
	{

		SCOPE_CYCLE_COUNTER(STAT_VulkanPSOHeaderInitTime);
		CreateGfxEntry(Initializer, DescriptorSetLayoutInfo, &Desc);
		Key = Desc.CreateKey2();
	}


	FVulkanGraphicsPipelineState* NewPSO = 0;
	{
		SCOPE_CYCLE_COUNTER(STAT_VulkanPSOLookupTime);
		FScopeLock Lock(&GraphicsPSOLockedCS);
		{
			FVulkanGraphicsPipelineState** PSO = GraphicsPSOLockedMap.Find(Key);
			if(PSO)
			{
				check(*PSO);
				if(!bIsPrecache)
				{
					LRUTouch(*PSO);
				}
				return *PSO;
			}
		}
	}



	{
		// Workers can be creating PSOs while FRHIResource::FlushPendingDeletes is running on the RHI thread
		// so let it get enqueued for a delete with Release() instead.  Only used for failed or duplicate PSOs...
		auto DeleteNewPSO = [](FVulkanGraphicsPipelineState* PSOPtr)
		{
			PSOPtr->AddRef();
			const uint32 RefCount = PSOPtr->Release();
			check(RefCount == 0);
		};

		SCOPE_CYCLE_COUNTER(STAT_VulkanPSOCreationTime);
		NewPSO = new FVulkanGraphicsPipelineState(Device, Initializer, Desc, &Key);
		{
			FVulkanLayout* Layout = FindOrAddLayout(DescriptorSetLayoutInfo, true, NewPSO->UsesBindless());
			NewPSO->Layout = Layout;
			NewPSO->bHasInputAttachments = Layout->GetDescriptorSetsLayout().HasInputAttachments();
		}

		if (!Device->GetOptionalExtensions().HasKHRDynamicRendering)
		{
			NewPSO->RenderPass = Device->GetImmediateContext().PrepareRenderPassForPSOCreation(Initializer);
		}
		
		{
			const FBoundShaderStateInput& BSI = Initializer.BoundShaderState;
			for (int32 StageIdx = 0; StageIdx < ShaderStage::NumGraphicsStages; ++StageIdx)
			{
				NewPSO->ShaderKeys[StageIdx] = GetShaderKeyForGfxStage(BSI, (ShaderStage::EStage)StageIdx);
			}

			if (Initializer.BoundShaderState.VertexDeclarationRHI)
			{
				check(BSI.VertexShaderRHI);
				FVulkanVertexShader* VS = ResourceCast(BSI.VertexShaderRHI);
				const FVulkanShaderHeader& VSHeader = VS->GetCodeHeader();
				NewPSO->VertexInputState.Generate(ResourceCast(Initializer.BoundShaderState.VertexDeclarationRHI), VSHeader.InOutMask);
			}

			if((!bIsPrecache || !LRUEvictImmediately()) 
	#if !UE_BUILD_SHIPPING
				&& 0 == CVarPipelineDebugForceEvictImmediately.GetValueOnAnyThread()
	#endif
				)
			{

				// Create the pipeline
				double BeginTime = FPlatformTime::Seconds();
				FVulkanShader* VulkanShaders[ShaderStage::NumGraphicsStages];
				GetVulkanGfxShaders(Initializer.BoundShaderState, VulkanShaders);

				for (int32 StageIdx = 0; StageIdx < ShaderStage::NumGraphicsStages; ++StageIdx)
				{
					uint64 key = GetShaderKeyForGfxStage(BSI, (ShaderStage::EStage)StageIdx);
					check(key == NewPSO->ShaderKeys[StageIdx]);
				}

			
				QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_CREATE_PART0);

				if(!CreateGfxPipelineFromEntry(NewPSO, VulkanShaders, Initializer.GetPSOPrecacheCompileType()))
				{
					DeleteNewPSO(NewPSO);
					return nullptr;
				}
				// Recover if we failed to create the pipeline.
				double EndTime = FPlatformTime::Seconds();
				double Delta = EndTime - BeginTime;
				if (Delta > HitchTime)
				{
					UE_LOGF(LogVulkanRHI, Verbose, "Hitchy gfx pipeline (%.3f ms)", (float)(Delta * 1000.0));
				}
			}
			FScopeLock Lock(&GraphicsPSOLockedCS); 
			FVulkanGraphicsPipelineState** MapPSO = GraphicsPSOLockedMap.Find(Key);
			if(MapPSO)//another thread could end up creating it.
			{
				DeleteNewPSO(NewPSO);
				NewPSO = *MapPSO;
			}
			else
			{
				GraphicsPSOLockedMap.Add(MoveTemp(Key), NewPSO);
				if (bUseLRU && NewPSO->VulkanPipeline != VK_NULL_HANDLE)
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState_LRU_PSOLock);
					// we add only created pipelines to the LRU
					FScopeLock LockRU(&LRUCS);
					NewPSO->bIsRegistered = true;
					LRUTrim(NewPSO->PipelineCacheSize);
					LRUAdd(NewPSO);
					if(bIsPrecache)
					{
						// immediately evict precache PSOs from the LRU.
						// precache PSOs can saturate the LRU. precache PSOs can end up being trimmed/evicted in the same frame which LRUTrim does not expect.
						// This means we are LRU-ing rendered PSOs only.
						LRURemove(NewPSO);
					}
				}
				else
				{
					NewPSO->bIsRegistered = true;
				}
			}
		}
	}
	return NewPSO;
}


FGraphicsPipelineStateRHIRef FVulkanDynamicRHI::RHICreateGraphicsPipelineState(const FGraphicsPipelineStateInitializer& PSOInitializer)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanGetOrCreatePipeline);
#endif
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateGraphicsPipelineState);
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanShaders);

	return Device->PipelineStateCache->RHICreateGraphicsPipelineState(PSOInitializer);
}

FVulkanComputePipeline* FVulkanPipelineStateCacheManager::RHICreateComputePipelineState(const FComputePipelineStateInitializer& Initializer)
{
	return Device->GetPipelineStateCache()->GetOrCreateComputePipeline(Initializer);
}

FComputePipelineStateRHIRef FVulkanDynamicRHI::RHICreateComputePipelineState(const FComputePipelineStateInitializer& Initializer)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanGetOrCreatePipeline);
#endif
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_RHICreateComputePipelineState);
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanShaders);

	return Device->PipelineStateCache->RHICreateComputePipelineState(Initializer);
}

FVulkanComputePipeline* FVulkanPipelineStateCacheManager::GetOrCreateComputePipeline(const FComputePipelineStateInitializer& Initializer)
{
	FVulkanComputeShader* ComputeShader = ResourceCast(Initializer.ComputeShader);
	check(ComputeShader);
	const FSpecializationContainerType& SpecializationConstants = Initializer.SpecializationConstants;
	const uint64 Key = CityHash64WithSeed((const char*)SpecializationConstants.GetData(), SpecializationConstants.NumBytes(), ComputeShader->GetShaderKey());

	{
		FRWScopeLock ScopeLock(ComputePipelineLock, SLT_ReadOnly);
		FVulkanComputePipeline** ComputePipelinePtr = ComputePipelineEntries.Find(Key);
		if (ComputePipelinePtr)
		{
			return *ComputePipelinePtr;
		}
	}

	// create pipeline of entry + store entry
	double BeginTime = FPlatformTime::Seconds();

	FVulkanComputePipeline* ComputePipeline = CreateComputePipeline(Initializer);

	double EndTime = FPlatformTime::Seconds();
	double Delta = EndTime - BeginTime;
	if (Delta > HitchTime)
	{
		UE_LOGF(LogVulkanRHI, Verbose, "Hitchy compute pipeline key CS (%.3f ms)", (float)(Delta * 1000.0));
	}

	{
		FRWScopeLock ScopeLock(ComputePipelineLock, SLT_Write);
		if(0 == ComputePipelineEntries.Find(Key))
		{
			ComputePipelineEntries.FindOrAdd(Key) = ComputePipeline;
		}
	}

	return ComputePipeline;
}

FVulkanComputePipeline* FVulkanPipelineStateCacheManager::CreateComputePipeline(const FComputePipelineStateInitializer& Initializer)
{
	FVulkanComputeShader* Shader = ResourceCast(Initializer.ComputeShader);
	const FSpecializationContainerType& SpecializationConstants = Initializer.SpecializationConstants;
	FVulkanComputePipeline* Pipeline = new FVulkanComputePipeline(Device, Initializer);

	FVulkanDescriptorSetsLayoutInfo DescriptorSetLayoutInfo;
	const FVulkanShaderHeader& CSHeader = Shader->GetCodeHeader();
	FUniformBufferGatherInfo UBGatherInfo;
	UBGatherInfo.CodeHeaders[ShaderStage::Compute] = &CSHeader;
	DescriptorSetLayoutInfo.FinalizeBindings<true>(*Device, UBGatherInfo, Shader->UsesBindless());
	FVulkanLayout* Layout = FindOrAddLayout(DescriptorSetLayoutInfo, false, Shader->UsesBindless());
	checkSlow(!Layout->IsGfxLayout());
	Pipeline->Layout = Layout;

	if (Device->SupportsShaderObjects())
	{
		return Pipeline;
	}

	TRefCountPtr<FVulkanShaderModule> ShaderModule = Shader->GetOrCreateHandle(Layout, Layout->GetDescriptorSetLayoutHash());

	// Entrypoint: main_00000000_00000000
	ANSICHAR EntryPoint[24];
	Shader->GetEntryPoint(EntryPoint, 24);

	VkComputePipelineCreateInfo PipelineInfo = {
		.sType = VK_STRUCTURE_TYPE_COMPUTE_PIPELINE_CREATE_INFO,
		.pNext = nullptr,
		.flags = 0,
		.stage = {
			.sType = VK_STRUCTURE_TYPE_PIPELINE_SHADER_STAGE_CREATE_INFO,
			.pNext = nullptr,
			.flags = 0,
			.stage = VK_SHADER_STAGE_COMPUTE_BIT,
			.module = ShaderModule->GetVkShaderModule(),
			.pName = EntryPoint,
			.pSpecializationInfo = nullptr
		},
		.layout = Layout->GetPipelineLayout(),
		.basePipelineHandle = VK_NULL_HANDLE,
		.basePipelineIndex = 0
	};


	VkSpecializationInfo SpecializationInfo;
	if (SpecializationConstants.Num())
	{
		checkf(SpecializationConstants.Num() < MaxNumSpecializationConstants, TEXT("Exceeded maximum number of specialization constants!  Raise FVulkanPipelineStateCacheManager::MaxNumSpecializationConstants."));
		SpecializationInfo = {
			.mapEntryCount = (uint32)SpecializationConstants.Num(),
			.pMapEntries = SpecializationMapEntries.GetData(),
			.dataSize = SpecializationConstants.NumBytes(),
			.pData = SpecializationConstants.GetData()
		};

		PipelineInfo.stage.pSpecializationInfo = &SpecializationInfo;
	}

	VkPipelineShaderStageRequiredSubgroupSizeCreateInfo RequiredSubgroupSizeCreateInfo;
	if ((CSHeader.WaveSize > 0) && Device->GetOptionalExtensions().HasEXTSubgroupSizeControl)
	{
		// Check if supported by this stage
		const VkPhysicalDeviceSubgroupSizeControlPropertiesEXT& SubgroupSizeControlProperties = Device->GetOptionalExtensionProperties().SubgroupSizeControlProperties;
		if (VKHasAllFlags(SubgroupSizeControlProperties.requiredSubgroupSizeStages, VK_SHADER_STAGE_COMPUTE_BIT))
		{
			// Check if requested size is supported
			if ((CSHeader.WaveSize >= SubgroupSizeControlProperties.minSubgroupSize) && (CSHeader.WaveSize <= SubgroupSizeControlProperties.maxSubgroupSize))
			{
				RequiredSubgroupSizeCreateInfo = {
					.sType = VK_STRUCTURE_TYPE_PIPELINE_SHADER_STAGE_REQUIRED_SUBGROUP_SIZE_CREATE_INFO,
					.pNext = nullptr,
					.requiredSubgroupSize = CSHeader.WaveSize
				};

				AddToPNext(PipelineInfo.stage, RequiredSubgroupSizeCreateInfo);
			}
		}
	}

	VkPipelineCreateFlags2CreateInfo Flags2CreateInfo;
	if (Shader->UsesBindless())
	{
		Flags2CreateInfo = {
			.sType = VK_STRUCTURE_TYPE_PIPELINE_CREATE_FLAGS_2_CREATE_INFO,
			.pNext = nullptr,
			.flags = PipelineInfo.flags | Device->GetBindlessDescriptorManager()->GetPipelineCreateFlag()
		};

		PipelineInfo.flags = 0;

		AddToPNext(PipelineInfo, Flags2CreateInfo);
	}

	VkShaderDescriptorSetAndBindingMappingInfoEXT BindingMappingInfo;
	if (Device->GetBindlessDescriptorManager()->UseDescriptorHeaps())
	{
		TConstArrayView<VkDescriptorSetAndBindingMappingEXT> BindingMappings = Device->GetBindlessDescriptorManager()->GetBindingMappings(SF_Compute, (CSHeader.PackedGlobalsSize > 0));
		BindingMappingInfo = {
			.sType = VK_STRUCTURE_TYPE_SHADER_DESCRIPTOR_SET_AND_BINDING_MAPPING_INFO_EXT,
			.pNext = nullptr,
			.mappingCount = (uint32)BindingMappings.Num(),
			.pMappings = BindingMappings.GetData()
		};

		AddToPNext(PipelineInfo.stage, BindingMappingInfo);
	}


	VkResult Result;
	{
#if WITH_ADDITIONAL_CRASH_CONTEXTS
		// during a crash write out the shader hashes and give the platform's pipeline crash handler an opportunity.
		const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
		auto AddToCrash = [VulkanShader=(FVulkanShader*)Shader, ThreadId](FCrashContextExtendedWriter& Writer)
		{
			constexpr bool bIsPrecompileJob = false;
			TConstArrayView<FVulkanShader*> ShaderView(&VulkanShader, (int32)1);

			TCHAR ExtraString[64] = { 0 };
			FCString::Snprintf(ExtraString, UE_ARRAY_COUNT(ExtraString), TEXT("(tid:%u)"), ThreadId);

			FVulkanPlatform::OnCreatePipelineCrash(Writer, ShaderView, ExtraString, bIsPrecompileJob);
		};

		UE_ADD_CRASH_CONTEXT_SCOPE((GEnableComputePipelineCrashContext ? MoveTemp(AddToCrash) : TUniqueFunction<void(FCrashContextExtendedWriter&)>([](FCrashContextExtendedWriter&) {})));
#endif

		QUICK_SCOPE_CYCLE_COUNTER(STAT_VulkanComputePSOCreate);
		FScopedPipelineCache PipelineCacheShared = GlobalPSOCache.Get(EPipelineCacheAccess::Shared);
		Result = VulkanRHI::vkCreateComputePipelines(Device->GetHandle(), PipelineCacheShared.Get(), 1, &PipelineInfo, VULKAN_CPU_ALLOCATOR, &Pipeline->Pipeline);
	}

	if (Result != VK_SUCCESS)
	{
		FString ComputeHash = Shader->GetHash().ToString();
		UE_LOGF(LogVulkanRHI, Error, "Failed to create compute pipeline.\nShaders in pipeline: CS: %ls", *ComputeHash);
		Pipeline->SetValid(false);
	}

	INC_DWORD_STAT(STAT_VulkanNumPSOs);

	return Pipeline;
}

void FVulkanPipelineStateCacheManager::NotifyDeletedComputePipeline(FVulkanComputePipeline* Pipeline)
{
	FVulkanComputeShader* ComputeShader = ResourceCast(Pipeline->GetComputeShader());
	if (ComputeShader)
	{
		const FSpecializationContainerType& SpecializationConstants = Pipeline->GetSpecializationConstants();
		const uint64 Key = CityHash64WithSeed((const char*)SpecializationConstants.GetData(), SpecializationConstants.NumBytes(), ComputeShader->GetShaderKey());
		FRWScopeLock ScopeLock(ComputePipelineLock, SLT_Write); 
		ComputePipelineEntries.Remove(Key);
	}
}

template<typename T>
static bool SerializeArray(FArchive& Ar, TArray<T>& Array)
{
	int32 Num = Array.Num();
	Ar << Num;
	if (Ar.IsLoading())
	{
		if (Num < 0)
		{
			return false;
		}
		else
		{
			Array.SetNum(Num);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				Ar << Array[Index];
			}
		}
	}
	else
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			Ar << Array[Index];
		}
	}
	return true;
}


void FVulkanPipelineStateCacheManager::FVulkanLRUCacheFile::Save(FArchive& Ar)
{
	// Modify VERSION if serialization changes
	Ar << Header.Version;
	Ar << Header.SizeOfPipelineSizes;

	SerializeArray(Ar, PipelineSizes);
}

bool FVulkanPipelineStateCacheManager::FVulkanLRUCacheFile::Load(FArchive& Ar)
{
	// Modify VERSION if serialization changes
	Ar << Header.Version;
	if (Header.Version != LRU_CACHE_VERSION)
	{
		UE_LOGF(LogVulkanRHI, Warning, "Unable to load lru pipeline cache due to mismatched Version %d != %d", Header.Version, (int32)LRU_CACHE_VERSION);
		return false;
	}

	Ar << Header.SizeOfPipelineSizes;
	if (Header.SizeOfPipelineSizes != (int32)(sizeof(FVulkanPipelineSize)))
	{
		UE_LOGF(LogVulkanRHI, Warning, "Unable to load lru pipeline cache due to mismatched size of FVulkanPipelineSize %d != %d; forgot to bump up LRU_CACHE_VERSION?", Header.SizeOfPipelineSizes, (int32)sizeof(FVulkanPipelineSize));
		return false;
	}

	if (!SerializeArray(Ar, PipelineSizes))
	{
		UE_LOGF(LogVulkanRHI, Warning, "Unable to load lru pipeline cache due to invalid archive data!");
		return false;
	}

	return true;
}



void GetVulkanGfxShaders(const FBoundShaderStateInput& BSI, FVulkanShader* OutShaders[ShaderStage::NumGraphicsStages])
{
	FMemory::Memzero(OutShaders, ShaderStage::NumGraphicsStages * sizeof(*OutShaders));

	OutShaders[ShaderStage::Vertex] = ResourceCast(BSI.VertexShaderRHI);

	if (BSI.PixelShaderRHI)
	{
		OutShaders[ShaderStage::Pixel] = ResourceCast(BSI.PixelShaderRHI);
	}

	if (BSI.GetMeshShader())
	{
		OutShaders[ShaderStage::Mesh] = ResourceCast(BSI.GetMeshShader());
	}

	if (BSI.GetAmplificationShader())
	{
		OutShaders[ShaderStage::Task] = ResourceCast(BSI.GetAmplificationShader());
	}

	if (BSI.GetGeometryShader())
	{
#if VULKAN_SUPPORTS_GEOMETRY_SHADERS
		OutShaders[ShaderStage::Geometry] = ResourceCast(BSI.GetGeometryShader());
#else
		ensureMsgf(0, TEXT("Geometry not supported!"));
#endif
	}
}

void GetVulkanGfxShaders(FVulkanDevice* Device, const FVulkanGraphicsPipelineState& GfxPipelineState, FVulkanShader* OutShaders[ShaderStage::NumGraphicsStages])
{
	FMemory::Memzero(OutShaders, ShaderStage::NumGraphicsStages * sizeof(*OutShaders));
	Device->GetShaderFactory().LookupGfxShaders(GfxPipelineState.ShaderKeys, OutShaders);
}

void FVulkanPipelineStateCacheManager::TickLRU()
{
	if(FVulkanChunkedPipelineCacheManager::IsEnabled())
	{
		FVulkanChunkedPipelineCacheManager::Get().Tick();
	}

	if (!bUseLRU || GVulkanPSOLRUEvictAfterUnusedFrames == 0)
	{
		return;
	}

	FScopeLock Lock(&LRUCS);
	const int MaxEvictsPerTick = 5;
	for(int i = 0 ; i<MaxEvictsPerTick; i++)
	{
		FVulkanGraphicsPipelineStateLRUNode* Node = LRU.GetTail();
		if (!Node)
		{
			return;
		}

		TRefCountPtr<FVulkanGraphicsPipelineState> PSO = Node->GetValue();
		bool bTimeToDie = PSO->LRUFrame + GVulkanPSOLRUEvictAfterUnusedFrames < GFrameNumberRenderThread;
		if (bTimeToDie)
		{
			LRUPRINT_DEBUG(TEXT("Evicting after %d frames of unuse (%d : %d) %d\n"), GVulkanPSOLRUEvictAfterUnusedFrames, PSO->LRUFrame, GFrameNumberRenderThread, PSO->PipelineCacheSize);
			LRURemove(PSO);
		}
		else
		{
			return;
		}
	}
}


void FVulkanPipelineStateCacheManager::LRUDump()
{
#if !UE_BUILD_SHIPPING
	uint32 tid = FPlatformTLS::GetCurrentThreadId();
	LRUPRINT(TEXT("//***** LRU DUMP *****\\\\\n"));
	FVulkanGraphicsPipelineStateLRUNode* Node= LRU.GetHead();
	uint32_t Size = 0;
	uint32_t Index = 0;
	while(Node)
	{
		FVulkanGraphicsPipelineState* PSO = Node->GetValue();
		Size += PSO->PipelineCacheSize;
		LRUPRINT(TEXT("\t%08x PSO %p :: %d  :: %06d \\ %06d\n"), tid, PSO, PSO->LRUFrame, PSO->PipelineCacheSize, Size);
		Node = Node->GetNextNode();
		Index++;
	}
	LRUPRINT(TEXT("\\\\***** LRU DUMP *****//\n"));
#endif
}



bool FVulkanPipelineStateCacheManager::LRUEvictImmediately()
{
	return bEvictImmediately && CVarEnableLRU.GetValueOnAnyThread() != 0;
}


void FVulkanPipelineStateCacheManager::LRUTrim(uint32 nSpaceNeeded)
{
	if(!bUseLRU)
	{
		return;
	}
	uint32 tid = FPlatformTLS::GetCurrentThreadId();
	uint32 MaxSize = (uint32)CVarLRUMaxPipelineSize.GetValueOnAnyThread();
	while (LRUUsedPipelineSize + nSpaceNeeded > MaxSize || LRUUsedPipelineCount > LRUUsedPipelineMax)
	{
		LRUPRINT_DEBUG(TEXT("%d EVICTING %d + %d > %d || %d > %d\n"), tid, LRUUsedPipelineSize , nSpaceNeeded, MaxSize ,LRUUsedPipelineCount ,LRUUsedPipelineMax);
		LRUEvictOne();
	}
}

void FVulkanPipelineStateCacheManager::LRUDebugEvictAll()
{
	check(bUseLRU);
	FScopeLock Lock(&LRUCS);
	int Count = 0;
	while(LRUEvictOne(true))
		Count++;

	LRUPRINT_DEBUG(TEXT("Evicted %d\n"), Count);
}

void FVulkanPipelineStateCacheManager::LRUAdd(FVulkanGraphicsPipelineState* PSO)
{
	if(!bUseLRU)
	{
		return;
	}

	FScopeLock Lock(&LRUCS);
	check(PSO->LRUNode == 0);
	check(PSO->GetVulkanPipeline());
	uint32 MaxSize = (uint32)CVarLRUMaxPipelineSize.GetValueOnAnyThread();
	uint32 PSOSize = PSO->PipelineCacheSize;

	LRUUsedPipelineSize += PSOSize;
	LRUUsedPipelineCount += 1;

	SET_DWORD_STAT(STAT_VulkanNumPSOLRUSize, LRUUsedPipelineSize);
	SET_DWORD_STAT(STAT_VulkanNumPSOLRU, LRUUsedPipelineCount);

	check(LRUUsedPipelineSize <= MaxSize); //should always be trimmed before.
	LRU.AddHead(PSO);
	PSO->LRUNode = LRU.GetHead();
	PSO->LRUFrame = GFrameNumberRenderThread;
	LRUPRINT_DEBUG(TEXT("LRUADD %p .. Frame %d :: %d    VKPSO %08x, cache size %d\n"), PSO, PSO->LRUFrame, GFrameNumberRenderThread, PSO->GetVulkanPipeline(), PSOSize);

}

void FVulkanPipelineStateCacheManager::LRUTouch(FVulkanGraphicsPipelineState* PSO)
{
	FScopeLock Lock(&LRUCS);
	check(!bUseLRU || ((PSO->GetVulkanPipeline() == 0) == (PSO->LRUNode == 0)));

	if(PSO->LRUNode)
	{
		check(PSO->GetVulkanPipeline());
		if(PSO->LRUNode != LRU.GetHead())
		{
			LRU.RemoveNode(PSO->LRUNode, false);
			LRU.AddHead(PSO->LRUNode);
		}
		PSO->LRUFrame = GFrameNumberRenderThread;
	}
	else
	{
		PSO->LRUFrame = GFrameNumberRenderThread;
		if(!PSO->GetVulkanPipeline())
		{
			// Create the pipeline
			double BeginTime = FPlatformTime::Seconds();
			FVulkanShader* VulkanShaders[ShaderStage::NumGraphicsStages];

			GetVulkanGfxShaders(Device, *PSO, VulkanShaders);

			QUICK_SCOPE_CYCLE_COUNTER(STAT_Vulkan_LRUMiss);

			if (!CreateGfxPipelineFromEntry(PSO, VulkanShaders, FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType::NotSet))
			{
				check(0);
			}
			double EndTime = FPlatformTime::Seconds();
			double Delta = EndTime - BeginTime;
			if (Delta > HitchTime)
			{
				UE_LOGF(LogVulkanRHI, Verbose, "Hitchy gfx pipeline (%.3f ms)", (float)(Delta * 1000.0));
			}

			if(bUseLRU)
			{
				LRUTrim(PSO->PipelineCacheSize);
				LRUAdd(PSO);
			}
		}
		else
		{
			check(!bUseLRU || PSO->LRUNode);
		}
	}
}

void FVulkanGraphicsPipelineState::DeleteVkPipeline(bool bImmediate)
{
	if (VulkanPipeline != VK_NULL_HANDLE)
	{
		if (bImmediate)
		{
			VulkanRHI::vkDestroyPipeline(Device->GetHandle(), VulkanPipeline, VULKAN_CPU_ALLOCATOR);
		}
		else
		{
			Device->GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::Pipeline, VulkanPipeline);
		}
		VulkanPipeline = VK_NULL_HANDLE;
	}

	Device->PipelineStateCache->LRUCheckNotInside(this);
}

void FVulkanGraphicsPipelineState::Bind(FVulkanCommandBuffer& CommandBuffer)
{
	if (Device->SupportsShaderObjects())
	{
		CommandBuffer.BindGraphicsShaderObjects(VulkanShaders);
	}
	else
	{
		CommandBuffer.BindPipeline(VK_PIPELINE_BIND_POINT_GRAPHICS, VulkanPipeline, UsesBindless());
	}
}

void FVulkanComputePipeline::Bind(FVulkanCommandBuffer& CommandBuffer)
{
	if (Device->SupportsShaderObjects())
	{
		check(ComputeShader.IsValid());
		FVulkanShader* VulkanShader = ResourceCast(GetComputeShader());
		CommandBuffer.BindComputeShaderObject(VulkanShader);
	}
	else
	{
		CommandBuffer.BindPipeline(VK_PIPELINE_BIND_POINT_COMPUTE, Pipeline, UsesBindless());
	}
}

void FVulkanPipelineStateCacheManager::LRUCheckNotInside(FVulkanGraphicsPipelineState* PSO)
{
	FScopeLock Lock(&LRUCS);


	FVulkanGraphicsPipelineStateLRUNode* Node = LRU.GetHead();
	uint32_t Size = 0;
	uint32_t Index = 0;
	while (Node)
	{
		FVulkanGraphicsPipelineState* foo = Node->GetValue();
		if (foo == PSO)
		{
			check(0 == foo->LRUNode);

		}
		check(foo != PSO);
		Node = Node->GetNextNode();
	}
	check(0 == PSO->LRUNode);
}

void FVulkanPipelineStateCacheManager::LRURemove(FVulkanGraphicsPipelineState* PSO)
{
	check(bUseLRU);
	if (PSO->LRUNode != 0)
	{
		bool bImmediate = PSO->LRUFrame + 3 < GFrameNumberRenderThread;
		LRU.RemoveNode(PSO->LRUNode);
		PSO->LRUNode = 0;

		LRUUsedPipelineSize -= PSO->PipelineCacheSize;
		LRUUsedPipelineCount--;

		PSO->DeleteVkPipeline(bImmediate);
		if (GVulkanReleaseShaderModuleWhenEvictingPSO)
		{
	        for (int ShaderStageIndex = 0; ShaderStageIndex < ShaderStage::NumGraphicsStages; ShaderStageIndex++)
	        {
				if (PSO->VulkanShaders[ShaderStageIndex] != nullptr)
				{
					PSO->VulkanShaders[ShaderStageIndex]->PurgeShaderModules();
				}
			}
		}
		SET_DWORD_STAT(STAT_VulkanNumPSOLRUSize, LRUUsedPipelineSize);
		SET_DWORD_STAT(STAT_VulkanNumPSOLRU, LRUUsedPipelineCount);
	}
	else
	{
		check(0 == PSO->GetVulkanPipeline());
	}
}

bool FVulkanPipelineStateCacheManager::LRUEvictOne(bool bOnlyOld)
{
	check(bUseLRU);
	uint32 tid = FPlatformTLS::GetCurrentThreadId();
	FVulkanGraphicsPipelineStateLRUNode* Node = LRU.GetTail();
	check(Node != 0);
	TRefCountPtr<FVulkanGraphicsPipelineState> PSO = Node->GetValue();

	bool bImmediate = PSO->LRUFrame + 3 < GFrameNumberRenderThread;
	if(bOnlyOld && !bImmediate)
	{
		return false;
	}
	check(PSO->LRUFrame != GFrameNumberRenderThread);

	LRURemove(PSO);
	return true;
}

void FVulkanPipelineStateCacheManager::LRURemoveAll()
{
	if (!bUseLRU)
	{
		return;
	}
	check(0);
}


