// Copyright Epic Games, Inc. All Rights Reserved..

/*=============================================================================
	VulkanPipeline.h: Private Vulkan RHI definitions.
=============================================================================*/

#pragma once

#include "VulkanResources.h"
#include "VulkanShaderResources.h"
#include "VulkanDescriptorSets.h"
#include "ShaderPipelineCache.h"


#define VULKAN_PSO_CACHE_DEBUG 0
#define VULKAN_USE_SHADERKEYS 1
class FVulkanDevice;
class FVulkanGraphicsPipelineState;


using FVulkanGraphicsPipelineStateLRU = TDoubleLinkedList<FVulkanGraphicsPipelineState*>;
using FVulkanGraphicsPipelineStateLRUNode = FVulkanGraphicsPipelineStateLRU::TDoubleLinkedListNode;


class FVulkanPSOKey : public TDataKey<FVulkanPSOKey, true> {};

template <typename TVulkanShader>
static inline uint64 GetShaderKey(FRHIGraphicsShader* ShaderType)
{
	TVulkanShader* VulkanShader = static_cast<TVulkanShader*>(ShaderType);
	FVulkanShader* BaseVulkanShader = static_cast<FVulkanShader*>(VulkanShader);
	return BaseVulkanShader ? BaseVulkanShader->GetShaderKey() : 0;
}

inline uint64 GetShaderKeyForGfxStage(const FBoundShaderStateInput& BSI, ShaderStage::EStage Stage)
{
	switch (Stage)
	{
	case ShaderStage::Vertex:
		return GetShaderKey<FVulkanVertexShader>(BSI.VertexShaderRHI);
	case ShaderStage::Pixel:
		return GetShaderKey<FVulkanPixelShader>(BSI.PixelShaderRHI);
	case ShaderStage::Geometry:
		return GetShaderKey<FVulkanGeometryShader>(BSI.GetGeometryShader());
	case ShaderStage::Mesh:
		return GetShaderKey<FVulkanMeshShader>(BSI.GetMeshShader());
	case ShaderStage::Task:
		return GetShaderKey<FVulkanTaskShader>(BSI.GetAmplificationShader());
	default:
		check(0);
	}

	return 0;
}


void GetVulkanGfxShaders(const FBoundShaderStateInput& BSI, FVulkanShader* OutShaders[ShaderStage::NumGraphicsStages]);
void GetVulkanGfxShaders(FVulkanDevice* Device, const FVulkanGraphicsPipelineState& GfxPipelineState, FVulkanShader* OutShaders[ShaderStage::NumGraphicsStages]);



struct FVulkanShaderHashes
{
	uint32 Hash;
	FShaderHash Stages[ShaderStage::MaxNumStages];

	FVulkanShaderHashes();
	FVulkanShaderHashes(const FGraphicsPipelineStateInitializer& PSOInitializer);

	friend inline uint32 GetTypeHash(const FVulkanShaderHashes& Hashes)
	{
		return Hashes.Hash;
	}

	inline void Finalize()
	{
		Hash = FCrc::MemCrc32(Stages, sizeof(Stages));
	}

	friend inline bool operator == (const FVulkanShaderHashes& A, const FVulkanShaderHashes& B)
	{
		for (int32 Index = 0; Index < ShaderStage::MaxNumStages; ++Index)
		{
			if (A.Stages[Index] != B.Stages[Index])
			{
				return false;
			}
		}

		return true;
	}
};

struct FVulkanPipelineSize
{
	//TODO: increase hash to 12 bytes to reduce collisions
	uint64 ShaderHash;
	uint32 PipelineSize;

	FVulkanPipelineSize()
		:ShaderHash(0), PipelineSize(0)
	{
	}
};

struct FDescriptorSetLayoutBinding
{
	uint32 Binding;
	uint8 DescriptorType;
	uint8 StageFlags;

	void ReadFrom(const VkDescriptorSetLayoutBinding& InState);
	void WriteInto(VkDescriptorSetLayoutBinding& OutState) const;

	bool operator==(const FDescriptorSetLayoutBinding& In) const
	{
		return  Binding == In.Binding &&
			DescriptorType == In.DescriptorType &&
			StageFlags == In.StageFlags;
	}
};




struct FGfxPipelineDesc
{
	FVulkanPSOKey CreateKey2() const;

	uint32 VertexInputKey;
	uint16 RasterizationSamples;
	uint32 Topology;
	struct FBlendAttachment
	{
		bool bBlend;
		uint8 ColorBlendOp;
		uint8 SrcColorBlendFactor;
		uint8 DstColorBlendFactor;
		uint8 AlphaBlendOp;
		uint8 SrcAlphaBlendFactor;
		uint8 DstAlphaBlendFactor;
		uint8 ColorWriteMask;

		void ReadFrom(const VkPipelineColorBlendAttachmentState& InState);
		void WriteInto(VkPipelineColorBlendAttachmentState& OutState) const;

		bool operator==(const FBlendAttachment& In) const
		{
			return bBlend == In.bBlend &&
				ColorBlendOp == In.ColorBlendOp &&
				SrcColorBlendFactor == In.SrcColorBlendFactor &&
				DstColorBlendFactor == In.DstColorBlendFactor &&
				AlphaBlendOp == In.AlphaBlendOp &&
				SrcAlphaBlendFactor == In.SrcAlphaBlendFactor &&
				DstAlphaBlendFactor == In.DstAlphaBlendFactor &&
				ColorWriteMask == In.ColorWriteMask;
		}
	};
	TArray<FBlendAttachment> ColorAttachmentStates;

	TArray<TArray<FDescriptorSetLayoutBinding>> DescriptorSetLayoutBindings;

	struct FVertexBinding
	{
		uint32 Stride;
		uint16 Binding;
		uint16 InputRate;

		void ReadFrom(const VkVertexInputBindingDescription& InState);
		void WriteInto(VkVertexInputBindingDescription& OutState) const;
		void WriteInto(VkVertexInputBindingDescription2EXT& OutState) const;

		bool operator==(const FVertexBinding& In) const
		{
			return Stride == In.Stride &&
				Binding == In.Binding &&
				InputRate == In.InputRate;
		}
	};
	TArray<FVertexBinding> VertexBindings;
	struct FVertexAttribute
	{
		uint32 Location;
		uint32 Binding;
		uint32 Format;
		uint32 Offset;

		void ReadFrom(const VkVertexInputAttributeDescription& InState);
		void WriteInto(VkVertexInputAttributeDescription& OutState) const;
		void WriteInto(VkVertexInputAttributeDescription2EXT& OutState) const;

		bool operator==(const FVertexAttribute& In) const
		{
			return Location == In.Location &&
				Binding == In.Binding &&
				Format == In.Format &&
				Offset == In.Offset;
		}
	};
	TArray<FVertexAttribute> VertexAttributes;

	struct FRasterizer
	{
		uint8 PolygonMode;
		uint8 CullMode;
		float DepthBiasSlopeScale;
		float DepthBiasConstantFactor;

		void ReadFrom(const VkPipelineRasterizationStateCreateInfo& InState);
		void WriteInto(VkPipelineRasterizationStateCreateInfo& OutState) const;

		bool operator==(const FRasterizer& In) const
		{
			return PolygonMode == In.PolygonMode &&
				CullMode == In.CullMode &&
				DepthBiasSlopeScale == In.DepthBiasSlopeScale &&
				DepthBiasConstantFactor == In.DepthBiasConstantFactor;
		}
	};
	FRasterizer Rasterizer;

	struct FDepthStencil
	{
		uint8 DepthCompareOp;
		bool bDepthTestEnable;
		bool bDepthWriteEnable;
		bool bStencilTestEnable;
		bool bDepthBoundsTestEnable;
		uint8 FrontFailOp;
		uint8 FrontPassOp;
		uint8 FrontDepthFailOp;
		uint8 FrontCompareOp;
		uint32 FrontCompareMask;
		uint32 FrontWriteMask;
		uint32 FrontReference;
		uint8 BackFailOp;
		uint8 BackPassOp;
		uint8 BackDepthFailOp;
		uint8 BackCompareOp;
		uint32 BackCompareMask;
		uint32 BackWriteMask;
		uint32 BackReference;

		void ReadFrom(const VkPipelineDepthStencilStateCreateInfo& InState);
		void WriteInto(VkPipelineDepthStencilStateCreateInfo& OutState) const;

		bool operator==(const FDepthStencil& In) const
		{
			return DepthCompareOp == In.DepthCompareOp &&
				bDepthTestEnable == In.bDepthTestEnable &&
				bDepthWriteEnable == In.bDepthWriteEnable &&
				bDepthBoundsTestEnable == In.bDepthBoundsTestEnable &&
				bStencilTestEnable == In.bStencilTestEnable &&
				FrontFailOp == In.FrontFailOp &&
				FrontPassOp == In.FrontPassOp &&
				FrontDepthFailOp == In.FrontDepthFailOp &&
				FrontCompareOp == In.FrontCompareOp &&
				FrontCompareMask == In.FrontCompareMask &&
				FrontWriteMask == In.FrontWriteMask &&
				FrontReference == In.FrontReference &&
				BackFailOp == In.BackFailOp &&
				BackPassOp == In.BackPassOp &&
				BackDepthFailOp == In.BackDepthFailOp &&
				BackCompareOp == In.BackCompareOp &&
				BackCompareMask == In.BackCompareMask &&
				BackWriteMask == In.BackWriteMask &&
				BackReference == In.BackReference;
		}
	};
	FDepthStencil DepthStencil;

#if VULKAN_USE_SHADERKEYS
	uint64 ShaderKeys[ShaderStage::NumGraphicsStages];
	uint64 ShaderKeyShared;
#else
	FVulkanShaderHashes ShaderHashes;
#endif
	uint64 SpecializationKey;

	struct FRenderTargets
	{
		struct FAttachmentRef
		{
			uint32 Attachment;
			uint64 Layout;

			void ReadFrom(const VkAttachmentReference& InState);
			void WriteInto(VkAttachmentReference& OutState) const;
			bool operator == (const FAttachmentRef& In) const
			{
				return Attachment == In.Attachment && Layout == In.Layout;
			}
		};

		struct FStencilAttachmentRef
		{
			uint64 Layout;

			void ReadFrom(const VkAttachmentReferenceStencilLayout& InState);
			void WriteInto(VkAttachmentReferenceStencilLayout& OutState) const;
			bool operator == (const FStencilAttachmentRef& In) const
			{
				return Layout == In.Layout;
			}
		};

		TArray<FAttachmentRef> ColorAttachments;
		TArray<FAttachmentRef> ResolveAttachments;
		FAttachmentRef Depth;
		FStencilAttachmentRef Stencil;
		FAttachmentRef FragmentDensity;

		struct FAttachmentDesc
		{
			uint32 Format;
			uint8 Flags;
			uint8 Samples;
			uint8 LoadOp;
			uint8 StoreOp;
			uint8 StencilLoadOp;
			uint8 StencilStoreOp;
			uint64 InitialLayout;
			uint64 FinalLayout;

			bool operator==(const FAttachmentDesc& In) const
			{
				return Format == In.Format &&
					Flags == In.Flags &&
					Samples == In.Samples &&
					LoadOp == In.LoadOp &&
					StoreOp == In.StoreOp &&
					StencilLoadOp == In.StencilLoadOp &&
					StencilStoreOp == In.StencilStoreOp &&
					InitialLayout == In.InitialLayout &&
					FinalLayout == In.FinalLayout;
			}

			void ReadFrom(const VkAttachmentDescription &InState);
			void WriteInto(VkAttachmentDescription& OutState) const;
		};

		struct FStencilAttachmentDesc
		{
			uint64 InitialLayout;
			uint64 FinalLayout;

			bool operator==(const FStencilAttachmentDesc& In) const
			{
				return InitialLayout == In.InitialLayout &&
					FinalLayout == In.FinalLayout;
			}

			void ReadFrom(const VkAttachmentDescriptionStencilLayout& InState);
			void WriteInto(VkAttachmentDescriptionStencilLayout& OutState) const;
		};

		TArray<FAttachmentDesc> Descriptions;
		FStencilAttachmentDesc StencilDescription;

		uint8 NumAttachments;
		uint8 NumColorAttachments;
		uint8 bHasDepthStencil;
		uint8 bHasResolveAttachments;
		uint8 bHasDepthStencilResolve;
		uint8 bHasFragmentDensityAttachment;
		uint8 NumUsedClearValues;
		uint32 RenderPassCompatibleHash;
		FVector Extent3D;

		void ReadFrom(const FVulkanRenderTargetLayout &InState);
		void WriteInto(FVulkanRenderTargetLayout& OutState) const;

		bool operator==(const FRenderTargets& In) const
		{
			return ColorAttachments == In.ColorAttachments &&
				ResolveAttachments == In.ResolveAttachments &&
				Depth == In.Depth &&
				Stencil == In.Stencil &&
				FragmentDensity == In.FragmentDensity &&
				Descriptions == In.Descriptions &&
				StencilDescription == In.StencilDescription &&
				NumAttachments == In.NumAttachments &&
				NumColorAttachments == In.NumColorAttachments &&
				bHasDepthStencil == In.bHasDepthStencil &&
				bHasResolveAttachments == In.bHasResolveAttachments &&
				bHasDepthStencilResolve == In.bHasDepthStencilResolve &&
				bHasFragmentDensityAttachment == In.bHasFragmentDensityAttachment &&
				NumUsedClearValues == In.NumUsedClearValues &&
				RenderPassCompatibleHash == In.RenderPassCompatibleHash &&
				Extent3D == In.Extent3D;
		}
	};
	FRenderTargets RenderTargets;

	uint8 SubpassIndex;

	uint8 UseAlphaToCoverage;

	EVRSShadingRate ShadingRate = EVRSShadingRate::VRSSR_1x1;
	EVRSRateCombiner Combiner = EVRSRateCombiner::VRSRB_Passthrough;

	bool operator==(const FGfxPipelineDesc& In) const
	{
		if (VertexInputKey != In.VertexInputKey)
		{
			return false;
		}

		if (RasterizationSamples != In.RasterizationSamples)
		{
			return false;
		}

		if (Topology != In.Topology)
		{
			return false;
		}

		if (ColorAttachmentStates != In.ColorAttachmentStates)
		{
			return false;
		}

		if (DescriptorSetLayoutBindings != In.DescriptorSetLayoutBindings)
		{
			return false;
		}

		if (!(Rasterizer == In.Rasterizer))
		{
			return false;
		}

		if (!(DepthStencil == In.DepthStencil))
		{
			return false;
		}

		if (!(SubpassIndex == In.SubpassIndex))
		{
			return false;
		}

		if (!(UseAlphaToCoverage == In.UseAlphaToCoverage))
		{
			return false;
		}
#if 0 == VULKAN_USE_SHADERKEYS
		if (!(ShaderHashes == In.ShaderHashes))
		{
			return false;
		}
#else
		if (0 != FMemory::Memcmp(ShaderKeys, In.ShaderKeys, sizeof(ShaderKeys)))
		{
			return false;
		}
#endif

		if (SpecializationKey != In.SpecializationKey)
		{
			return false;
		}

		if (!(RenderTargets == In.RenderTargets))
		{
			return false;
		}

		if (VertexBindings != In.VertexBindings)
		{
			return false;
		}

		if (VertexAttributes != In.VertexAttributes)
		{
			return false;
		}

		if (ShadingRate != In.ShadingRate)
		{
			return false;
		}
		
		if (Combiner != In.Combiner)
		{
			return false;
		}

		return true;
	}
};




class FVulkanPipelineStateCacheManager
{
public:

	// Array of potential cache locations; first entries have highest priority. Only one cache file is loaded. If unsuccessful, tries next entry in the array.
	void InitAndLoad(const TArray<FString>& CacheFilenames);
	void Save(const FString& CacheFilename);

	FVulkanPipelineStateCacheManager(FVulkanDevice* InParent);
	~FVulkanPipelineStateCacheManager();

	void RebuildCache();

	FVulkanComputePipeline* GetOrCreateComputePipeline(const FComputePipelineStateInitializer& Initializer);
	void NotifyDeletedComputePipeline(FVulkanComputePipeline* Pipeline);

	VkResult CreateRayTracingPipeline(const VkRayTracingPipelineCreateInfoKHR& CreateInfo, bool bIsPartial, VkPipeline& OutPipeline);

private:
	class FPipelineCache;

	/** Delegate handlers to track the ShaderPipelineCache precompile. */
	void OnShaderPipelineCacheOpened(FString const& Name, EShaderPlatform Platform, uint32 Count, const FGuid& VersionGuid, FShaderPipelineCache::FShaderCachePrecompileContext& ShaderCachePrecompileContext);
	void OnShaderPipelineCachePrecompilationComplete(uint32 Count, double Seconds, const FShaderPipelineCache::FShaderCachePrecompileContext& ShaderCachePrecompileContext);

	void CreateGfxEntry(const FGraphicsPipelineStateInitializer& PSOInitializer, FVulkanDescriptorSetsLayoutInfo& DescriptorSetLayoutInfo, FGfxPipelineDesc* Desc);
	bool Load(const TArray<FString>& CacheFilenames, FPipelineCache& Cache);
	void SavePSOCache(const FString& CacheFilename, FPipelineCache& Cache);
	void DestroyCache();

	FGraphicsPipelineStateRHIRef RHICreateGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer);
	FVulkanComputePipeline* RHICreateComputePipelineState(const FComputePipelineStateInitializer& Initializer);
	void NotifyDeletedGraphicsPSO(FRHIGraphicsPipelineState* PSO);
	bool CreateGfxPipelineFromEntry(FVulkanGraphicsPipelineState* PSO, FVulkanShader* Shaders[ShaderStage::NumGraphicsStages], FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType PSOCompileType);
	bool CreateGfxPipelineFromLibrary(FVulkanGraphicsPipelineState& GraphicsPipelineState, FVulkanShader* Shaders[ShaderStage::NumGraphicsStages], FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType PSOCompileType);

	void CreateVertexInputLibraries();
	VkPipeline VertexInputPipelineLibraryVS = VK_NULL_HANDLE;
	VkPipeline VertexInputPipelineLibraryMS = VK_NULL_HANDLE;

	void CreateEmptyFragmentStageLibrary();
	VkPipeline EmptyFragmentStageLibrary = VK_NULL_HANDLE;

	VkPipeline FindOrCreateFragmentOutputStateLibrary(const FVulkanGraphicsPipelineState& GraphicsPipelineState);
	TMap<uint32, VkPipeline> FragmentOutputStatePipelineLibraries;
	FCriticalSection FragmentOutputStateCS;

	VkResult CreateVKPipeline(FVulkanGraphicsPipelineState* PSO, FVulkanShader* Shaders[ShaderStage::NumGraphicsStages], const VkGraphicsPipelineCreateInfo& PipelineInfo, FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType PSOCompileType);

	FVulkanLayout* FindOrAddLayout(const FVulkanDescriptorSetsLayoutInfo& DescriptorSetLayoutInfo, bool bGfxLayout, bool bUsesBindless);
	FVulkanComputePipeline* CreateComputePipeline(const FComputePipelineStateInitializer& Initializer);

	/** LRU Related functions */
	void TickLRU();
	bool LRUEvictImmediately();
	void LRUTrim(uint32 nSpaceNeeded);
	void LRUAdd(FVulkanGraphicsPipelineState* PSO);
	void LRUTouch(FVulkanGraphicsPipelineState* PSO);
	bool LRUEvictOne(bool bOnlyOld = false);
	void LRURemoveAll();
	void LRUDump();
	void LRUDebugEvictAll(); //evict all that are safe to evict without stalling..
	void LRURemove(FVulkanGraphicsPipelineState* PSO);
	void LRUCheckNotInside(FVulkanGraphicsPipelineState* PSO);


	FVulkanDevice* Device;
	bool bEvictImmediately;
	FString CompiledPSOCacheTopFolderPath;
	FString CompiledPSOCacheFolderName;
	FDelegateHandle OnShaderPipelineCacheOpenedDelegate;
	FDelegateHandle OnShaderPipelineCachePrecompilationCompleteDelegate;


	FRWLock ComputePipelineLock;
	TMap<uint64, FVulkanComputePipeline*> ComputePipelineEntries;

	template<typename TType>
	class FScopedRWAccessor
	{
		bool bWriteAccess;
		TType& ProtectedObj;
		FRWLock& RWLock;
	public:
		FScopedRWAccessor(bool bWriteAccessIn, TType& ProtectedObjIn, FRWLock& RWLockIn) : bWriteAccess(bWriteAccessIn), ProtectedObj(ProtectedObjIn), RWLock(RWLockIn) { bWriteAccess ? RWLock.WriteLock() : RWLock.ReadLock(); }
		~FScopedRWAccessor() { bWriteAccess ? RWLock.WriteUnlock() : RWLock.ReadUnlock(); }
		TType& Get() { return ProtectedObj; }
	};

	using FScopedPipelineCache = FScopedRWAccessor<VkPipelineCache>;

	enum class EPipelineCacheAccess : uint8
	{
		Shared,			// 'read' access, or for use when the API does its own synchronization.
		Exclusive		// 'write' access, excludes all other usage for the duration. 
	};
	class FPipelineCache
	{
		VkPipelineCache PipelineCache = VK_NULL_HANDLE;
		FRWLock PipelineCacheLock;
	public:
		FScopedPipelineCache Get(EPipelineCacheAccess PipelineAccessType) { return FScopedPipelineCache(PipelineAccessType == EPipelineCacheAccess::Exclusive, PipelineCache, PipelineCacheLock); }
	};
	FPipelineCache GlobalPSOCache;		// contains all PSO caches opened during the program run as well as PSO objects created on the fly

	FPipelineCache CurrentPrecompilingPSOCache;
	// if true, we will link to the PSOFC, loading later, when we have that guid and only if the guid matches, saving only if there is no match, and only saving after the PSOFC is done.
	bool bPrecompilingCacheLoadedFromFile;
	FGuid CurrentPrecompilingPSOCacheGuid;

	TSet<FGuid> CompiledPSOCaches;

	FCriticalSection LayoutMapCS;
	TMap<FVulkanDescriptorSetsLayoutInfo, FVulkanLayout*> LayoutMap;
	FVulkanDescriptorSetLayoutMap DSetLayoutMap;


	FCriticalSection GraphicsPSOLockedCS;
	TMap<FVulkanPSOKey, FVulkanGraphicsPipelineState*> GraphicsPSOLockedMap;

	
	FCriticalSection LRUCS;
	FVulkanGraphicsPipelineStateLRU LRU;
	uint32 LRUUsedPipelineSize = 0;
	uint32 LRUUsedPipelineCount = 0;
	uint32 LRUUsedPipelineMax = 0;
	TMap<uint64, FVulkanPipelineSize> LRU2SizeList;	// key: Shader hash (FShaderHash), value: pipeline size
	bool bUseLRU = true;
	friend class FVulkanDynamicRHI;
	friend class FVulkanCommandListContext;
	friend class FVulkanGraphicsPipelineState;

	struct FVulkanLRUCacheFile
	{
		enum
		{
			LRU_CACHE_VERSION = 2,
		};
		struct FFileHeader
		{
			int32 Version = -1;
			int32 SizeOfPipelineSizes = -1;
		} Header;

		TArray<FVulkanPipelineSize> PipelineSizes;

		void Save(FArchive& Ar);
		bool Load(FArchive& Ar);
	};

	static constexpr uint32 MaxNumSpecializationConstants = 32;
	TStaticArray<VkSpecializationMapEntry, MaxNumSpecializationConstants> SpecializationMapEntries;
};

struct VulkanResourceFrameCounter
{
	void Set(uint32_t Frame) const
	{
		LastUsedFrame.store(Frame, std::memory_order::relaxed);
	}
	
	uint32 Get() const
	{
		return LastUsedFrame.load(std::memory_order::relaxed);
	}

	// Note that the frame counter may be updated from any number of threads, and is not entirely synchronized
	mutable std::atomic<uint32> LastUsedFrame{0};
};

// Common pipeline class
class FVulkanPipeline
{
public:
	FVulkanPipeline(FVulkanDevice* InDevice);
	virtual ~FVulkanPipeline();

	inline VkPipeline GetHandle() const
	{
		return Pipeline;
	}

	inline const FVulkanLayout& GetLayout() const
	{
		return *Layout;
	}
#if VULKAN_ENABLE_LRU_CACHE
	inline void DeleteVkPipeline(bool ImmediateDestroy)
	{
		if (Pipeline != VK_NULL_HANDLE)
		{
			if (ImmediateDestroy)
			{
				VulkanRHI::vkDestroyPipeline(Device->GetHandle(), Pipeline, VULKAN_CPU_ALLOCATOR);
			}
			else
			{
				Device->GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::Pipeline, Pipeline);
			}
			Pipeline = VK_NULL_HANDLE;
		}
	}
#endif

	VulkanResourceFrameCounter FrameCounter;
	
protected:
	FVulkanDevice* Device;
	VkPipeline Pipeline;
	FVulkanLayout* Layout; /*owned by FVulkanPipelineStateCacheManager, do not delete yourself !*/

	friend class FVulkanPipelineStateCacheManager;
	friend class FVulkanGraphicsPipelineState;
	friend class FVulkanComputePipelineDescriptorState;
};

class FVulkanComputePipeline : public FVulkanPipeline, public FRHIComputePipelineState
{
public:
	FVulkanComputePipeline(FVulkanDevice* InDevice, const FComputePipelineStateInitializer& Initializer);
	virtual ~FVulkanComputePipeline();

	const FVulkanShaderHeader& GetShaderCodeHeader() const
	{
		return ResourceCast(GetComputeShader())->GetCodeHeader();
	}

	const FVulkanLayout& GetLayout() const
	{
		return *Layout;
	}

	void Bind(FVulkanCommandBuffer& CommandBuffer);

	bool UsesBindless() const
	{
		return bUsesBindless;
	}

	const FSpecializationContainerType& GetSpecializationConstants() const
	{
		return SpecializationConstants;
	}

protected:
	const bool bUsesBindless;
	const FSpecializationContainerType SpecializationConstants;
};

class FVulkanGraphicsPipelineState : public FRHIGraphicsPipelineState
{
public:
	FVulkanGraphicsPipelineState(FVulkanDevice* Device, const FGraphicsPipelineStateInitializer& PSOInitializer, const FGfxPipelineDesc& Desc, FVulkanPSOKey* Key);
	~FVulkanGraphicsPipelineState();

	const FVulkanVertexInputStateInfo& GetVertexInputState() const
	{
		return VertexInputState;
	}

	const FVulkanLayout& GetLayout() const
	{
		return *Layout;
	}

	void Bind(FVulkanCommandBuffer& CommandBuffer);

	const uint64 GetShaderKey(EShaderFrequency Frequency) const
	{
		ShaderStage::EStage Stage = ShaderStage::GetStageForFrequency(Frequency);
		return ShaderKeys[Stage];
	}

	const FVulkanShader* GetVulkanShader(EShaderFrequency Frequency) const
	{
		ShaderStage::EStage Stage = ShaderStage::GetStageForFrequency(Frequency);
		return VulkanShaders[Stage];
	}

	VkPipeline GetVulkanPipeline() const
	{
		return VulkanPipeline;
	}

	FRHIGraphicsShader* GetShader(EShaderFrequency Frequency) const override
	{
		switch (Frequency)
		{
		case SF_Vertex: return static_cast<FVulkanVertexShader*>(VulkanShaders[ShaderStage::Vertex]);
		case SF_Pixel: return static_cast<FVulkanPixelShader*>(VulkanShaders[ShaderStage::Pixel]);
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case SF_Mesh: return static_cast<FVulkanMeshShader*>(VulkanShaders[ShaderStage::Mesh]);
		case SF_Amplification: return static_cast<FVulkanTaskShader*>(VulkanShaders[ShaderStage::Task]);
#endif
#if VULKAN_SUPPORTS_GEOMETRY_SHADERS
		case SF_Geometry: return static_cast<FVulkanGeometryShader*>(VulkanShaders[ShaderStage::Geometry]);
#endif
		default: return nullptr;
		}
	}

	void DeleteVkPipeline(bool bImmediate);
	void GetOrCreateShaderModules(TRefCountPtr<FVulkanShaderModule>(&ShaderModulesOUT)[ShaderStage::NumGraphicsStages], FVulkanShader* const* Shaders);
	FVulkanShader::FSpirvCode GetPatchedSpirvCode(FVulkanShader* Shader);
	void PurgeShaderModules(FVulkanShader*const* Shaders);

	bool UsesBindless() const
	{
		return bUsesBindless;
	}


	bool								bHasInputAttachments = false;
	bool								bIsRegistered;
	bool								bUsesBindless = false;

	uint64								ShaderKeys[ShaderStage::NumGraphicsStages];
	TEnumAsByte<EPrimitiveType>			PrimitiveType;


	VkPipeline VulkanPipeline;
	FVulkanVertexInputStateInfo VertexInputState;
	FVulkanLayout* Layout;
	FVulkanDevice* Device;
	const FGfxPipelineDesc Desc;
	FVulkanShader* VulkanShaders[ShaderStage::NumGraphicsStages];
	FSpecializationContainerType SpecializationConstants[ShaderStage::NumGraphicsStages];
	const FVulkanRenderPass* RenderPass;


	FVulkanGraphicsPipelineStateLRUNode* LRUNode = nullptr;
	uint32 LRUFrame = UINT32_MAX;
	uint32 PipelineCacheSize = UINT32_MAX;
	uint64	PrecacheKey;// hash of elements relevant to the PSO cache
	FVulkanPSOKey							VulkanKey;

	VulkanResourceFrameCounter FrameCounter;

#if VULKAN_PSO_CACHE_DEBUG
	FPixelShaderRHIRef					PixelShaderRHI;
	FVertexShaderRHIRef					VertexShaderRHI;
	FVertexDeclarationRHIRef			VertexDeclarationRHI;

#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	FGeometryShaderRHIRef				GeometryShaderRHI;
#endif

	FGraphicsPipelineStateInitializer		PSOInitializer;
#endif

};

// Returns a structure with all input attachments pre-configured
inline VkRenderingInputAttachmentIndexInfo GetDefaultRenderingInputAttachmentIndexInfo(uint32 NumColorAttachments)
{
	static uint32 InputAttachmentIndices[] = {
		(uint32)FVulkanShaderHeader::EAttachmentType::Depth,
		(uint32)FVulkanShaderHeader::EAttachmentType::Color0,
		(uint32)FVulkanShaderHeader::EAttachmentType::Color1,
		(uint32)FVulkanShaderHeader::EAttachmentType::Color2,
		(uint32)FVulkanShaderHeader::EAttachmentType::Color3,
		(uint32)FVulkanShaderHeader::EAttachmentType::Color4,
		(uint32)FVulkanShaderHeader::EAttachmentType::Color5,
		(uint32)FVulkanShaderHeader::EAttachmentType::Color6,
		(uint32)FVulkanShaderHeader::EAttachmentType::Color7,
	};

	static_assert(UE_ARRAY_COUNT(InputAttachmentIndices) == (uint32)FVulkanShaderHeader::EAttachmentType::Count,
		"Missing input attachment indices in static array!");

	VkRenderingInputAttachmentIndexInfo RenderingInputAttachmentIndexInfo = {
			.sType = VK_STRUCTURE_TYPE_RENDERING_INPUT_ATTACHMENT_INDEX_INFO,
			.pNext = nullptr,
			.colorAttachmentCount = NumColorAttachments,
			.pColorAttachmentInputIndices = &InputAttachmentIndices[(uint32)FVulkanShaderHeader::EAttachmentType::Color0],
			.pDepthInputAttachmentIndex = &InputAttachmentIndices[(uint32)FVulkanShaderHeader::EAttachmentType::Depth],
			.pStencilInputAttachmentIndex = nullptr
	};
	return RenderingInputAttachmentIndexInfo;
}


template<>
struct TVulkanResourceTraits<class FRHIComputePipelineState>
{
	typedef class FVulkanComputePipeline TConcreteType;
};

template<>
struct TVulkanResourceTraits<FRHIGraphicsPipelineState>
{
	typedef FVulkanGraphicsPipelineState TConcreteType;
};
