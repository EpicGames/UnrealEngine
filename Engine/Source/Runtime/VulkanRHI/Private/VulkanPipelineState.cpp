// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanStatePipeline.cpp: Vulkan pipeline state implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanPipelineState.h"
#include "VulkanResources.h"
#include "VulkanPipeline.h"
#include "VulkanContext.h"
#include "VulkanPendingState.h"
#include "VulkanPipeline.h"
#include "VulkanLLM.h"
#include "VulkanBindlessDescriptorManager.h"
#include "RHICoreShader.h"
#include "GlobalRenderResources.h"  // For GBlackTexture


#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
static TAutoConsoleVariable<int32> GAlwaysWriteDS(
	TEXT("r.Vulkan.AlwaysWriteDS"),
	0,
	TEXT(""),
	ECVF_RenderThreadSafe
);
#endif

static bool ShouldAlwaysWriteDescriptors()
{
#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
	return (GAlwaysWriteDS.GetValueOnAnyThread() != 0);
#else
	return false;
#endif
}

FVulkanComputePipelineDescriptorState::FVulkanComputePipelineDescriptorState(FVulkanDevice& InDevice, FVulkanComputePipeline* InComputePipeline)
	: FVulkanCommonPipelineDescriptorState(InDevice, ShaderStage::NumComputeStages, InComputePipeline->UsesBindless())
	, ComputePipeline(InComputePipeline)
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanShaders);
	check(InComputePipeline);
	const FVulkanShaderHeader& CodeHeader = InComputePipeline->GetShaderCodeHeader();
	PackedUniformBufferState.Init(CodeHeader);

	DescriptorSetsLayout = &InComputePipeline->GetLayout().GetDescriptorSetsLayout();

	UsedSetsMask = (CodeHeader.Bindings.Num() > 0) ? 1 : 0;

	CreateDescriptorWriteInfos();
	InComputePipeline->AddRef();

	for (const FVulkanShaderHeader::FGlobalSamplerInfo& GlobalSamplerInfo : CodeHeader.GlobalSamplerInfos)
	{
		checkSlow(!bUseBindless);
		SetSamplerState(ShaderStage::Compute, GlobalSamplerInfo.BindingIndex, &Device.GetGlobalSamplers(GlobalSamplerInfo.Type));
	}

	ensure(DSWriter.Num() == 0 || DSWriter.Num() == 1);
}

void FVulkanCommonPipelineDescriptorState::CreateDescriptorWriteInfos()
{
	check(DSWriteContainer.DescriptorWrites.Num() == 0);
	check(UsedSetsMask <= (uint32)(((uint32)1 << MaxNumSets) - 1));

	for (uint32 Set = 0; Set < MaxNumSets; ++Set)
	{
		const FVulkanDescriptorSetsLayoutInfo::FStageInfo& StageInfo = DescriptorSetsLayout->StageInfos[Set];
		if (!StageInfo.Types.Num())
		{
			continue;
		}
		
		if (UseVulkanDescriptorCache())
		{
			DSWriteContainer.HashableDescriptorInfo.AddZeroed(StageInfo.Types.Num() + 1); // Add 1 for the Layout
		}
		DSWriteContainer.DescriptorWrites.AddZeroed(StageInfo.Types.Num());
		DSWriteContainer.DescriptorImageInfo.AddZeroed(StageInfo.NumImageInfos);
		DSWriteContainer.DescriptorBufferInfo.AddZeroed(StageInfo.NumBufferInfos);
		DSWriteContainer.AccelerationStructureWrites.AddZeroed(StageInfo.NumAccelerationStructures);
		DSWriteContainer.AccelerationStructures.AddZeroed(StageInfo.NumAccelerationStructures);

		checkf(StageInfo.Types.Num() < 255, TEXT("Need more bits for BindingToDynamicOffsetMap (currently 8)! Requires %d descriptor bindings in a set!"), StageInfo.Types.Num());
		DSWriteContainer.BindingToDynamicOffsetMap.AddUninitialized(StageInfo.Types.Num());
	}

	FMemory::Memset(DSWriteContainer.BindingToDynamicOffsetMap.GetData(), 255, DSWriteContainer.BindingToDynamicOffsetMap.Num());

	check(DSWriter.Num() == 0);
	DSWriter.AddDefaulted(MaxNumSets);

	const FVulkanSamplerState& DefaultSampler = Device.GetDefaultSampler();
	const FVulkanView::FTextureView& DefaultImageView = ResourceCast(GBlackTexture->TextureRHI)->DefaultView->GetTextureView();

	FVulkanHashableDescriptorInfo* CurrentHashableDescriptorInfo = nullptr;
	if (UseVulkanDescriptorCache())
	{
		CurrentHashableDescriptorInfo = DSWriteContainer.HashableDescriptorInfo.GetData();
	}
	VkWriteDescriptorSet* CurrentDescriptorWrite = DSWriteContainer.DescriptorWrites.GetData();
	VkDescriptorImageInfo* CurrentImageInfo = DSWriteContainer.DescriptorImageInfo.GetData();
	VkDescriptorBufferInfo* CurrentBufferInfo = DSWriteContainer.DescriptorBufferInfo.GetData();
	VkWriteDescriptorSetAccelerationStructureKHR* CurrentAccelerationStructuresWriteDescriptors = DSWriteContainer.AccelerationStructureWrites.GetData();
	VkAccelerationStructureKHR* CurrentAccelerationStructures = DSWriteContainer.AccelerationStructures.GetData();

	uint8* CurrentBindingToDynamicOffsetMap = DSWriteContainer.BindingToDynamicOffsetMap.GetData();
	TArray<uint32> DynamicOffsetsStart;
	DynamicOffsetsStart.AddZeroed(MaxNumSets);
	uint32 TotalNumDynamicOffsets = 0;

	for (uint32 Set = 0; Set < MaxNumSets; ++Set)
	{
		const FVulkanDescriptorSetsLayoutInfo::FStageInfo& StageInfo = DescriptorSetsLayout->StageInfos[Set];
		if (!StageInfo.Types.Num())
		{
			continue;
		}

		DynamicOffsetsStart[Set] = TotalNumDynamicOffsets;

		const uint32 NumDynamicOffsets = DSWriter[Set].SetupDescriptorWrites(
			StageInfo.Types, CurrentHashableDescriptorInfo,
			CurrentDescriptorWrite, CurrentImageInfo, CurrentBufferInfo, CurrentBindingToDynamicOffsetMap,
			CurrentAccelerationStructuresWriteDescriptors,
			CurrentAccelerationStructures,
			DefaultSampler, DefaultImageView);

		if (bUseBindless)
		{
			DSWriter[Set].BindlessUBIndices.SetNum(StageInfo.Types.Num());
		}
		else
		{
			DSWriter[Set].SetExcludedResources(GetExcludedResources(Set));
		}

		TotalNumDynamicOffsets += NumDynamicOffsets;

		if (CurrentHashableDescriptorInfo) // UseVulkanDescriptorCache()
		{
			CurrentHashableDescriptorInfo += StageInfo.Types.Num();
			CurrentHashableDescriptorInfo->Layout.Max0 = UINT32_MAX;
			CurrentHashableDescriptorInfo->Layout.Max1 = UINT32_MAX;
			CurrentHashableDescriptorInfo->Layout.LayoutId = DescriptorSetsLayout->GetHandleIds()[Set];
			++CurrentHashableDescriptorInfo;
		}

		CurrentDescriptorWrite += StageInfo.Types.Num();
		CurrentImageInfo += StageInfo.NumImageInfos;
		CurrentBufferInfo += StageInfo.NumBufferInfos;
		CurrentAccelerationStructuresWriteDescriptors += StageInfo.NumAccelerationStructures;
		CurrentAccelerationStructures += StageInfo.NumAccelerationStructures;

		CurrentBindingToDynamicOffsetMap += StageInfo.Types.Num();
	}

	DynamicOffsets.AddZeroed(TotalNumDynamicOffsets);
	for (uint32 Set = 0; Set < MaxNumSets; ++Set)
	{
		DSWriter[Set].DynamicOffsets = DynamicOffsetsStart[Set] + DynamicOffsets.GetData();
	}

	DescriptorSetHandles.AddZeroed(MaxNumSets);
}

static inline VulkanRHI::FVulkanAllocation UpdatePackedUniformBuffers(const FPackedUniformBuffers& PackedUniformBuffers, FVulkanCommandListContext& InContext)
{
	const FPackedUniformBuffers::FPackedBuffer& StagedUniformBuffer = PackedUniformBuffers.GetBuffer();

	const uint32 UBSize = (uint32)StagedUniformBuffer.Num();
	const uint32 UBAlign = (uint32)InContext.Device.GetLimits().minUniformBufferOffsetAlignment;

	VulkanRHI::FVulkanAllocation TempAllocation;
	uint8* MappedPointer = InContext.Device.GetTempBlockAllocator().Alloc(UBSize, UBAlign, InContext, TempAllocation);

	FMemory::Memcpy(MappedPointer, StagedUniformBuffer.GetData(), UBSize);

	return TempAllocation;
}


static void SetDefaultDescriptorsForMissingBuffers(FVulkanDescriptorSetWriter& DescriptorSetWriter, FRHIShader* Shader, const uint32 SpecializationHash)
{
	// Grab defaults from system resources that we can use on descriptors we know won't be accessed
	const FVulkanView::FStructuredBufferView* DefaultStructureBufferUAV = nullptr;
	if (GEmptyStructuredBufferWithUAV && GEmptyStructuredBufferWithUAV->UnorderedAccessViewRHI.IsValid())
	{
		DefaultStructureBufferUAV = &ResourceCast(GEmptyStructuredBufferWithUAV->UnorderedAccessViewRHI.GetReference())->GetStructuredBufferView();
	}
	const FVulkanView::FTypedBufferView* DefaultTypedBufferSRV = nullptr;
	if (GEmptyVertexBufferWithUAV && GEmptyVertexBufferWithUAV->ShaderResourceViewRHI.IsValid())
	{
		DefaultTypedBufferSRV = &ResourceCast(GEmptyVertexBufferWithUAV->ShaderResourceViewRHI.GetReference())->GetTypedBufferView();
	}
	const FVulkanView::FTypedBufferView* DefaultTypedBufferUAV = nullptr;
	if (GEmptyVertexBufferWithUAV && GEmptyVertexBufferWithUAV->UnorderedAccessViewRHI.IsValid())
	{
		DefaultTypedBufferUAV = &ResourceCast(GEmptyVertexBufferWithUAV->UnorderedAccessViewRHI.GetReference())->GetTypedBufferView();
	}

	TArrayView<VkWriteDescriptorSet> DescriptorWrites(const_cast<VkWriteDescriptorSet*>(DescriptorSetWriter.GetWriteDescriptors()), DescriptorSetWriter.GetNumWrites());
	for (VkWriteDescriptorSet& Write : DescriptorWrites)
	{
#if RHI_INCLUDE_SHADER_DEBUG_DATA
		const bool bIsExcludedFromSpecialization = Shader->IsExcludedFromSpecialization(Write.dstBinding, SpecializationHash);
#else
		constexpr bool bIsExcludedFromSpecialization = true;
#endif

		switch (Write.descriptorType)
		{
		case VK_DESCRIPTOR_TYPE_STORAGE_BUFFER:
			if (DefaultStructureBufferUAV && (Write.pBufferInfo->buffer == VK_NULL_HANDLE))
			{
				checkf(bIsExcludedFromSpecialization, TEXT("Missing a storage buffer binding that isn't excluded in this specialization!"));
				VkDescriptorBufferInfo* BufferInfo = const_cast<VkDescriptorBufferInfo*>(Write.pBufferInfo);
				BufferInfo->buffer = DefaultStructureBufferUAV->Buffer;
				BufferInfo->offset = DefaultStructureBufferUAV->Offset;
				BufferInfo->range = DefaultStructureBufferUAV->Size;
			}
			break;
		case VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER:
			if (DefaultTypedBufferUAV && (Write.pTexelBufferView == nullptr))
			{
				checkf(bIsExcludedFromSpecialization, TEXT("Missing a storage texel buffer binding that isn't excluded in this specialization!"));
				VkWriteDescriptorSet& BufferInfo = const_cast<VkWriteDescriptorSet&>(Write);
				Write.pTexelBufferView = &DefaultTypedBufferUAV->View;
			}
			break;
		case VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER:
			if (DefaultTypedBufferSRV && (Write.pTexelBufferView == nullptr))
			{
				checkf(bIsExcludedFromSpecialization, TEXT("Missing a uniform texel buffer binding that isn't excluded in this specialization!"));
				Write.pTexelBufferView = &DefaultTypedBufferSRV->View;
			}
			break;
		default:
			// Do nothing: textures already have defaults and other types aren't excluded by specialization constants.
			break;
		}
	}
}



template<bool bUseDynamicGlobalUBs>
bool FVulkanComputePipelineDescriptorState::InternalUpdateDescriptorSets(FVulkanCommandListContext& Context)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanUpdateDescriptorSets);
#endif

	// Early exit
	if (!UsedSetsMask)
	{
		return false;
	}

	if (PackedUniformBufferState.Dirty != 0)
	{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
		SCOPE_CYCLE_COUNTER(STAT_VulkanApplyPackedUniformBuffers);
#endif
		SubmitPackedUniformBuffers<bUseDynamicGlobalUBs>(DSWriter[ShaderStage::Compute], UpdatePackedUniformBuffers(PackedUniformBufferState.UniformBuffers, Context));
		PackedUniformBufferState.Dirty = 0;
	}

	// We are not using UseVulkanDescriptorCache() for compute pipelines
	// Compute tend to use volatile resources that polute descriptor cache

	if (!Context.AcquirePoolSetAndDescriptorsIfNeeded(*DescriptorSetsLayout, true, DescriptorSetHandles.GetData()))
	{
		return false;
	}

	const VkDescriptorSet DescriptorSet = DescriptorSetHandles[0];
	DSWriter[0].SetDescriptorSet(DescriptorSet);
#if VULKAN_VALIDATE_DESCRIPTORS_WRITTEN
	for(FVulkanDescriptorSetWriter& Writer : DSWriter)
	{
		Writer.CheckAllWritten();
	}
#endif

	{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
		INC_DWORD_STAT_BY(STAT_VulkanNumUpdateDescriptors, DSWriteContainer.DescriptorWrites.Num());
		INC_DWORD_STAT(STAT_VulkanNumDescSets);
		SCOPE_CYCLE_COUNTER(STAT_VulkanVkUpdateDS);
#endif
		// Certain layers don't optimize SPIRV enough to know if resources are truly used,
		// so set defaults to the ones we know will be excluded to silence errors
		if (ComputePipeline && ComputePipeline->GetComputeShader()->HasSpecializationConstants())
		{
			check(ComputePipeline->GetSpecializationConstants().Num());
			const uint32 SpecializationHash = GetTypeHash(ComputePipeline->GetSpecializationConstants());
			SetDefaultDescriptorsForMissingBuffers(DSWriter[ShaderStage::Compute], ComputePipeline->GetComputeShader(), SpecializationHash);
		}

		VulkanRHI::vkUpdateDescriptorSets(Device.GetHandle(), DSWriteContainer.DescriptorWrites.Num(), DSWriteContainer.DescriptorWrites.GetData(), 0, nullptr);
	}

	return true;
}

void FVulkanComputePipelineDescriptorState::UpdateBindlessDescriptors(FVulkanCommandListContext& Context)
{
	check(bUseBindless);

	// We should only have uniform buffers at this point
	check(DSWriteContainer.DescriptorBufferInfo.Num() == DSWriteContainer.DescriptorWrites.Num());
	check(DSWriteContainer.DescriptorImageInfo.Num() == 0);

	const FVulkanShaderHeader& Header = ComputePipeline->GetShaderCodeHeader();
	checkf(Header.NumBoundUniformBuffers <= VulkanBindless::MaxUniformBuffersPerStage, TEXT("Compute shader has more uniform buffers than is permitted in bindless."));
	check(Header.NumBoundUniformBuffers == DSWriter[ShaderStage::EStage::Compute].NumWrites);

	if (Header.NumBoundUniformBuffers == 0)
	{
		return;
	}

	const VkDeviceSize UBOffsetAlignment = Device.GetLimits().minUniformBufferOffsetAlignment;

	auto GetGlobalUBAlloc = [&]()
	{
		VulkanRHI::FVulkanAllocation TempAllocation;
		check(PackedUniformBufferState.Mask <= 1);
		if (PackedUniformBufferState.Mask != 0)
		{
			const FPackedUniformBuffers::FPackedBuffer& StagedUniformBuffer = PackedUniformBufferState.UniformBuffers.GetBuffer();
			const int32 UBSize = StagedUniformBuffer.Num();

			uint8* MappedPointer = Device.GetTempBlockAllocator().Alloc(UBSize, UBOffsetAlignment, Context, TempAllocation, nullptr);
			FMemory::Memcpy(MappedPointer, StagedUniformBuffer.GetData(), UBSize);
		}
		return TempAllocation;
	};

	if (Device.GetBindlessDescriptorManager()->UseDescriptorHeaps())
	{
		uint64 GlobalAddressOrFirstBinding = DSWriter[ShaderStage::EStage::Compute].BindlessUBIndices[0];
		if (Header.PackedGlobalsSize > 0)
		{
			check(PackedUniformBufferState.Mask <= 1);
			if (PackedUniformBufferState.Mask != 0)
			{
				const FPackedUniformBuffers::FPackedBuffer& StagedUniformBuffer = PackedUniformBufferState.UniformBuffers.GetBuffer();
				const int32 UBSize = StagedUniformBuffer.Num();

				VkDescriptorAddressInfoEXT AddressInfo;
				VulkanRHI::FVulkanAllocation TempAllocation;
				uint8* MappedPointer = Device.GetTempBlockAllocator().Alloc(UBSize, UBOffsetAlignment, Context, TempAllocation, &AddressInfo);
				FMemory::Memcpy(MappedPointer, StagedUniformBuffer.GetData(), UBSize);

				GlobalAddressOrFirstBinding = AddressInfo.address;
			}
			else
			{
				GlobalAddressOrFirstBinding = MAX_uint64;
			}
		}

		TConstArrayView<uint32> Bindings;
		if (Header.NumBoundUniformBuffers > 1)
		{
			Bindings = TConstArrayView<uint32>(&DSWriter[ShaderStage::EStage::Compute].BindlessUBIndices[1], Header.NumBoundUniformBuffers - 1);
		}

		Context.GetCommandBuffer().SetBindingPushConstants(SF_Compute, GlobalAddressOrFirstBinding, Bindings);
	}
	else
	{
		// Create a local copy of the writes
		TArray<VkWriteDescriptorSet, TInlineAllocator<VulkanBindless::MaxUniformBuffersPerStage>> UBWrites;
		UBWrites.SetNumZeroed(Header.NumBoundUniformBuffers);
		FMemory::Memcpy(UBWrites.GetData(), DSWriter[ShaderStage::EStage::Compute].WriteDescriptors, Header.NumBoundUniformBuffers * sizeof(VkWriteDescriptorSet));

		VkDescriptorBufferInfo ConstantBufferInfo;

		// UBs are currently set from a fresh batch of descriptors for every call, so ignore PackedUniformBuffersDirty
		check(PackedUniformBufferState.Mask <= 1);
		if (PackedUniformBufferState.Mask != 0)
		{
			const FPackedUniformBuffers::FPackedBuffer& StagedUniformBuffer = PackedUniformBufferState.UniformBuffers.GetBuffer();
			const int32 UBSize = StagedUniformBuffer.Num();

			VulkanRHI::FVulkanAllocation TempAllocation;
			uint8* MappedPointer = Device.GetTempBlockAllocator().Alloc(UBSize, UBOffsetAlignment, Context, TempAllocation, nullptr);
			FMemory::Memcpy(MappedPointer, StagedUniformBuffer.GetData(), UBSize);

			VkWriteDescriptorSet& WriteDescriptorSet = UBWrites[0];
			check(WriteDescriptorSet.dstBinding == 0);
			check(WriteDescriptorSet.dstArrayElement == 0);
			check(WriteDescriptorSet.descriptorCount == 1);
			check(WriteDescriptorSet.descriptorType == VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER);

			WriteDescriptorSet.pBufferInfo = &ConstantBufferInfo;

			ConstantBufferInfo.buffer = TempAllocation.GetBufferHandle();
			ConstantBufferInfo.offset = TempAllocation.Offset;
			ConstantBufferInfo.range = UBSize;

			PackedUniformBufferState.Dirty = 0;
		}

		// Send to descriptor manager
		Device.GetBindlessDescriptorManager()->RegisterUniformBuffers(Context, VK_SHADER_STAGE_COMPUTE_BIT, UBWrites);
	}
}

TArray<uint32> FVulkanComputePipelineDescriptorState::GetExcludedResources(uint32 SetIndex) const
{
	check(SetIndex == 0);

	TArray<uint32> ExcludedResources;
	FRHIShader* RHIShader = ComputePipeline->GetComputeShader();
	if (RHIShader && RHIShader->HasSpecializationConstants())
	{
		const uint32 SpecializationHash = GetTypeHash(ComputePipeline->GetSpecializationConstants());

		for (auto& Pair : RHIShader->GetSpecializationExclusions())
		{
			if (Pair.Value.Contains(SpecializationHash))
			{
				ExcludedResources.Add(Pair.Key);
			}
		}
	}
	return ExcludedResources;
}


FVulkanGraphicsPipelineDescriptorState::FVulkanGraphicsPipelineDescriptorState(FVulkanDevice& InDevice, FVulkanGraphicsPipelineState* InGfxPipeline)
	: FVulkanCommonPipelineDescriptorState(InDevice, ShaderStage::NumGraphicsStages, InGfxPipeline->UsesBindless())
	, GfxPipeline(InGfxPipeline)
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanShaders);

	check(InGfxPipeline && InGfxPipeline->Layout && InGfxPipeline->Layout->IsGfxLayout());
	DescriptorSetsLayout = &InGfxPipeline->Layout->GetDescriptorSetsLayout();

	UsedSetsMask = 0;

	const FVulkanShaderFactory& ShaderFactory = Device.GetShaderFactory();

	TStaticArray<const FVulkanShaderHeader*, ShaderStage::NumGraphicsStages> StageHeaders(InPlace, nullptr);

	auto InitStage = [this, &StageHeaders](const FVulkanShader* InShader, ShaderStage::EStage InStage)
	{
		check(InShader);

		PackedUniformBufferStates[InStage].Init(InShader->GetCodeHeader());
		UsedSetsMask |= InShader->GetCodeHeader().Bindings.Num() ? (1u << InStage) : 0u;
		StageHeaders[InStage] = &InShader->GetCodeHeader();
	};

	if (uint64 VertexShaderKey = InGfxPipeline->GetShaderKey(SF_Vertex))
	{
		const FVulkanVertexShader* VertexShader = ShaderFactory.LookupShader<FVulkanVertexShader>(VertexShaderKey);
		InitStage(VertexShader, ShaderStage::Vertex);
	}

	if (uint64 PixelShaderKey = InGfxPipeline->GetShaderKey(SF_Pixel))
	{
		const FVulkanPixelShader* PixelShader = ShaderFactory.LookupShader<FVulkanPixelShader>(PixelShaderKey);
		InitStage(PixelShader, ShaderStage::Pixel);
	}

#if PLATFORM_SUPPORTS_MESH_SHADERS
	if (uint64 MeshShaderKey = InGfxPipeline->GetShaderKey(SF_Mesh))
	{
		const FVulkanMeshShader* MeshShader = ShaderFactory.LookupShader<FVulkanMeshShader>(MeshShaderKey);
		InitStage(MeshShader, ShaderStage::Mesh);
	}

	if (uint64 TaskShaderKey = InGfxPipeline->GetShaderKey(SF_Amplification))
	{
		const FVulkanTaskShader* TaskShader = ShaderFactory.LookupShader<FVulkanTaskShader>(TaskShaderKey);
		InitStage(TaskShader, ShaderStage::Task);
	}
#endif

#if VULKAN_SUPPORTS_GEOMETRY_SHADERS
	if (uint64 GeometryShaderKey = InGfxPipeline->GetShaderKey(SF_Geometry))
	{
		const FVulkanGeometryShader* GeometryShader = ShaderFactory.LookupShader<FVulkanGeometryShader>(GeometryShaderKey);
		InitStage(GeometryShader, ShaderStage::Geometry);
	}
#endif

	CreateDescriptorWriteInfos();
	//UE_LOGF(LogVulkanRHI, Warning, "GfxPSOState %p For PSO %p Writes:%d", this, InGfxPipeline, DSWriteContainer.DescriptorWrites.Num());

	InGfxPipeline->AddRef();

	for (int32 StageIndex = 0; StageIndex < StageHeaders.Num(); ++StageIndex)
	{
		if (const FVulkanShaderHeader* CodeHeader = StageHeaders[StageIndex])
		{
			for (const FVulkanShaderHeader::FGlobalSamplerInfo& GlobalSamplerInfo : CodeHeader->GlobalSamplerInfos)
			{
				checkSlow(!bUseBindless);
				SetSamplerState(StageIndex, GlobalSamplerInfo.BindingIndex, &Device.GetGlobalSamplers(GlobalSamplerInfo.Type));
			}
		}
	}
}

template<bool bUseDynamicGlobalUBs>
bool FVulkanGraphicsPipelineDescriptorState::InternalUpdateDescriptorSets(FVulkanCommandListContext& Context)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanUpdateDescriptorSets);
#endif

	// Early exit
	if (!UsedSetsMask)
	{
		return false;
	}

	// Process updates
	{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
		SCOPE_CYCLE_COUNTER(STAT_VulkanApplyPackedUniformBuffers);
#endif
		for (int32 Stage = 0; Stage < ShaderStage::NumGraphicsStages; ++Stage)
		{
			FVulkanPackedUniformBufferState& State = PackedUniformBufferStates[Stage];
			if (State.Dirty != 0)
			{
				MarkDirty(SubmitPackedUniformBuffers<bUseDynamicGlobalUBs>(DSWriter[Stage], UpdatePackedUniformBuffers(State.UniformBuffers, Context)));
				State.Dirty = 0;
			}
		}
	}

	if (UseVulkanDescriptorCache() && !HasVolatileResources())
	{
		if (bIsResourcesDirty)
		{
			Device.GetDescriptorSetCache().GetDescriptorSets(GetDSetsKey(), *DescriptorSetsLayout, DSWriter, DescriptorSetHandles.GetData());
			bIsResourcesDirty = false;
		}
	}
	else
	{
		const bool bNeedsWrite = (bIsResourcesDirty || ShouldAlwaysWriteDescriptors());

		// Allocate sets based on what changed
		if (Context.AcquirePoolSetAndDescriptorsIfNeeded(*DescriptorSetsLayout, bNeedsWrite, DescriptorSetHandles.GetData()))
		{
			{
				uint32 RemainingSetsMask = UsedSetsMask;
				uint32 Set = 0;
				uint32 NumSets = 0;
				while (RemainingSetsMask)
				{
					if (RemainingSetsMask & 1)
					{
						const VkDescriptorSet DescriptorSet = DescriptorSetHandles[Set];
						DSWriter[Set].SetDescriptorSet(DescriptorSet);
#if VULKAN_VALIDATE_DESCRIPTORS_WRITTEN
						DSWriter[Set].CheckAllWritten();
#endif
						++NumSets;
					}

					++Set;
					RemainingSetsMask >>= 1;
				}
#if VULKAN_ENABLE_AGGRESSIVE_STATS
				INC_DWORD_STAT_BY(STAT_VulkanNumUpdateDescriptors, DSWriteContainer.DescriptorWrites.Num());
				INC_DWORD_STAT_BY(STAT_VulkanNumDescSets, NumSets);
#endif
			}

			// Certain layers don't optimize SPIRV enough to know if resources are truly used,
			// so set defaults to the ones we know will be excluded to silence errors
			for (uint32 Stage = 0; Stage < MaxNumSets; ++Stage)
			{
				FRHIShader* Shader = GfxPipeline->GetShader(GetFrequencyForGfxStage((ShaderStage::EStage)Stage));
				if (Shader && Shader->HasSpecializationConstants())
				{
					const uint32 SpecializationHash = GetTypeHash(GfxPipeline->SpecializationConstants[Stage]);
					SetDefaultDescriptorsForMissingBuffers(DSWriter[Stage], Shader, SpecializationHash);
				}
			}

#if VULKAN_ENABLE_AGGRESSIVE_STATS
				SCOPE_CYCLE_COUNTER(STAT_VulkanVkUpdateDS);
#endif
			VulkanRHI::vkUpdateDescriptorSets(Device.GetHandle(), DSWriteContainer.DescriptorWrites.Num(), DSWriteContainer.DescriptorWrites.GetData(), 0, nullptr);

			bIsResourcesDirty = false;
		}
	}

	return true;
}

void FVulkanGraphicsPipelineDescriptorState::UpdateBindlessDescriptors(FVulkanCommandListContext& Context)
{
	check(bUseBindless);

	// We should only have uniform buffers at this point
	check(DSWriteContainer.DescriptorBufferInfo.Num() == DSWriteContainer.DescriptorWrites.Num());
	check(DSWriteContainer.DescriptorImageInfo.Num() == 0);

	checkf(!GfxPipeline->GetVulkanShader(SF_Vertex) || !GfxPipeline->GetVulkanShader(SF_Geometry) ||
		GfxPipeline->GetVulkanShader(SF_Vertex)->GetCodeHeader().NumBoundUniformBuffers +
		GfxPipeline->GetVulkanShader(SF_Geometry)->GetCodeHeader().NumBoundUniformBuffers <
		VulkanBindless::MaxUniformBuffersPerStage, TEXT("Sum of Vertex UB count and Gemetry UB count exceeds bindless limit."));

	checkf(!GfxPipeline->GetVulkanShader(SF_Mesh) || !GfxPipeline->GetVulkanShader(SF_Amplification) ||
		GfxPipeline->GetVulkanShader(SF_Mesh)->GetCodeHeader().NumBoundUniformBuffers +
		GfxPipeline->GetVulkanShader(SF_Amplification)->GetCodeHeader().NumBoundUniformBuffers <
		VulkanBindless::MaxUniformBuffersPerStage, TEXT("Sum of Mesh UB count and Task UB count exceeds bindless limit."));

	const VkDeviceSize UBOffsetAlignment = Device.GetLimits().minUniformBufferOffsetAlignment;

	auto GetGlobalUBAlloc = [&](int32 Stage)
	{
		VulkanRHI::FVulkanAllocation TempAllocation;

		FVulkanPackedUniformBufferState& State = PackedUniformBufferStates[Stage];
		check(State.Mask <= 1);
		if (State.Mask != 0)
		{
			const FPackedUniformBuffers::FPackedBuffer& StagedUniformBuffer = State.UniformBuffers.GetBuffer();
			const int32 UBSize = StagedUniformBuffer.Num();

			uint8* MappedPointer = Device.GetTempBlockAllocator().Alloc(UBSize, UBOffsetAlignment, Context, TempAllocation, nullptr);
			FMemory::Memcpy(MappedPointer, StagedUniformBuffer.GetData(), UBSize);

			State.Dirty = 0;
		}

		return TempAllocation;
	};

	if (Device.GetBindlessDescriptorManager()->UseDescriptorHeaps())
	{
		for (int32 Stage = 0; Stage < ShaderStage::NumGraphicsStages; ++Stage)
		{
			const EShaderFrequency Frequency = GetFrequencyForGfxStage((ShaderStage::EStage)Stage);
			const FVulkanShader* VulkanShader = GfxPipeline->GetVulkanShader(Frequency);
			if (!VulkanShader)
			{
				continue;
			}

			const FVulkanShaderHeader& Header = VulkanShader->GetCodeHeader();
			const TArray<uint32>& BindlessUBIndices = DSWriter[Stage].BindlessUBIndices;
			check(BindlessUBIndices.Num() == Header.NumBoundUniformBuffers);

			if (Header.NumBoundUniformBuffers == 0)
			{
				continue;
			}

			uint64 GlobalAddressOrFirstBinding = DSWriter[Stage].BindlessUBIndices[0];
			if (Header.PackedGlobalsSize > 0)
			{
				check(PackedUniformBufferStates[Stage].Mask <= 1);
				if (PackedUniformBufferStates[Stage].Mask != 0)
				{
					const FPackedUniformBuffers::FPackedBuffer& StagedUniformBuffer = PackedUniformBufferStates[Stage].UniformBuffers.GetBuffer();
					const int32 UBSize = StagedUniformBuffer.Num();

					VkDescriptorAddressInfoEXT AddressInfo;
					VulkanRHI::FVulkanAllocation TempAllocation;
					uint8* MappedPointer = Device.GetTempBlockAllocator().Alloc(UBSize, UBOffsetAlignment, Context, TempAllocation, &AddressInfo);
					FMemory::Memcpy(MappedPointer, StagedUniformBuffer.GetData(), UBSize);

					GlobalAddressOrFirstBinding = AddressInfo.address;
				}
				else
				{
					GlobalAddressOrFirstBinding = MAX_uint64;
				}
			}

			TConstArrayView<uint32> Bindings;
			if (Header.NumBoundUniformBuffers > 1)
			{
				Bindings = TConstArrayView<uint32>(&BindlessUBIndices[1], Header.NumBoundUniformBuffers - 1);
			}

			Context.GetCommandBuffer().SetBindingPushConstants(Frequency, GlobalAddressOrFirstBinding, Bindings);
		}
	}
	else
	{
		TArray<VkWriteDescriptorSet, TInlineAllocator<VulkanBindless::MaxUniformBuffersTotal>> UBCompactedWrites;
		TStaticArray<VkDescriptorBufferInfo, ShaderStage::NumGraphicsStages> ConstantBufferInfos;

		VkShaderStageFlags StageFlags = 0;

#if VULKAN_ENABLE_AGGRESSIVE_STATS
		SCOPE_CYCLE_COUNTER(STAT_VulkanApplyPackedUniformBuffers);
#endif
		for (int32 Stage = 0; Stage < ShaderStage::NumGraphicsStages; ++Stage)
		{
			const EShaderFrequency Frequency = GetFrequencyForGfxStage((ShaderStage::EStage)Stage);
			const FVulkanShader* VulkanShader = GfxPipeline->GetVulkanShader(Frequency);
			if (!VulkanShader)
			{
				continue;
			}

			StageFlags |= UEFrequencyToVKStageBit(Frequency);

			const FVulkanShaderHeader& Header = VulkanShader->GetCodeHeader();

			// Move the writes to compacted binding indices
			const int32 BindingOffset = VulkanBindless::GetOffsetForFrequency(Frequency);
			for (int32 UBIndex = 0; UBIndex < (int32)Header.NumBoundUniformBuffers; ++UBIndex)
			{
				const VkWriteDescriptorSet& WriteDescriptorSet = DSWriter[Stage].WriteDescriptors[UBIndex];
				check(WriteDescriptorSet.dstBinding == UBIndex);
				check(WriteDescriptorSet.dstArrayElement == 0);
				check(WriteDescriptorSet.descriptorCount == 1);
				check(WriteDescriptorSet.descriptorType == VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER);

				VkWriteDescriptorSet& CompactedWrite = UBCompactedWrites.Add_GetRef(WriteDescriptorSet);
				if (BindingOffset < 0)
				{
					CompactedWrite.dstBinding = VulkanBindless::MaxUniformBuffersTotal - UBIndex - 1;
				}
				else
				{
					CompactedWrite.dstBinding = BindingOffset + UBIndex;
				}

				if (UBIndex == 0)
				{
					VulkanRHI::FVulkanAllocation TempAllocation = GetGlobalUBAlloc(Stage);
					if (TempAllocation.IsValid())
					{
						VkDescriptorBufferInfo& ConstantBufferInfo = ConstantBufferInfos[Stage];

						CompactedWrite.pBufferInfo = &ConstantBufferInfo;

						ConstantBufferInfo.buffer = TempAllocation.GetBufferHandle();
						ConstantBufferInfo.offset = TempAllocation.Offset;
						ConstantBufferInfo.range = TempAllocation.Size;
					}
				}
			}
		}

		// Send to descriptor manager
		Device.GetBindlessDescriptorManager()->RegisterUniformBuffers(Context, StageFlags, UBCompactedWrites);
	}
}

TArray<uint32> FVulkanGraphicsPipelineDescriptorState::GetExcludedResources(uint32 SetIndex) const
{
	check(SetIndex < ShaderStage::NumGraphicsStages);

	TArray<uint32> ExcludedResources;
	FRHIShader* RHIShader = GfxPipeline->GetShader(GetFrequencyForGfxStage((ShaderStage::EStage)SetIndex));
	if (RHIShader && RHIShader->HasSpecializationConstants())
	{
		const uint32 SpecializationHash = GetTypeHash(GfxPipeline->SpecializationConstants[SetIndex]);

		for (auto& Pair : RHIShader->GetSpecializationExclusions())
		{
			if (Pair.Value.Contains(SpecializationHash))
			{
				ExcludedResources.Add(Pair.Key);
			}
		}
	}
	return ExcludedResources;
}



template bool FVulkanGraphicsPipelineDescriptorState::InternalUpdateDescriptorSets<true>(FVulkanCommandListContext& Context);
template bool FVulkanGraphicsPipelineDescriptorState::InternalUpdateDescriptorSets<false>(FVulkanCommandListContext& Context);
template bool FVulkanComputePipelineDescriptorState::InternalUpdateDescriptorSets<true>(FVulkanCommandListContext& Context);
template bool FVulkanComputePipelineDescriptorState::InternalUpdateDescriptorSets<false>(FVulkanCommandListContext& Context);


void FVulkanCommonPipelineDescriptorState::SetSRV(bool bCompute, uint8 DescriptorSet, uint32 BindingIndex, FVulkanShaderResourceView* SRV)
{
	check(!bUseBindless);

	switch (SRV->GetViewType())
	{
	case FVulkanView::EType::Null:
		checkf(false, TEXT("Attempt to bind a null SRV."));
		break;
		
	case FVulkanView::EType::TypedBuffer:
		MarkDirty(DSWriter[DescriptorSet].WriteUniformTexelBuffer(BindingIndex, SRV->GetTypedBufferView()));
		break;

	case FVulkanView::EType::Texture:
		{
			const ERHIAccess Access = bCompute ? ERHIAccess::SRVCompute : ERHIAccess::SRVGraphics;
			const FVulkanTexture* VulkanTexture = ResourceCast(SRV->GetTexture());
			const VkImageLayout Layout = FVulkanPipelineBarrier::GetDefaultLayout(*VulkanTexture, Access);
			MarkDirty(DSWriter[DescriptorSet].WriteImage(BindingIndex, SRV->GetTextureView(), Layout));
		}
		break;

	case FVulkanView::EType::StructuredBuffer:
		check((ResourceCast(SRV->GetBuffer())->GetBufferUsageFlags() & VK_BUFFER_USAGE_STORAGE_BUFFER_BIT) == VK_BUFFER_USAGE_STORAGE_BUFFER_BIT);
		MarkDirty(DSWriter[DescriptorSet].WriteStorageBuffer(BindingIndex, SRV->GetStructuredBufferView()));
		break;

	case FVulkanView::EType::AccelerationStructure:
		MarkDirty(DSWriter[DescriptorSet].WriteAccelerationStructure(BindingIndex, SRV->GetAccelerationStructureView().Handle));
		break;
	}
}

void FVulkanCommonPipelineDescriptorState::SetUAV(bool bCompute, uint8 DescriptorSet, uint32 BindingIndex, FVulkanUnorderedAccessView* UAV)
{
	check(!bUseBindless);

	switch (UAV->GetViewType())
	{
	case FVulkanView::EType::Null:
		checkf(false, TEXT("Attempt to bind a null UAV."));
		break;

	case FVulkanView::EType::TypedBuffer:
		MarkDirty(DSWriter[DescriptorSet].WriteStorageTexelBuffer(BindingIndex, UAV->GetTypedBufferView()));
		break;

	case FVulkanView::EType::Texture:
		{
			const ERHIAccess Access = bCompute ? ERHIAccess::UAVCompute : ERHIAccess::UAVGraphics;
			const FVulkanTexture* VulkanTexture = ResourceCast(UAV->GetTexture());
			const VkImageLayout ExpectedLayout = FVulkanPipelineBarrier::GetDefaultLayout(*VulkanTexture, Access);
			MarkDirty(DSWriter[DescriptorSet].WriteStorageImage(BindingIndex, UAV->GetTextureView(), ExpectedLayout));
		}
		break;

	case FVulkanView::EType::StructuredBuffer:
		check((ResourceCast(UAV->GetBuffer())->GetBufferUsageFlags() & VK_BUFFER_USAGE_STORAGE_BUFFER_BIT) == VK_BUFFER_USAGE_STORAGE_BUFFER_BIT);
		MarkDirty(DSWriter[DescriptorSet].WriteStorageBuffer(BindingIndex, UAV->GetStructuredBufferView()));
		break;

	case FVulkanView::EType::AccelerationStructure:
		MarkDirty(DSWriter[DescriptorSet].WriteAccelerationStructure(BindingIndex, UAV->GetAccelerationStructureView().Handle));
		break;
	}
}
