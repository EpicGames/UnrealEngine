// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanPipelineState.h: Vulkan pipeline state definitions.
=============================================================================*/

#pragma once

#include "VulkanConfiguration.h"
#include "VulkanMemory.h"
#include "VulkanCommandBuffer.h"
#include "VulkanDescriptorSets.h"
#include "VulkanLayout.h"
#include "VulkanPipeline.h"
#include "VulkanRHIPrivate.h"
#include "Containers/ArrayView.h"

class FVulkanComputePipeline;
extern TAutoConsoleVariable<int32> GDynamicGlobalUBs;


// Common Pipeline state
class FVulkanCommonPipelineDescriptorState
{
public:
	FVulkanCommonPipelineDescriptorState(FVulkanDevice& InDevice, uint32 InMaxNumSets, bool InUsesBindless)
		: Device(InDevice)
		, MaxNumSets(InMaxNumSets)
		, bUseBindless(InUsesBindless)
	{
	}

	virtual ~FVulkanCommonPipelineDescriptorState() = default;

	const FVulkanDSetsKey& GetDSetsKey() const
	{
		check(UseVulkanDescriptorCache());
		if (bIsDSetsKeyDirty)
		{
			DSetsKey.GenerateFromData(DSWriteContainer.HashableDescriptorInfo.GetData(),
				sizeof(FVulkanHashableDescriptorInfo) * DSWriteContainer.HashableDescriptorInfo.Num());
			bIsDSetsKeyDirty = false;
		}
		return DSetsKey;
	}

	bool HasVolatileResources() const
	{
		for (const FVulkanDescriptorSetWriter& Writer : DSWriter)
		{
			if (Writer.bHasVolatileResources)
			{
				return true;
			}
		}
		return false;
	}

	inline void MarkDirty(bool bDirty)
	{
		bIsResourcesDirty |= bDirty;
		bIsDSetsKeyDirty |= bDirty;
	}

	void SetSRV(bool bCompute, uint8 DescriptorSet, uint32 BindingIndex, FVulkanShaderResourceView* SRV);
	void SetUAV(bool bCompute, uint8 DescriptorSet, uint32 BindingIndex, FVulkanUnorderedAccessView* UAV);

	inline void SetTexture(uint8 DescriptorSet, uint32 BindingIndex, const FVulkanTexture* Texture, VkImageLayout Layout)
	{
		check(!bUseBindless);
		check(Texture && Texture->PartialView);

		// If the texture doesn't support sampling, then we read it through a UAV
		if (Texture->SupportsSampling())
		{
			MarkDirty(DSWriter[DescriptorSet].WriteImage(BindingIndex, Texture->PartialView->GetTextureView(), Layout));
		}
		else
		{
			MarkDirty(DSWriter[DescriptorSet].WriteStorageImage(BindingIndex, Texture->PartialView->GetTextureView(), Layout));
		}
	}

	inline void SetSamplerState(uint8 DescriptorSet, uint32 BindingIndex, const FVulkanSamplerState* Sampler)
	{
		check(!bUseBindless);
		check(Sampler && Sampler->Sampler != VK_NULL_HANDLE);
		MarkDirty(DSWriter[DescriptorSet].WriteSampler(BindingIndex, *Sampler));
	}

	inline void SetInputAttachment(uint8 DescriptorSet, uint32 BindingIndex, const FVulkanView::FTextureView& TextureView, VkImageLayout Layout)
	{
		check(!bUseBindless);
		MarkDirty(DSWriter[DescriptorSet].WriteInputAttachment(BindingIndex, TextureView, Layout));
	}

	inline void SetUniformBuffer(uint8 DescriptorSet, uint32 BindingIndex, const FVulkanUniformBuffer* UniformBuffer, bool bDynamic)
	{
		const VulkanRHI::FVulkanAllocation& Allocation = UniformBuffer->GetAllocation();
		const VkDeviceSize Range = UniformBuffer->IsUniformView() ? PLATFORM_MAX_UNIFORM_BUFFER_RANGE : UniformBuffer->GetSize();

		if (bUseBindless)
		{
			check(!bDynamic);
			MarkDirty(DSWriter[DescriptorSet].WriteBindlessUniformBuffer(Device, BindingIndex, UniformBuffer));
		}
		else if (bDynamic)
		{
			MarkDirty(DSWriter[DescriptorSet].WriteDynamicUniformBuffer(BindingIndex, Allocation.GetBufferHandle(), Allocation.HandleId, 0, Range, UniformBuffer->GetOffset()));
		}
		else
		{
			MarkDirty(DSWriter[DescriptorSet].WriteUniformBuffer(BindingIndex, Allocation.GetBufferHandle(), Allocation.HandleId, UniformBuffer->GetOffset(), Range));
		}
	}

	inline void SetUniformBufferDynamicOffset(uint8 DescriptorSet, uint32 BindingIndex, uint32 DynamicOffset)
	{
		const uint8 DynamicOffsetIndex = DSWriter[DescriptorSet].BindingToDynamicOffsetMap[BindingIndex];
		DSWriter[DescriptorSet].DynamicOffsets[DynamicOffsetIndex] = DynamicOffset;
	}

	VkDescriptorType GetDescriptorType(uint8 DescriptorSet, uint32 BindingIndex) const
	{
		const TArray<FVulkanDescriptorSetsLayout::FSetLayout>& Layouts = DescriptorSetsLayout->GetLayouts();
		return Layouts[DescriptorSet].LayoutBindings[BindingIndex].descriptorType;
	}

	virtual TArray<uint32> GetExcludedResources(uint32 SetIndex) const = 0;

protected:
	void Reset()
	{
		for(FVulkanDescriptorSetWriter& Writer : DSWriter)
		{
			Writer.Reset();
		}
	}
	inline void Bind(VkCommandBuffer CmdBuffer, VkPipelineLayout PipelineLayout, VkPipelineBindPoint BindPoint)
	{
		// Bindless will replace with global sets
		if (!bUseBindless && UsedSetsMask)
		{
			const uint32 FirstSet = FMath::CountTrailingZeros(UsedSetsMask);
			const uint32 NumSets = 32 - FMath::CountLeadingZeros(UsedSetsMask) - FirstSet;
			check(FirstSet + NumSets <= (uint32)DescriptorSetHandles.Num());

			VulkanRHI::vkCmdBindDescriptorSets(CmdBuffer,
				BindPoint,
				PipelineLayout,
				FirstSet, NumSets, &DescriptorSetHandles[FirstSet],
				(uint32)DynamicOffsets.Num(), DynamicOffsets.GetData());
		}
	}

	void CreateDescriptorWriteInfos();

	FVulkanDevice& Device;

	const uint32 MaxNumSets;

	//#todo-rco: Won't work multithreaded!
	FVulkanDescriptorSetWriteContainer DSWriteContainer;
	const FVulkanDescriptorSetsLayout* DescriptorSetsLayout = nullptr;

	//#todo-rco: Won't work multithreaded!
	TArray<VkDescriptorSet> DescriptorSetHandles;

	// Bitmask of sets that exist in this pipeline
	//#todo-rco: Won't work multithreaded!
	uint32			UsedSetsMask = 0;

	//#todo-rco: Won't work multithreaded!
	TArray<uint32> DynamicOffsets;

	bool bIsResourcesDirty = true;

	TArray<FVulkanDescriptorSetWriter> DSWriter;
	
	mutable FVulkanDSetsKey DSetsKey;
	mutable bool bIsDSetsKeyDirty = true;

	const bool bUseBindless;
};

struct FVulkanPackedUniformBufferState
{
	FPackedUniformBuffers UniformBuffers;
	uint32 Mask = 0;
	uint32 Dirty = 0;

	void Init(const FVulkanShaderHeader& CodeHeader)
	{
		UniformBuffers.Init(CodeHeader, Mask);
	}

	void Reset()
	{
		Dirty = Mask;
	}

	void SetParameter(uint32 ByteOffset, uint32 NumBytes, const void* NewValue)
	{
		UniformBuffers.SetPackedGlobalParameter(ByteOffset, NumBytes, NewValue, Dirty);
	}
};

class FVulkanComputePipelineDescriptorState : public FVulkanCommonPipelineDescriptorState
{
public:
	FVulkanComputePipelineDescriptorState(FVulkanDevice& InDevice, FVulkanComputePipeline* InComputePipeline);
	virtual ~FVulkanComputePipelineDescriptorState()
	{
		ComputePipeline->Release();
	}

	void Reset()
	{
		FVulkanCommonPipelineDescriptorState::Reset();
		PackedUniformBufferState.Reset();
	}

	FVulkanPackedUniformBufferState& GetUniformBuffers(uint8 Stage)
	{
		return PackedUniformBufferState;
	}

	void SetParameter(uint32 ByteOffset, uint32 NumBytes, const void* NewValue)
	{
		PackedUniformBufferState.SetParameter(ByteOffset, NumBytes, NewValue);
	}

	bool UpdateDescriptorSets(FVulkanCommandListContext& Context)
	{
		check(!bUseBindless);
		if (GDynamicGlobalUBs->GetInt() > 0)
		{
			return InternalUpdateDescriptorSets<true>(Context);
		}

		return InternalUpdateDescriptorSets<false>(Context);
	}

	void UpdateBindlessDescriptors(FVulkanCommandListContext& Context);

	void BindDescriptorSets(VkCommandBuffer CmdBuffer)
	{
		Bind(CmdBuffer, ComputePipeline->GetLayout().GetPipelineLayout(), VK_PIPELINE_BIND_POINT_COMPUTE);
	}

	TArray<uint32> GetExcludedResources(uint32 SetIndex) const override final;

protected:
	FVulkanPackedUniformBufferState PackedUniformBufferState;

	FVulkanComputePipeline* ComputePipeline;

	template<bool bUseDynamicGlobalUBs>
	bool InternalUpdateDescriptorSets(FVulkanCommandListContext& Context);

	friend class FVulkanPendingComputeState;
	friend class FVulkanCommandListContext;
};

class FVulkanGraphicsPipelineDescriptorState : public FVulkanCommonPipelineDescriptorState
{
public:
	FVulkanGraphicsPipelineDescriptorState(FVulkanDevice& InDevice, FVulkanGraphicsPipelineState* InGfxPipeline);
	virtual ~FVulkanGraphicsPipelineDescriptorState()
	{
		GfxPipeline->Release();
	}

	FVulkanPackedUniformBufferState& GetUniformBuffers(uint8 Stage)
	{
		return PackedUniformBufferStates[Stage];
	}

	void SetParameter(uint8 Stage, uint32 ByteOffset, uint32 NumBytes, const void* NewValue)
	{
		PackedUniformBufferStates[Stage].SetParameter(ByteOffset, NumBytes, NewValue);
	}

	bool UpdateDescriptorSets(FVulkanCommandListContext& Context)
	{
		check(!bUseBindless);

		const bool bUseDynamicGlobalUBs = (GDynamicGlobalUBs->GetInt() > 0);
		if (bUseDynamicGlobalUBs)
		{
			return InternalUpdateDescriptorSets<true>(Context);
		}
		else
		{
			return InternalUpdateDescriptorSets<false>(Context);
		}
	}

	void UpdateBindlessDescriptors(FVulkanCommandListContext& Context);

	inline void BindDescriptorSets(VkCommandBuffer CmdBuffer)
	{
		Bind(CmdBuffer, GfxPipeline->GetLayout().GetPipelineLayout(), VK_PIPELINE_BIND_POINT_GRAPHICS);
	}

	void Reset()
	{
		for (FVulkanPackedUniformBufferState& State : PackedUniformBufferStates)
		{
			State.Reset();
		}

		FVulkanCommonPipelineDescriptorState::Reset();
		bIsResourcesDirty = true;
	}

	TArray<uint32> GetExcludedResources(uint32 SetIndex) const override final;

protected:
	TStaticArray<FVulkanPackedUniformBufferState, ShaderStage::NumGraphicsStages> PackedUniformBufferStates;

	FVulkanGraphicsPipelineState* GfxPipeline;

	template<bool bUseDynamicGlobalUBs>
	bool InternalUpdateDescriptorSets(FVulkanCommandListContext& Context);

	friend class FVulkanPendingGfxState;
	friend class FVulkanCommandListContext;
};

template <bool bIsDynamic>
static inline bool SubmitPackedUniformBuffers(FVulkanDescriptorSetWriter& DescriptorWriteSet, const VulkanRHI::FVulkanAllocation& TempAllocation)
{
	const int32 BindingIndex = 0;  // Packed uniform buffers are only used for globals at binding 0
	if (bIsDynamic)
	{
		return DescriptorWriteSet.WriteDynamicUniformBuffer(BindingIndex, TempAllocation.GetBufferHandle(), TempAllocation.HandleId, 0, TempAllocation.Size, TempAllocation.Offset);
	}
	else
	{
		return DescriptorWriteSet.WriteUniformBuffer(BindingIndex, TempAllocation.GetBufferHandle(), TempAllocation.HandleId, TempAllocation.Offset, TempAllocation.Size);
	}
}
