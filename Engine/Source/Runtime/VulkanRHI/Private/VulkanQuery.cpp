// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanQuery.cpp: Vulkan query RHI implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanDevice.h"
#include "VulkanResources.h"
#include "VulkanContext.h"
#include "VulkanCommandBuffer.h"
#include "VulkanQuery.h"
#include "EngineGlobals.h"
#include "RenderCore.h"

#if VULKAN_QUERY_CALLSTACK
#include "HAL/PlatformStackwalk.h"
#endif

static int32 GTimestampQueryStage = 0;
FAutoConsoleVariableRef CVarTimestampQueryStage(
	TEXT("r.Vulkan.TimestampQueryStage"),
	GTimestampQueryStage,
	TEXT("Defines which pipeline stage is used for timestamp queries.\n")
	TEXT(" 0: Use VK_PIPELINE_STAGE_TOP_OF_PIPE_BIT, less precise measures but less likely to alter performance (default)\n")
	TEXT(" 1: Use VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT, more precise measures but may alter performance on some platforms"),
	ECVF_RenderThreadSafe
);

static int32 GTimingQueryPoolSize = 128;
FAutoConsoleVariableRef CVarTimingQueryPoolSize(
	TEXT("r.Vulkan.TimingQueryPoolSize"),
	GTimingQueryPoolSize,
	TEXT("Amount of timing queries per pool in reusable query pools. (Default: 128)\n"),
	ECVF_ReadOnly
);

static int32 GQueryPoolDeletionDelay = 10;
FAutoConsoleVariableRef CVarQueryPoolDeletionDelay(
	TEXT("r.Vulkan.QueryPoolDeletionDelay"),
	GQueryPoolDeletionDelay,
	TEXT("Amount of frames to wait before deleting an unused query pools. (Default: 10)\n"),
	ECVF_ReadOnly
);

static int32 GQueryUseControlPrecisionBit = 1;
FAutoConsoleVariableRef CVarQueryUseControlPrecisionBit(
	TEXT("r.Vulkan.QueryUseControlPrecisionBit"),
	GQueryUseControlPrecisionBit,
	TEXT("Whether to use VK_QUERY_CONTROL_PRECISE_BIT when doing a render query. (Default : 1 (on), 0 (off))\n"),
	ECVF_ReadOnly
);


FVulkanQueryPool::FVulkanQueryPool(FVulkanDevice& InDevice, uint32 InMaxQueries, EVulkanQueryPoolType InQueryType)
	: Device(InDevice)
	, QueryPool(VK_NULL_HANDLE)
	, MaxQueries(InMaxQueries)
	, QueryType(InQueryType)
{
	INC_DWORD_STAT(STAT_VulkanNumQueryPools);
	VkQueryPoolCreateInfo PoolCreateInfo = {
		.sType = VK_STRUCTURE_TYPE_QUERY_POOL_CREATE_INFO,
		.pNext = nullptr,
		.flags = 0,
		.queryType = GetVkQueryType(InQueryType),
		.queryCount = MaxQueries,
		.pipelineStatistics = 0
	};

	// With maintenance9, always set an initial state for the queries
	if (Device.GetOptionalExtensions().HasKHRMaintenance9)
	{
		QueryRefs.AddDefaulted(MaxQueries / QueryStride);
		QueryResults.SetNumZeroed(MaxQueries / QueryStride);
		PoolCreateInfo.flags |= VK_QUERY_POOL_CREATE_RESET_BIT_KHR;
		bIsDirty = false;
	}

	VERIFYVULKANRESULT(VulkanRHI::vkCreateQueryPool(Device.GetHandle(), &PoolCreateInfo, VULKAN_CPU_ALLOCATOR, &QueryPool));
}

FVulkanQueryPool::~FVulkanQueryPool()
{
	DEC_DWORD_STAT(STAT_VulkanNumQueryPools);
	VulkanRHI::vkDestroyQueryPool(Device.GetHandle(), QueryPool, VULKAN_CPU_ALLOCATOR);
	QueryPool = VK_NULL_HANDLE;
}

void FVulkanQueryPool::ReserveQuery(FVulkanRenderQuery* Query)
{
	checkSlow(QueryType == Query->GetQueryPoolType());
	const uint32 QuerySlot = CurrentQueryCount++;
	Query->IndexInPool = QuerySlot * QueryStride;
	QueryRefs[QuerySlot] = Query;
	QueryResults[QuerySlot] = &Query->Result;
	bIsDirty = true;
}

uint32 FVulkanQueryPool::ReserveQuery(uint64* ResultPtr)
{
	checkSlow(QueryType == EVulkanQueryPoolType::Timestamp);
	const uint32 QuerySlot = CurrentQueryCount++;
	const uint32 IndexInPool = QuerySlot * QueryStride;
	QueryRefs[QuerySlot] = nullptr;
	QueryResults[QuerySlot] = ResultPtr;
	bIsDirty = true;
	return IndexInPool;
}

void FVulkanQueryPool::Reset(FVulkanCommandBuffer* InCmdBuffer, uint32 InQueryStride)
{
	CurrentQueryCount = 0;
	QueryStride = InQueryStride;
	UnusedFrameCount = 0;

	QueryRefs.Empty(MaxQueries);
	QueryResults.Empty(MaxQueries);
	QueryRefs.AddDefaulted(MaxQueries / QueryStride);
	QueryResults.SetNumZeroed(MaxQueries / QueryStride);

	if (InCmdBuffer)
	{
		VulkanRHI::vkCmdResetQueryPool(InCmdBuffer->GetHandle(), QueryPool, 0, MaxQueries);
	}
	else
	{
		checkf(Device.GetOptionalExtensions().HasEXTHostQueryReset, TEXT("VK_EXT_host_query_reset must be supported to reset queries without a command buffer."));
		VulkanRHI::vkResetQueryPoolEXT(Device.GetHandle(), QueryPool, 0, MaxQueries);
	}

	bIsDirty = false;
}

bool FVulkanQueryPool::IsStale() const
{
	return UnusedFrameCount >= GQueryPoolDeletionDelay;
}

void FVulkanCommandListContext::BeginOcclusionQueryBatch(uint32 NumQueriesInBatch, uint32 MultiViewCount)
{
	const uint32 QueryStride = MultiViewCount == 0 ? 1 : MultiViewCount;
	TArray<FVulkanQueryPool*>& OcclusionPoolArray = GetQueryPoolArray(EVulkanQueryPoolType::Occlusion);
	FVulkanQueryPool* NewOcclusionQueryPool = Device.AcquireOcclusionQueryPool(NumQueriesInBatch * QueryStride);
	if (NewOcclusionQueryPool->IsDirty() || (NewOcclusionQueryPool->GetQueryStride() != QueryStride))
	{
		FVulkanCommandBuffer* CommandBuffer = Device.GetOptionalExtensions().HasEXTHostQueryReset ? nullptr : &GetCommandBuffer();
		NewOcclusionQueryPool->Reset(CommandBuffer, QueryStride);
	}
	OcclusionPoolArray.Add(NewOcclusionQueryPool);
}

FVulkanQueryPool* FVulkanDevice::AcquireOcclusionQueryPool(uint32 NumQueries)
{
	FScopeLock Lock(&QueryPoolLock);

	// At least add one query
	NumQueries = FMath::Max(1u, AlignArbitrary(NumQueries, 256));
	OcclusionQueryPoolSize = FMath::Max(OcclusionQueryPoolSize, NumQueries);

	TArray<FVulkanQueryPool*>& FreeOcclusionPools = FreeQueryPools[(int32)EVulkanQueryPoolType::Occlusion];

	// Destroy pools that can't accomodate our new minimum size
	for (int32 Index = FreeOcclusionPools.Num() - 1; Index >= 0; --Index)
	{
		FVulkanQueryPool* Pool = FreeOcclusionPools[Index];
		checkSlow(Pool && (Pool->GetPoolType() == EVulkanQueryPoolType::Occlusion));
		if (Pool->GetMaxQueries() < OcclusionQueryPoolSize)
		{
			delete Pool;
			FreeOcclusionPools.RemoveAtSwap(Index, EAllowShrinking::No);
		}
	}

	if (FreeOcclusionPools.Num())
	{
		FVulkanQueryPool* Pool = FreeOcclusionPools.Pop(EAllowShrinking::No);
		checkSlow(Pool && (Pool->GetPoolType() == EVulkanQueryPoolType::Occlusion));
		return Pool;
	}

	FVulkanQueryPool* Pool = new FVulkanQueryPool(*this, OcclusionQueryPoolSize, EVulkanQueryPoolType::Occlusion);
	return Pool;
}

FVulkanQueryPool* FVulkanDevice::AcquireTimingQueryPool()
{
	FScopeLock Lock(&QueryPoolLock);

	TArray<FVulkanQueryPool*>& FreeTimingPools = FreeQueryPools[(int32)EVulkanQueryPoolType::Timestamp];
	if (FreeTimingPools.Num())
	{
		FVulkanQueryPool* Pool = FreeTimingPools.Pop(EAllowShrinking::No);
		checkSlow(Pool && (Pool->GetPoolType() == EVulkanQueryPoolType::Timestamp));
		return Pool;
	}
	return new FVulkanQueryPool(*this, GTimingQueryPoolSize, EVulkanQueryPoolType::Timestamp);
}

void FVulkanDevice::ReleaseQueryPool(FVulkanQueryPool* Pool)
{
	FScopeLock Lock(&QueryPoolLock);
	Pool->SetDirty();
	FreeQueryPools[(int32)Pool->GetPoolType()].Add(Pool);
}

void FVulkanDevice::RemoveStaleQueryPools()
{
	FScopeLock Lock(&QueryPoolLock);
	for (TArray<FVulkanQueryPool*>& PoolArray : FreeQueryPools)
	{
		for (int32 Index = PoolArray.Num() - 1; Index >= 0; --Index)
		{
			FVulkanQueryPool* Pool = PoolArray[Index];
			checkSlow(Pool);
			if (Pool->IsStale())
			{
				delete Pool;
				PoolArray.RemoveAtSwap(Index, EAllowShrinking::No);
			}
			else
			{
				Pool->IncrementUnusedFrameCount();
			}
		}
	}
}

FVulkanRenderQuery::FVulkanRenderQuery(ERenderQueryType InType)
	: QueryType(InType)
{
	INC_DWORD_STAT(STAT_VulkanNumQueries);
}

FVulkanRenderQuery::~FVulkanRenderQuery()
{
	check(!SyncPoint.IsValid() || SyncPoint->IsComplete());
	DEC_DWORD_STAT(STAT_VulkanNumQueries);
}

FRenderQueryRHIRef FVulkanDynamicRHI::RHICreateRenderQuery(ERenderQueryType QueryType)
{
	ensureMsgf((QueryType == RQT_Occlusion) || (QueryType == RQT_AbsoluteTime), TEXT("Unknown QueryType %d"), QueryType);
	return new FVulkanRenderQuery(QueryType);
}

bool FVulkanDynamicRHI::RHIGetRenderQueryResult(FRHIRenderQuery* QueryRHI, uint64& OutQueryResult, bool bWait, uint32 GPUIndex)
{
	FVulkanRenderQuery* Query = ResourceCast(QueryRHI);

	if (!ensureMsgf(Query->SyncPoint, TEXT("Attempt to get result data for an FRHIRenderQuery that was never used in a command list.")))
	{
		OutQueryResult = 0;
		return false;
	}

	if (!Query->SyncPoint->IsComplete())
	{
		if (bWait)
		{
			FRenderThreadIdleScope IdleScope(ERenderThreadIdleTypes::WaitingForGPUQuery);
			ProcessInterruptQueueUntil(Query->SyncPoint);
		}
		else
		{
			return false;
		}
	}

	checkSlow(Query->SyncPoint->IsComplete());

	if (Query->QueryType == RQT_Occlusion)
	{
		OutQueryResult = Query->Result;
		return true;
	}
	else if (Query->QueryType == RQT_AbsoluteTime)
	{
		const VkPhysicalDeviceLimits& Limits = Device->GetDeviceProperties().limits;
		const double TimingFrequency = (double)((uint64)((1000.0 * 1000.0 * 1000.0) / Limits.timestampPeriod));
		OutQueryResult = (uint64)((double(Query->Result) / TimingFrequency) * 1000.0 * 1000.0);
		return true;
	}

	return false;
}

void FVulkanDynamicRHI::RHIEndRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery)
{
	const uint32 GPUIndex = 0;

	FVulkanRenderQuery* Query = ResourceCast(RenderQuery);
	auto& QueryBatchData = RHICmdList.GetQueryBatchData(Query->QueryType);

	if (QueryBatchData[GPUIndex])
	{
		// This query belongs to a batch. Use the sync point we created earlier
		Query->SyncPoint = static_cast<FVulkanSyncPoint*>(QueryBatchData[GPUIndex]);
	}
	else
	{
		// Queries issued outside of a batch use one sync point per query.
//		check(Query->SyncPoint == nullptr);
		Query->SyncPoint = FVulkanSyncPoint::Create(EVulkanSyncPointType::GPUAndCPU, TEXT("UnbatchedRenderQuerySyncPoint"));

		RHICmdList.EnqueueLambda([SyncPoint = Query->SyncPoint](FRHICommandListBase& ExecutingCmdList)
		{
			FVulkanCommandListContext& Context = FVulkanCommandListContext::Get(ExecutingCmdList);
			Context.AddPendingSyncPoint(SyncPoint);
		});
	}

	// Enqueue the RHI command to record the EndQuery() call on the context.
	FDynamicRHI::RHIEndRenderQuery_TopOfPipe(RHICmdList, RenderQuery);
}

void FVulkanDynamicRHI::RHIBeginRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType)
{
	// Each query batch uses a single sync point to signal when the results are ready.
	const uint32 GPUIndex = 0;

	auto& QueryBatchData = RHICmdList.GetQueryBatchData(QueryType);
	checkf(QueryBatchData[GPUIndex] == nullptr, TEXT("A query batch for this type has already begun on this command list."));

	FVulkanSyncPointRef SyncPoint = FVulkanSyncPoint::Create(EVulkanSyncPointType::GPUAndCPU, TEXT("BatchedRenderQuerySyncPoint"));

	// Keep a reference in the RHI command list, so we can retrieve it later in BeginQuery/EndQuery/EndBatch.
	QueryBatchData[GPUIndex] = SyncPoint.GetReference();
	SyncPoint->AddRef();
}

void FVulkanDynamicRHI::RHIEndRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType)
{
	const uint32 GPUIndex = 0;

	auto& QueryBatchData = RHICmdList.GetQueryBatchData(QueryType);
	checkf(QueryBatchData[GPUIndex], TEXT("A query batch for this type is not open on this command list."));

	FVulkanSyncPointRef SyncPoint = static_cast<FVulkanSyncPoint*>(QueryBatchData[GPUIndex]);

	// Clear the sync point reference on the RHI command list
	SyncPoint->Release();
	QueryBatchData[GPUIndex] = nullptr;

	RHICmdList.EnqueueLambda([SyncPoint = MoveTemp(SyncPoint)](FRHICommandListBase& ExecutingCmdList)
	{
		FVulkanCommandListContext& Context = FVulkanCommandListContext::Get(ExecutingCmdList);
		Context.AddPendingSyncPoint(SyncPoint);
	});
}

void FVulkanCommandListContext::RHIBeginRenderQuery(FRHIRenderQuery* QueryRHI)
{
	FVulkanRenderQuery* Query = ResourceCast(QueryRHI);
	if (Query->QueryType == RQT_Occlusion)
	{
		FVulkanQueryPool* CurrentOcclusionQueryPool = GetCurrentOcclusionQueryPool();
		CurrentOcclusionQueryPool->ReserveQuery(Query);

		VkQueryControlFlags Flags = 0;
		if (GQueryUseControlPrecisionBit > 0)
		{
			Flags |= VK_QUERY_CONTROL_PRECISE_BIT;
		}
		VulkanRHI::vkCmdBeginQuery(GetCommandBuffer().GetHandle(), CurrentOcclusionQueryPool->GetHandle(), Query->IndexInPool, Flags);
	}
	else if (Query->QueryType == RQT_AbsoluteTime)
	{
		ensureMsgf(0, TEXT("Timing queries should NOT call RHIBeginRenderQuery()!"));
	}
}

void FVulkanCommandListContext::RHIEndRenderQuery(FRHIRenderQuery* QueryRHI)
{
	FVulkanRenderQuery* Query = ResourceCast(QueryRHI);
	check(Query->SyncPoint.IsValid());
	if (Query->QueryType == RQT_Occlusion)
	{
		FVulkanQueryPool* CurrentPool = GetCurrentOcclusionQueryPool();
		VulkanRHI::vkCmdEndQuery(GetCommandBuffer().GetHandle(), CurrentPool->GetHandle(), Query->IndexInPool);
	}
	else if (Query->QueryType == RQT_AbsoluteTime)
	{
		FVulkanQueryPool* CurrentPool = GetCurrentTimestampQueryPool();
		CurrentPool->ReserveQuery(Query);

		const VkPipelineStageFlagBits QueryPipelineStage = GTimestampQueryStage ? VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT : VK_PIPELINE_STAGE_TOP_OF_PIPE_BIT;
		VulkanRHI::vkCmdWriteTimestamp(GetCommandBuffer().GetHandle(), QueryPipelineStage, CurrentPool->GetHandle(), Query->IndexInPool);
	}
}

FVulkanQueryPool* FVulkanCommandListContext::GetCurrentOcclusionQueryPool()
{
	TArray<FVulkanQueryPool*>& OcclusionPoolArray = GetQueryPoolArray(EVulkanQueryPoolType::Occlusion);
	checkSlow(OcclusionPoolArray.Num() && !OcclusionPoolArray.Last()->IsFull());
	return OcclusionPoolArray.Last();
}

FVulkanQueryPool* FVulkanContextCommon::GetCurrentTimestampQueryPool(FVulkanPayload& Payload)
{
	TArray<FVulkanQueryPool*>& TimestampPoolArray = Payload.QueryPools[(int32)EVulkanQueryPoolType::Timestamp];
	uint32 DesiredQueryStride = 1u;
	if (Payload.CommandBuffers.Num() > 0 && Payload.CommandBuffers.Last()->IsInsideRenderPass())
	{
		DesiredQueryStride = Payload.CommandBuffers.Last()->CurrentMultiViewCount;
		DesiredQueryStride = DesiredQueryStride == 0 ? 1 : DesiredQueryStride;
	}
	// Note: In theory, it's possible to have both multiview and non-multiview timing queries in one pool.
	// However, this would add significant complexity to fetching results.
	// For now, to keep things simple, switch to a new query pool when the stride changes.
	if ((TimestampPoolArray.Num() == 0) || TimestampPoolArray.Last()->IsFull() || TimestampPoolArray.Last()->GetQueryStride() != DesiredQueryStride)
	{
		FVulkanQueryPool* NewPool = Device.AcquireTimingQueryPool();
		if (NewPool->IsDirty() || (NewPool->GetQueryStride() != DesiredQueryStride))
		{
			FVulkanCommandBuffer* CommandBuffer = nullptr;
			if (!Device.GetOptionalExtensions().HasEXTHostQueryReset)
			{
				if (Payload.CommandBuffers.Num() == 0)
				{
					PrepareNewCommandBuffer(Payload);
				}
				CommandBuffer = Payload.CommandBuffers.Last();
			}
			NewPool->Reset(CommandBuffer, DesiredQueryStride);
		}

		TimestampPoolArray.Add(NewPool);
	}
	return TimestampPoolArray.Last();
}
