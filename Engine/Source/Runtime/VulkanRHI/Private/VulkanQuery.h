// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanQuery.h: Private Vulkan RHI definitions.
=============================================================================*/

#pragma once

#include "VulkanPlatform.h"

class FVulkanDevice;
class FVulkanQueue;
class FVulkanContextCommon;
class FVulkanCommandBuffer;
class FVulkanRenderQuery;
class FVulkanDynamicRHI;

class FVulkanSyncPoint;
using FVulkanSyncPointRef = TRefCountPtr<FVulkanSyncPoint>;


enum class EVulkanQueryPoolType : uint8
{
	Occlusion,
	PipelineStats,
	Timestamp,
	ASCompactedSize,

	Count
};

constexpr VkQueryType GetVkQueryType(EVulkanQueryPoolType Type)
{
	switch (Type)
	{
	case EVulkanQueryPoolType::Occlusion:       return VK_QUERY_TYPE_OCCLUSION;
	case EVulkanQueryPoolType::PipelineStats:   return VK_QUERY_TYPE_PIPELINE_STATISTICS;
	case EVulkanQueryPoolType::Timestamp:       return VK_QUERY_TYPE_TIMESTAMP;
	case EVulkanQueryPoolType::ASCompactedSize: return VK_QUERY_TYPE_ACCELERATION_STRUCTURE_COMPACTED_SIZE_KHR;

	case EVulkanQueryPoolType::Count:
	default:
		checkNoEntry();
		return VK_QUERY_TYPE_MAX_ENUM;
	}
}

class FVulkanQueryPool
{
public:
	FVulkanQueryPool(FVulkanDevice& InDevice, uint32 InMaxQueries, EVulkanQueryPoolType InQueryType);
	virtual ~FVulkanQueryPool();

	uint32 GetMaxQueries() const
	{
		return MaxQueries;
	}

	VkQueryPool GetHandle() const
	{
		return QueryPool;
	}

	EVulkanQueryPoolType GetPoolType() const
	{
		return QueryType;
	}

	FVulkanDevice& GetDevice()
	{
		return Device;
	}

	uint32 GetQueryStride() const
	{
		return QueryStride;
	}

	bool IsFull() const
	{
		return (CurrentQueryCount * QueryStride >= MaxQueries);
	}

	void ReserveQuery(FVulkanRenderQuery* Query);
	uint32 ReserveQuery(uint64* ResultPtr);

	void Reset(FVulkanCommandBuffer* InCmdBuffer, uint32 InQueryStride = 1u);
	bool IsStale() const;

	void IncrementUnusedFrameCount()
	{
		UnusedFrameCount++;
	}

	void SetDirty()
	{
		bIsDirty = true;
	}

	bool IsDirty() const
	{
		return bIsDirty;
	}

protected:
	FVulkanDevice& Device;
	VkQueryPool QueryPool;
	const uint32 MaxQueries;
	const EVulkanQueryPoolType QueryType;
	TArray<TRefCountPtr<FVulkanRenderQuery>> QueryRefs;
	TArray<uint64*> QueryResults;
	uint32 CurrentQueryCount = 0;
	/** When queries are issued during a multiview render pass, vkCmdBeginQuery and vkCmdEndQuery
	 *  each correspond to multiple subsequent queries in the pool, one for each view.
	 *  The interpretation of these per-view values is implementation dependent, but they
	 *  can be summed together to get the actual value. */
	uint32 QueryStride = 1u;
	int32 UnusedFrameCount = 0;
	bool bIsDirty = true;

	friend FVulkanDynamicRHI;
};

class FVulkanRenderQuery : public FRHIRenderQuery
{
public:
	FVulkanRenderQuery(ERenderQueryType InType);
	virtual ~FVulkanRenderQuery();

	EVulkanQueryPoolType GetQueryPoolType()
	{
		if (QueryType == RQT_Occlusion)
		{
			return EVulkanQueryPoolType::Occlusion;
		}
		else 
		{
			check(QueryType == RQT_AbsoluteTime);
			return EVulkanQueryPoolType::Timestamp;
		}
	}

	const ERenderQueryType QueryType;
	uint64 Result = 0;
	uint32 IndexInPool = UINT32_MAX;
	FVulkanSyncPointRef SyncPoint;
};
