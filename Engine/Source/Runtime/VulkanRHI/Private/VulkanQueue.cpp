// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanQueue.cpp: Vulkan Queue implementation.
=============================================================================*/

#include "VulkanQueue.h"
#include "VulkanDevice.h"
#include "VulkanMemory.h"
#include "VulkanContext.h"
#include "VulkanCommandBuffer.h"

int32 GWaitForIdleOnSubmit = 0;
FAutoConsoleVariableRef CVarVulkanWaitForIdleOnSubmit(
	TEXT("r.Vulkan.WaitForIdleOnSubmit"),
	GWaitForIdleOnSubmit,
	TEXT("Waits for the GPU to be idle after submitting a command buffer. Useful for tracking GPU hangs.\n")
	TEXT(" 0: Do not wait (default)\n")
	TEXT(" 1: Wait on every submit\n"),
	ECVF_Default
	);

int32 GAllowTimelineSemaphores = 1;
FAutoConsoleVariableRef CVarVulkanSubmissionAllowTimelineSemaphores(
	TEXT("r.Vulkan.Submission.AllowTimelineSemaphores"),
	GAllowTimelineSemaphores,
	TEXT("If supported, use timeline semaphores for queue submission to reduce API calls.\n")
	TEXT(" 0: Use normal fences\n")
	TEXT(" 1: Use timeline semaphores if available (default)\n"),
	ECVF_ReadOnly
);

// :todo-jn: Merge payloads
//static int32 GVulkanMergePayloads = 1;
//static FAutoConsoleVariableRef CVarVulkanSubmissionMergePayloads(
//	TEXT("r.Vulkan.Submission.MergePayloads"),
//	GVulkanMergePayloads,
//	TEXT("0: Submit payloads individually\n")
//	TEXT("1: Merge consecutive payloads without syncs going to same queue (default)\n"),
//	ECVF_ReadOnly
//);


static TAutoConsoleVariable<int32> CVarVulkanExtendedLifetimeFrames(
	TEXT("r.Vulkan.Aftermath.ExtendedLifetimeFrames"),
	2,
	TEXT("Number of frames to keep breadcrumbs alive for Aftermath checkpoints."),
	ECVF_ReadOnly
);

int32 GVulkanDiagnosticBuffer = 1;
static FAutoConsoleVariableRef CVarVulkanDiagnosticBuffer(
	TEXT("r.Vulkan.DiagnosticBuffer"),
	GVulkanDiagnosticBuffer,
	TEXT("0: Disable the diagnostic buffer\n")
	TEXT("1: Enable the diagnostic buffer with less perf impact (default)")
	TEXT("2: Enable the diagnostic buffer with more precision"),
	ECVF_ReadOnly
);


FVulkanQueue::FVulkanQueue(FVulkanDevice& InDevice, uint32 InFamilyIndex, EVulkanQueueType InQueueType)
	: Queue(VK_NULL_HANDLE)
	, FamilyIndex(InFamilyIndex)
	, QueueIndex(0)
	, QueueType(InQueueType)
	, Device(InDevice)
	, bUseTimelineSemaphores(InDevice.GetOptionalExtensions().HasKHRTimelineSemaphore&& GAllowTimelineSemaphores)
{
	VulkanRHI::vkGetDeviceQueue(Device.GetHandle(), FamilyIndex, QueueIndex, &Queue);

	FillSupportedStageBits();

	if (bUseTimelineSemaphores)
	{
		// Use ImmediateDeletion for the timeline since it's deleted after the deferrered deletion queue
		const EVulkanSemaphoreFlags SemaphoreFlags = EVulkanSemaphoreFlags::Timeline | EVulkanSemaphoreFlags::ImmediateDeletion;
		TimelineSempahore = new FVulkanSemaphore(InDevice, SemaphoreFlags, CompletedTimelineSemaphoreValue);
		TimelineSempahore->AddRef();
	}
}

FVulkanQueue::~FVulkanQueue()
{
	if (TimelineSempahore)
	{
		TimelineSempahore->Release();
	}

	for (TArray<FVulkanCommandBufferPool*>& CommandBufferPoolArray : CommandBufferPools)
	{
		for (FVulkanCommandBufferPool* CommandBufferPool : CommandBufferPoolArray)
		{
			delete CommandBufferPool;
		}
		CommandBufferPoolArray.Reset();
	}
}


void FVulkanQueue::BindSparseResources(FVulkanPayload& Payload)
{
	TArray<TArray<VkSparseMemoryBind>, TInlineAllocator<1>> SparseMemoryBinds;
	TArray<VkSparseBufferMemoryBindInfo, TInlineAllocator<1>> BufferMemoryBindInfo;
	TArray<VkSparseImageOpaqueMemoryBindInfo, TInlineAllocator<1>> ImageMemoryBindInfo;
	for (const FVulkanCommitReservedResourceDesc& CommitDesc : Payload.ReservedResourcesToCommit)
	{
		if (CommitDesc.Resource->GetType() == RRT_Buffer)
		{
			FVulkanBuffer* Buffer = ResourceCast(static_cast<FRHIBuffer*>(CommitDesc.Resource.GetReference()));
			TArray<VkSparseMemoryBind> BufferSparseMemoryBinds = Buffer->CommitReservedResource(CommitDesc.CommitSizeInBytes);
			if (BufferSparseMemoryBinds.Num())
			{
				VkSparseBufferMemoryBindInfo BindInfo;
				BindInfo.buffer = Buffer->GetHandle();
				BindInfo.bindCount = BufferSparseMemoryBinds.Num();
				BindInfo.pBinds = BufferSparseMemoryBinds.GetData();
				BufferMemoryBindInfo.Emplace(BindInfo);

				SparseMemoryBinds.Emplace(MoveTemp(BufferSparseMemoryBinds));
			}
		}
		else if (CommitDesc.Resource->GetType() == RRT_Texture)
		{
			FVulkanTexture* Texture = ResourceCast(static_cast<FRHITexture*>(CommitDesc.Resource.GetReference()));
			TArray<VkSparseMemoryBind> ImageSparseMemoryBinds = Texture->CommitReservedResource(CommitDesc.CommitSizeInBytes);
			if (ImageSparseMemoryBinds.Num())
			{
				VkSparseImageOpaqueMemoryBindInfo BindInfo;
				BindInfo.image = Texture->Image;
				BindInfo.bindCount = ImageSparseMemoryBinds.Num();
				BindInfo.pBinds = ImageSparseMemoryBinds.GetData();
				ImageMemoryBindInfo.Emplace(BindInfo);

				SparseMemoryBinds.Emplace(MoveTemp(ImageSparseMemoryBinds));
			}
		}
		else
		{
			checkNoEntry();
		}
	}

	if (SparseMemoryBinds.Num())
	{
		TArray<VkSemaphore> SemaphoreHandles;
		SemaphoreHandles.Reserve(Payload.WaitSemaphores.Num());
		for (FVulkanSemaphore* Semaphore : Payload.WaitSemaphores)
		{
			SemaphoreHandles.Add(Semaphore->GetHandle());
		}
		Payload.WaitSemaphores.Reset();

		VkBindSparseInfo BindSparseInfo;
		ZeroVulkanStruct(BindSparseInfo, VK_STRUCTURE_TYPE_BIND_SPARSE_INFO);
		BindSparseInfo.waitSemaphoreCount = SemaphoreHandles.Num();
		BindSparseInfo.pWaitSemaphores = SemaphoreHandles.GetData();
		BindSparseInfo.bufferBindCount = BufferMemoryBindInfo.Num();
		BindSparseInfo.pBufferBinds = BufferMemoryBindInfo.GetData();
		BindSparseInfo.imageOpaqueBindCount = ImageMemoryBindInfo.Num();
		BindSparseInfo.pImageOpaqueBinds = ImageMemoryBindInfo.GetData();
		VERIFYVULKANRESULT(VulkanRHI::vkQueueBindSparse(Queue, 1, &BindSparseInfo, VK_NULL_HANDLE));

		if (GWaitForIdleOnSubmit != 0)
		{
			VERIFYVULKANRESULT(VulkanRHI::vkDeviceWaitIdle(Device.GetHandle()));
		}
	}
}


int32 FVulkanQueue::SubmitQueuedPayloads(TMap<VkSemaphore, FBinarySemaphoreSignalInfo>& SignaledSemas)
{
	auto CanBeProcessed = [&SignaledSemas, &Device=Device](FVulkanPayload* Payload)
	{
		const TArray<TRefCountPtr<FVulkanSemaphore>>& WaitSemas = Payload->WaitSemaphores;
		for (const TRefCountPtr<FVulkanSemaphore>& WaitSema : WaitSemas)
		{
			const VkSemaphore SemaphoreHandle = WaitSema->GetHandle();
			if (!WaitSema->IsExternallySignaled() && !SignaledSemas.Contains(SemaphoreHandle))
			{
				return false;
			}
		}

		// Find the maximum timeline value to wait on for each queue
		uint64 MaxTimelineWaitValue[(int32)EVulkanQueueType::Count] = {};
		for (FVulkanSemaphore* WaitSema : WaitSemas)
		{
			if (!WaitSema->IsExternallySignaled())
			{
				const VkSemaphore SemaphoreHandle = WaitSema->GetHandle();
				FBinarySemaphoreSignalInfo& SignalInfo = SignaledSemas[SemaphoreHandle];

				const int32 QueueTypeIndex = (int32)SignalInfo.QueueType;
				MaxTimelineWaitValue[QueueTypeIndex] = FMath::Max(MaxTimelineWaitValue[QueueTypeIndex], SignalInfo.TimelineValue);
			}
		}

		// Create the profiler events
		const uint64 SubmitTime = FPlatformTime::Cycles64();
		for (int32 QueueTypeIndex = 0; QueueTypeIndex < (int32)EVulkanQueueType::Count; ++QueueTypeIndex)
		{
			if (MaxTimelineWaitValue[QueueTypeIndex] > 0)
			{
				FVulkanQueue* SignalQueue = Device.GetQueue((EVulkanQueueType)QueueTypeIndex);
				check(SignalQueue);
				Payload->EventStream.Emplace<UE::RHI::GPUProfiler::FEvent::FWaitFence>(
					SubmitTime, MaxTimelineWaitValue[QueueTypeIndex], SignalQueue->GetProfilerQueue());
			}
		}

		// We can only remove them from the list if they are all present
		for (const TRefCountPtr<FVulkanSemaphore>& WaitSema : WaitSemas)
		{
			if (!WaitSema->IsExternallySignaled())
			{
				const VkSemaphore SemaphoreHandle = WaitSema->GetHandle();
				const int32 NumRemoved = SignaledSemas.Remove(SemaphoreHandle);
				checkSlow(NumRemoved > 0);
			}
		}

		return true;
	};

	// Accumulate a list of the payloads we can submit
	TArray<FVulkanPayload*> Payloads;
	while (FVulkanPayload * Payload = PendingSubmission.Peek())
	{
		// We can only submit the payload if all its wait semas have been signaled
		if (!CanBeProcessed(Payload))
		{
			break;
		}

		Payloads.Add(Payload);
		PendingSubmission.Pop();
	}

	if (Payloads.Num())
	{
		SubmitPayloads(Payloads, SignaledSemas);
	}

	return Payloads.Num();
}

void FVulkanQueue::SubmitPayloads(TArrayView<FVulkanPayload*> Payloads, TMap<VkSemaphore, FBinarySemaphoreSignalInfo>& SignaledSemas)
{
	TArray<VkSemaphore> SemaphoreStorage;
	TArray<VkCommandBuffer> CommandBufferStorage;
	TArray<VkSubmitInfo> SubmitInfos;
	TArray<VkTimelineSemaphoreSubmitInfo> TimelineInfos;
	TArray<uint64> TimelineValues;

	// Presize the arrays so that we don't reallocate
	const uint32 NumPayloads = Payloads.Num();
	uint32 NumWaitSemaphores = 0;
	uint32 NumSignalSemaphores = 0;
	uint32 NumCommandBuffers = 0;

	for (FVulkanPayload* Payload : Payloads)
	{
		NumWaitSemaphores += Payload->WaitSemaphores.Num();
		NumSignalSemaphores += Payload->SignalSemaphores.Num();
		NumCommandBuffers += Payload->CommandBuffers.Num();
	}

	if (bUseTimelineSemaphores)
	{
		// We will add a timeline sempahore per payload
		NumSignalSemaphores += NumPayloads;

		TimelineInfos.Reserve(NumPayloads);
		TimelineValues.Reserve(NumSignalSemaphores);
	}

	SemaphoreStorage.Reserve(NumWaitSemaphores + NumSignalSemaphores);
	CommandBufferStorage.Reserve(NumCommandBuffers);
	SubmitInfos.Reserve(NumPayloads);

	if (Device.UseMinimalSubmits())
	{
		for (int32 PayloadIndex = Payloads.Num()-1; PayloadIndex >= 1; PayloadIndex--)
		{
			FVulkanPayload* PreviousPayload = Payloads[PayloadIndex - 1];
			FVulkanPayload* CurrentPayload = Payloads[PayloadIndex];

			// We can't merge if we have syncs or action that need to happen between the command buffers
			if (CurrentPayload->WaitSemaphores.IsEmpty() && 
				!CurrentPayload->PreExecuteCallback &&
				PreviousPayload->SignalSemaphores.IsEmpty() &&
				PreviousPayload->ReservedResourcesToCommit.IsEmpty() &&
				!PreviousPayload->Timing.IsSet())
			{
				bool bMovableSyncs = true;
				for (FVulkanSyncPointRef& SyncPoint : PreviousPayload->SyncPoints)
				{
					if (SyncPoint->GetType() != EVulkanSyncPointType::Context)
					{
						bMovableSyncs = false;
						break;
					}
				}

				if (!bMovableSyncs)
				{
					continue;
				}

				PreviousPayload->CommandBuffers.Append(MoveTemp(CurrentPayload->CommandBuffers));
			}
		}
	}

	const uint64 SubmitTime = FPlatformTime::Cycles64();

	for (FVulkanPayload* Payload : Payloads)
	{
		Payload->PreExecute();

		const bool bHasExternalGPUTime = Payload->ExternalGPUTime.IsSet();
		if (bHasExternalGPUTime)
		{
			Payload->EventStream.Emplace<UE::RHI::GPUProfiler::FEvent::FFrameTime>(*Payload->ExternalGPUTime);
		}

		if (Payload->EndFrameEvent.IsSet())
		{
			Payload->EndFrameEvent->CPUTimestamp = SubmitTime;
			Payload->EventStream.Emplace<UE::RHI::GPUProfiler::FEvent::FFrameBoundary>(*Payload->EndFrameEvent);
		}

		if (Payload->Timing.IsSet())
		{
			if (FVulkanTiming* LocalTiming = *Payload->Timing)
			{
				SCOPED_NAMED_EVENT(CalibrateClocks, FColor::Red);
				Device.GetCalibrationTimestamp(*LocalTiming);
			}
		}

		// Some payloads have nothing to submit because they are only used to trigger CPU events
		if (!Payload->WaitSemaphores.Num() && !Payload->CommandBuffers.Num() && !Payload->SignalSemaphores.Num() && !Payload->ReservedResourcesToCommit.Num())
		{
			// Consider complete when previous workload is done
			Payload->TimelineSemaphoreValue = NextTimelineSemaphoreValue - 1;
			continue;
		}

		// Bind memory for sparse resources
		if (Payload->ReservedResourcesToCommit.Num())
		{
			if (SubmitInfos.Num() > 0)
			{
				Submit(SubmitInfos, nullptr);
				SubmitInfos.Reset();
			}

			BindSparseResources(*Payload);
		}

		VkSubmitInfo& SubmitInfo = SubmitInfos.AddZeroed_GetRef();
		SubmitInfo.sType = VK_STRUCTURE_TYPE_SUBMIT_INFO;

		VkTimelineSemaphoreSubmitInfo* TimelineInfo = nullptr;
		if (bUseTimelineSemaphores)
		{
			TimelineInfo = &TimelineInfos.AddZeroed_GetRef();
			TimelineInfo->sType = VK_STRUCTURE_TYPE_TIMELINE_SEMAPHORE_SUBMIT_INFO_KHR;
			SubmitInfo.pNext = TimelineInfo;
		}

		// Wait Semaphores
		if (Payload->WaitSemaphores.Num())
		{
			const int32 FirstSempahoreIndex = SemaphoreStorage.Num();
			for (FVulkanSemaphore* Semaphore : Payload->WaitSemaphores)
			{
				SemaphoreStorage.Add(Semaphore->GetHandle());
			}
			SubmitInfo.waitSemaphoreCount = Payload->WaitSemaphores.Num();
			SubmitInfo.pWaitSemaphores = &SemaphoreStorage[FirstSempahoreIndex];
			SubmitInfo.pWaitDstStageMask = Payload->WaitFlags.GetData();
		}

		// Command buffers
		if (Payload->CommandBuffers.Num())
		{
			const int32 FirstIndex = CommandBufferStorage.Num();
			for (FVulkanCommandBuffer* CommandBuffer : Payload->CommandBuffers)
			{
				CommandBufferStorage.Add(CommandBuffer->GetHandle());

				CommandBuffer->FlushProfilerEvents(Payload->EventStream, SubmitTime);
				for (FVulkanCommandBuffer* SecondaryCommandBuffer : CommandBuffer->ExecutedSecondaryCommandBuffers)
				{
					SecondaryCommandBuffer->FlushProfilerEvents(Payload->EventStream, SubmitTime);
				}

				CommandBuffer->SetSubmitted();
			}
			SubmitInfo.commandBufferCount = Payload->CommandBuffers.Num();
			SubmitInfo.pCommandBuffers = &CommandBufferStorage[FirstIndex];
		}

		// Signal Semaphores (and timeline semaphore if supported)
		if (Payload->SignalSemaphores.Num())
		{
			const int32 FirstSempahoreIndex = SemaphoreStorage.Num();
			for (FVulkanSemaphore* Semaphore : Payload->SignalSemaphores)
			{
				SemaphoreStorage.Add(Semaphore->GetHandle());

				// Externally-waited semaphores (like rendering done semaphore) are never waited by a payload
				if (!Semaphore->IsExternallyWaited())
				{
					SignaledSemas.Add(Semaphore->GetHandle(), { NextTimelineSemaphoreValue, QueueType });
				}
			}
			SubmitInfo.signalSemaphoreCount = Payload->SignalSemaphores.Num();
			SubmitInfo.pSignalSemaphores = &SemaphoreStorage[FirstSempahoreIndex];
		}

		Payload->EventStream.Emplace<UE::RHI::GPUProfiler::FEvent::FSignalFence>(
			SubmitTime, NextTimelineSemaphoreValue
		);

		if (bUseTimelineSemaphores)
		{
			SemaphoreStorage.Add(TimelineSempahore->GetHandle());
			if (SubmitInfo.pSignalSemaphores)
			{
				checkSlow(SubmitInfo.signalSemaphoreCount > 0);
				++SubmitInfo.signalSemaphoreCount;
			}
			else
			{
				SubmitInfo.pSignalSemaphores = &SemaphoreStorage[SemaphoreStorage.Num()-1];
				SubmitInfo.signalSemaphoreCount = 1;
			}

			const int32 FirstValueIndex = TimelineValues.Num();
			TimelineInfo->signalSemaphoreValueCount = SubmitInfo.signalSemaphoreCount;
			TimelineValues.AddZeroed(SubmitInfo.signalSemaphoreCount);
			TimelineInfo->pSignalSemaphoreValues = (uint64_t*)&TimelineValues[FirstValueIndex];

			Payload->TimelineSemaphoreValue = NextTimelineSemaphoreValue;
			TimelineValues.Last() = NextTimelineSemaphoreValue;
		}
		else
		{
			// If timeline semaphores aren't supported, we need to use Fences.
			// Because there can only be a single Fence per call to QueueSubmit()
			// we need to submit each payload individually.
			Payload->TimelineSemaphoreValue = NextTimelineSemaphoreValue;
			Payload->Fence = Device.GetFenceManager().AllocateFence();
			Submit(SubmitInfos, Payload->Fence);
			SubmitInfos.Reset();
		}

		NextTimelineSemaphoreValue++;
	}

	if (bUseTimelineSemaphores && SubmitInfos.Num())
	{
		Submit(SubmitInfos, nullptr);
	}

	// Queue the submitted payloads in the interrupt queue
	for (FVulkanPayload* Payload : Payloads)
	{
		for (FGraphEventRef& SubmissionEvent : Payload->SubmissionEvents)
		{
			SubmissionEvent->DispatchSubsequents();
		}
		Payload->SubmissionEvents.Reset();

		PendingInterrupt.Enqueue(Payload);
	}
}


void FVulkanQueue::Submit(TArrayView<VkSubmitInfo> InSubmitInfos, FVulkanFence* Fence)
{
	SCOPE_CYCLE_COUNTER(STAT_VulkanQueueSubmit);
	SCOPED_NAMED_EVENT(VulkanQueueSubmit, FColor::Purple);

	const VkFence FenceHandle = Fence ? Fence->GetHandle() : VK_NULL_HANDLE;
	VERIFYVULKANRESULT(VulkanRHI::vkQueueSubmit(Queue, InSubmitInfos.Num(), InSubmitInfos.GetData(), FenceHandle));

	if (GWaitForIdleOnSubmit != 0)
	{
		VERIFYVULKANRESULT(VulkanRHI::vkDeviceWaitIdle(Device.GetHandle()));

		if (Fence)
		{
			const bool bSuccess = Device.GetFenceManager().WaitForFence(Fence, 500ULL * 1000 * 1000);
			ensure(bSuccess);
			ensure(Device.GetFenceManager().IsFenceSignaled(Fence));
		}
	}
}

int32 FVulkanQueue::ProcessInterruptQueue(uint64 Timeout)
{
	SCOPED_NAMED_EVENT_TEXT("ProcessInterruptQueue", FColor::Orange);

	if (bUseTimelineSemaphores)
	{
		checkSlow(TimelineSempahore);
		TimelineSempahore->GetTimelineSemaphoreValue(CompletedTimelineSemaphoreValue);
	}

	int32 NumPayloads = 0;
	bool bSuccess;
	do
	{
		bSuccess = false;
		if (FVulkanPayload* Payload = PendingInterrupt.Peek())
		{
			if (Payload->TimelineSemaphoreValue <= CompletedTimelineSemaphoreValue)
			{
				bSuccess = true;
			}
			else if (bUseTimelineSemaphores)
			{
				checkSlow(Payload->TimelineSemaphoreValue > 0);
				if (Timeout > 0)
				{
					bSuccess = TimelineSempahore->WaitForTimelineSemaphoreValue(Payload->TimelineSemaphoreValue, Timeout);
					TimelineSempahore->GetTimelineSemaphoreValue(CompletedTimelineSemaphoreValue);
				}
			}
			else
			{
				checkSlow(Payload->Fence);
				bSuccess = (Timeout == 0) ?
					Device.GetFenceManager().IsFenceSignaled(Payload->Fence) :
					Device.GetFenceManager().WaitForFence(Payload->Fence, Timeout);

				if (bSuccess)
				{
					checkSlow(Payload->TimelineSemaphoreValue > 0);
					check(CompletedTimelineSemaphoreValue < Payload->TimelineSemaphoreValue);
					CompletedTimelineSemaphoreValue = Payload->TimelineSemaphoreValue;
				}
			}

			if (bSuccess)
			{
				++NumPayloads;

				// Resolve any pending actions from the payload being completed
				FVulkanDynamicRHI::Get().CompletePayload(Payload);

				PendingInterrupt.Pop();
				delete Payload;
			}
		}
	} while (bSuccess);

	return NumPayloads;
}

void FVulkanQueue::FillSupportedStageBits()
{
	check((int32)FamilyIndex < Device.GetQueueFamilyProps().Num());

	const VkQueueFamilyProperties& QueueProps = Device.GetQueueFamilyProps()[FamilyIndex];

	SupportedStages = 
		VK_PIPELINE_STAGE_TOP_OF_PIPE_BIT | 
		VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT |
		VK_PIPELINE_STAGE_HOST_BIT |
		VK_PIPELINE_STAGE_ALL_COMMANDS_BIT;

	SupportedAccess =
		VK_ACCESS_HOST_READ_BIT |
		VK_ACCESS_HOST_WRITE_BIT |
		VK_ACCESS_MEMORY_READ_BIT |
		VK_ACCESS_MEMORY_WRITE_BIT;

	if (VKHasAnyFlags(QueueProps.queueFlags, VK_QUEUE_GRAPHICS_BIT))
	{
		SupportedStages |=
			VK_PIPELINE_STAGE_DRAW_INDIRECT_BIT |
			VK_PIPELINE_STAGE_VERTEX_INPUT_BIT |
			VK_PIPELINE_STAGE_VERTEX_SHADER_BIT |
			VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT |
			VK_PIPELINE_STAGE_EARLY_FRAGMENT_TESTS_BIT |
			VK_PIPELINE_STAGE_LATE_FRAGMENT_TESTS_BIT |
			VK_PIPELINE_STAGE_COLOR_ATTACHMENT_OUTPUT_BIT |
			VK_PIPELINE_STAGE_TRANSFER_BIT |
			VK_PIPELINE_STAGE_ALL_GRAPHICS_BIT;

		SupportedAccess |=
			VK_ACCESS_TRANSFER_READ_BIT |
			VK_ACCESS_TRANSFER_WRITE_BIT |
			VK_ACCESS_INDIRECT_COMMAND_READ_BIT |
			VK_ACCESS_UNIFORM_READ_BIT |
			VK_ACCESS_SHADER_READ_BIT |
			VK_ACCESS_SHADER_WRITE_BIT |
			VK_ACCESS_INDEX_READ_BIT |
			VK_ACCESS_VERTEX_ATTRIBUTE_READ_BIT |
			VK_ACCESS_INPUT_ATTACHMENT_READ_BIT |
			VK_ACCESS_COLOR_ATTACHMENT_READ_BIT |
			VK_ACCESS_COLOR_ATTACHMENT_WRITE_BIT |
			VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_READ_BIT |
			VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_WRITE_BIT;

		if (Device.GetPhysicalDeviceFeatures().Core_1_0.geometryShader)
		{
			SupportedStages |= VK_PIPELINE_STAGE_GEOMETRY_SHADER_BIT;
		}
		if (Device.GetOptionalExtensions().HasKHRFragmentShadingRate)
		{
			SupportedStages |= VK_PIPELINE_STAGE_FRAGMENT_SHADING_RATE_ATTACHMENT_BIT_KHR;
			SupportedAccess |= VK_ACCESS_FRAGMENT_SHADING_RATE_ATTACHMENT_READ_BIT_KHR;
		}
		if (Device.GetOptionalExtensions().HasEXTFragmentDensityMap)
		{
			SupportedStages |= VK_PIPELINE_STAGE_FRAGMENT_DENSITY_PROCESS_BIT_EXT;
			SupportedAccess |= VK_ACCESS_FRAGMENT_DENSITY_MAP_READ_BIT_EXT;
		}
		if (Device.GetOptionalExtensions().HasEXTMeshShader)
		{
			SupportedStages |= VK_PIPELINE_STAGE_TASK_SHADER_BIT_EXT | VK_PIPELINE_STAGE_MESH_SHADER_BIT_EXT;
		}
	}

	if (VKHasAnyFlags(QueueProps.queueFlags, VK_QUEUE_COMPUTE_BIT))
	{
		SupportedStages |=
			VK_PIPELINE_STAGE_COMPUTE_SHADER_BIT |
			VK_PIPELINE_STAGE_DRAW_INDIRECT_BIT |
			VK_PIPELINE_STAGE_TRANSFER_BIT;

		SupportedAccess |=
			VK_ACCESS_TRANSFER_READ_BIT |
			VK_ACCESS_TRANSFER_WRITE_BIT |
			VK_ACCESS_INDIRECT_COMMAND_READ_BIT |
			VK_ACCESS_UNIFORM_READ_BIT |
			VK_ACCESS_SHADER_READ_BIT |
			VK_ACCESS_SHADER_WRITE_BIT;

		if (Device.GetOptionalExtensions().HasAccelerationStructure)
		{
			SupportedStages |= VK_PIPELINE_STAGE_ACCELERATION_STRUCTURE_BUILD_BIT_KHR;
			SupportedAccess |= VK_ACCESS_ACCELERATION_STRUCTURE_READ_BIT_KHR | VK_ACCESS_ACCELERATION_STRUCTURE_WRITE_BIT_KHR;
		}

		if (Device.GetOptionalExtensions().HasRayTracingPipeline)
		{
			SupportedStages |= VK_PIPELINE_STAGE_RAY_TRACING_SHADER_BIT_KHR;
		}
	}

	if (VKHasAnyFlags(QueueProps.queueFlags, VK_QUEUE_TRANSFER_BIT))
	{
		SupportedStages |= VK_PIPELINE_STAGE_TRANSFER_BIT;
		SupportedAccess |= VK_ACCESS_TRANSFER_READ_BIT | VK_ACCESS_TRANSFER_WRITE_BIT;
	}
}

UE::RHI::GPUProfiler::FQueue FVulkanQueue::GetProfilerQueue() const
{
	UE::RHI::GPUProfiler::FQueue ProfilerQueue;
	ProfilerQueue.GPU = 0;
	ProfilerQueue.Index = 0;

	switch (QueueType)
	{
	default: checkNoEntry(); [[fallthrough]];
	case EVulkanQueueType::Graphics:     ProfilerQueue.Type = UE::RHI::GPUProfiler::FQueue::EType::Graphics; break;
	case EVulkanQueueType::AsyncCompute: ProfilerQueue.Type = UE::RHI::GPUProfiler::FQueue::EType::Compute;  break;
	case EVulkanQueueType::Transfer:     ProfilerQueue.Type = UE::RHI::GPUProfiler::FQueue::EType::Copy;     break;
	}

	return ProfilerQueue;
}

FVulkanCommandBufferPool* FVulkanQueue::AcquireCommandBufferPool(EVulkanCommandBufferType CommandBufferType)
{
	FScopeLock Lock(&CommandBufferPoolCS);
	TArray<FVulkanCommandBufferPool*>& CommandBufferPoolArray = CommandBufferPools[(int32)CommandBufferType];
	if (CommandBufferPoolArray.Num())
	{
		return CommandBufferPoolArray.Pop(EAllowShrinking::No);
	}
	return new FVulkanCommandBufferPool(Device, *this, CommandBufferType);
}

void FVulkanQueue::ReleaseCommandBufferPool(FVulkanCommandBufferPool* CommandBufferPool)
{
	FScopeLock Lock(&CommandBufferPoolCS);
	check(&CommandBufferPool->GetQueue() == this);
	TArray<FVulkanCommandBufferPool*>& CommandBufferPoolArray = CommandBufferPools[(int32)CommandBufferPool->GetCommandBufferType()];
	CommandBufferPoolArray.Add(CommandBufferPool);
}

void FVulkanQueue::InitDiagnosticBuffer()
{
	check(!DiagnosticBuffer.IsValid());
	if (GVulkanDiagnosticBuffer != 0)
	{
		DiagnosticBuffer = MakeUnique<FVulkanDiagnosticBuffer>(Device, *this);
	}
}

FVulkanDiagnosticBuffer::FVulkanDiagnosticBuffer(FVulkanDevice& InDevice, FVulkanQueue& InQueue)
	: Device(InDevice)
	, Queue(InQueue)
{
	VkBufferCreateInfo CreateInfo;
	ZeroVulkanStruct(CreateInfo, VK_STRUCTURE_TYPE_BUFFER_CREATE_INFO);
	CreateInfo.size = SizeInBytes;
	CreateInfo.usage = VK_BUFFER_USAGE_TRANSFER_DST_BIT;
	VERIFYVULKANRESULT(VulkanRHI::vkCreateBuffer(Device.GetHandle(), &CreateInfo, VULKAN_CPU_ALLOCATOR, &Buffer));

	const VulkanRHI::EVulkanAllocationFlags MemoryFlags =
		VulkanRHI::EVulkanAllocationFlags::Dedicated |
		VulkanRHI::EVulkanAllocationFlags::AutoBind |
		VulkanRHI::EVulkanAllocationFlags::HostVisible |
		VulkanRHI::EVulkanAllocationFlags::HostCached;
	Device.GetMemoryManager().AllocateBufferMemory(Allocation, Buffer, MemoryFlags, TEXT("DiagnosticBuffer"));

	Data = (FQueue*)Allocation.GetMappedPointer(&InDevice);
	check(Data);
	FMemory::Memzero(Data, SizeInBytes);

#if WITH_RHI_BREADCRUMBS
	if (Device.GetOptionalExtensions().HasNVDiagnosticCheckpoints &&
		UE::RHI::UseGPUCrashBreadcrumbs())
	{
		ExtendedBreadcrumbAllocators.SetNumZeroed(CVarVulkanExtendedLifetimeFrames->GetInt());
	}
#endif
}

FVulkanDiagnosticBuffer::~FVulkanDiagnosticBuffer()
{
	if (Buffer != VK_NULL_HANDLE)
	{
		VulkanRHI::vkDestroyBuffer(Device.GetHandle(), Buffer, VULKAN_CPU_ALLOCATOR);
		Buffer = VK_NULL_HANDLE;
	}

	if (Allocation.IsValid())
	{
		Device.GetMemoryManager().FreeVulkanAllocation(Allocation, VulkanRHI::EVulkanFreeFlag_DontDefer);
	}
}

#if WITH_RHI_BREADCRUMBS
static void ManuallyWriteMarker(const VkCommandBuffer CommandBufferHandle, const VkBuffer BufferHandle, const uint64 WriteOffset, const uint32 BreadcrumbID)
{
	// When using precise barriers, make sure any work before or after can't overlap with the marker's write.
	// When aiming for minimal cost, use a single barrier before the writing the marker (the fact we write markers on in and out should mean we're never more than off by one)
	const bool bUsePreciseBarriers = (GVulkanDiagnosticBuffer > 1);
	const VkPipelineStageFlags DestStage = bUsePreciseBarriers ? VK_PIPELINE_STAGE_TOP_OF_PIPE_BIT : VK_PIPELINE_STAGE_TRANSFER_BIT;
	VulkanDynamicAPI::vkCmdPipelineBarrier(CommandBufferHandle, VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT, DestStage, 0, 0, nullptr, 0, nullptr, 0, nullptr);

	VulkanDynamicAPI::vkCmdFillBuffer(CommandBufferHandle, BufferHandle, WriteOffset, sizeof(BreadcrumbID), BreadcrumbID);

	if (bUsePreciseBarriers)
	{
		VulkanDynamicAPI::vkCmdPipelineBarrier(CommandBufferHandle, VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT, DestStage, 0, 0, nullptr, 0, nullptr, 0, nullptr);
	}

}

void FVulkanDiagnosticBuffer::WriteMarkerIn(FVulkanCommandBuffer& CommandBuffer, FRHIBreadcrumbNode* Breadcrumb) const
{
	const VkCommandBuffer CommandBufferHandle = CommandBuffer.GetHandle();
	const uint64 WriteOffset = offsetof(FQueue, MarkerIn);

	if (!Breadcrumb)
	{
		Breadcrumb = FRHIBreadcrumbNode::Sentinel;
	}

	if (Device.GetOptionalExtensions().HasAMDBufferMarker)
	{
		VulkanDynamicAPI::vkCmdWriteBufferMarkerAMD(CommandBufferHandle, VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT, Buffer, WriteOffset, Breadcrumb->ID);
	}
	else
	{
		// Buffer writes cannot be recorded inside of a render pass
		if (CommandBuffer.IsOutsideRenderPass())
		{
			ManuallyWriteMarker(CommandBufferHandle, Buffer, WriteOffset, Breadcrumb->ID);
		}
	}

#if NV_AFTERMATH
	if (Device.GetOptionalExtensions().HasNVDiagnosticCheckpoints)
	{
		VulkanDynamicAPI::vkCmdSetCheckpointNV(CommandBufferHandle, Breadcrumb);
	}
#endif // NV_AFTERMATH
}

void FVulkanDiagnosticBuffer::WriteMarkerOut(FVulkanCommandBuffer& CommandBuffer, FRHIBreadcrumbNode* Breadcrumb) const
{
	const VkCommandBuffer CommandBufferHandle = CommandBuffer.GetHandle();
	const uint64 WriteOffset = offsetof(FQueue, MarkerOut);

	if (!Breadcrumb)
	{
		Breadcrumb = FRHIBreadcrumbNode::Sentinel;
	}

	if (Device.GetOptionalExtensions().HasAMDBufferMarker)
	{
		VulkanDynamicAPI::vkCmdWriteBufferMarkerAMD(CommandBufferHandle, VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT, Buffer, WriteOffset, Breadcrumb->ID);
	}
	else
	{
		// Buffer writes cannot be recorded inside of a render pass
		if (CommandBuffer.IsOutsideRenderPass())
		{
			ManuallyWriteMarker(CommandBufferHandle, Buffer, WriteOffset, Breadcrumb->ID);
		}
	}

#if NV_AFTERMATH
	if (Device.GetOptionalExtensions().HasNVDiagnosticCheckpoints)
	{
		VulkanDynamicAPI::vkCmdSetCheckpointNV(CommandBufferHandle, Breadcrumb->GetParent());
	}
#endif // NV_AFTERMATH
}

#endif // WITH_RHI_BREADCRUMBS