// Copyright Epic Games, Inc. All Rights Reserved..

/*=============================================================================
	VulkanQueue.h: Private Vulkan RHI definitions.
=============================================================================*/

#pragma once

#include "VulkanConfiguration.h"
#include "VulkanBarriers.h"
#include "VulkanCommandBuffer.h"
#include "RHIDiagnosticBuffer.h"
#include "RHICoreNvidiaAftermath.h"

class FVulkanDynamicRHI;
class FVulkanDevice;
class FVulkanPayload;
class FVulkanTiming;

namespace VulkanRHI
{
	class FFence;
	class FSemaphore;
}

enum class EVulkanQueueType : uint8
{
	Graphics = 0,
	AsyncCompute,
	Transfer,

	Count,
};

inline const TCHAR* GetVulkanQueueTypeName(EVulkanQueueType QueueType)
{
	switch (QueueType)
	{
	default: checkNoEntry(); // fallthrough
	case EVulkanQueueType::Graphics:      return TEXT("Graphics");
	case EVulkanQueueType::AsyncCompute:  return TEXT("AsyncCompute");
	case EVulkanQueueType::Transfer:      return TEXT("Transfer");
	}
}

struct FBinarySemaphoreSignalInfo
{
	uint64 TimelineValue;
	EVulkanQueueType QueueType;
};

class FVulkanQueue
{
public:
	FVulkanQueue(FVulkanDevice& InDevice, uint32 InFamilyIndex, EVulkanQueueType InQueueType);
	~FVulkanQueue();

	EVulkanQueueType GetQueueType() const
	{
		return QueueType;
	}

	uint32 GetFamilyIndex() const
	{
		return FamilyIndex;
	}

	uint32 GetQueueIndex() const
	{
		return QueueIndex;
	}

	VkQueue GetHandle() const
	{
		return Queue;
	}

	FVulkanCommandBufferPool* AcquireCommandBufferPool(EVulkanCommandBufferType CommandBufferType);
	void ReleaseCommandBufferPool(FVulkanCommandBufferPool* CommandBufferPool);

	VkPipelineStageFlags GetSupportedStageBits() const
	{
		return SupportedStages;
	}

	VkAccessFlags GetSupportedAccessFlags() const
	{
		return SupportedAccess;
	}

	const FVulkanSemaphore* GetTimelineSemaphore() const
	{
		return TimelineSempahore;
	}

	uint64 GetLastSubmittedTimelineSemaphoreValue() const
	{
		return NextTimelineSemaphoreValue - 1;
	}

	uint64 GetCompletedTimelineSemaphoreValue() const
	{
		return CompletedTimelineSemaphoreValue;
	}

	int32 SubmitQueuedPayloads(TMap<VkSemaphore, FBinarySemaphoreSignalInfo>& SignaledSemas);
	int32 ProcessInterruptQueue(uint64 Timeout);

	UE::RHI::GPUProfiler::FQueue GetProfilerQueue() const;

	void InitDiagnosticBuffer();
	class FVulkanDiagnosticBuffer* GetDiagnosticBuffer()
	{
		return DiagnosticBuffer.Get();
	}

private:
	void SubmitPayloads(TArrayView<FVulkanPayload*> Payloads, TMap<VkSemaphore, FBinarySemaphoreSignalInfo>& InOutSignaledSemas);
	void Submit(TArrayView<VkSubmitInfo> InSubmitInfos, FVulkanFence* Fence);

	void BindSparseResources(FVulkanPayload& Payload);

	friend FVulkanDynamicRHI;

	VkQueue Queue;
	uint32 FamilyIndex;
	uint32 QueueIndex;
	EVulkanQueueType QueueType;
	FVulkanDevice& Device;

	FCriticalSection CommandBufferPoolCS;
	TStaticArray<TArray<FVulkanCommandBufferPool*>, (int32)EVulkanCommandBufferType::Count> CommandBufferPools;

	const bool bUseTimelineSemaphores;
	FVulkanSemaphore* TimelineSempahore = nullptr;
	uint64 NextTimelineSemaphoreValue = 1;
	uint64 CompletedTimelineSemaphoreValue = 0;

	uint64 SubmitCounter;
	VkPipelineStageFlags SupportedStages = VK_PIPELINE_STAGE_NONE;
	VkAccessFlags SupportedAccess = VK_ACCESS_NONE;

	class FVulkanPayloadQueue
	{
		TSpscQueue<FVulkanPayload*> Queue;

	public:
		void Enqueue(FVulkanPayload* Payload)
		{
			check(Payload);
			Queue.Enqueue(Payload);
		}

		FVulkanPayload* Peek()
		{
			if (FVulkanPayload** PayloadPtr = Queue.Peek())
			{
				return *PayloadPtr;
			}

			return nullptr;
		}

		void Pop()
		{
			Queue.Dequeue();
		}

		auto begin() { return Queue.begin(); }
		auto end  () { return Queue.end  (); }
	};

	FVulkanPayloadQueue PendingSubmission;
	FVulkanPayloadQueue PendingInterrupt;

	// The active timing struct on this queue. Updated / accessed by the interrupt thread.
	FVulkanTiming* Timing = nullptr;

	TUniquePtr<class FVulkanDiagnosticBuffer> DiagnosticBuffer;

	void FillSupportedStageBits();
};


//
// Per Queue diagnostic buffer. Stays accessible after a GPU crash to allow readback of diagnostic messages.
// Also used to track the progress of the GPU via breadcrumb markers.
//
class FVulkanDiagnosticBuffer : public FRHIDiagnosticBuffer
{
public:
	FVulkanDiagnosticBuffer(FVulkanDevice& InDevice, FVulkanQueue& InQueue);
	~FVulkanDiagnosticBuffer();

	// :todo: Bindings for shader asserts, etc

#if WITH_RHI_BREADCRUMBS
	void WriteMarkerIn(FVulkanCommandBuffer& CommandBuffer, FRHIBreadcrumbNode* Breadcrumb) const;
	void WriteMarkerOut(FVulkanCommandBuffer& CommandBuffer, FRHIBreadcrumbNode* Breadcrumb) const;

	uint32 ReadMarkerIn()
	{
		Allocation.InvalidateMappedMemory(&Device); 
		return Data->MarkerIn; 
	}
	uint32 ReadMarkerOut()
	{
		Allocation.InvalidateMappedMemory(&Device);
		return Data->MarkerOut; 
	}
#endif // WITH_RHI_BREADCRUMBS

	bool IsValid() const { return (Buffer != VK_NULL_HANDLE); }

private:
	FVulkanDevice& Device;
	FVulkanQueue& Queue;
	VkBuffer Buffer = VK_NULL_HANDLE;
	VulkanRHI::FVulkanAllocation Allocation;

#if WITH_RHI_BREADCRUMBS
	// Extend lifetime of breadcrumbs so we can use the pointers directly
	// One array per frame, and cycle which array is used on EndFrame
	TArray<FRHIBreadcrumbAllocatorArray> ExtendedBreadcrumbAllocators;
	int32 CurrentExtendedIndex = 0;
#endif // WITH_RHI_BREADCRUMBS

	friend FVulkanDynamicRHI;
};