// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanRHI.cpp: Vulkan device RHI implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "BuildSettings.h"
#include "HardwareInfo.h"
#include "VulkanShaderResources.h"
#include "VulkanResources.h"
#include "VulkanPendingState.h"
#include "VulkanContext.h"
#include "VulkanBarriers.h"
#include "Misc/CommandLine.h"
#include "GenericPlatform/GenericPlatformDriver.h"
#include "Modules/ModuleManager.h"
#include "VulkanPipelineState.h"
#include "Misc/FileHelper.h"
#include "VulkanLLM.h"
#include "Misc/EngineVersion.h"
#include "GlobalShader.h"
#include "RenderingThread.h"
#include "RHICore.h"
#include "VulkanResourceCollection.h"
#include "RHIValidation.h"
#include "RHIUtilities.h"
#include "ShaderDiagnostics.h"
#include "IHeadMountedDisplayModule.h"
#include "VulkanRenderpass.h"
#include "VulkanTransientResourceAllocator.h"
#include "VulkanExtensions.h"
#include "VulkanRayTracing.h"
#include "VulkanChunkedPipelineCache.h"
#include "VulkanBindlessDescriptorManager.h"
#if PLATFORM_ANDROID
#include "Android/AndroidPlatformMisc.h"
#endif

// Use Vulkan Profiles to verify feature level support on startup
void VulkanProfilePrint(const char* Msg)
{
	UE_LOGF(LogVulkanRHI, Log, "   - %s", Msg);
}
#define VP_DEBUG_MESSAGE_CALLBACK VulkanProfilePrint
#include "vulkan_profiles_ue.h"
#undef VP_DEBUG_MESSAGE_CALLBACK


static_assert(sizeof(VkStructureType) == sizeof(int32), "ZeroVulkanStruct() assumes VkStructureType is int32!");

TAtomic<uint64> GVulkanBufferHandleIdCounter{ 0 };
TAtomic<uint64> GVulkanBufferViewHandleIdCounter{ 0 };
TAtomic<uint64> GVulkanImageViewHandleIdCounter{ 0 };
TAtomic<uint64> GVulkanSamplerHandleIdCounter{ 0 };
TAtomic<uint64> GVulkanDSetLayoutHandleIdCounter{ 0 };

#if VULKAN_ENABLE_DESKTOP_HMD_SUPPORT
#include "IHeadMountedDisplayModule.h"
#endif

#define LOCTEXT_NAMESPACE "VulkanRHI"


///////////////////////////////////////////////////////////////////////////////

TAutoConsoleVariable<int32> GRHIThreadCvar(
	TEXT("r.Vulkan.RHIThread"),
	2,
	TEXT("0 to only use Render Thread\n")
	TEXT("1 to use ONE RHI Thread\n")
	TEXT("2 to use multiple RHI Thread\n")
);

int32 GVulkanInputAttachmentShaderRead = 0;
static FAutoConsoleVariableRef GCVarInputAttachmentShaderRead(
	TEXT("r.Vulkan.InputAttachmentShaderRead"),
	GVulkanInputAttachmentShaderRead,
	TEXT("Whether to use VK_ACCESS_SHADER_READ_BIT an input attachments to workaround rendering issues\n")
	TEXT("0 use: VK_ACCESS_INPUT_ATTACHMENT_READ_BIT (default)\n")
	TEXT("1 use: VK_ACCESS_INPUT_ATTACHMENT_READ_BIT | VK_ACCESS_SHADER_READ_BIT\n"),
	ECVF_ReadOnly
);

int32 GVulkanEnableTransientResourceAllocator = 1;
static FAutoConsoleVariableRef GCVarEnableTransientResourceAllocator(
	TEXT("r.Vulkan.EnableTransientResourceAllocator"),
	GVulkanEnableTransientResourceAllocator,
	TEXT("Whether to enable the TransientResourceAllocator to reduce memory usage\n")
	TEXT("0 to disabled (default)\n")
	TEXT("1 to enable\n"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<bool> CVarAllowVulkanPSOPrecache(
	TEXT("r.Vulkan.AllowPSOPrecaching"),
	true,
	TEXT("true: if r.PSOPrecaching=1 Vulkan RHI will use precaching. (default)\n")
	TEXT("false: Vulkan RHI will disable precaching (even if r.PSOPrecaching=1)."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

// If precaching is active we should not need the file cache.
// however, precaching and filecache are compatible with each other, there maybe some scenarios in which both could be used.
static TAutoConsoleVariable<bool> CVarEnableVulkanPSOFileCacheWhenPrecachingActive(
	TEXT("r.Vulkan.EnablePSOFileCacheWhenPrecachingActive"),
	false,
	TEXT("false: If precaching is available (r.PSOPrecaching=1, r.Vulkan.UseChunkedPSOCache=1) then disable the PSO filecache. (default)\n")
	TEXT("true: Allow both PSO file cache and precaching."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

int32 GVulkanAMDCompatibilityMode = 1;
static FAutoConsoleVariableRef GCVarVulkanAMDCompatibilityMode(
	TEXT("r.Vulkan.AMDCompatibilityMode"),
	GVulkanAMDCompatibilityMode,
	TEXT("Used to tweak enabled Vulkan feature set in order to ensure wider compatibility with all AMD GPUs on all platforms. (default:1)"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<bool> GCVarVulkanSupportsDedicatedSRVGraphicsNonPixelAccess(
	TEXT("r.Vulkan.SupportsDedicatedSRVGraphicsNonPixelAccess"),
	false,
	TEXT("Enable to use dedicated SRVGraphicsPixel/SRVGraphicsNonPixel access for texture srv parameters instead of using the SRVGraphics mask everywhere. (default:0)"),
	ECVF_ReadOnly
);

extern TAutoConsoleVariable<int32> GVulkanRayTracingCVar;

// All the graphic shader stages supported by the VkDevice (VK_PIPELINE_STAGE_VERTEX_SHADER_BIT, VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT, etc)
VkPipelineStageFlags GVulkanDeviceGraphicsPipelineStageBits = 0;

DEFINE_LOG_CATEGORY(LogVulkan)

// Selects the device to us for the provided instance
static VkPhysicalDevice SelectPhysicalDevice(VkInstance InInstance)
{
	VkResult Result;

	uint32 PhysicalDeviceCount = 0;
	Result = VulkanRHI::vkEnumeratePhysicalDevices(InInstance, &PhysicalDeviceCount, nullptr);
	if ((Result != VK_SUCCESS) || (PhysicalDeviceCount == 0))
	{
		UE_LOGF(LogVulkanRHI, Log, 
			"SelectPhysicalDevice could not find a compatible Vulkan device or driver (EnumeratePhysicalDevices returned '%ls' and %d devices).  "
			"Make sure your video card supports Vulkan and try updating your video driver to a more recent version (proceed with any pending reboots)."
			, VK_TYPE_TO_STRING(VkResult, Result), PhysicalDeviceCount
		);

		return VK_NULL_HANDLE;
	}

	TArray<VkPhysicalDevice> PhysicalDevices;
	PhysicalDevices.AddZeroed(PhysicalDeviceCount);
	VERIFYVULKANRESULT(VulkanRHI::vkEnumeratePhysicalDevices(InInstance, &PhysicalDeviceCount, PhysicalDevices.GetData()));
	checkf(PhysicalDeviceCount >= 1, TEXT("Couldn't enumerate physical devices on second attempt! Make sure your drivers are up to date and that you are not pending a reboot."));

	struct FPhysicalDeviceInfo
	{
		FPhysicalDeviceInfo() = delete;
		FPhysicalDeviceInfo(uint32 InOriginalIndex, VkPhysicalDevice InPhysicalDevice)
			: OriginalIndex(InOriginalIndex)
			, PhysicalDevice(InPhysicalDevice)
		{
			ZeroVulkanStruct(PhysicalDeviceProperties2, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_PROPERTIES_2);
			ZeroVulkanStruct(PhysicalDeviceIDProperties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_ID_PROPERTIES);
			PhysicalDeviceProperties2.pNext = &PhysicalDeviceIDProperties;
			VulkanRHI::vkGetPhysicalDeviceProperties2(PhysicalDevice, &PhysicalDeviceProperties2);
		}

		uint32 OriginalIndex;
		VkPhysicalDevice PhysicalDevice;
		VkPhysicalDeviceProperties2 PhysicalDeviceProperties2;
		VkPhysicalDeviceIDProperties PhysicalDeviceIDProperties;
	};
	TArray<FPhysicalDeviceInfo> PhysicalDeviceInfos;
	PhysicalDeviceInfos.Reserve(PhysicalDeviceCount);

	// Fill the array with each devices properties
	for (uint32 Index = 0; Index < PhysicalDeviceCount; ++Index)
	{
		PhysicalDeviceInfos.Emplace(Index, PhysicalDevices[Index]);
	}

	// Allow HMD to override which graphics adapter is chosen, so we pick the adapter where the HMD is connected
#if VULKAN_ENABLE_DESKTOP_HMD_SUPPORT
	if (IHeadMountedDisplayModule::IsAvailable())
	{
		static_assert(sizeof(uint64) == VK_LUID_SIZE);
		const uint64 HmdGraphicsAdapterLuid = IHeadMountedDisplayModule::Get().GetGraphicsAdapterLuid();

		for (int32 Index = 0; Index < PhysicalDeviceInfos.Num(); ++Index)
		{
			if (FMemory::Memcmp(&HmdGraphicsAdapterLuid, &PhysicalDeviceInfos[Index].PhysicalDeviceIDProperties.deviceLUID, VK_LUID_SIZE_KHR) == 0)
			{
				UE_LOGF(LogVulkanRHI, Log, "HMD device at index %d of %u being used as default...", Index, PhysicalDeviceCount);
				return PhysicalDeviceInfos[Index].PhysicalDevice;
			}
		}
	}
#endif

	// Use the device as forced by CVar or CommandLine arg
	auto* CVarGraphicsAdapter = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GraphicsAdapter"));
	int32 ExplicitAdapterValue = CVarGraphicsAdapter ? CVarGraphicsAdapter->GetValueOnAnyThread() : -1;
	const bool bUsingCmdLine = FParse::Value(FCommandLine::Get(), TEXT("graphicsadapter="), ExplicitAdapterValue);
	const ANSICHAR* GraphicsAdapterOriginTxt = bUsingCmdLine ? "command line" : "'r.GraphicsAdapter'";
	if (ExplicitAdapterValue >= 0)  // Use adapter at the specified index
	{
		if (ExplicitAdapterValue >= PhysicalDeviceInfos.Num())
		{
			UE_LOGF(LogVulkanRHI, Warning, "Tried to use graphics adapter at index %d as specified by %s, but only %d Adapter(s) found. Falling back to first device...", 
				ExplicitAdapterValue, GraphicsAdapterOriginTxt, PhysicalDeviceInfos.Num());
			ExplicitAdapterValue = 0;
		}

		UE_LOGF(LogVulkanRHI, Log, "Using device at index %d of %u as specfified by %s...", ExplicitAdapterValue, PhysicalDeviceCount, GraphicsAdapterOriginTxt);
		return PhysicalDeviceInfos[ExplicitAdapterValue].PhysicalDevice;
	}
	else if (ExplicitAdapterValue == -2)  // Take the first one that fulfills the criteria
	{
		UE_LOGF(LogVulkanRHI, Log, "Using first device (of %u) without any sorting as specfified by %s...", PhysicalDeviceCount, GraphicsAdapterOriginTxt);
		return PhysicalDeviceInfos[0].PhysicalDevice;
	}
	else if (ExplicitAdapterValue == -1)  // Favour non integrated because there are usually faster
	{
		// Reoreder the list to place discrete adapters first
		PhysicalDeviceInfos.Sort([](const FPhysicalDeviceInfo& Lhs, const FPhysicalDeviceInfo& Rhs)
			{
				// For devices of the same type, jsut keep the original order
				if (Lhs.PhysicalDeviceProperties2.properties.deviceType == Rhs.PhysicalDeviceProperties2.properties.deviceType)
				{
					return Lhs.OriginalIndex < Rhs.OriginalIndex;
				}
				
				// Prefer discrete GPUs first, then integrated, then CPU
				return (Lhs.PhysicalDeviceProperties2.properties.deviceType == VK_PHYSICAL_DEVICE_TYPE_DISCRETE_GPU) || 
					(Rhs.PhysicalDeviceProperties2.properties.deviceType == VK_PHYSICAL_DEVICE_TYPE_CPU);
			});
	}


	// If a preferred vendor is specified, return the first device from that vendor
	const EGpuVendorId PreferredVendor = RHIGetPreferredAdapterVendor();
	if (PreferredVendor != EGpuVendorId::Unknown)
	{
		// Check for preferred
		for (int32 Index = 0; Index < PhysicalDeviceInfos.Num(); ++Index)
		{
			if (RHIConvertToGpuVendorId(PhysicalDeviceInfos[Index].PhysicalDeviceProperties2.properties.vendorID) == PreferredVendor)
			{
				UE_LOGF(LogVulkanRHI, Log, "Using preferred vendor device at index %d of %u...", Index, PhysicalDeviceCount);
				return PhysicalDeviceInfos[Index].PhysicalDevice;
			}
		}
	}

	// Skip all CPU devices if they aren't permitted
	const bool bAllowCPUDevices = FParse::Param(FCommandLine::Get(), TEXT("AllowCPUDevices")) || FParse::Param(FCommandLine::Get(), TEXT("AllowSoftwareRendering"));
	for (int32 Index = 0; Index < PhysicalDeviceInfos.Num(); ++Index)
	{
		if (!bAllowCPUDevices && (PhysicalDeviceInfos[Index].PhysicalDeviceProperties2.properties.deviceType == VK_PHYSICAL_DEVICE_TYPE_CPU))
		{
			continue;
		}

		return PhysicalDeviceInfos[Index].PhysicalDevice;
	}

	UE_LOGF(LogVulkanRHI, Warning, "None of the %u devices meet all the criteria!", PhysicalDeviceCount);
	return VK_NULL_HANDLE;
}

static uint32 GetVulkanApiVersionForFeatureLevel(ERHIFeatureLevel::Type FeatureLevel, bool bRaytracing)
{
	const FString ProfileName = FVulkanPlatform::GetVulkanProfileNameForFeatureLevel(FeatureLevel, bRaytracing);
	VpProfileProperties ProfileProperties;
	FMemory::Memzero(ProfileProperties);
	FCStringAnsi::Strncpy(ProfileProperties.profileName, TCHAR_TO_ANSI(*ProfileName), VP_MAX_PROFILE_NAME_SIZE);

	const uint32 minApiVersion = vpGetProfileAPIVersion(&ProfileProperties);
	if (minApiVersion)
	{
		return minApiVersion;
	}

	UE_LOGF(LogVulkanRHI, Log, "Using default apiVersion for platform...");
	return UE_VK_API_VERSION;
}

// Returns the API version for the provided feautre level, returns 0 if not supported
static bool CheckVulkanProfile(ERHIFeatureLevel::Type FeatureLevel, bool bRaytracing)
{
	const FString ProfileName = FVulkanPlatform::GetVulkanProfileNameForFeatureLevel(FeatureLevel, bRaytracing);

	if (!FVulkanGenericPlatform::SupportsProfileChecks())
	{
		UE_LOGF(LogVulkanRHI, Log, "Skipping Vulkan Profile check for %ls:", *ProfileName);
		return true;
	}

	UE_LOGF(LogVulkanRHI, Log, "Starting Vulkan Profile check for %ls:", *ProfileName);
	ON_SCOPE_EXIT
	{
		UE_LOGF(LogVulkanRHI, Log, "Vulkan Profile check complete.");
	};

	VpProfileProperties ProfileProperties;
	FMemory::Memzero(ProfileProperties);
	FCStringAnsi::Strncpy(ProfileProperties.profileName, TCHAR_TO_ANSI(*ProfileName), VP_MAX_PROFILE_NAME_SIZE);

	VkBool32 bInstanceSupported = VK_FALSE;
	VkResult InstanceResult = vpGetInstanceProfileSupport(nullptr, &ProfileProperties, &bInstanceSupported);  // :todo-jn: no VERIFYVULKANRESULT, this can fail and it's fine
	if ((InstanceResult == VK_SUCCESS) && bInstanceSupported)
	{
		VkInstanceCreateInfo InstanceCreateInfo;
		ZeroVulkanStruct(InstanceCreateInfo, VK_STRUCTURE_TYPE_INSTANCE_CREATE_INFO);

		VpInstanceCreateInfo ProfileInstanceCreateInfo;
		FMemory::Memzero(ProfileInstanceCreateInfo);
		ProfileInstanceCreateInfo.enabledFullProfileCount = 1;
		ProfileInstanceCreateInfo.pEnabledFullProfiles = &ProfileProperties;
		ProfileInstanceCreateInfo.pCreateInfo = &InstanceCreateInfo;

		VkInstance TempInstance = VK_NULL_HANDLE;
		VERIFYVULKANRESULT(vpCreateInstance(&ProfileInstanceCreateInfo, VULKAN_CPU_ALLOCATOR, &TempInstance));

		// Use FVulkanGenericPlatform on purpose here, we only want basic common functionality (no platform specific stuff)
		FVulkanGenericPlatform::LoadVulkanInstanceFunctions(TempInstance);

		ON_SCOPE_EXIT
		{
			// Keep nothing around from the temporary instance we created
			if (TempInstance)
			{
				VulkanRHI::vkDestroyInstance(TempInstance, VULKAN_CPU_ALLOCATOR);
				TempInstance = VK_NULL_HANDLE;
				FVulkanPlatform::ClearVulkanInstanceFunctions();
			}
		};

		// Pick the device we would use on this instance
		const VkPhysicalDevice PhysicalDevice = SelectPhysicalDevice(TempInstance);
		if (PhysicalDevice)
		{
			VkBool32 bDeviceSupported = VK_FALSE;
			VERIFYVULKANRESULT(vpGetPhysicalDeviceProfileSupport(TempInstance, PhysicalDevice, &ProfileProperties, &bDeviceSupported));
			if (bDeviceSupported)
			{
				return true;
			}
			else if (vpGetProfileAPIVersion(&ProfileProperties) >= VK_API_VERSION_1_2)
			{
				// If we're in Vulkan 1.2+, print more information about the device being tested when it fails
				VkPhysicalDeviceVulkan12Properties Vulkan12Properties;
				ZeroVulkanStruct(Vulkan12Properties, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_VULKAN_1_2_PROPERTIES);
				VkPhysicalDeviceProperties2 PhysicalDeviceProperties2;
				ZeroVulkanStruct(PhysicalDeviceProperties2, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_PROPERTIES_2);
				PhysicalDeviceProperties2.pNext = &Vulkan12Properties;
				VulkanRHI::vkGetPhysicalDeviceProperties2(PhysicalDevice, &PhysicalDeviceProperties2);
				UE_LOGF(LogVulkanRHI, Log, "   - Selected PhysicalDevice: [%s | %s | %s]",
					PhysicalDeviceProperties2.properties.deviceName, Vulkan12Properties.driverName, Vulkan12Properties.driverInfo);
			}
		}
	}

	return false;
}

void FVulkanDynamicRHIModule::StartupModule()
{
#if VULKAN_USE_LLM
	LLM(VulkanLLM::Initialize());
#endif
}

bool FVulkanDynamicRHIModule::IsSupported()
{
	if (FVulkanPlatform::IsSupported())
	{
		return FVulkanPlatform::LoadVulkanLibrary();
	}
	return false;
}

bool FVulkanDynamicRHIModule::IsSupported(ERHIFeatureLevel::Type FeatureLevel)
{
	if (IsSupported())
	{
		if (FeatureLevel == ERHIFeatureLevel::ES3_1)
		{
			return !GIsEditor;
		}
		else if (!FVulkanPlatform::SupportsProfileChecks())
		{
			return true;
		}
		else
		{
			return CheckVulkanProfile(FeatureLevel, false);
		}
	}

	return false;
}

FDynamicRHI* FVulkanDynamicRHIModule::CreateRHI(ERHIFeatureLevel::Type InRequestedFeatureLevel)
{
	GMaxRHIFeatureLevel = FVulkanPlatform::GetFeatureLevel(InRequestedFeatureLevel);
	checkf(GMaxRHIFeatureLevel != ERHIFeatureLevel::Num, TEXT("Invalid feature level requested!"));

	EShaderPlatform ShaderPlatformForFeatureLevel[ERHIFeatureLevel::Num];
	FVulkanPlatform::SetupFeatureLevels(ShaderPlatformForFeatureLevel);
	GMaxRHIShaderPlatform = ShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel];
	checkf(GMaxRHIShaderPlatform != SP_NumPlatforms, TEXT("Requested feature level [%s] mapped to unsupported shader platform!"), *LexToString(InRequestedFeatureLevel));

	UE_LOGF(LogVulkanRHI, Display, "Vulkan RHI ShaderPlatform for %ls: %ls.", *LexToString(InRequestedFeatureLevel), *LexToString(GMaxRHIShaderPlatform, false));

	GVulkanRHI = new FVulkanDynamicRHI();
	FDynamicRHI* FinalRHI = GVulkanRHI;

#if ENABLE_RHI_VALIDATION
	if (FParse::Param(FCommandLine::Get(), TEXT("RHIValidation")))
	{
		FinalRHI = new FValidationRHI(FinalRHI);
	}
#endif

	for (int32 Index = 0; Index < ERHIFeatureLevel::Num; ++Index)
	{
		if (ShaderPlatformForFeatureLevel[Index] != SP_NumPlatforms)
		{
			check(GMaxTextureSamplers >= (int32)FDataDrivenShaderPlatformInfo::GetMaxSamplers(ShaderPlatformForFeatureLevel[Index]));
			if (GMaxTextureSamplers < (int32)FDataDrivenShaderPlatformInfo::GetMaxSamplers(ShaderPlatformForFeatureLevel[Index]))
			{
				UE_LOGF(LogVulkanRHI, Error, "Shader platform requires at least: %d samplers, device supports: %d.", FDataDrivenShaderPlatformInfo::GetMaxSamplers(ShaderPlatformForFeatureLevel[Index]), GMaxTextureSamplers);
			}
		}
	}

	return FinalRHI;
}

IMPLEMENT_MODULE(FVulkanDynamicRHIModule, VulkanRHI);

FVulkanDynamicRHI::FVulkanDynamicRHI()
	: Instance(VK_NULL_HANDLE)
	, Device(nullptr)
{
	// This should be called once at the start 
	check(IsInGameThread());
	check(!GIsThreadedRendering);

	GPoolSizeVRAMPercentage = 0;
	GTexturePoolSize = 0;
	GRHISupportsMultithreading = true;
	GRHISupportsMultithreadedResources = true;
	GRHITransitionPrivateData_SizeInBytes = sizeof(FVulkanTransitionData);
	GRHITransitionPrivateData_AlignInBytes = alignof(FVulkanTransitionData);
	GConfig->GetInt(TEXT("TextureStreaming"), TEXT("PoolSizeVRAMPercentage"), GPoolSizeVRAMPercentage, GEngineIni);

	GRHIGlobals.SupportsBarycentricsSemantic = true;

	GRHISupportsPSOPrecaching = CVarAllowVulkanPSOPrecache.GetValueOnAnyThread();
	GRHISupportsPipelineFileCache = !GRHISupportsPSOPrecaching || CVarEnableVulkanPSOFileCacheWhenPrecachingActive.GetValueOnAnyThread();
	UE_LOGF(LogVulkanRHI, Log, "Vulkan PSO Precaching = %d, PipelineFileCache = %d", GRHISupportsPSOPrecaching, GRHISupportsPipelineFileCache);

	// Vulkan always uses SPIR-V. This is used to differentiate standard and preview shaders in the DDC.
	GRHIGlobals.PreferredPreviewShaderCodeFormat = TEXT("SPIRV");

	// Copy source requires its own image layout.
	EnumRemoveFlags(GRHIMergeableAccessMask, ERHIAccess::CopySrc);

	// Setup the validation requests ready before we load dlls
	SetupValidationRequests();

	UE_LOGF(LogVulkanRHI, Display, "Built with Vulkan header version %u.%u.%u", VK_API_VERSION_MAJOR(VK_HEADER_VERSION_COMPLETE), VK_API_VERSION_MINOR(VK_HEADER_VERSION_COMPLETE), VK_API_VERSION_PATCH(VK_HEADER_VERSION_COMPLETE));

	CreateInstance();
	SelectDevice();
}

FVulkanDynamicRHI::~FVulkanDynamicRHI() = default;

void FVulkanDynamicRHI::Init()
{
	InitInstance();

	bIsStandaloneStereoDevice = IHeadMountedDisplayModule::IsAvailable() && IHeadMountedDisplayModule::Get().IsStandaloneStereoOnlyDevice();

	static const auto CVarStreamingTexturePoolSize = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Streaming.PoolSize"));
	int32 StreamingPoolSizeValue = CVarStreamingTexturePoolSize->GetValueOnAnyThread();
			
	if (GPoolSizeVRAMPercentage > 0)
	{
		const uint64 TotalGPUMemory = Device->GetDeviceMemoryManager().GetTotalMemory(true);

		float PoolSize = float(GPoolSizeVRAMPercentage) * 0.01f * float(TotalGPUMemory);

		// Truncate GTexturePoolSize to MB (but still counted in bytes)
		GTexturePoolSize = int64(FGenericPlatformMath::TruncToFloat(PoolSize / 1024.0f / 1024.0f)) * 1024 * 1024;

		UE_LOGF(LogRHI, Log, "Texture pool is %llu MB (%d%% of %llu MB)",
			GTexturePoolSize / 1024 / 1024,
			GPoolSizeVRAMPercentage,
			TotalGPUMemory / 1024 / 1024);
	}
	else if (StreamingPoolSizeValue > 0)
	{
		GTexturePoolSize = (int64)StreamingPoolSizeValue * 1024 * 1024;

		const uint64 TotalGPUMemory = Device->GetDeviceMemoryManager().GetTotalMemory(true);
		UE_LOGF(LogRHI, Log, "Texture pool is %llu MB (of %llu MB total graphics mem)",
				GTexturePoolSize / 1024 / 1024,
				TotalGPUMemory / 1024 / 1024);
	}
}

void FVulkanDynamicRHI::PostInit()
{
	if (GRHISupportsRayTracing)
	{
		Device->InitializeRayTracing();
	}
}

void FVulkanDynamicRHI::Shutdown()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("savevulkanpsocacheonexit")))
	{
		SavePipelineCache();
	}

	check(IsInGameThread() && IsInRenderingThread());
	check(Device);

	Device->PrepareForDestroy();

	EmptyCachedBoundShaderStates();

	FVulkanVertexDeclaration::EmptyCache();

	if (GIsRHIInitialized)
	{
		// Reset the RHI initialized flag.
		GIsRHIInitialized = false;

		FVulkanPlatform::OverridePlatformHandlers(false);

		GRHINeedsExtraDeletionLatency = false;

		check(!GIsCriticalError);

		// Ask all initialized FRenderResources to release their RHI resources.
		FRenderResource::ReleaseRHIForAllResources();

		for (const TPair<uint32, TRefCountPtr<FVulkanSamplerState>>& Pair : Device->SamplerMap)
		{
			FVulkanSamplerState* SamplerState = Pair.Value.GetReference();
			VulkanRHI::vkDestroySampler(Device->GetHandle(), SamplerState->Sampler, VULKAN_CPU_ALLOCATOR);

			if (SamplerState->BindlessHandle.IsValid())
			{
				Device->GetBindlessDescriptorManager()->FreeDescriptor(SamplerState->BindlessHandle);
			}
		}
		Device->SamplerMap.Empty();

		Device->CleanUpRayTracing();

		// Flush all pending deletes before destroying the device.
		FRHICommandListImmediate::Get().ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

		ShutdownSubmissionPipe();
	}

	Device->Destroy();

	delete Device;
	Device = nullptr;

	// Release the early HMD interface used to query extra extensions - if any was used
	HMDVulkanExtensions = nullptr;

#if VULKAN_HAS_DEBUGGING_ENABLED
	RemoveDebugCallbacks();
#endif

	VulkanRHI::vkDestroyInstance(Instance, VULKAN_CPU_ALLOCATOR);

	IConsoleManager::Get().UnregisterConsoleObject(SavePipelineCacheCmd);
	IConsoleManager::Get().UnregisterConsoleObject(RebuildPipelineCacheCmd);

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
	IConsoleManager::Get().UnregisterConsoleObject(DumpMemoryCmd);
	IConsoleManager::Get().UnregisterConsoleObject(DumpMemoryFullCmd);
	IConsoleManager::Get().UnregisterConsoleObject(DumpStagingMemoryCmd);
	IConsoleManager::Get().UnregisterConsoleObject(DumpLRUCmd);
	IConsoleManager::Get().UnregisterConsoleObject(TrimLRUCmd);
#endif

	FVulkanPlatform::FreeVulkanLibrary();

#if VULKAN_ENABLE_DUMP_LAYER
	VulkanRHI::FlushDebugWrapperLog();
#endif
}

void FVulkanDynamicRHI::CreateInstance()
{
	// Engine registration can be disabled via console var. Also disable automatically if ShaderDevelopmentMode is on.
	auto* CVarDisableEngineAndAppRegistration = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DisableEngineAndAppRegistration"));
	bool bDisableEngineRegistration = (CVarDisableEngineAndAppRegistration && CVarDisableEngineAndAppRegistration->GetValueOnAnyThread() != 0) ||
		IsShaderDevelopmentModeEnabled();

	// Use the API version stored in the profile
	ApiVersion = GetVulkanApiVersionForFeatureLevel(GMaxRHIFeatureLevel, false);

	// Run a profile check to see if this device can support our raytacing requirements since it might change the required API version of the instance
	if (FVulkanPlatform::SupportsProfileChecks() && GVulkanRayTracingCVar.GetValueOnAnyThread())
	{
		const bool bRayTracingAllowedOnCurrentShaderPlatform = (GMaxRHIShaderPlatform == SP_VULKAN_SM6 || IsVulkanMobileSM5Platform(GMaxRHIShaderPlatform));

		if (CheckVulkanProfile(GMaxRHIFeatureLevel, true) && bRayTracingAllowedOnCurrentShaderPlatform)
		{
			// Raytracing is supported, update the required API version
			ApiVersion = GetVulkanApiVersionForFeatureLevel(GMaxRHIFeatureLevel, true);
		}
		else
		{
			// Raytracing is not supported, disable it completely instead of only loading parts of it
			GVulkanRayTracingCVar->Set(0, ECVF_SetByCode);

			if (!bRayTracingAllowedOnCurrentShaderPlatform)
			{
				UE_LOGF(LogVulkanRHI, Display, "Vulkan RayTracing disabled because SM6 shader platform is required.");
			}
			else
			{
				UE_LOGF(LogVulkanRHI, Display, "Vulkan RayTracing disabled because of failed profile check.");
			}
		}
	}

	UE_LOGF(LogVulkanRHI, Log, "Using API Version %u.%u.", VK_API_VERSION_MAJOR(ApiVersion), VK_API_VERSION_MINOR(ApiVersion));

	// EngineName will be of the form "UnrealEngine4.21", with the minor version ("21" in this example)
	// updated with every quarterly release
	FString EngineName = FApp::GetEpicProductIdentifier() + FEngineVersion::Current().ToString(EVersionComponent::Minor);
	FTCHARToUTF8 EngineNameConverter(*EngineName);
	FTCHARToUTF8 ProjectNameConverter(FApp::GetProjectName());

	VkApplicationInfo AppInfo;
	ZeroVulkanStruct(AppInfo, VK_STRUCTURE_TYPE_APPLICATION_INFO);
	AppInfo.pApplicationName = bDisableEngineRegistration ? nullptr : ProjectNameConverter.Get();
	AppInfo.applicationVersion = static_cast<uint32>(BuildSettings::GetCurrentChangelist()) | (BuildSettings::IsLicenseeVersion() ? 0x80000000 : 0);
	AppInfo.pEngineName = bDisableEngineRegistration ? nullptr : EngineNameConverter.Get();
	AppInfo.engineVersion = FEngineVersion::Current().GetMinor();
	AppInfo.apiVersion = ApiVersion;

	VkInstanceCreateInfo InstInfo;
	ZeroVulkanStruct(InstInfo, VK_STRUCTURE_TYPE_INSTANCE_CREATE_INFO);
	InstInfo.pApplicationInfo = &AppInfo;

	FVulkanInstanceExtensionArray UEInstanceExtensions = FVulkanInstanceExtension::GetUESupportedInstanceExtensions(ApiVersion);
	InstanceLayers = SetupInstanceLayers(UEInstanceExtensions);
	for (TUniquePtr<FVulkanInstanceExtension>& Extension : UEInstanceExtensions)
	{
		if (Extension->InUse())
		{
			InstanceExtensions.Add(Extension->GetExtensionName());
			Extension->PreCreateInstance(InstInfo, OptionalInstanceExtensions);
		}
	}

	InstInfo.enabledExtensionCount = InstanceExtensions.Num();
	InstInfo.ppEnabledExtensionNames = InstInfo.enabledExtensionCount > 0 ? (const ANSICHAR* const*)InstanceExtensions.GetData() : nullptr;
	
	InstInfo.enabledLayerCount = InstanceLayers.Num();
	InstInfo.ppEnabledLayerNames = InstInfo.enabledLayerCount > 0 ? InstanceLayers.GetData() : nullptr;

	VkResult Result = VulkanRHI::vkCreateInstance(&InstInfo, VULKAN_CPU_ALLOCATOR, &Instance);
	
	if (Result == VK_ERROR_LAYER_NOT_PRESENT)
	{
		UE_LOGF(LogVulkanRHI, Warning, "Vulkan instance creation returned an error with the requested layers (%d):", InstanceLayers.Num());

		for (const ANSICHAR* AnsiLayerName : InstanceLayers)
		{
			UE_LOGF(LogVulkanRHI, Warning, "- %s", AnsiLayerName);
		}

		const EAppReturnType::Type MsgBoxResult = FPlatformMisc::MessageBoxExt(EAppMsgType::YesNo, TEXT(
			"ERROR: Vulkan driver couldn't load one of the requested layers (see log for details).\n\n"
			"Retry without layers?"),
			TEXT("Incompatible Vulkan layer found!"));

		if (MsgBoxResult == EAppReturnType::Yes)
		{
			InstInfo.enabledLayerCount = 0;
			Result = VulkanRHI::vkCreateInstance(&InstInfo, VULKAN_CPU_ALLOCATOR, &Instance);
		}
		else
		{
			FPlatformMisc::RequestExitWithStatus(true, 1);
			// unreachable
			return;
		}
	}

	FVulkanPlatform::NotifyFoundInstanceLayersAndExtensions(InstanceLayers, InstanceExtensions);

	if (Result == VK_ERROR_INCOMPATIBLE_DRIVER)
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT(
			"Cannot find a compatible Vulkan driver (ICD).\n\nPlease look at the Getting Started guide for "
			"additional information."), TEXT("Incompatible Vulkan driver found!"));
		FPlatformMisc::RequestExitWithStatus(true, 1);
		// unreachable
		return;
	}
	else if(Result == VK_ERROR_EXTENSION_NOT_PRESENT)
	{
		// Check for missing extensions 
		FString MissingExtensions;

		uint32_t PropertyCount;
		VulkanRHI::vkEnumerateInstanceExtensionProperties(nullptr, &PropertyCount, nullptr);

		TArray<VkExtensionProperties> Properties;
		Properties.SetNum(PropertyCount);
		VulkanRHI::vkEnumerateInstanceExtensionProperties(nullptr, &PropertyCount, Properties.GetData());

		for (const ANSICHAR* Extension : InstanceExtensions)
		{
			bool bExtensionFound = false;

			for (uint32_t PropertyIndex = 0; PropertyIndex < PropertyCount; PropertyIndex++)
			{
				const char* PropertyExtensionName = Properties[PropertyIndex].extensionName;

				if (!FCStringAnsi::Strcmp(PropertyExtensionName, Extension))
				{
					bExtensionFound = true;
					break;
				}
			}

			if (!bExtensionFound)
			{
				UE_LOGF(LogVulkanRHI, Error, "Missing required Vulkan extension: %s", Extension);
				MissingExtensions += FString(Extension) + TEXT("\n");
			}
		}

		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *FString::Printf(TEXT(
			"Vulkan driver doesn't contain specified extensions:\n%s;\n\
			make sure your layers path is set appropriately."), *MissingExtensions), TEXT("Incomplete Vulkan driver found!"));
	}
	else if (Result != VK_SUCCESS)
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT(
			"Vulkan failed to create instance.\n\nDo you have a compatible Vulkan "
			 "driver (ICD) installed?\nPlease look at "
			 "the Getting Started guide for additional information."), TEXT("No Vulkan driver found!"));
		FPlatformMisc::RequestExitWithStatus(true, 1);
		// unreachable
		return;
	}

	VERIFYVULKANRESULT(Result);

	if (!FVulkanPlatform::LoadVulkanInstanceFunctions(Instance))
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT(
			"Failed to find all required Vulkan entry points! Try updating your driver."), TEXT("No Vulkan entry points found!"));
	}

#if VULKAN_HAS_DEBUGGING_ENABLED
	SetupDebugCallbacks();
#endif
}

void FVulkanDynamicRHI::SelectDevice()
{
	VkPhysicalDevice PhysicalDevice = SelectPhysicalDevice(Instance);
	if (PhysicalDevice == VK_NULL_HANDLE)
	{
		// Shouldn't be possible if profile checks passed prior to this
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, 
			TEXT("Vulkan failed to select physical device after passing profile checks."), TEXT("No Vulkan driver found!"));
		FPlatformMisc::RequestExitWithStatus(true, 1);
		return;
	}

	UE_LOGF(LogVulkanRHI, Log, "Creating Vulkan Device using VkPhysicalDevice 0x%p.", reinterpret_cast<const void*>(PhysicalDevice));
	Device = new FVulkanDevice(this, PhysicalDevice);

	const VkPhysicalDeviceProperties& Props = Device->GetDeviceProperties();
	bool bUseVendorIdAsIs = true;
	if (Props.vendorID > 0xffff)
	{
		bUseVendorIdAsIs = false;
		VkVendorId VendorId = (VkVendorId)Props.vendorID;
		switch (VendorId)
		{
		case VK_VENDOR_ID_VIV:		GRHIVendorId = (uint32)EGpuVendorId::Vivante; break;
		case VK_VENDOR_ID_VSI:		GRHIVendorId = (uint32)EGpuVendorId::VeriSilicon; break;
		case VK_VENDOR_ID_KAZAN:	GRHIVendorId = (uint32)EGpuVendorId::Kazan; break;
		case VK_VENDOR_ID_CODEPLAY:	GRHIVendorId = (uint32)EGpuVendorId::Codeplay; break;
		case VK_VENDOR_ID_MESA:		GRHIVendorId = (uint32)EGpuVendorId::Mesa; break;
		default:
			// Unhandled case
			UE_LOGF(LogVulkanRHI, Warning, "Unhandled VkVendorId %d", (int32)VendorId);
			bUseVendorIdAsIs = true;
			break;
		}
	}

	if (bUseVendorIdAsIs)
	{
		GRHIVendorId = Props.vendorID;
	}
	GRHIAdapterName = ANSI_TO_TCHAR(Props.deviceName);

	if (PLATFORM_ANDROID)
	{
		GRHIAdapterName.Append(TEXT(" Vulkan"));
		// On Android GL version string often contains extra information such as an actual driver version on the device.
#if PLATFORM_ANDROID
		FString GLVersion = FAndroidMisc::GetGLVersion();
#else
		FString GLVersion;
#endif
		GRHIAdapterInternalDriverVersion = FString::Printf(TEXT("%d.%d.%d|%s"), VK_VERSION_MAJOR(Props.apiVersion), VK_VERSION_MINOR(Props.apiVersion), VK_VERSION_PATCH(Props.apiVersion), *GLVersion);
		UE_LOGF(LogVulkanRHI, Log, "API Version: %ls", *GRHIAdapterInternalDriverVersion);
	}
	else if (PLATFORM_WINDOWS)
	{
		GRHIDeviceId = Props.deviceID;
		UE_LOGF(LogVulkanRHI, Log, "API Version: %d.%d.%d", VK_VERSION_MAJOR(Props.apiVersion), VK_VERSION_MINOR(Props.apiVersion), VK_VERSION_PATCH(Props.apiVersion));
	}
	else if(PLATFORM_UNIX)
	{
		if (Device->GetVendorId() == EGpuVendorId::Nvidia)
		{
			UNvidiaDriverVersion NvidiaVersion;
			static_assert(sizeof(NvidiaVersion) == sizeof(Props.driverVersion), "Mismatched Nvidia pack driver version!");
			NvidiaVersion.Packed = Props.driverVersion;
			GRHIAdapterUserDriverVersion = FString::Printf(TEXT("%d.%02d"), NvidiaVersion.Major, NvidiaVersion.Minor);
		}
		else
		{
			GRHIAdapterUserDriverVersion = FString::Printf(TEXT("%d.%d.%d"), VK_VERSION_MAJOR(Props.driverVersion), VK_VERSION_MINOR(Props.driverVersion), VK_VERSION_PATCH(Props.driverVersion));
		}

		GRHIDeviceId = Props.deviceID;
		GRHIAdapterInternalDriverVersion = GRHIAdapterUserDriverVersion;
		GRHIAdapterDriverDate = TEXT("01-01-01");  // Unused on unix systems, pick a date that will fail test if compared but passes IsValid() check
		UE_LOGF(LogVulkanRHI, Log, "     API Version: %d.%d.%d", VK_VERSION_MAJOR(Props.apiVersion), VK_VERSION_MINOR(Props.apiVersion), VK_VERSION_PATCH(Props.apiVersion));
	}

	GRHIPersistentThreadGroupCount = 1440; // TODO: Revisit based on vendor/adapter/perf query

	GRHIGlobals.SupportsTimestampRenderQueries = FVulkanPlatform::SupportsTimestampRenderQueries() && (Device->GetLimits().timestampPeriod > 0.0f);
}

void FVulkanDynamicRHI::InitInstance()
{
	check(IsInGameThread());

	if (!GIsRHIInitialized)
	{
		// Wait for the rendering thread to go idle.
		FlushRenderingCommands();

		FVulkanPlatform::OverridePlatformHandlers(true);

		GRHISupportsAsyncTextureCreation = false;

		Device->InitGPU();

#if VULKAN_HAS_DEBUGGING_ENABLED
		if (GRenderDocFound)
		{
			EnableIdealGPUCaptureOptions(true);
		}
#endif

		const VkPhysicalDeviceProperties& Props = Device->GetDeviceProperties();
		const VkPhysicalDeviceLimits& Limits = Device->GetLimits();
		const FVulkanPhysicalDeviceFeatures& Features = Device->GetPhysicalDeviceFeatures();

		// Initialize the RHI capabilities.
		GRHIGlobals.SupportsDedicatedSRVGraphicsNonPixelAccess = GCVarVulkanSupportsDedicatedSRVGraphicsNonPixelAccess.GetValueOnAnyThread();
		GRHINeedsSRVGraphicsNonPixelWorkaround = !GRHIGlobals.SupportsDedicatedSRVGraphicsNonPixelAccess;
		GRHISupportsFirstInstance = true;
		GRHISupportsDynamicResolution = FVulkanPlatform::SupportsDynamicResolution();
		GRHISupportsFrameCyclesBubblesRemoval = true;
		GSupportsDepthBoundsTest = Features.Core_1_0.depthBounds != 0;
		GSupportsRenderTargetFormat_PF_G8 = false;	// #todo-rco
		GRHISupportsTextureStreaming = true;
		GRHISupportsGPUTimestampBubblesRemoval = true;
		GSupportsMobileMultiView = Device->GetOptionalExtensions().HasKHRMultiview ? true : false;
		GRHISupportsMSAAShaderResolve = Device->GetOptionalExtensions().HasQcomRenderPassShaderResolve ? true : false;
		GRHISupportsRayTracing = RHI_RAYTRACING && RHISupportsRayTracing(GMaxRHIShaderPlatform) && Device->GetOptionalExtensions().HasRaytracingExtensions();
		GRHIGlobals.SupportsMapWriteNoOverwrite = true;

		GRHIGlobals.NeedsExtraTransitions = true;

		if (GRHISupportsRayTracing)
		{
			GRHISupportsRayTracingShaders = RHISupportsRayTracingShaders(GMaxRHIShaderPlatform) && Device->GetOptionalExtensions().HasRayTracingPipeline && UE::RHICore::AreRayTracingShadersEnabled();
			GRHISupportsInlineRayTracing = RHISupportsInlineRayTracing(GMaxRHIShaderPlatform) && Device->GetOptionalExtensions().HasRayQuery;

			// Inline RayTracing SBT is needed if raytracing position fetch isn't available
			GRHIGlobals.RayTracing.RequiresInlineRayTracingSBT = !VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH;

			static auto CVarRayTracingAllowCompaction = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Vulkan.RayTracing.AllowCompaction"));
			GRHIGlobals.RayTracing.SupportsAccelerationStructureCompaction = CVarRayTracingAllowCompaction->GetInt() != 0;

			GRHIRayTracingAccelerationStructureAlignment = 256; // TODO (currently handled by FVulkanAccelerationStructureBuffer)
			//Some devices have 64 for min AS offset alignment meanwhile engine AS alignment is 256. hence using round up value
			GRHIRayTracingScratchBufferAlignment = FPlatformMath::Max<uint32>(GRHIRayTracingAccelerationStructureAlignment, 
													Device->GetOptionalExtensionProperties().AccelerationStructureProps.minAccelerationStructureScratchOffsetAlignment);

			GRHIRayTracingInstanceDescriptorSize = uint32(sizeof(VkAccelerationStructureInstanceKHR));

			// Loose parameters are always placed in the shader record after the FVulkanHitGroupSystemParameters in Vulkan (see VulkanRayTracing.h and VulkanCommon.ush)
			GRHIGlobals.RayTracing.SupportsLooseParamsInShaderRecord = true;
		}

		// Use this compatibility mode avoid known issues at launch time with latest drivers at the time of release 5.8.
		// This will disable ray tracing with proprietary driver and older MESA versions
		const bool bUseAMDCompatibilityMode = GVulkanAMDCompatibilityMode && (Device->GetVendorId() == EGpuVendorId::Amd);
		if (GVulkanAMDCompatibilityMode)
		{
			// Disable ray tracing if driver doesn't support acceleration structures in the mutable descriptor set
			if (GRHISupportsRayTracing && !Device->GetBindlessDescriptorManager()->SupportsBindlessAccelerationStructures())
			{
				GRHISupportsRayTracing = false;
				GRHISupportsRayTracingShaders = false;
				GRHISupportsInlineRayTracing = false;
				UE_LOGF(LogVulkanRHI, Warning, "The driver does not support acceleration structures in the mutable descriptor set, ray tracing disabled.");
			}

			if (Device->GetOptionalExtensionProperties().PhysicalDeviceDriverProperties.driverID == VK_DRIVER_ID_MESA_RADV)
			{
				// Some crashes were found in rt pipeline creation in version 25.X.X and some
				// distros max out at versions prior to 26.X.X, let them use the engine without ray tracing.
				if (GRHISupportsRayTracing && (Props.driverVersion < VK_MAKE_VERSION(26, 0, 0)))
				{
					GRHISupportsRayTracing = false;
					GRHISupportsRayTracingShaders = false;
					GRHISupportsInlineRayTracing = false;
					UE_LOGF(LogVulkanRHI, Warning, "Using MESA RADV version prior to 26.X.X, ray tracing disabled.");
				}
			}
			else if ((Device->GetOptionalExtensionProperties().PhysicalDeviceDriverProperties.driverID == VK_DRIVER_ID_AMD_PROPRIETARY) ||
					(Device->GetOptionalExtensionProperties().PhysicalDeviceDriverProperties.driverID == VK_DRIVER_ID_AMD_OPEN_SOURCE))
			{
				// Some GPUs have issues with inline rt, others have issues with rt shaders, disable until we can properly map it.
				GRHISupportsRayTracing = false;
				GRHISupportsRayTracingShaders = false;
				GRHISupportsInlineRayTracing = false;
				static auto CVarPathTracing = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PathTracing"));
				if (CVarPathTracing)
				{
					CVarPathTracing->Set(0);
				}
				
				UE_LOGF(LogVulkanRHI, Display, "Using AMD Propriertary Driver, ray tracing disabled.");
			}
		}

#if VULKAN_ENABLE_DUMP_LAYER
		// Disable RHI thread by default if the dump layer is enabled
		GRHISupportsRHIThread = false;
		GRHISupportsParallelRHIExecute = false;
#else
		GRHISupportsRHIThread = GRHIThreadCvar->GetInt() != 0;
		GRHISupportsParallelRHIExecute = Device->SupportsParallelRendering() && (GRHIThreadCvar->GetInt() > 1);
#endif

		GRHIGlobals.ParallelRenderPasses.Supported                   = GRHISupportsParallelRHIExecute;
		GRHIGlobals.ParallelRenderPasses.ParentWaitForChildTranslate = GRHISupportsParallelRHIExecute;
		GRHIGlobals.ParallelRenderPasses.ChildWaitForParentBegin     = GRHISupportsParallelRHIExecute;
		GRHIGlobals.ParallelRenderPasses.UsesCustomContexts = true;

		GRHISupportsUAVFormatAliasing = true;

		// Some platforms might only have CPU for an RHI thread, but not for parallel tasks
		GSupportsParallelRenderingTasksWithSeparateRHIThread = GRHISupportsRHIThread ? FVulkanPlatform::SupportParallelRenderingTasks() : false;

		//#todo-rco: Add newer Nvidia also
		GSupportsEfficientAsyncCompute = Device->HasAsyncComputeQueue();
		UE_LOGF(LogVulkanRHI, Display, "Vulkan Async Compute has been %s.", GSupportsEfficientAsyncCompute ? "ENABLED" : "DISABLED");

		GSupportsVolumeTextureRendering = FVulkanPlatform::SupportsVolumeTextureRendering();

		// Indicate that the RHI needs to use the engine's deferred deletion queue.
		GRHINeedsExtraDeletionLatency = true;

		GMaxShadowDepthBufferSizeX =  FPlatformMath::Min<int32>(Props.limits.maxImageDimension2D, GMaxShadowDepthBufferSizeX);
		GMaxShadowDepthBufferSizeY =  FPlatformMath::Min<int32>(Props.limits.maxImageDimension2D, GMaxShadowDepthBufferSizeY);
		GMaxTextureDimensions = Props.limits.maxImageDimension2D;
		GRHIGlobals.MaxViewDimensionForTypedBuffer = Props.limits.maxTexelBufferElements;
		GRHIGlobals.MaxViewSizeBytesForNonTypedBuffer = FMath::Min<uint64>(Props.limits.maxStorageBufferRange, GRHIGlobals.MaxViewSizeBytesForNonTypedBuffer);  // Might be set by maintenance4
		GMaxComputeSharedMemory = Props.limits.maxComputeSharedMemorySize;
		GMaxTextureMipCount = FPlatformMath::CeilLogTwo( GMaxTextureDimensions ) + 1;
		GMaxTextureMipCount = FPlatformMath::Min<int32>( MAX_TEXTURE_MIP_COUNT, GMaxTextureMipCount );
		GMaxCubeTextureDimensions = Props.limits.maxImageDimensionCube;
		GMaxVolumeTextureDimensions = Props.limits.maxImageDimension3D;
		GMaxWorkGroupInvocations = Props.limits.maxComputeWorkGroupInvocations;
		GMaxTextureArrayLayers = Props.limits.maxImageArrayLayers;
		GRHISupportsBaseVertexIndex = true;
		GSupportsSeparateRenderTargetBlendState = true;
		GSupportsDualSrcBlending = Features.Core_1_0.dualSrcBlend == VK_TRUE;
		GRHISupportsSeparateDepthStencilCopyAccess = Device->SupportsParallelRendering();
		GRHIGlobals.bSupportsBindless = Device->SupportsBindless() && FDataDrivenShaderPlatformInfo::GetSupportsBindless(GMaxRHIShaderPlatform);
		GMaxTextureSamplers = (int32)FMath::Min<uint32>(MAX_int32, Props.limits.maxPerStageDescriptorSamplers);
		GRHISupportsLossyFramebufferCompression = Device->GetOptionalExtensions().HasEXTImageCompressionControl;
		GRHIMaxDispatchThreadGroupsPerDimension.X = FMath::Min<uint32>(Limits.maxComputeWorkGroupCount[0], 0x7fffffff);
		GRHIMaxDispatchThreadGroupsPerDimension.Y = FMath::Min<uint32>(Limits.maxComputeWorkGroupCount[1], 0x7fffffff);
		GRHIMaxDispatchThreadGroupsPerDimension.Z = FMath::Min<uint32>(Limits.maxComputeWorkGroupCount[2], 0x7fffffff);
		GRHISupportsBindingTexArrayPerSlice = true;

		// Check a few extra things to ensure ease of use (currently met by all vendors)
		const uint32 GraphicsQueueFamilyIndex = Device->GetGraphicsQueue()->GetFamilyIndex();
		const VkQueueFlags GraphicsQueueFlags = Device->GetQueueFamilyProps()[GraphicsQueueFamilyIndex].queueFlags;
		const bool bGraphicsQueueSupportsSparseBinding = VKHasAllFlags(GraphicsQueueFlags, VK_QUEUE_SPARSE_BINDING_BIT) &&
			Device->SupportsParallelRendering(); // resource commits are only handled in the sync2 path
		if (bGraphicsQueueSupportsSparseBinding)
		{
			GRHIGlobals.ReservedResources.Supported =
				Features.Core_1_0.sparseBinding &&
				Features.Core_1_0.sparseResidencyBuffer &&
				Features.Core_1_0.sparseResidencyImage2D &&
				Props.sparseProperties.residencyNonResidentStrict;

			// :todo-jn: Wait for maintenance9 to be released
			GRHIGlobals.ReservedResources.SupportsVolumeTextures = false;
		}

		// Note: While the 2022/2024 profile limits state a minimum of 16, other profiles (even core) go down to 4
		//       (see https://vulkan.lunarg.com/doc/view/1.3.290.0/windows/profiles_definitions.html)
		// Since the RHI has historically always supported 8 UAV's, let's leave those specific devices out
		uint32 DeviceMaxStorageDescriptorPerStage = FMath::Min(Props.limits.maxPerStageDescriptorStorageBuffers, Props.limits.maxPerStageDescriptorStorageImages);
		GRHIGlobals.MaxSimultaneousUAVs = DeviceMaxStorageDescriptorPerStage >= 16 ? 16 : 8;

		FVulkanPlatform::SetupFeatureLevels(GRHIGlobals.ShaderPlatformForFeatureLevel);

		GRHIRequiresRenderTargetForPixelShaderUAVs = true;

		GUseTexture3DBulkDataRHI = false;

		// these are supported by all devices
		GVulkanDeviceGraphicsPipelineStageBits = VK_PIPELINE_STAGE_VERTEX_SHADER_BIT | VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT;
		VkShaderStageFlags VulkanDeviceShaderStageBits = VK_SHADER_STAGE_VERTEX_BIT | VK_SHADER_STAGE_FRAGMENT_BIT | VK_SHADER_STAGE_COMPUTE_BIT;

		// optional shader stages
		if (Features.Core_1_0.geometryShader)
		{
			GVulkanDeviceGraphicsPipelineStageBits |= VK_PIPELINE_STAGE_GEOMETRY_SHADER_BIT;
			VulkanDeviceShaderStageBits |= VK_SHADER_STAGE_GEOMETRY_BIT;
		}

#if PLATFORM_SUPPORTS_MESH_SHADERS
		// If mesh shaders are enabled in DDPI (currently SM6), then the profile check will ensure it's supported
		if (!bUseAMDCompatibilityMode && Device->GetOptionalExtensions().HasEXTMeshShader)
		{
			GRHIGlobals.SupportsMeshShadersTier0 = RHISupportsMeshShadersTier0(GMaxRHIShaderPlatform);
			GRHIGlobals.SupportsMeshShadersTier1 = RHISupportsMeshShadersTier1(GMaxRHIShaderPlatform);

			GVulkanDeviceGraphicsPipelineStageBits |= VK_PIPELINE_STAGE_TASK_SHADER_BIT_EXT | VK_PIPELINE_STAGE_MESH_SHADER_BIT_EXT;
			VulkanDeviceShaderStageBits |= VK_SHADER_STAGE_TASK_BIT_EXT | VK_SHADER_STAGE_MESH_BIT_EXT;
		}
#endif

		const VkShaderStageFlags RequiredSubgroupShaderStageFlags = FVulkanPlatform::RequiredWaveOpsShaderStageFlags(VulkanDeviceShaderStageBits);
		
		// Check for wave ops support (only filled on platforms creating Vulkan 1.1 or greater instances)
		const VkSubgroupFeatureFlags RequiredSubgroupFlags =	VK_SUBGROUP_FEATURE_BASIC_BIT | VK_SUBGROUP_FEATURE_VOTE_BIT | 
																VK_SUBGROUP_FEATURE_ARITHMETIC_BIT | VK_SUBGROUP_FEATURE_BALLOT_BIT | 
																VK_SUBGROUP_FEATURE_SHUFFLE_BIT;
		GRHISupportsWaveOperations = VKHasAllFlags(Device->GetDeviceSubgroupProperties().supportedStages, RequiredSubgroupShaderStageFlags) &&
			VKHasAllFlags(Device->GetDeviceSubgroupProperties().supportedOperations, RequiredSubgroupFlags);

		if (GRHISupportsWaveOperations)
		{
			// Use default size if VK_EXT_subgroup_size_control didn't fill them
			if (!GRHIMinimumWaveSize || !GRHIMaximumWaveSize)
			{
				GRHIMinimumWaveSize = GRHIMaximumWaveSize = Device->GetDeviceSubgroupProperties().subgroupSize;
			}

			UE_LOGF(LogVulkanRHI, Display, "Wave Operations have been ENABLED (wave size: min=%d max=%d).", GRHIMinimumWaveSize, GRHIMaximumWaveSize);
		}
		else
		{
			const uint32 MissingStageFlags = (Device->GetDeviceSubgroupProperties().supportedStages & RequiredSubgroupShaderStageFlags) ^ RequiredSubgroupShaderStageFlags;
			const uint32 MissingOperationFlags = (Device->GetDeviceSubgroupProperties().supportedOperations & RequiredSubgroupFlags) ^ RequiredSubgroupFlags;
			UE_LOGF(LogVulkanRHI, Display, "Wave Operations have been DISABLED (missing stages=0x%x operations=0x%x).", MissingStageFlags, MissingOperationFlags);
		}

		FHardwareInfo::RegisterHardwareInfo(NAME_RHI, TEXT("Vulkan"));

		SavePipelineCacheCmd = IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("r.Vulkan.SavePipelineCache"),
			TEXT("Save pipeline cache."),
			FConsoleCommandDelegate::CreateStatic(SavePipelineCache),
			ECVF_Default
			);

		RebuildPipelineCacheCmd = IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("r.Vulkan.RebuildPipelineCache"),
			TEXT("Rebuilds pipeline cache."),
			FConsoleCommandDelegate::CreateStatic(RebuildPipelineCache),
			ECVF_Default
			);

#if VULKAN_SUPPORTS_VALIDATION_CACHE
#if VULKAN_HAS_DEBUGGING_ENABLED
		if (GValidationCvar.GetValueOnAnyThread() > 0)
		{
			SaveValidationCacheCmd = IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("r.Vulkan.SaveValidationCache"),
				TEXT("Save validation cache."),
				FConsoleCommandDelegate::CreateStatic(SaveValidationCache),
				ECVF_Default
				);
		}
#endif
#endif

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
		DumpMemoryCmd = IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("r.Vulkan.DumpMemory"),
			TEXT("Dumps memory map."),
			FConsoleCommandDelegate::CreateStatic(DumpMemory),
			ECVF_Default
			);
		DumpMemoryFullCmd = IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("r.Vulkan.DumpMemoryFull"),
			TEXT("Dumps full memory map."),
			FConsoleCommandDelegate::CreateStatic(DumpMemoryFull),
			ECVF_Default
		);
		DumpStagingMemoryCmd = IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("r.Vulkan.DumpStagingMemory"),
			TEXT("Dumps staging memory map."),
			FConsoleCommandDelegate::CreateStatic(DumpStagingMemory),
			ECVF_Default
		);

		DumpLRUCmd = IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("r.Vulkan.DumpPSOLRU"),
			TEXT("Dumps Vulkan PSO LRU."),
			FConsoleCommandDelegate::CreateStatic(DumpLRU),
			ECVF_Default
		);
		TrimLRUCmd = IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("r.Vulkan.TrimPSOLRU"),
			TEXT("Trim Vulkan PSO LRU."),
			FConsoleCommandDelegate::CreateStatic(TrimLRU),
			ECVF_Default
		);

#endif

#if PLATFORM_WINDOWS || PLATFORM_UNIX
		GRHIDeviceIsIntegrated = (Device->GetDeviceProperties().deviceType == VK_PHYSICAL_DEVICE_TYPE_INTEGRATED_GPU);
		UE_LOGF(LogVulkanRHI, Log, "Integrated GPU (iGPU): %s", GRHIDeviceIsIntegrated ? "true" : "false");
#endif

		InitializeSubmissionPipe();

		FRenderResource::InitPreRHIResources();
		GIsRHIInitialized = true;
	}
}

void FVulkanDynamicRHI::RHIEndFrame_RenderThread(FRHICommandListImmediate& RHICmdList)
{
	// @todo dev-pr - refactor RHIEndFrame_RenderThread to reduce use of the immediate command list, and move cleanup work to RHIEndFrame() below.

	RHICmdList.EnqueueLambdaMultiPipe(ERHIPipeline::Graphics, FRHICommandListBase::EThreadFence::Enabled, TEXT("Vulkan EndFrame"),
		[this](FVulkanContextArray const& Contexts)
	{
		FVulkanCommandListContext& Context = *Contexts[ERHIPipeline::Graphics];

		check(Context.IsImmediate());

		bool bTrimMemory = true;
		Context.FreeUnusedCmdBuffers(bTrimMemory);

		Context.Device.GetStagingManager().ProcessPendingFree(false, true);
		Context.Device.GetMemoryManager().ReleaseFreedPages(Contexts);
		Context.Device.GetDeferredDeletionQueue().ReleaseResources();

		if (UseVulkanDescriptorCache())
		{
			Context.Device.GetDescriptorSetCache().GC();
		}
		Context.Device.GetDescriptorPoolsManager().GC();

		Context.Device.RemoveStaleQueryPools();

		Context.Device.GetPipelineStateCache()->TickLRU();

		Context.Device.GetTempBlockAllocator().UpdateBlocks();
	});

	FDynamicRHI::RHIEndFrame_RenderThread(RHICmdList);

	RHICmdList.EnqueueLambdaMultiPipe(ERHIPipeline::Graphics, FRHICommandListBase::EThreadFence::Enabled, TEXT("Vulkan BeginFrame"),
		[this](FVulkanContextArray const& Contexts)
	{
		FVulkanCommandListContext& Context = *Contexts[ERHIPipeline::Graphics];

		check(Context.IsImmediate());

		extern uint32 GVulkanRHIDeletionFrameNumber;
		++GVulkanRHIDeletionFrameNumber;

		if (GRHISupportsRayTracing)
		{
			Context.Device.GetRayTracingCompactionRequestHandler()->Update(Context);
		}
	});
}

void FVulkanDynamicRHI::RHIEndFrame(const FRHIEndFrameArgs& Args)
{
	if (GetDevice())
	{
		GetDevice()->DeviceMemoryManager.UpdateMemoryProperties();
	}

	// Close the previous frame's timing and start a new one
	auto Lambda = [this, OldTiming = MoveTemp(CurrentTimingPerQueue)]()
	{
		TArray<UE::RHI::GPUProfiler::FEventStream, TInlineAllocator<(int32)EVulkanQueueType::Count>> Streams;
		for (auto const& Timing : OldTiming)
		{
			Streams.Add(MoveTemp(Timing->EventStream));
		}
		UE::RHI::GPUProfiler::ProcessEvents(Streams);
	};

	EnqueueEndOfPipeTask(MoveTemp(Lambda), [&](FVulkanPayload& Payload)
	{
		// Modify the payloads the EOP task will submit to include
		// a new timing struct and a frame boundary event.

		Payload.Timing = CurrentTimingPerQueue.CreateNew(Payload.Queue);

		ERHIPipeline Pipeline;
		switch (Payload.Queue.QueueType)
		{
		default: checkNoEntry(); [[fallthrough]];
		case EVulkanQueueType::Graphics:     Pipeline = ERHIPipeline::Graphics; break;
		case EVulkanQueueType::AsyncCompute: Pipeline = ERHIPipeline::AsyncCompute; break;

		case EVulkanQueueType::Transfer:
			// There is currently no high level RHI copy queue support
			Pipeline = ERHIPipeline::None;
			break;
		}

		if (ExternalGPUTime.IsSet())
		{
			Payload.ExternalGPUTime = ExternalGPUTime.GetValue();
			ExternalGPUTime.Reset();
		}

		// CPU timestamp for the frame boundary event is filled in by the submission thread
		Payload.EndFrameEvent = UE::RHI::GPUProfiler::FEvent::FFrameBoundary(Pipeline, Args, 0);
	});

	// Pump the interrupt queue to gather completed events
	// (required if we're not using an interrupt thread).
	ProcessInterruptQueueUntil(nullptr);
}

FVulkanTiming::FVulkanTiming(FVulkanQueue& InQueue)
	: Queue(InQueue)
	, EventStream(InQueue.GetProfilerQueue())
{
}

void FVulkanDynamicRHI::RHIEndDrawingViewport(FRHICommandListImmediate& RHICmdList, FRHIViewport* ViewportRHI, const FRHIPresentArgs& PresentArgs)
{
	//
	// Make sure all prior graphics and async compute work has been submitted. This is necessary because Vulkan RHI handles the present from inside the below lambda.
	// Since the present is not yet pipelined via payloads etc, we must ensure prior work has been handed to RHISubmitCommandLists before we translate the lambda.
	//
	// In future, Present() itself should be an enqueued command, and platform RHIs should never implicitly send commands to the GPU queues during RHI translation.
	// All GPU work should be submitted via RHISubmitCommandLists.
	//
	// This is also why there is a task dependency between RHI translate and RHI submission tasks in RHICommandList.cpp.
	// See GRHIGlobals.SupportsConcurrentTranslateAndSubmit
	//
	RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);

	{
		FRHICommandListScopedPipeline PipelineScope(RHICmdList, ERHIPipeline::Graphics);
		RHICmdList.EnqueueLambda(TEXT("RHIEndDrawingViewport"), [ViewportRHI, PresentArgs](FRHICommandListImmediate& ExecutingCmdList)
		{
			LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanMisc);

			// Not all platforms have control on the Present Counter, correlate the information FrameCounter -> PresentCounter before Presenting the frame
			RHIFrameInfoSetPresentCounter(PresentArgs.FrameCounter, GRHIPresentCounter);

			FVulkanViewport* Viewport = ResourceCast(ViewportRHI);

			FVulkanCommandListContext& Context = FVulkanCommandListContext::Get(ExecutingCmdList);

			FVulkanCommandBuffer& CommandBuffer = Context.GetCommandBuffer();
			check(!CommandBuffer.HasEnded() && !CommandBuffer.IsInsideRenderPass());

			const bool bNativePresent = Viewport->Present(Context, PresentArgs);
			if (bNativePresent)
			{
				//#todo-rco: Check for r.FinishCurrentFrame
			}
		});
	}

	// Switch to the next viewport back buffer
	ResourceCast(ViewportRHI)->AdvanceBackBufferFrame(RHICmdList, PresentArgs.bPresent);
}

#if WITH_RHI_BREADCRUMBS
	void FVulkanCommandListContext::RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb)
	{
		if (UE::RHI::UseGPUCrashBreadcrumbs() && Queue.GetDiagnosticBuffer())
		{
			Queue.GetDiagnosticBuffer()->WriteMarkerIn(GetCommandBuffer(), Breadcrumb);
		}

		const TCHAR* NameStr = nullptr;
		FRHIBreadcrumb::FBuffer Buffer;
		auto GetNameStr = [&NameStr, &Buffer, Breadcrumb]()
		{
			if (!NameStr)
			{
				NameStr = Breadcrumb->GetTCHAR(Buffer);
			}
			return NameStr;
		};

		const FColor Color = FColor::White;

		if (ShouldEmitBreadcrumbs())
		{
		#if VULKAN_ENABLE_DRAW_MARKERS
			if (auto CmdBeginLabel = Device.GetCmdBeginDebugLabel())
			{
				FTCHARToUTF8 Converter(GetNameStr());
				VkDebugUtilsLabelEXT Label;
				ZeroVulkanStruct(Label, VK_STRUCTURE_TYPE_DEBUG_UTILS_LABEL_EXT);
				Label.pLabelName = Converter.Get();
				FLinearColor LColor(Color);
				Label.color[0] = LColor.R;
				Label.color[1] = LColor.G;
				Label.color[2] = LColor.B;
				Label.color[3] = LColor.A;
				CmdBeginLabel(GetCommandBuffer().GetHandle(), &Label);
			}
		#endif

		#if VULKAN_ENABLE_DUMP_LAYER
			// only valid on immediate context currently.  needs to be fixed for parallel rhi execute
			if (IsImmediate())
			{
				VulkanRHI::DumpLayerPushMarker(GetNameStr());
			}
		#endif
		}

		if (bSupportsBreadcrumbs)
		{
			FlushProfilerStats();

			auto& Event = GetCommandBuffer().EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FBeginBreadcrumb>(Breadcrumb);
			FVulkanQueryPool* CurrentPool = GetCurrentTimestampQueryPool();
			const uint32 IndexInPool = CurrentPool->ReserveQuery(&Event.GPUTimestampTOP);
			VulkanRHI::vkCmdWriteTimestamp(GetCommandBuffer().GetHandle(), VK_PIPELINE_STAGE_TOP_OF_PIPE_BIT, CurrentPool->GetHandle(), IndexInPool);
		}
	}

	void FVulkanCommandListContext::RHIEndBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb)
	{
		if (bSupportsBreadcrumbs)
		{
			FlushProfilerStats();

			auto& Event = GetCommandBuffer().EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FEndBreadcrumb>(Breadcrumb);
			FVulkanQueryPool* CurrentPool = GetCurrentTimestampQueryPool();
			const uint32 IndexInPool = CurrentPool->ReserveQuery(&Event.GPUTimestampBOP);
			VulkanRHI::vkCmdWriteTimestamp(GetCommandBuffer().GetHandle(), VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT, CurrentPool->GetHandle(), IndexInPool);
		}

		if (ShouldEmitBreadcrumbs())
		{
		#if VULKAN_ENABLE_DUMP_LAYER
			if (IsImmediate())
			{
				VulkanRHI::DumpLayerPopMarker();
			}
		#endif

		#if VULKAN_ENABLE_DRAW_MARKERS
			if (auto CmdEndLabel = Device.GetCmdEndDebugLabel())
			{
				CmdEndLabel(GetCommandBuffer().GetHandle());
			}
		#endif
		}

		if (UE::RHI::UseGPUCrashBreadcrumbs() && Queue.GetDiagnosticBuffer())
		{
			Queue.GetDiagnosticBuffer()->WriteMarkerOut(GetCommandBuffer(), Breadcrumb);
		}
	}
#endif // WITH_RHI_BREADCRUMBS

void FVulkanDynamicRHI::RHIGetSupportedResolution( uint32 &Width, uint32 &Height )
{
}

bool FVulkanDynamicRHI::RHIGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate)
{
	return false;
}

// IVulkanDynamicRHI interface

uint32 FVulkanDynamicRHI::RHIGetVulkanVersion() const
{
	return ApiVersion;
}

VkInstance FVulkanDynamicRHI::RHIGetVkInstance() const
{
	return GetInstance();
}

VkDevice FVulkanDynamicRHI::RHIGetVkDevice() const
{
	if (Device)
	{
		return Device->GetHandle();
	}
	return VK_NULL_HANDLE;
}

const uint8* FVulkanDynamicRHI::RHIGetVulkanDeviceUUID() const
{
	return GetDevice()->GetDeviceIdProperties().deviceUUID;
}

VkPhysicalDevice FVulkanDynamicRHI::RHIGetVkPhysicalDevice() const
{
	return Device->GetPhysicalHandle();
}

const VkAllocationCallbacks* FVulkanDynamicRHI::RHIGetVkAllocationCallbacks()
{
	return VULKAN_CPU_ALLOCATOR;
}

VkQueue FVulkanDynamicRHI::RHIGetGraphicsVkQueue() const
{
	return GetDevice()->GetGraphicsQueue()->GetHandle();
}

uint32 FVulkanDynamicRHI::RHIGetGraphicsQueueIndex() const
{
	return GetDevice()->GetGraphicsQueue()->GetQueueIndex();
}

uint32 FVulkanDynamicRHI::RHIGetGraphicsQueueFamilyIndex() const
{
	return GetDevice()->GetGraphicsQueue()->GetFamilyIndex();
}

VkCommandBuffer FVulkanDynamicRHI::RHIGetVkCommandBuffer(FRHICommandListBase& ExecutingCmdList) const
{
	return FVulkanCommandListContext::Get(ExecutingCmdList).GetActiveCmdBuffer()->GetHandle();
}

uint64 FVulkanDynamicRHI::RHIGetGraphicsAdapterLUID(VkPhysicalDevice InPhysicalDevice) const
{
	uint64 AdapterLUID = 0;
#if VULKAN_SUPPORTS_DRIVER_PROPERTIES

	VkPhysicalDeviceProperties2KHR GpuProps2;
	ZeroVulkanStruct(GpuProps2, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_PROPERTIES_2);

	VkPhysicalDeviceIDPropertiesKHR GpuIdProps;
	ZeroVulkanStruct(GpuIdProps, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_ID_PROPERTIES);

	GpuProps2.pNext = &GpuIdProps;

	VulkanRHI::vkGetPhysicalDeviceProperties2(InPhysicalDevice, &GpuProps2);
	if (GpuIdProps.deviceLUIDValid)
	{
		AdapterLUID = reinterpret_cast<const uint64&>(GpuIdProps.deviceLUID);
	}
#endif
	return AdapterLUID;
}

bool FVulkanDynamicRHI::RHIDoesAdapterMatchDevice(const void* InAdapterId) const
{
#if PLATFORM_WINDOWS
	const VkPhysicalDeviceIDPropertiesKHR& vkPhysicalDeviceIDProperties = GetDevice()->GetDeviceIdProperties();
	if (vkPhysicalDeviceIDProperties.deviceLUIDValid)
	{
		return FMemory::Memcmp(InAdapterId, &vkPhysicalDeviceIDProperties.deviceLUID, sizeof(LUID)) == 0;
	}
#endif

	// Not enough information.  Assume the adapter matches
	return true;
}

void* FVulkanDynamicRHI::RHIGetVkDeviceProcAddr(const char* InName) const
{
	return (void*)VulkanRHI::vkGetDeviceProcAddr(Device->GetHandle(), InName);
}

void* FVulkanDynamicRHI::RHIGetVkInstanceProcAddr(const char* InName) const
{
	return (void*)VulkanRHI::vkGetInstanceProcAddr(Instance, InName);
}

void* FVulkanDynamicRHI::RHIGetVkInstanceGlobalProcAddr(const char* InName) const
{
	return (void*)VulkanRHI::vkGetInstanceProcAddr(nullptr, InName);
}

VkFormat FVulkanDynamicRHI::RHIGetSwapChainVkFormat(EPixelFormat InFormat) const
{
	// UE renders a gamma-corrected image so we need to use an sRGB format if available
	return UEToVkTextureFormat(GPixelFormats[InFormat].UnrealFormat, true);
}

bool FVulkanDynamicRHI::RHISupportsEXTFragmentDensityMap2() const
{
	return GetDevice()->GetOptionalExtensions().HasEXTFragmentDensityMap2;
}

TArray<VkExtensionProperties> FVulkanDynamicRHI::RHIGetAllInstanceExtensions() const
{
	uint32_t ExtensionCount = 0;
	VulkanRHI::vkEnumerateInstanceExtensionProperties(nullptr, &ExtensionCount, nullptr);

	TArray<VkExtensionProperties> Extensions;
	Extensions.SetNumUninitialized(ExtensionCount);

	VulkanRHI::vkEnumerateInstanceExtensionProperties(nullptr, &ExtensionCount, Extensions.GetData());

	return Extensions;
}

TArray<VkExtensionProperties> FVulkanDynamicRHI::RHIGetAllDeviceExtensions(VkPhysicalDevice InPhysicalDevice) const
{
	uint32_t ExtensionCount = 0;
	VulkanRHI::vkEnumerateDeviceExtensionProperties(InPhysicalDevice, nullptr, &ExtensionCount, nullptr);

	TArray<VkExtensionProperties> Extensions;
	Extensions.SetNumUninitialized(ExtensionCount);

	VulkanRHI::vkEnumerateDeviceExtensionProperties(InPhysicalDevice, nullptr, &ExtensionCount, Extensions.GetData());

	return Extensions;
}

TArray<FAnsiString> FVulkanDynamicRHI::RHIGetLoadedDeviceExtensions() const
{
	// Create copies to prevent issues
	TArray<FAnsiString> OutExtensions;
	const TArray<const ANSICHAR*>& DeviceExtensions = GetDevice()->DeviceExtensions;
	for (const ANSICHAR* ExtensionName : DeviceExtensions)
	{
		OutExtensions.Emplace(ExtensionName);
	}
	return OutExtensions;
}

VkImage FVulkanDynamicRHI::RHIGetVkImage(FRHITexture* InTexture) const
{
	FVulkanTexture* VulkanTexture = ResourceCast(InTexture);
	return VulkanTexture->Image;
}

VkFormat FVulkanDynamicRHI::RHIGetViewVkFormat(FRHITexture* InTexture) const
{
	FVulkanTexture* VulkanTexture = ResourceCast(InTexture);
	return VulkanTexture->ViewFormat;
}

FVulkanRHIAllocationInfo FVulkanDynamicRHI::RHIGetAllocationInfo(FRHITexture* InTexture) const
{
	FVulkanTexture* VulkanTexture = ResourceCast(InTexture);

	FVulkanRHIAllocationInfo NewInfo{};
	NewInfo.Handle = VulkanTexture->GetAllocationHandle();
	NewInfo.Offset = VulkanTexture->GetAllocationOffset();
	NewInfo.Size = VulkanTexture->GetMemorySize();
	NewInfo.Flags = VulkanTexture->MemProps;

	return NewInfo;
}

FVulkanRHIImageViewInfo FVulkanDynamicRHI::RHIGetImageViewInfo(FRHITexture* InTexture) const
{
	FVulkanTexture* VulkanTexture = ResourceCast(InTexture);

	const FRHITextureDesc& Desc = InTexture->GetDesc();

	FVulkanRHIImageViewInfo Info{};
	Info.ImageView = VulkanTexture->DefaultView->GetTextureView().View;
	Info.Image = VulkanTexture->DefaultView->GetTextureView().Image;
	Info.Format = VulkanTexture->ViewFormat;
	Info.Width = Desc.Extent.X;
	Info.Height = Desc.Extent.Y;
	Info.Depth = Desc.Depth;
	Info.UEFlags = Desc.Flags;

	Info.SubresourceRange.aspectMask = VulkanTexture->GetFullAspectMask();
	Info.SubresourceRange.layerCount = VulkanTexture->GetNumberOfArrayLevels();
	Info.SubresourceRange.levelCount = Desc.NumMips;

	// TODO: do we need these?
	Info.SubresourceRange.baseMipLevel = 0;
	Info.SubresourceRange.baseArrayLayer = 0;

	return Info;
}

FVulkanRHIAllocationInfo FVulkanDynamicRHI::RHIGetAllocationInfo(FRHIBuffer* InBuffer) const
{
	FVulkanBuffer* VulkanBuffer = ResourceCast(InBuffer);
	const VulkanRHI::FVulkanAllocation& Allocation = VulkanBuffer->GetCurrentAllocation();

	FVulkanRHIAllocationInfo NewInfo{};
	NewInfo.Handle = Allocation.GetDeviceMemoryHandle(GetDevice());
	NewInfo.Offset = Allocation.Offset;
	NewInfo.Size = Allocation.Size;
	NewInfo.Flags = Allocation.GetMemoryPropertyFlags(GetDevice());

	return NewInfo;
}

void FVulkanDynamicRHI::RHIFinishExternalComputeWork(VkCommandBuffer InCommandBuffer)
{
	FVulkanCommandListContext& ImmediateContext = GetDevice()->GetImmediateContext();
	check(InCommandBuffer == ImmediateContext.GetActiveCmdBuffer()->GetHandle());

	ImmediateContext.GetPendingComputeState()->Reset();
	ImmediateContext.GetPendingGfxState()->Reset();
}

void FVulkanDynamicRHI::RHIVerifyResult(VkResult Result, const ANSICHAR* VkFuntion, const ANSICHAR* Filename, uint32 Line)
{
	VulkanRHI::VerifyVulkanResult(Result, VkFuntion, Filename, Line);
}

void* FVulkanDynamicRHI::RHIGetNativeDevice()
{
	return (void*)Device->GetHandle();
}

void* FVulkanDynamicRHI::RHIGetNativePhysicalDevice()
{
	return (void*)Device->GetPhysicalHandle();
}

void* FVulkanDynamicRHI::RHIGetNativeGraphicsQueue()
{
	return (void*)Device->GetGraphicsQueue()->GetHandle();
}

void* FVulkanDynamicRHI::RHIGetNativeComputeQueue()
{
	return Device->HasAsyncComputeQueue() ? 
		(void*)Device->GetComputeQueue()->GetHandle() :
		(void*)Device->GetGraphicsQueue()->GetHandle();
}

void FVulkanDynamicRHI::RHIGetMemoryStats(FRHIMemoryStats& OutStats)
{
	if (GetDevice())
	{
		OutStats = GetDevice()->GetDeviceMemoryManager().GetRHIMemoryStats();
	}
}

IRHICommandContext* FVulkanDynamicRHI::RHIGetDefaultContext()
{
	return &Device->GetImmediateContext();
}

uint64 FVulkanDynamicRHI::RHIGetMinimumAlignmentForBufferBackedSRV(EPixelFormat Format)
{
	const VkPhysicalDeviceLimits& Limits = Device->GetLimits();
	return Limits.minTexelBufferOffsetAlignment;
}

FTextureRHIRef FVulkanDynamicRHI::RHICreateTexture2DFromResource(EPixelFormat Format, uint32 SizeX, uint32 SizeY, uint32 NumMips, uint32 NumSamples, VkImage Resource, ETextureCreateFlags Flags, const FClearValueBinding& ClearValueBinding, const FVulkanRHIExternalImageDeleteCallbackInfo& ExternalImageDeleteCallbackInfo)
{
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("VulkanTexture2DFromResource"), SizeX, SizeY, Format)
		.SetClearValue(ClearValueBinding)
		.SetFlags(Flags)
		.SetNumMips(NumMips)
		.SetNumSamples(NumSamples)
		.DetermineInititialState();

	return new FVulkanTexture(*Device, Desc, Resource, ExternalImageDeleteCallbackInfo);
}

#if PLATFORM_ANDROID
FTextureRHIRef FVulkanDynamicRHI::RHICreateTexture2DFromAndroidHardwareBuffer(AHardwareBuffer* HardwareBuffer)
{
	check(HardwareBuffer);

	AHardwareBuffer_Desc HardwareBufferDesc;
	AHardwareBuffer_describe(HardwareBuffer, &HardwareBufferDesc);
	check((HardwareBufferDesc.usage & AHARDWAREBUFFER_USAGE_GPU_SAMPLED_IMAGE) != 0);

	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("VulkanTexture2DFromAndroidHardwareBuffer"), HardwareBufferDesc.width, HardwareBufferDesc.height, PF_Unknown)
		.SetFlags(ETextureCreateFlags::External)
		.DetermineInititialState();

	return new FVulkanTexture(*Device, Desc, HardwareBufferDesc, HardwareBuffer);
}
#endif

FTextureRHIRef FVulkanDynamicRHI::RHICreateTexture2DArrayFromResource(EPixelFormat Format, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, VkImage Resource, ETextureCreateFlags Flags, const FClearValueBinding& ClearValueBinding)
{
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2DArray(TEXT("VulkanTextureArrayFromResource"), SizeX, SizeY, ArraySize, Format)
		.SetClearValue(ClearValueBinding)
		.SetFlags(Flags)
		.SetNumMips(NumMips)
		.SetNumSamples(NumSamples)
		.DetermineInititialState();

	return new FVulkanTexture(*Device, Desc, Resource, {});
}

FTextureRHIRef FVulkanDynamicRHI::RHICreateTextureCubeFromResource(EPixelFormat Format, uint32 Size, bool bArray, uint32 ArraySize, uint32 NumMips, VkImage Resource, ETextureCreateFlags Flags, const FClearValueBinding& ClearValueBinding)
{
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create(TEXT("VulkanTextureCubeFromResource"), ArraySize > 1 ? ETextureDimension::TextureCubeArray : ETextureDimension::TextureCube)
		.SetExtent(Size)
		.SetArraySize(ArraySize)
		.SetFormat(Format)
		.SetClearValue(ClearValueBinding)
		.SetFlags(Flags)
		.SetNumMips(NumMips)
		.DetermineInititialState();

	return new FVulkanTexture(*Device, Desc, Resource, {});
}

void FVulkanDynamicRHI::RHIAliasTextureResources(FTextureRHIRef& DestTextureRHI, FTextureRHIRef& SrcTextureRHI)
{
	if (DestTextureRHI && SrcTextureRHI)
	{
		FVulkanTexture* DestTexture = ResourceCast(DestTextureRHI);
		DestTexture->AliasTextureResources(SrcTextureRHI);
	}
}

FTextureRHIRef FVulkanDynamicRHI::RHICreateAliasedTexture(FTextureRHIRef& SourceTextureRHI)
{
	const FString Name = SourceTextureRHI->GetName().ToString() + TEXT("Alias");
	FRHITextureCreateDesc Desc(SourceTextureRHI->GetDesc(), ERHIAccess::SRVMask, *Name);
	return new FVulkanTexture(*Device, Desc, SourceTextureRHI);
}

FVulkanRenderPass::FVulkanRenderPass(FVulkanDevice& InDevice, const FVulkanRenderTargetLayout& InRTLayout) :
	Layout(InRTLayout),
	RenderPass(VK_NULL_HANDLE),
	NumUsedClearValues(InRTLayout.GetNumUsedClearValues()),
	Device(InDevice)
{
	INC_DWORD_STAT(STAT_VulkanNumRenderPasses);
	RenderPass = CreateVulkanRenderPass(InDevice, InRTLayout);
}

FVulkanRenderPass::~FVulkanRenderPass()
{
	DEC_DWORD_STAT(STAT_VulkanNumRenderPasses);

	Device.GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::RenderPass, RenderPass);
	RenderPass = VK_NULL_HANDLE;
}

void FVulkanDynamicRHI::SavePipelineCache()
{
	FString CacheFile = VulkanRHI::GetPipelineCacheFilename();

	GVulkanRHI->Device->PipelineStateCache->Save(CacheFile);
}

void FVulkanDynamicRHI::RebuildPipelineCache()
{
	GVulkanRHI->Device->PipelineStateCache->RebuildCache();
}

#if VULKAN_SUPPORTS_VALIDATION_CACHE
void FVulkanDynamicRHI::SaveValidationCache()
{
	VkValidationCacheEXT ValidationCache = GVulkanRHI->Device->GetValidationCache();
	if (ValidationCache != VK_NULL_HANDLE)
	{
		VkDevice Device = GVulkanRHI->Device->GetHandle();
		PFN_vkGetValidationCacheDataEXT vkGetValidationCacheData = (PFN_vkGetValidationCacheDataEXT)(void*)VulkanRHI::vkGetDeviceProcAddr(Device, "vkGetValidationCacheDataEXT");
		check(vkGetValidationCacheData);
		size_t CacheSize = 0;
		VkResult Result = vkGetValidationCacheData(Device, ValidationCache, &CacheSize, nullptr);
		if (Result == VK_SUCCESS)
		{
			if (CacheSize > 0)
			{
				TArray<uint8> Data;
				Data.AddUninitialized(CacheSize);
				Result = vkGetValidationCacheData(Device, ValidationCache, &CacheSize, Data.GetData());
				if (Result == VK_SUCCESS)
				{
					FString CacheFilename = VulkanRHI::GetValidationCacheFilename();
					if (FFileHelper::SaveArrayToFile(Data, *CacheFilename))
					{
						UE_LOGF(LogVulkanRHI, Display, "Saved validation cache file '%ls', %d bytes", *CacheFilename, Data.Num());
					}
				}
				else
				{
					UE_LOGF(LogVulkanRHI, Warning, "Failed to query Vulkan validation cache data, VkResult=%d", Result);
				}
			}
		}
		else
		{
			UE_LOGF(LogVulkanRHI, Warning, "Failed to query Vulkan validation cache size, VkResult=%d", Result);
		}
	}
}
#endif

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
void FVulkanDynamicRHI::DumpMemory()
{
	GVulkanRHI->Device->GetMemoryManager().DumpMemory(false);
}
void FVulkanDynamicRHI::DumpMemoryFull()
{
	GVulkanRHI->Device->GetMemoryManager().DumpMemory(true);
}
void FVulkanDynamicRHI::DumpStagingMemory()
{
	GVulkanRHI->Device->GetStagingManager().DumpMemory();
}
void FVulkanDynamicRHI::DumpLRU()
{
	GVulkanRHI->Device->PipelineStateCache->LRUDump();
}
void FVulkanDynamicRHI::TrimLRU()
{
	GVulkanRHI->Device->PipelineStateCache->LRUDebugEvictAll();
}
#endif

void FVulkanDynamicRHI::VulkanSetImageLayout(VkCommandBuffer CmdBuffer, VkImage Image, VkImageLayout OldLayout, VkImageLayout NewLayout, const VkImageSubresourceRange& SubresourceRange)
{
	FVulkanPipelineBarrier Barrier;
	Barrier.AddImageLayoutTransition(Image, OldLayout, NewLayout, SubresourceRange);
	Barrier.Execute(CmdBuffer);
}

IRHITransientResourceAllocator* FVulkanDynamicRHI::RHICreateTransientResourceAllocator()
{
#if VULKAN_SUPPORTS_TRANSIENT_RESOURCE_ALLOCATOR
	if (GVulkanEnableTransientResourceAllocator)
	{
		return new FVulkanTransientResourceAllocator(Device->GetOrCreateTransientHeapCache());
	}
#endif
	return nullptr;
}

uint32 FVulkanDynamicRHI::GetPrecachePSOHashVersion()
{
	static const uint32 PrecacheHashVersion = 6;
	return PrecacheHashVersion;
}

// If you modify this function bump then GetPrecachePSOHashVersion, this will invalidate any previous uses of the hash.
// i.e. pre-existing PSO caches must be rebuilt.
uint64 FVulkanDynamicRHI::RHIComputeStatePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer)
{
	struct FHashKey
	{
		uint32 VertexDeclaration;
		uint32 VertexShader;
		uint32 PixelShader;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		uint32 GeometryShader;
#endif // PLATFORM_SUPPORTS_GEOMETRY_SHADERS
#if PLATFORM_SUPPORTS_MESH_SHADERS
		uint32 MeshShader;
		uint32 TaskShader;
#endif // PLATFORM_SUPPORTS_MESH_SHADERS
		uint32 BlendState;
		uint32 RasterizerState;
		uint32 DepthStencilState;

		uint32 DrawShadingRate : 8;
		uint32 PrimitiveType : 8;
		uint32 bDepthBounds : 1;
		uint32 bAllowVariableRateShading : 1;
		uint32 Unused : 14;
	} HashKey;

	FMemory::Memzero(&HashKey, sizeof(FHashKey));

	// We know for sure that on ARM MALI GPUs vertex decl does not affect PSO
	const bool bVertexDeclAffectsPSO = (GRHIVendorId != (uint32)EGpuVendorId::Arm);
	if (bVertexDeclAffectsPSO)
	{ 
		HashKey.VertexDeclaration = Initializer.BoundShaderState.VertexDeclarationRHI ? Initializer.BoundShaderState.VertexDeclarationRHI->GetPrecachePSOHash() : 0;
	}
	HashKey.VertexShader = Initializer.BoundShaderState.GetVertexShader() ? GetTypeHash(Initializer.BoundShaderState.GetVertexShader()->GetHash()) : 0;
	HashKey.PixelShader = Initializer.BoundShaderState.GetPixelShader() ? GetTypeHash(Initializer.BoundShaderState.GetPixelShader()->GetHash()) : 0;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	HashKey.GeometryShader = Initializer.BoundShaderState.GetGeometryShader() ? GetTypeHash(Initializer.BoundShaderState.GetGeometryShader()->GetHash()) : 0;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
	HashKey.MeshShader = Initializer.BoundShaderState.GetMeshShader() ? GetTypeHash(Initializer.BoundShaderState.GetMeshShader()->GetHash()) : 0;
	HashKey.TaskShader = Initializer.BoundShaderState.GetAmplificationShader() ? GetTypeHash(Initializer.BoundShaderState.GetAmplificationShader()->GetHash()) : 0;
#endif

	FBlendStateInitializerRHI BlendStateInitializerRHI;
	if (Initializer.BlendState && Initializer.BlendState->GetInitializer(BlendStateInitializerRHI))
	{
		HashKey.BlendState = GetTypeHash(BlendStateInitializerRHI);
	}
	FRasterizerStateInitializerRHI RasterizerStateInitializerRHI;
	if (Initializer.RasterizerState && Initializer.RasterizerState->GetInitializer(RasterizerStateInitializerRHI))
	{
		HashKey.RasterizerState = GetTypeHash(RasterizerStateInitializerRHI);
	}
	FDepthStencilStateInitializerRHI DepthStencilStateInitializerRHI;
	if (Initializer.DepthStencilState && Initializer.DepthStencilState->GetInitializer(DepthStencilStateInitializerRHI))
	{
		HashKey.DepthStencilState = GetTypeHash(DepthStencilStateInitializerRHI);
	}

	HashKey.DrawShadingRate = Initializer.ShadingRate;
	HashKey.PrimitiveType = Initializer.PrimitiveType;
	HashKey.bDepthBounds = Initializer.bDepthBounds;
	HashKey.bAllowVariableRateShading = Initializer.bAllowVariableRateShading;

	uint64 PrecachePSOHash = CityHash64((const char*)&HashKey, sizeof(FHashKey));

	return PrecachePSOHash;
}

// If you modify this function bump then GetPrecachePSOHashVersion, this will invalidate any previous uses of the hash.
// i.e. pre-existing PSO caches must be rebuilt.
uint64 FVulkanDynamicRHI::RHIComputePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer)
{

	// When compute precache PSO hash we assume a valid state precache PSO hash is already provided
	uint64 StatePrecachePSOHash = Initializer.StatePrecachePSOHash;
	if (StatePrecachePSOHash == 0)
	{
		StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(Initializer);
	}

	// All members which are not part of the state objects
	struct FNonStateHashKey
	{
		uint64							StatePrecachePSOHash;

		uint32							RenderTargetsEnabled;
		FGraphicsPipelineStateInitializer::TRenderTargetFormats	RenderTargetFormats;
		FGraphicsPipelineStateInitializer::TRenderTargetFlags RenderTargetFlags;
// AJB: temporarily disabling depth stencil properties as they do not appear to be required and it causes us to miss some permutations.
//		EPixelFormat					DepthStencilTargetFormat;
//		ETextureCreateFlags				DepthStencilTargetFlag;
		uint16							NumSamples;
		ESubpassHint					SubpassHint;
		uint8							SubpassIndex;
		uint8							MultiViewCount;
		bool							bHasFragmentDensityAttachment;
		EConservativeRasterization		ConservativeRasterization;
	} HashKey;

	FMemory::Memzero(&HashKey, sizeof(FNonStateHashKey));

	HashKey.StatePrecachePSOHash = StatePrecachePSOHash;

	HashKey.RenderTargetsEnabled = Initializer.RenderTargetsEnabled;
	HashKey.RenderTargetFormats = Initializer.RenderTargetFormats;
	HashKey.RenderTargetFlags = Initializer.RenderTargetFlags;
//	HashKey.DepthStencilTargetFormat = Initializer.DepthStencilTargetFormat;
//	HashKey.DepthStencilTargetFlag = Initializer.DepthStencilTargetFlag;
	HashKey.NumSamples = Initializer.NumSamples;
	HashKey.SubpassHint = Initializer.SubpassHint;
	HashKey.SubpassIndex = Initializer.SubpassIndex;
	HashKey.MultiViewCount = Initializer.MultiViewCount;
	HashKey.bHasFragmentDensityAttachment = Initializer.bHasFragmentDensityAttachment;
	HashKey.ConservativeRasterization = Initializer.ConservativeRasterization;
	
	// TODO: check if any RT flags actually affect PSO in VK
	for (ETextureCreateFlags& Flags : HashKey.RenderTargetFlags)
	{
		Flags = Flags & FGraphicsPipelineStateInitializer::RelevantRenderTargetFlagMask;
	}
// 	HashKey.DepthStencilTargetFlag = (HashKey.DepthStencilTargetFlag & FGraphicsPipelineStateInitializer::RelevantDepthStencilFlagMask);

	return CityHash64((const char*)&HashKey, sizeof(FNonStateHashKey));
}

bool FVulkanDynamicRHI::RHIMatchPrecachePSOInitializers(const FGraphicsPipelineStateInitializer& LHS, const FGraphicsPipelineStateInitializer& RHS)
{
	// first check non pointer objects
	if (LHS.PrimitiveType != RHS.PrimitiveType ||
		LHS.bDepthBounds != RHS.bDepthBounds ||
		LHS.MultiViewCount != RHS.MultiViewCount ||
		LHS.ShadingRate != RHS.ShadingRate ||
		LHS.bHasFragmentDensityAttachment != RHS.bHasFragmentDensityAttachment ||
		LHS.bAllowVariableRateShading != RHS.bAllowVariableRateShading ||
		LHS.RenderTargetsEnabled != RHS.RenderTargetsEnabled ||
		LHS.RenderTargetFormats != RHS.RenderTargetFormats ||
		!FGraphicsPipelineStateInitializer::RelevantRenderTargetFlagsEqual(LHS.RenderTargetFlags, RHS.RenderTargetFlags) ||
		LHS.DepthStencilTargetFormat != RHS.DepthStencilTargetFormat ||
		!FGraphicsPipelineStateInitializer::RelevantDepthStencilFlagsEqual(LHS.DepthStencilTargetFlag, RHS.DepthStencilTargetFlag) ||
		LHS.NumSamples != RHS.NumSamples ||
		LHS.SubpassHint != RHS.SubpassHint ||
		LHS.SubpassIndex != RHS.SubpassIndex ||
		LHS.StatePrecachePSOHash != RHS.StatePrecachePSOHash ||
		LHS.ConservativeRasterization != RHS.ConservativeRasterization)
	{
		return false;
	}

	// check the RHI shaders (pointer check for shaders should be fine)
	if (LHS.BoundShaderState.VertexShaderRHI != RHS.BoundShaderState.VertexShaderRHI ||
		LHS.BoundShaderState.PixelShaderRHI != RHS.BoundShaderState.PixelShaderRHI ||
		LHS.BoundShaderState.GetMeshShader() != RHS.BoundShaderState.GetMeshShader() ||
		LHS.BoundShaderState.GetAmplificationShader() != RHS.BoundShaderState.GetAmplificationShader() ||
		LHS.BoundShaderState.GetGeometryShader() != RHS.BoundShaderState.GetGeometryShader())
	{
		return false;
	}

	// Full compare the of the vertex declaration
	if (!MatchRHIState<FRHIVertexDeclaration, FVertexDeclarationElementList>(LHS.BoundShaderState.VertexDeclarationRHI, RHS.BoundShaderState.VertexDeclarationRHI))
	{
		return false;
	}

	// Check actual state content (each initializer can have it's own state and not going through a factory)
	if (!MatchRHIState<FRHIBlendState, FBlendStateInitializerRHI>(LHS.BlendState, RHS.BlendState)
		|| !MatchRHIState<FRHIRasterizerState, FRasterizerStateInitializerRHI>(LHS.RasterizerState, RHS.RasterizerState)
		|| !MatchRHIState<FRHIDepthStencilState, FDepthStencilStateInitializerRHI>(LHS.DepthStencilState, RHS.DepthStencilState)
		)
	{
		return false;
	}

	return true;
}

void FVulkanDynamicRHI::RHIReplaceResources(FRHICommandListBase& RHICmdList, TArray<FRHIResourceReplaceInfo>&& ReplaceInfos)
{
	RHICmdList.EnqueueLambdaMultiPipe(GetEnabledRHIPipelines(), FRHICommandListBase::EThreadFence::Enabled, TEXT("FVulkanDynamicRHI::RHIReplaceResources"),
		[ReplaceInfos = MoveTemp(ReplaceInfos)](const FVulkanContextArray& Contexts)
		{
			for (FRHIResourceReplaceInfo const& Info : ReplaceInfos)
			{
				switch (Info.GetType())
				{
				default:
					checkNoEntry();
					break;

				case FRHIResourceReplaceInfo::EType::Buffer:
					{
						FVulkanBuffer* Dst = ResourceCast(Info.GetBuffer().Dst);
						FVulkanBuffer* Src = ResourceCast(Info.GetBuffer().Src);

						if (Src)
						{
							// The source buffer should not have any associated views.
							check(!Src->HasLinkedViews());

							Dst->TakeOwnership(*Src);
						}
						else
						{
							Dst->ReleaseOwnership();
						}

						Dst->UpdateLinkedViews(Contexts);
					}
					break;

				case FRHIResourceReplaceInfo::EType::RTGeometry:
					{
						FVulkanRayTracingGeometry* Src = ResourceCast(Info.GetRTGeometry().Src);
						FVulkanRayTracingGeometry* Dst = ResourceCast(Info.GetRTGeometry().Dst);

						if (!Src)
						{
							TRefCountPtr<FVulkanRayTracingGeometry> DeletionProxy = new FVulkanRayTracingGeometry(NoInit);
							Dst->RemoveCompactionRequest();
							Dst->Swap(*DeletionProxy);
						}
						else
						{
							Dst->Swap(*Src);
						}
					}
					break;
				}
			}
		}
	);
}

void FVulkanDynamicRHI::RHIWriteResourceGPUAddresses(FRHICommandListBase& RHICmdList, uint32 GPUIndex, TConstArrayView<FRHIResource*> Resources, uint64* AddressesPtr)
{
	RHICmdList.EnqueueLambda([AddressesPtr, Resources](FRHICommandListBase&)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(RHIWriteResourceGPUAddresses);

			for (int32 ResourceIndex = 0; ResourceIndex < Resources.Num(); ++ResourceIndex)
			{
				const FRHIResource* Resource = Resources[ResourceIndex];

				if (Resource->GetType() == RRT_Buffer)
				{
					const FVulkanBuffer* Buffer = ResourceCast(static_cast<const FRHIBuffer*>(Resource));
					AddressesPtr[ResourceIndex] = Buffer->GetDeviceAddress();
				}
				else if (Resource->GetType() == RRT_Texture)
				{
					unimplemented();
				}
				else if (Resource->GetType() == RRT_RayTracingGeometry)
				{
					const FVulkanRayTracingGeometry* RayTracingGeometry = ResourceCast(static_cast<const FRHIRayTracingGeometry*>(Resource));
					AddressesPtr[ResourceIndex] = RayTracingGeometry->GetDeviceAddress();
				}
				else
				{
					checkNoEntry();
				}
			}
		});
}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
FRHIResourceCollectionRef FVulkanDynamicRHI::RHICreateResourceCollection(FRHICommandListBase& RHICmdList, TConstArrayView<FRHIResourceCollectionMember> InMembers)
{
	return UE::RHICore::CreateGenericResourceCollection(RHICmdList, InMembers);
}

void FVulkanDynamicRHI::RHIUpdateResourceCollection(FRHICommandListBase& RHICmdList, FRHIResourceCollection* InResourceCollection, uint32 InStartIndex, TConstArrayView<FRHIResourceCollectionMember> InMemberUpdates)
{
	UE::RHICore::UpdateGenericResourceCollection(RHICmdList, ResourceCast(InResourceCollection), InStartIndex, InMemberUpdates);
}
#endif

void FVulkanDynamicRHI::RHISetFDMOffsets(const FVector2f GazePoint[2], FIntPoint FramebufferSize)
{
	if (GRHISupportsVariableRateShadingImageOffsets)
	{
		// Eye Offsets are in -1.0..1.0 space
		const float LeftXOffset = FMath::Clamp(GazePoint[0].X, -1.0, 1.0);
		const float LeftYOffset = FMath::Clamp(GazePoint[0].Y, -1.0, 1.0);
		const float RightXOffset = FMath::Clamp(GazePoint[1].X, -1.0, 1.0);
		const float RightYOffset = FMath::Clamp(GazePoint[1].Y, -1.0, 1.0);

		// Sanity check, these two should never be 0.  If so, for some reason the extension property initialization has failed
		check(GRHIVariableRateShadingImageOffsetGranularity.X > 0 && GRHIVariableRateShadingImageOffsetGranularity.Y > 0);

		// Offsets need to be quantized to the pixel size defined in the extension's properties struct
		const FVector2D QuantizedGranularity((float)FramebufferSize.X / (float)GRHIVariableRateShadingImageOffsetGranularity.X,
			(float)FramebufferSize.Y / (float)GRHIVariableRateShadingImageOffsetGranularity.Y);

		// Quantize the offsets to the minimum offset allowed by the FDM Offset granularity
		GRHIVariableRateShadingImageOffsets[0].X = FMath::RoundToInt(LeftXOffset * 0.5 * QuantizedGranularity.X) * GRHIVariableRateShadingImageOffsetGranularity.X;
		GRHIVariableRateShadingImageOffsets[0].Y = FMath::RoundToInt(LeftYOffset * 0.5 * QuantizedGranularity.Y) * GRHIVariableRateShadingImageOffsetGranularity.Y;
		GRHIVariableRateShadingImageOffsets[1].X = FMath::RoundToInt(RightXOffset * 0.5 * QuantizedGranularity.X) * GRHIVariableRateShadingImageOffsetGranularity.X;
		GRHIVariableRateShadingImageOffsets[1].Y = FMath::RoundToInt(RightYOffset * 0.5 * QuantizedGranularity.Y) * GRHIVariableRateShadingImageOffsetGranularity.Y;
	}
}

#undef LOCTEXT_NAMESPACE
