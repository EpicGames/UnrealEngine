// Copyright Epic Games, Inc. All Rights Reserved..

#pragma once

// Dependencies
#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "Misc/ScopeLock.h"
#include "RHI.h"
#include "RenderUtils.h"
#include "RHIValidation.h"

// let the platform set up the headers and some defines
#include "VulkanPlatform.h"

// the configuration will set up anything not set up by the platform
#include "VulkanConfiguration.h"

#include "VulkanCommandWrappers.h"

#include "VulkanState.h"
#include "VulkanResources.h"
#include "VulkanUtil.h"
#include "VulkanViewport.h"
#include "VulkanDynamicRHI.h"
#include "RHI.h"

#include "GPUProfiler.h"
#include "VulkanQueue.h"
#include "VulkanCommandBuffer.h"
#include "Stats/Stats.h"

class FVulkanQueue;
class FVulkanCommandBuffer;
class FVulkanShader;
class FVulkanDescriptorSetsLayout;
class FVulkanGfxPipeline;
class FVulkanRenderPass;
class FValidationContext;


template<typename BitsType>
constexpr bool VKHasAllFlags(VkFlags Flags, BitsType Contains)
{
	return (Flags & Contains) == Contains;
}

template<typename BitsType>
constexpr bool VKHasAnyFlags(VkFlags Flags, BitsType Contains)
{
	return (Flags & Contains) != 0;
}

inline VkShaderStageFlagBits UEFrequencyToVKStageBit(EShaderFrequency InStage)
{
	switch (InStage)
	{
	case SF_Vertex:			return VK_SHADER_STAGE_VERTEX_BIT;
	case SF_Mesh:			return VK_SHADER_STAGE_MESH_BIT_EXT;
	case SF_Amplification:	return VK_SHADER_STAGE_TASK_BIT_EXT;
	case SF_Pixel:			return VK_SHADER_STAGE_FRAGMENT_BIT;
	case SF_Geometry:		return VK_SHADER_STAGE_GEOMETRY_BIT;
	case SF_Compute:		return VK_SHADER_STAGE_COMPUTE_BIT;
	case SF_RayGen:			return VK_SHADER_STAGE_RAYGEN_BIT_KHR;
	case SF_RayMiss:		return VK_SHADER_STAGE_MISS_BIT_KHR;
	case SF_RayHitGroup:	return VK_SHADER_STAGE_CLOSEST_HIT_BIT_KHR; // vkrt todo: How to handle VK_SHADER_STAGE_ANY_HIT_BIT_KHR?
	case SF_RayCallable:	return VK_SHADER_STAGE_CALLABLE_BIT_KHR;
	default:
		checkf(false, TEXT("Undefined shader stage %d"), (int32)InStage);
		break;
	}

	return VK_SHADER_STAGE_ALL;
}

inline EShaderFrequency VkStageBitToUEFrequency(VkShaderStageFlagBits FlagBits)
{
	switch (FlagBits)
	{
	case VK_SHADER_STAGE_VERTEX_BIT:					return SF_Vertex;
	case VK_SHADER_STAGE_FRAGMENT_BIT:					return SF_Pixel;
	case VK_SHADER_STAGE_MESH_BIT_EXT:					return SF_Mesh;
	case VK_SHADER_STAGE_TASK_BIT_EXT:					return SF_Amplification;
	case VK_SHADER_STAGE_GEOMETRY_BIT:					return SF_Geometry;
	case VK_SHADER_STAGE_COMPUTE_BIT:					return SF_Compute;

	case VK_SHADER_STAGE_RAYGEN_BIT_KHR:				return SF_RayGen;
	case VK_SHADER_STAGE_MISS_BIT_KHR:					return SF_RayMiss;
	case VK_SHADER_STAGE_CALLABLE_BIT_KHR:				return SF_RayCallable;

	// Hit group frequencies
	case VK_SHADER_STAGE_ANY_HIT_BIT_KHR:
	case VK_SHADER_STAGE_CLOSEST_HIT_BIT_KHR:
	case VK_SHADER_STAGE_INTERSECTION_BIT_KHR:
		return SF_RayHitGroup;

	default:
		checkf(false, TEXT("Undefined VkShaderStageFlagBits %d"), (int32)FlagBits);
		break;
	}

	return SF_NumFrequencies;
}

static constexpr int32 GetNumStagesForBindPoint(VkPipelineBindPoint BindPoint)
{
	switch (BindPoint)
	{
	case VK_PIPELINE_BIND_POINT_GRAPHICS:
		return ShaderStage::NumGraphicsStages;

	case VK_PIPELINE_BIND_POINT_COMPUTE:
		return ShaderStage::NumComputeStages;

	case VK_PIPELINE_BIND_POINT_RAY_TRACING_KHR:
		return ShaderStage::NumRayTracingStages;
	};

	return 0;
}

union UNvidiaDriverVersion
{
	struct
	{
#if PLATFORM_LITTLE_ENDIAN
		uint32 Tertiary		: 6;
		uint32 Secondary	: 8;
		uint32 Minor		: 8;
		uint32 Major		: 10;
#else
		uint32 Major		: 10;
		uint32 Minor		: 8;
		uint32 Secondary	: 8;
		uint32 Tertiary		: 6;
#endif
	};
	uint32 Packed;
};

// Transitions an image to the specified layout. This does not update the layout cached internally by the RHI; the calling code must do that explicitly via FVulkanCommandListContext::GetLayoutManager() if necessary.
void VulkanSetImageLayout(FVulkanCommandBuffer* CmdBuffer, VkImage Image, VkImageLayout OldLayout, VkImageLayout NewLayout, const VkImageSubresourceRange& SubresourceRange);


// Regular pipeline creation stats
DECLARE_STATS_GROUP(TEXT("Vulkan PSO"), STATGROUP_VulkanPSO, STATCAT_Advanced);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("PSO LRU Elements"), STAT_VulkanNumPSOLRU, STATGROUP_VulkanPSO, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("PSO LRU Size"), STAT_VulkanNumPSOLRUSize, STATGROUP_VulkanPSO, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num PSOs"), STAT_VulkanNumPSOs, STATGROUP_VulkanPSO, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Graphics PSOs"), STAT_VulkanNumGraphicsPSOs, STATGROUP_VulkanPSO, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Compute  PSOs"), STAT_VulkanNumComputePSOs, STATGROUP_VulkanPSO, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("VulkanPSOKey Memory"), STAT_VulkanPSOKeyMemory, STATGROUP_VulkanPSO, );

DECLARE_CYCLE_STAT_EXTERN(TEXT("PSO HeaderInit time"), STAT_VulkanPSOHeaderInitTime, STATGROUP_VulkanPSO, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PSO Lookup time"), STAT_VulkanPSOLookupTime, STATGROUP_VulkanPSO, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PSO Creation time"), STAT_VulkanPSOCreationTime, STATGROUP_VulkanPSO, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PSO Vulkan Creation time"), STAT_VulkanPSOVulkanCreationTime, STATGROUP_VulkanPSO, );


// Graphic pipeline library stats
DECLARE_STATS_GROUP(TEXT("Vulkan Graphic Pipeline Libraries"), STATGROUP_VulkanGraphicPipelineLibs, STATCAT_Advanced);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Regular PSO Compile Count"), STAT_VulkanRegPSOCompileCount, STATGROUP_VulkanGraphicPipelineLibs, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Library PSO Compile Count"), STAT_VulkanLibPSOCompileCount, STATGROUP_VulkanGraphicPipelineLibs, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num VertexInput Libraries"), STAT_VulkanNumVertexInputLibs, STATGROUP_VulkanGraphicPipelineLibs, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num VS Libraries"), STAT_VulkanNumVSLibs, STATGROUP_VulkanGraphicPipelineLibs, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num MS Libraries"), STAT_VulkanNumMSLibs, STATGROUP_VulkanGraphicPipelineLibs, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num PS Libraries"), STAT_VulkanNumPSLibs, STATGROUP_VulkanGraphicPipelineLibs, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num FragmentOutputState Libraries"), STAT_VulkanNumFragOutputStateLibs, STATGROUP_VulkanGraphicPipelineLibs, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Shader Modules"), STAT_VulkanNumShaderModule, STATGROUP_VulkanGraphicPipelineLibs, );


// General Stats
DECLARE_STATS_GROUP(TEXT("Vulkan RHI"), STATGROUP_VulkanRHI, STATCAT_Advanced);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Draw call time"), STAT_VulkanDrawCallTime, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Dispatch call time"), STAT_VulkanDispatchCallTime, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Draw call prep time"), STAT_VulkanDrawCallPrepareTime, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CustomPresent time"), STAT_VulkanCustomPresentTime, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Dispatch call prep time"), STAT_VulkanDispatchCallPrepareTime, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Get Or Create Pipeline"), STAT_VulkanGetOrCreatePipeline, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Get DescriptorSet"), STAT_VulkanGetDescriptorSet, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Pipeline Bind"), STAT_VulkanPipelineBind, STATGROUP_VulkanRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Cmd Buffers"), STAT_VulkanNumCmdBuffers, STATGROUP_VulkanRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Render Passes"), STAT_VulkanNumRenderPasses, STATGROUP_VulkanRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Frame Buffers"), STAT_VulkanNumFrameBuffers, STATGROUP_VulkanRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Buffer Views"), STAT_VulkanNumBufferViews, STATGROUP_VulkanRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Image Views"), STAT_VulkanNumImageViews, STATGROUP_VulkanRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Physical Mem Allocations"), STAT_VulkanNumPhysicalMemAllocations, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Uniform Buffer Creation Time"), STAT_VulkanUniformBufferCreateTime, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Apply Packed Uniform Buffers"), STAT_VulkanApplyPackedUniformBuffers, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("SRV Update Time"), STAT_VulkanSRVUpdateTime, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAV Update Time"), STAT_VulkanUAVUpdateTime, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Deletion Queue"), STAT_VulkanDeletionQueue, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Queue Submit"), STAT_VulkanQueueSubmit, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Queue Present"), STAT_VulkanQueuePresent, STATGROUP_VulkanRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Queries"), STAT_VulkanNumQueries, STATGROUP_VulkanRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Query Pools"), STAT_VulkanNumQueryPools, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Wait For Fence"), STAT_VulkanWaitFence, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Wait For Swapchain"), STAT_VulkanWaitSwapchain, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Acquire Backbuffer"), STAT_VulkanAcquireBackBuffer, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Staging Buffer Mgmt"), STAT_VulkanStagingBuffer, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("VkCreateDescriptorPool"), STAT_VulkanVkCreateDescriptorPool, STATGROUP_VulkanRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Created DescSet Pools"), STAT_VulkanNumDescPools, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdateUniformBuffers"), STAT_VulkanUpdateUniformBuffers, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdateUniformBuffersRename"), STAT_VulkanUpdateUniformBuffersRename, STATGROUP_VulkanRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Pooled Barrier Events"), STAT_VulkanNumPooledBarrierEvents, STATGROUP_VulkanRHI, );
#if VULKAN_ENABLE_AGGRESSIVE_STATS
DECLARE_CYCLE_STAT_EXTERN(TEXT("Barrier Time"), STAT_VulkanBarrierTime, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update DescriptorSets"), STAT_VulkanUpdateDescriptorSets, STATGROUP_VulkanRHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num Desc Sets Updated"), STAT_VulkanNumDescSets, STATGROUP_VulkanRHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num WriteDescriptors Cmd"), STAT_VulkanNumUpdateDescriptors, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Set unif Buffer"), STAT_VulkanSetUniformBufferTime, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("VkUpdate DS"), STAT_VulkanVkUpdateDS, STATGROUP_VulkanRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Bind Vertex Streams"), STAT_VulkanBindVertexStreamsTime, STATGROUP_VulkanRHI, );
#endif
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Max Potential Desc Sets"), STAT_VulkanNumDescSetsTotal, STATGROUP_VulkanRHI, );

namespace VulkanRHI
{
	static uint32 GetNumBitsPerPixel(VkFormat Format)
	{
		switch (Format)
		{
		case VK_FORMAT_B8G8R8A8_UNORM:
		case VK_FORMAT_B10G11R11_UFLOAT_PACK32:
		case VK_FORMAT_X8_D24_UNORM_PACK32:
		case VK_FORMAT_D24_UNORM_S8_UINT:
		case VK_FORMAT_A2B10G10R10_UNORM_PACK32:
		case VK_FORMAT_D32_SFLOAT:
		case VK_FORMAT_R16G16_UNORM:
		case VK_FORMAT_R16G16_SFLOAT:
		case VK_FORMAT_R32_UINT:
		case VK_FORMAT_R32_SINT:
		case VK_FORMAT_R8G8B8A8_UNORM:
		case VK_FORMAT_R8G8B8A8_UINT:
		case VK_FORMAT_R8G8B8A8_SNORM:
		case VK_FORMAT_R16G16_UINT:
		case VK_FORMAT_R32_SFLOAT:
			return 32;
		case VK_FORMAT_R8_UNORM:
		case VK_FORMAT_R8_UINT:
			return 8;
		case VK_FORMAT_R16_UNORM:
		case VK_FORMAT_D16_UNORM:
		case VK_FORMAT_R16_UINT:
		case VK_FORMAT_R16_SINT:
		case VK_FORMAT_R16_SFLOAT:
		case VK_FORMAT_R5G6B5_UNORM_PACK16:
		case VK_FORMAT_R8G8_UNORM:
			return 16;
		case VK_FORMAT_R16G16B16A16_SFLOAT:
		case VK_FORMAT_R32G32_SFLOAT:
		case VK_FORMAT_R32G32_UINT:
		case VK_FORMAT_R32G32_SINT:
		case VK_FORMAT_R16G16B16A16_UNORM:
		case VK_FORMAT_R16G16B16A16_SNORM:
		case VK_FORMAT_R16G16B16A16_UINT:
		case VK_FORMAT_R16G16B16A16_SINT:
		case VK_FORMAT_R64_UINT:
		case VK_FORMAT_R64_SINT:
			return 64;
		case VK_FORMAT_R32G32B32A32_SFLOAT:
		case VK_FORMAT_R32G32B32A32_UINT:
			return 128;
		case VK_FORMAT_D32_SFLOAT_S8_UINT:
			return 8;
		case VK_FORMAT_BC1_RGB_UNORM_BLOCK:
		case VK_FORMAT_BC1_RGB_SRGB_BLOCK:
		case VK_FORMAT_BC1_RGBA_UNORM_BLOCK:
		case VK_FORMAT_BC1_RGBA_SRGB_BLOCK:
			return 4;
		case VK_FORMAT_BC2_UNORM_BLOCK:
		case VK_FORMAT_BC2_SRGB_BLOCK:
		case VK_FORMAT_BC3_UNORM_BLOCK:
		case VK_FORMAT_BC3_SRGB_BLOCK:
		case VK_FORMAT_BC4_UNORM_BLOCK:
		case VK_FORMAT_BC4_SNORM_BLOCK:
		case VK_FORMAT_BC5_UNORM_BLOCK:
		case VK_FORMAT_BC5_SNORM_BLOCK:
			return 8;
		case VK_FORMAT_BC6H_UFLOAT_BLOCK:
		case VK_FORMAT_BC6H_SFLOAT_BLOCK:
		case VK_FORMAT_BC7_UNORM_BLOCK:
		case VK_FORMAT_BC7_SRGB_BLOCK:
			return 8;

			// No pixel, only blocks!
#if PLATFORM_DESKTOP
			//MapFormatSupport(PF_DXT1, VK_FORMAT_BC1_RGB_UNORM_BLOCK);	// Also what OpenGL expects (RGBA instead RGB, but not SRGB)
			//MapFormatSupport(PF_DXT3, VK_FORMAT_BC2_UNORM_BLOCK);
			//MapFormatSupport(PF_DXT5, VK_FORMAT_BC3_UNORM_BLOCK);
			//MapFormatSupport(PF_BC4, VK_FORMAT_BC4_UNORM_BLOCK);
			//MapFormatSupport(PF_BC5, VK_FORMAT_BC5_UNORM_BLOCK);
			//MapFormatSupport(PF_BC6H, VK_FORMAT_BC6H_UFLOAT_BLOCK);
			//MapFormatSupport(PF_BC7, VK_FORMAT_BC7_UNORM_BLOCK);
#elif PLATFORM_ANDROID
			//MapFormatSupport(PF_ASTC_4x4, VK_FORMAT_ASTC_4x4_UNORM_BLOCK);
			//MapFormatSupport(PF_ASTC_6x6, VK_FORMAT_ASTC_6x6_UNORM_BLOCK);
			//MapFormatSupport(PF_ASTC_8x8, VK_FORMAT_ASTC_8x8_UNORM_BLOCK);
			//MapFormatSupport(PF_ASTC_10x10, VK_FORMAT_ASTC_10x10_UNORM_BLOCK);
			//MapFormatSupport(PF_ASTC_12x12, VK_FORMAT_ASTC_12x12_UNORM_BLOCK);
			//MapFormatSupport(PF_ETC1, VK_FORMAT_ETC2_R8G8B8_UNORM_BLOCK);
			//MapFormatSupport(PF_ETC2_RGB, VK_FORMAT_ETC2_R8G8B8_UNORM_BLOCK);
			//MapFormatSupport(PF_ETC2_RGBA, VK_FORMAT_ETC2_R8G8B8A8_UNORM_BLOCK);
#endif
		default:
			break;
		}

		checkf(0, TEXT("Unhandled bits per pixel for VkFormat %d"), (uint32)Format);
		return 8;
	}

	static VkImageAspectFlags GetAspectMaskFromUEFormat(EPixelFormat Format, bool bIncludeStencil, bool bIncludeDepth = true)
	{
		switch (Format)
		{
		case PF_X24_G8:
			return VK_IMAGE_ASPECT_STENCIL_BIT;
		case PF_DepthStencil:
			return (bIncludeDepth ? VK_IMAGE_ASPECT_DEPTH_BIT : 0) | (bIncludeStencil ? VK_IMAGE_ASPECT_STENCIL_BIT : 0);
		case PF_ShadowDepth:
		case PF_D24:
			return VK_IMAGE_ASPECT_DEPTH_BIT;
		default:
			return VK_IMAGE_ASPECT_COLOR_BIT;
		}
	}

	static bool VulkanFormatHasStencil(VkFormat Format)
	{
		switch (Format)
		{
		case VK_FORMAT_D16_UNORM_S8_UINT:
		case VK_FORMAT_D24_UNORM_S8_UINT:
		case VK_FORMAT_D32_SFLOAT_S8_UINT:
		case VK_FORMAT_S8_UINT:
			return true;
		default:
			return false;
		}
	}
}

#if VULKAN_HAS_DEBUGGING_ENABLED
extern TAutoConsoleVariable<int32> GValidationCvar;
#endif

static inline VkAttachmentLoadOp RenderTargetLoadActionToVulkan(ERenderTargetLoadAction InLoadAction)
{
	VkAttachmentLoadOp OutLoadAction = VK_ATTACHMENT_LOAD_OP_MAX_ENUM;

	switch (InLoadAction)
	{
	case ERenderTargetLoadAction::ELoad:		OutLoadAction = VK_ATTACHMENT_LOAD_OP_LOAD;			break;
	case ERenderTargetLoadAction::EClear:		OutLoadAction = VK_ATTACHMENT_LOAD_OP_CLEAR;		break;
	case ERenderTargetLoadAction::ENoAction:	OutLoadAction = VK_ATTACHMENT_LOAD_OP_DONT_CARE;	break;
	default:																						break;
	}

	// Check for missing translation
	check(OutLoadAction != VK_ATTACHMENT_LOAD_OP_MAX_ENUM);
	return OutLoadAction;
}

static inline VkAttachmentStoreOp RenderTargetStoreActionToVulkan(ERenderTargetStoreAction InStoreAction)
{
	VkAttachmentStoreOp OutStoreAction = VK_ATTACHMENT_STORE_OP_MAX_ENUM;

	switch (InStoreAction)
	{
	case ERenderTargetStoreAction::EStore:
		OutStoreAction = VK_ATTACHMENT_STORE_OP_STORE;
		break;
	case ERenderTargetStoreAction::ENoAction:
	case ERenderTargetStoreAction::EMultisampleResolve:
		OutStoreAction = VK_ATTACHMENT_STORE_OP_DONT_CARE;
		break;
	default:
		break;
	}

	// Check for missing translation
	check(OutStoreAction != VK_ATTACHMENT_STORE_OP_MAX_ENUM);
	return OutStoreAction;
}

extern VkFormat GVulkanSRGBFormat[PF_MAX];
inline VkFormat UEToVkTextureFormat(EPixelFormat UEFormat, const bool bIsSRGB)
{
	if (bIsSRGB)
	{
		return GVulkanSRGBFormat[UEFormat];
	}
	else
	{
		return (VkFormat)GPixelFormats[UEFormat].PlatformFormat;
	}
}

static inline VkFormat UEToVkBufferFormat(EVertexElementType Type)
{
	switch (Type)
	{
	case VET_Float1:
		return VK_FORMAT_R32_SFLOAT;
	case VET_Float2:
		return VK_FORMAT_R32G32_SFLOAT;
	case VET_Float3:
		return VK_FORMAT_R32G32B32_SFLOAT;
	case VET_PackedNormal:
		return VK_FORMAT_R8G8B8A8_SNORM;
	case VET_UByte4:
		return VK_FORMAT_R8G8B8A8_UINT;
	case VET_UByte4N:
		return VK_FORMAT_R8G8B8A8_UNORM;
	case VET_Color:
		return VK_FORMAT_B8G8R8A8_UNORM;
	case VET_Short2:
		return VK_FORMAT_R16G16_SINT;
	case VET_Short4:
		return VK_FORMAT_R16G16B16A16_SINT;
	case VET_Short2N:
		return VK_FORMAT_R16G16_SNORM;
	case VET_Half2:
		return VK_FORMAT_R16G16_SFLOAT;
	case VET_Half4:
		return VK_FORMAT_R16G16B16A16_SFLOAT;
	case VET_Short4N:		// 4 X 16 bit word: normalized
		return VK_FORMAT_R16G16B16A16_SNORM;
	case VET_UShort2:
		return VK_FORMAT_R16G16_UINT;
	case VET_UShort4:
		return VK_FORMAT_R16G16B16A16_UINT;
	case VET_UShort2N:		// 16 bit word normalized to (value/65535.0:value/65535.0:0:0:1)
		return VK_FORMAT_R16G16_UNORM;
	case VET_UShort4N:		// 4 X 16 bit word unsigned: normalized
		return VK_FORMAT_R16G16B16A16_UNORM;
	case VET_Float4:
		return VK_FORMAT_R32G32B32A32_SFLOAT;
	case VET_URGB10A2N:
		return VK_FORMAT_A2B10G10R10_UNORM_PACK32;
	case VET_UInt:
		return VK_FORMAT_R32_UINT;
	default:
		break;
	}

	check(!"Undefined vertex-element format conversion");
	return VK_FORMAT_UNDEFINED;
}

static inline bool IsAstcLdrFormat(VkFormat Format)
{
	return Format >= VK_FORMAT_ASTC_4x4_UNORM_BLOCK && Format <= VK_FORMAT_ASTC_12x12_SRGB_BLOCK;
}

static inline bool IsAstcSrgbFormat(VkFormat Format)
{
	switch (Format)
	{
	case VK_FORMAT_ASTC_4x4_SRGB_BLOCK:
	case VK_FORMAT_ASTC_5x4_SRGB_BLOCK:
	case VK_FORMAT_ASTC_5x5_SRGB_BLOCK:
	case VK_FORMAT_ASTC_6x5_SRGB_BLOCK:
	case VK_FORMAT_ASTC_6x6_SRGB_BLOCK:
	case VK_FORMAT_ASTC_8x5_SRGB_BLOCK:
	case VK_FORMAT_ASTC_8x6_SRGB_BLOCK:
	case VK_FORMAT_ASTC_8x8_SRGB_BLOCK:
	case VK_FORMAT_ASTC_10x5_SRGB_BLOCK:
	case VK_FORMAT_ASTC_10x6_SRGB_BLOCK:
	case VK_FORMAT_ASTC_10x8_SRGB_BLOCK:
	case VK_FORMAT_ASTC_10x10_SRGB_BLOCK:
	case VK_FORMAT_ASTC_12x10_SRGB_BLOCK:
	case VK_FORMAT_ASTC_12x12_SRGB_BLOCK:
		return true;
	default:
		return false;
	}
}

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
extern TAutoConsoleVariable<int32> CVarVulkanDebugBarrier;
#endif

namespace VulkanRHI
{
	static inline FString GetPipelineCacheFilename()
	{
		return FPaths::ProjectSavedDir() / TEXT("VulkanPSO.cache");
	}

	static inline FString GetValidationCacheFilename()
	{
		return FPaths::ProjectSavedDir() / TEXT("VulkanValidation.cache");
	}

#if VULKAN_ENABLE_DRAW_MARKERS
	inline void SetDebugName(PFN_vkSetDebugUtilsObjectNameEXT SetDebugName, VkDevice Device, VkImage Image, const char* Name)
	{
		VkDebugUtilsObjectNameInfoEXT Info;
		ZeroVulkanStruct(Info, VK_STRUCTURE_TYPE_DEBUG_UTILS_OBJECT_NAME_INFO_EXT);
		Info.objectType = VK_OBJECT_TYPE_IMAGE;
		Info.objectHandle = (uint64)Image;
		Info.pObjectName = Name;
		SetDebugName(Device, &Info);
}
#endif

	// Merge a depth and a stencil layout for drivers that don't support VK_KHR_separate_depth_stencil_layouts
	inline VkImageLayout GetMergedDepthStencilLayout(VkImageLayout DepthLayout, VkImageLayout StencilLayout)
	{
		if ((DepthLayout == VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL) || (StencilLayout == VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL))
		{
			checkf(StencilLayout == DepthLayout,
				TEXT("You can't merge transfer src layout without anything else than transfer src (%s != %s).  ")
				TEXT("You need either VK_KHR_separate_depth_stencil_layouts or GRHISupportsSeparateDepthStencilCopyAccess enabled."),
				VK_TYPE_TO_STRING(VkImageLayout, DepthLayout), VK_TYPE_TO_STRING(VkImageLayout, StencilLayout));
			return VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL;
		}

		if ((DepthLayout == VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL) || (StencilLayout == VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL))
		{
			checkf(StencilLayout == DepthLayout,
				TEXT("You can't merge transfer dst layout without anything else than transfer dst (%s != %s).  ")
				TEXT("You need either VK_KHR_separate_depth_stencil_layouts or GRHISupportsSeparateDepthStencilCopyAccess enabled."),
				VK_TYPE_TO_STRING(VkImageLayout, DepthLayout), VK_TYPE_TO_STRING(VkImageLayout, StencilLayout));
			return VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL;
		}

		if ((DepthLayout == VK_IMAGE_LAYOUT_UNDEFINED) && (StencilLayout == VK_IMAGE_LAYOUT_UNDEFINED))
		{
			return VK_IMAGE_LAYOUT_UNDEFINED;
		}

		// Depth formats used on textures that aren't targets (like GBlackTextureDepthCube)
		if ((DepthLayout == VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL) && (StencilLayout == VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL))
		{
			return VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL;
		}

		auto IsMergedLayout = [](VkImageLayout Layout)
		{
			return	(Layout == VK_IMAGE_LAYOUT_DEPTH_STENCIL_ATTACHMENT_OPTIMAL) ||
					(Layout == VK_IMAGE_LAYOUT_DEPTH_STENCIL_READ_ONLY_OPTIMAL) ||
					(Layout == VK_IMAGE_LAYOUT_DEPTH_ATTACHMENT_STENCIL_READ_ONLY_OPTIMAL) ||
					(Layout == VK_IMAGE_LAYOUT_DEPTH_READ_ONLY_STENCIL_ATTACHMENT_OPTIMAL);
		};

		if (IsMergedLayout(DepthLayout) || IsMergedLayout(StencilLayout))
		{
			checkf(StencilLayout == DepthLayout,
				TEXT("Layouts were already merged but they are mismatched (%s != %s)."),
				VK_TYPE_TO_STRING(VkImageLayout, DepthLayout), VK_TYPE_TO_STRING(VkImageLayout, StencilLayout));
			return DepthLayout;
		}

		if (DepthLayout == VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL)
		{
			if ((StencilLayout == VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL) || (StencilLayout == VK_IMAGE_LAYOUT_UNDEFINED))
			{
				return VK_IMAGE_LAYOUT_DEPTH_STENCIL_ATTACHMENT_OPTIMAL;
			}
			else
			{
				check(StencilLayout == VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL);
				return VK_IMAGE_LAYOUT_DEPTH_ATTACHMENT_STENCIL_READ_ONLY_OPTIMAL;
			}
		}
		else if (DepthLayout == VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL)
		{
			if ((StencilLayout == VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL) || (StencilLayout == VK_IMAGE_LAYOUT_UNDEFINED))
			{
				return VK_IMAGE_LAYOUT_DEPTH_STENCIL_READ_ONLY_OPTIMAL;
			}
			else
			{
				check(StencilLayout == VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL);
				return VK_IMAGE_LAYOUT_DEPTH_READ_ONLY_STENCIL_ATTACHMENT_OPTIMAL;
			}
		}
		else
		{
			return (StencilLayout == VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL) ? VK_IMAGE_LAYOUT_DEPTH_STENCIL_ATTACHMENT_OPTIMAL : VK_IMAGE_LAYOUT_DEPTH_STENCIL_READ_ONLY_OPTIMAL;
		}
	}

	inline void HeavyWeightBarrier(VkCommandBuffer CmdBuffer)
	{
		VkMemoryBarrier Barrier;
		ZeroVulkanStruct(Barrier, VK_STRUCTURE_TYPE_MEMORY_BARRIER);
		Barrier.srcAccessMask = VK_ACCESS_INDIRECT_COMMAND_READ_BIT |
			VK_ACCESS_INDEX_READ_BIT |
			VK_ACCESS_VERTEX_ATTRIBUTE_READ_BIT |
			VK_ACCESS_UNIFORM_READ_BIT |
			VK_ACCESS_INPUT_ATTACHMENT_READ_BIT |
			VK_ACCESS_SHADER_READ_BIT |
			VK_ACCESS_SHADER_WRITE_BIT |
			VK_ACCESS_COLOR_ATTACHMENT_READ_BIT |
			VK_ACCESS_COLOR_ATTACHMENT_WRITE_BIT |
			VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_READ_BIT |
			VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_WRITE_BIT |
			VK_ACCESS_TRANSFER_READ_BIT |
			VK_ACCESS_TRANSFER_WRITE_BIT |
			VK_ACCESS_HOST_READ_BIT |
			VK_ACCESS_HOST_WRITE_BIT |
			VK_ACCESS_ACCELERATION_STRUCTURE_READ_BIT_KHR |
			VK_ACCESS_ACCELERATION_STRUCTURE_WRITE_BIT_KHR | 
			VK_ACCESS_FRAGMENT_DENSITY_MAP_READ_BIT_EXT |
			VK_ACCESS_FRAGMENT_SHADING_RATE_ATTACHMENT_READ_BIT_KHR;
		Barrier.dstAccessMask = VK_ACCESS_INDIRECT_COMMAND_READ_BIT |
			VK_ACCESS_INDEX_READ_BIT |
			VK_ACCESS_VERTEX_ATTRIBUTE_READ_BIT |
			VK_ACCESS_UNIFORM_READ_BIT |
			VK_ACCESS_INPUT_ATTACHMENT_READ_BIT |
			VK_ACCESS_SHADER_READ_BIT |
			VK_ACCESS_SHADER_WRITE_BIT |
			VK_ACCESS_COLOR_ATTACHMENT_READ_BIT |
			VK_ACCESS_COLOR_ATTACHMENT_WRITE_BIT |
			VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_READ_BIT |
			VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_WRITE_BIT |
			VK_ACCESS_TRANSFER_READ_BIT |
			VK_ACCESS_TRANSFER_WRITE_BIT |
			VK_ACCESS_HOST_READ_BIT |
			VK_ACCESS_HOST_WRITE_BIT |
			VK_ACCESS_ACCELERATION_STRUCTURE_READ_BIT_KHR |
			VK_ACCESS_ACCELERATION_STRUCTURE_WRITE_BIT_KHR |
			VK_ACCESS_FRAGMENT_DENSITY_MAP_READ_BIT_EXT |
			VK_ACCESS_FRAGMENT_SHADING_RATE_ATTACHMENT_READ_BIT_KHR;
		VulkanRHI::vkCmdPipelineBarrier(CmdBuffer, VK_PIPELINE_STAGE_ALL_COMMANDS_BIT, VK_PIPELINE_STAGE_ALL_COMMANDS_BIT, 0, 1, &Barrier, 0, nullptr, 0, nullptr);
	}

	inline void DebugHeavyWeightBarrier(VkCommandBuffer CmdBuffer, int32 CVarConditionMask)
	{
#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
		if (CVarVulkanDebugBarrier.GetValueOnAnyThread() & CVarConditionMask)
		{
			HeavyWeightBarrier(CmdBuffer);
		}
#endif
	}
}

inline bool UseVulkanDescriptorCache()
{
	// Descriptor cache path for WriteAccelerationStructure() is not implemented, so disable if RT is enabled
	return GMaxRHIFeatureLevel <= ERHIFeatureLevel::ES3_1;
}

inline bool ValidateShadingRateDataType()
{
	switch (GRHIVariableRateShadingImageDataType)
	{
	case VRSImage_Palette:
		return true;

	case VRSImage_Fractional:
		return true;

	case VRSImage_NotSupported:
		checkf(false, TEXT("A texture was marked as a shading rate source but attachment VRS is not supported on this device. Ensure GRHISupportsAttachmentVariableRateShading and GRHIAttachmentVariableRateShadingEnabled are true before specifying a shading rate attachment."));
		break;

	default:
		checkf(false, TEXT("Unrecognized shading rate image data type. Specified type was %d"), (int)GRHIVariableRateShadingImageDataType);
		break;
	}

	return false;
}

extern int32 GWaitForIdleOnSubmit;

#if VULKAN_HAS_DEBUGGING_ENABLED
extern bool GRenderDocFound;
#endif

const int GMaxCrashBufferEntries = 2048;

extern VULKANRHI_API class FVulkanDynamicRHI* GVulkanRHI;

extern TAtomic<uint64> GVulkanBufferHandleIdCounter;
extern TAtomic<uint64> GVulkanBufferViewHandleIdCounter;
extern TAtomic<uint64> GVulkanImageViewHandleIdCounter;
extern TAtomic<uint64> GVulkanSamplerHandleIdCounter;
extern TAtomic<uint64> GVulkanDSetLayoutHandleIdCounter;
