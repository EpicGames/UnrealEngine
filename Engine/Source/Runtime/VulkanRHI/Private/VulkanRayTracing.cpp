// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanRayTracing.h"

#include "VulkanContext.h"
#include "VulkanDescriptorSets.h"
#include "VulkanBindlessDescriptorManager.h"
#include "BuiltInRayTracingShaders.h"
#include "Experimental/Containers/SherwoodHashTable.h"
#include "Async/ParallelFor.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "ShaderParameterMacros.h"

static int32 GVulkanRayTracingAllowCompaction = 1;
static FAutoConsoleVariableRef CVarVulkanRayTracingAllowCompaction(
	TEXT("r.Vulkan.RayTracing.AllowCompaction"),
	GVulkanRayTracingAllowCompaction,
	TEXT("Whether to automatically perform compaction for static acceleration structures to save GPU memory. (default = 1)\n"),
	ECVF_ReadOnly
);

static int32 GVulkanRayTracingMaxBatchedCompaction = 64;
static FAutoConsoleVariableRef CVarVulkanRayTracingMaxBatchedCompaction(
	TEXT("r.Vulkan.RayTracing.MaxBatchedCompaction"),
	GVulkanRayTracingMaxBatchedCompaction,
	TEXT("Maximum of amount of compaction requests and rebuilds per frame. (default = 64)\n"),
	ECVF_ReadOnly
);

static int32 GVulkanRayTracingCompactionMinPrimitiveCount = 128;
static FAutoConsoleVariableRef CVarVulkanRayTracingCompactionMinPrimitiveCount(
	TEXT("r.Vulkan.RayTracing.Compaction.MinPrimitiveCount"),
	GVulkanRayTracingCompactionMinPrimitiveCount,
	TEXT("Sets the minimum primitive count threshold below which geometry skips the compaction. (default = 128)\n")
);

static int32 GVulkanRayTracingMaxShaderGroupStride = 512;
static FAutoConsoleVariableRef GCVarVulkanRayTracingMaxShaderGroupStride(
	TEXT("r.Vulkan.RayTracing.MaxShaderGroupStride"),
	GVulkanRayTracingMaxShaderGroupStride,
	TEXT("The default size to allocate for each record (default: 512)."),
	ECVF_ReadOnly
);

// Ray tracing stat counters

DECLARE_STATS_GROUP(TEXT("Vulkan: Ray Tracing"), STATGROUP_VulkanRayTracing, STATCAT_Advanced);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Created pipelines (total)"), STAT_VulkanRayTracingCreatedPipelines, STATGROUP_VulkanRayTracing);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Compiled shaders (total)"), STAT_VulkanRayTracingCompiledShaders, STATGROUP_VulkanRayTracing);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Allocated bottom level acceleration structures"), STAT_VulkanRayTracingAllocatedBLAS, STATGROUP_VulkanRayTracing);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Allocated top level acceleration structures"), STAT_VulkanRayTracingAllocatedTLAS, STATGROUP_VulkanRayTracing);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Triangles in all BL acceleration structures"), STAT_VulkanRayTracingTrianglesBLAS, STATGROUP_VulkanRayTracing);

DECLARE_DWORD_COUNTER_STAT(TEXT("Built BL AS (per frame)"), STAT_VulkanRayTracingBuiltBLAS, STATGROUP_VulkanRayTracing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Updated BL AS (per frame)"), STAT_VulkanRayTracingUpdatedBLAS, STATGROUP_VulkanRayTracing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Built TL AS (per frame)"), STAT_VulkanRayTracingBuiltTLAS, STATGROUP_VulkanRayTracing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Updated TL AS (per frame)"), STAT_VulkanRayTracingUpdatedTLAS, STATGROUP_VulkanRayTracing);

DECLARE_MEMORY_STAT(TEXT("Total BL AS Memory"), STAT_VulkanRayTracingBLASMemory, STATGROUP_VulkanRayTracing);
DECLARE_MEMORY_STAT(TEXT("Static BL AS Memory"), STAT_VulkanRayTracingStaticBLASMemory, STATGROUP_VulkanRayTracing);
DECLARE_MEMORY_STAT(TEXT("Dynamic BL AS Memory"), STAT_VulkanRayTracingDynamicBLASMemory, STATGROUP_VulkanRayTracing);
DECLARE_MEMORY_STAT(TEXT("TL AS Memory"), STAT_VulkanRayTracingTLASMemory, STATGROUP_VulkanRayTracing);
DECLARE_MEMORY_STAT(TEXT("Total Used Video Memory"), STAT_VulkanRayTracingUsedVideoMemory, STATGROUP_VulkanRayTracing);

DECLARE_CYCLE_STAT(TEXT("RTPSO Compile Shader"), STAT_RTPSO_CompileShader, STATGROUP_VulkanRayTracing);
DECLARE_CYCLE_STAT(TEXT("RTPSO Create Pipeline"), STAT_RTPSO_CreatePipeline, STATGROUP_VulkanRayTracing);


bool FVulkanRayTracingPlatform::CheckVulkanInstanceFunctions(VkInstance inInstance)
{
	bool bFoundAllEntryPoints = true;
#define CHECK_VK_ENTRYPOINTS(Type,Func) if (VulkanDynamicAPI::Func == NULL) { bFoundAllEntryPoints = false; UE_LOGF(LogRHI, Warning, "Failed to find entry point for %ls", TEXT(#Func)); }
#if UE_BUILD_DEBUG
	ENUM_VK_ENTRYPOINTS_RAYTRACING(CHECK_VK_ENTRYPOINTS);
#endif
	return bFoundAllEntryPoints;
}

enum class EBLASBuildDataUsage
{	
	// Uses provided VB/IB when filling out BLAS build data
	Rendering = 0,

	// Does not use VB/IB. Special mode for estimating BLAS size.
	Size = 1
};

// Temporary brute force allocation helper, this should be handled by the memory sub-allocator
static uint32 FindMemoryType(VkPhysicalDevice Gpu, uint32 Filter, VkMemoryPropertyFlags RequestedProperties)
{
	VkPhysicalDeviceMemoryProperties Properties = {};
	VulkanRHI::vkGetPhysicalDeviceMemoryProperties(Gpu, &Properties);

	uint32 Result = UINT32_MAX;
	for (uint32 i = 0; i < Properties.memoryTypeCount; ++i)
	{
		const bool bTypeFilter = Filter & (1 << i);
		const bool bPropFilter = (Properties.memoryTypes[i].propertyFlags & RequestedProperties) == RequestedProperties;
		if (bTypeFilter && bPropFilter)
		{
			Result = i;
			break;
		}
	}

	check(Result < UINT32_MAX);
	return Result;
}

static void AddAccelerationStructureBuildBarrier(VkCommandBuffer CommandBuffer)
{
	VkMemoryBarrier Barrier;
	ZeroVulkanStruct(Barrier, VK_STRUCTURE_TYPE_MEMORY_BARRIER);
	Barrier.srcAccessMask = VK_ACCESS_ACCELERATION_STRUCTURE_WRITE_BIT_KHR;
	Barrier.dstAccessMask = VK_ACCESS_ACCELERATION_STRUCTURE_READ_BIT_KHR | VK_ACCESS_ACCELERATION_STRUCTURE_WRITE_BIT_KHR;
    
    // TODO: Revisit the compute stages here as we don't always need barrier to compute
    VkPipelineStageFlags srcStage = VK_PIPELINE_STAGE_ACCELERATION_STRUCTURE_BUILD_BIT_KHR | VK_PIPELINE_STAGE_COMPUTE_SHADER_BIT;
    VkPipelineStageFlags dstStage = VK_PIPELINE_STAGE_ACCELERATION_STRUCTURE_BUILD_BIT_KHR | VK_PIPELINE_STAGE_COMPUTE_SHADER_BIT;
    
	VulkanRHI::vkCmdPipelineBarrier(CommandBuffer, srcStage, dstStage, 0, 1, &Barrier, 0, nullptr, 0, nullptr);
}

static bool ShouldCompactAfterBuild(ERayTracingAccelerationStructureFlags BuildFlags)
{
	return EnumHasAllFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowCompaction | ERayTracingAccelerationStructureFlags::FastTrace)
		&& !EnumHasAnyFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate);
}

static VkBuildAccelerationStructureFlagBitsKHR TranslateRayTracingAccelerationStructureFlags(ERayTracingAccelerationStructureFlags Flags)
{
	uint32 Result = {};

	auto HandleFlag = [&Flags, &Result](ERayTracingAccelerationStructureFlags Engine, VkBuildAccelerationStructureFlagBitsKHR Native)
		{
			if (EnumHasAllFlags(Flags, Engine))
			{
				Result |= (uint32)Native;
				EnumRemoveFlags(Flags, Engine);
			}
		};

	HandleFlag(ERayTracingAccelerationStructureFlags::AllowUpdate, VK_BUILD_ACCELERATION_STRUCTURE_ALLOW_UPDATE_BIT_KHR);
	HandleFlag(ERayTracingAccelerationStructureFlags::AllowCompaction, VK_BUILD_ACCELERATION_STRUCTURE_ALLOW_COMPACTION_BIT_KHR);
	HandleFlag(ERayTracingAccelerationStructureFlags::FastTrace, VK_BUILD_ACCELERATION_STRUCTURE_PREFER_FAST_TRACE_BIT_KHR);
	HandleFlag(ERayTracingAccelerationStructureFlags::FastBuild, VK_BUILD_ACCELERATION_STRUCTURE_PREFER_FAST_BUILD_BIT_KHR);
	HandleFlag(ERayTracingAccelerationStructureFlags::MinimizeMemory, VK_BUILD_ACCELERATION_STRUCTURE_LOW_MEMORY_BIT_KHR);

	checkf(!EnumHasAnyFlags(Flags, Flags), TEXT("Some ERayTracingAccelerationStructureFlags entries were not handled"));

#if VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH
	Result |= (uint32)VK_BUILD_ACCELERATION_STRUCTURE_ALLOW_DATA_ACCESS_KHR;
#endif

	return VkBuildAccelerationStructureFlagBitsKHR(Result);
}

static ERayTracingAccelerationStructureFlags GetRayTracingAccelerationStructureBuildFlags(const FRayTracingGeometryInitializer& Initializer)
{
	ERayTracingAccelerationStructureFlags BuildFlags = ERayTracingAccelerationStructureFlags::None;

	if (Initializer.bFastBuild)
	{
		BuildFlags = ERayTracingAccelerationStructureFlags::FastBuild;
	}
	else
	{
		BuildFlags = ERayTracingAccelerationStructureFlags::FastTrace;
	}

	if (Initializer.bAllowUpdate)
	{
		EnumAddFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate);
	}

	if (!Initializer.bFastBuild && !Initializer.bAllowUpdate && Initializer.bAllowCompaction && GVulkanRayTracingAllowCompaction && (uint32(GVulkanRayTracingCompactionMinPrimitiveCount) < Initializer.TotalPrimitiveCount))
	{
		EnumAddFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowCompaction);
	}

	return BuildFlags;
}

static void GetBLASBuildData(
	const VkDevice Device,
	const TConstArrayView<FRayTracingGeometrySegment> Segments,
	const ERayTracingGeometryType GeometryType,
	const FBufferRHIRef IndexBufferRHI,
	const uint32 IndexBufferOffset,
	ERayTracingAccelerationStructureFlags BuildFlags,
	const EAccelerationStructureBuildMode BuildMode,
	const EBLASBuildDataUsage Usage,
	FVkRtBLASBuildData& BuildData)
{
	FVulkanBuffer* const IndexBuffer = ResourceCast(IndexBufferRHI.GetReference());
	VkDeviceOrHostAddressConstKHR IndexBufferDeviceAddress = {};
	
	// We only need to get IB/VB address when we are getting data for rendering. For estimating BLAS size we set them to 0.
	// According to vulkan spec any VkDeviceOrHostAddressKHR members are ignored in vkGetAccelerationStructureBuildSizesKHR.
	uint32 IndexStrideInBytes = 0;
	if (IndexBufferRHI)
	{
		IndexBufferDeviceAddress.deviceAddress = Usage == EBLASBuildDataUsage::Rendering ? IndexBuffer->GetDeviceAddress() + IndexBufferOffset : 0;

		// In case we are just calculating size but index buffer is not yet in valid state we assume the geometry is using uint32 format
		IndexStrideInBytes = Usage == EBLASBuildDataUsage::Rendering
			? IndexBuffer->GetStride()
			: (IndexBuffer->GetSize() > 0) ? IndexBuffer->GetStride() : 4;
	}

	// Pre allocate LSS Segments to avoid memory reallocation
	if (GeometryType == RTGT_LinearSweptSpheres)
	{
		BuildData.LSSSegments.Reserve(Segments.Num());
	}

	TArray<uint32, TInlineAllocator<1>> PrimitiveCounts;

	for (int32 SegmentIndex = 0; SegmentIndex < Segments.Num(); ++SegmentIndex)
	{
		const FRayTracingGeometrySegment& Segment = Segments[SegmentIndex];

		FVulkanBuffer* const VertexBuffer = ResourceCast(Segment.VertexBuffer.GetReference());

		VkDeviceOrHostAddressConstKHR VertexBufferDeviceAddress = {};
		VertexBufferDeviceAddress.deviceAddress = Usage == EBLASBuildDataUsage::Rendering
			? VertexBuffer->GetDeviceAddress() + Segment.VertexBufferOffset
			: 0;

		VkAccelerationStructureGeometryKHR SegmentGeometry;
		ZeroVulkanStruct(SegmentGeometry, VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_GEOMETRY_KHR);

		if (Segment.bForceOpaque)
		{
			SegmentGeometry.flags |= VK_GEOMETRY_OPAQUE_BIT_KHR;
		}

		if (!Segment.bAllowDuplicateAnyHitShaderInvocation)
		{
			// Allow only a single any-hit shader invocation per primitive
			SegmentGeometry.flags |= VK_GEOMETRY_NO_DUPLICATE_ANY_HIT_INVOCATION_BIT_KHR;
		}

		VkAccelerationStructureGeometryLinearSweptSpheresDataNV LSSSegmentGeometry;
		ZeroVulkanStruct(LSSSegmentGeometry, VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_GEOMETRY_LINEAR_SWEPT_SPHERES_DATA_NV);

		uint32 PrimitiveOffset = 0;
		switch (GeometryType)
		{
			case RTGT_Triangles:
				SegmentGeometry.geometryType = VK_GEOMETRY_TYPE_TRIANGLES_KHR;

				SegmentGeometry.geometry.triangles.sType = VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_GEOMETRY_TRIANGLES_DATA_KHR;
				SegmentGeometry.geometry.triangles.vertexFormat = VK_FORMAT_R32G32B32_SFLOAT;
				SegmentGeometry.geometry.triangles.vertexData = VertexBufferDeviceAddress;
				SegmentGeometry.geometry.triangles.maxVertex = Segment.MaxVertices;
				SegmentGeometry.geometry.triangles.vertexStride = Segment.VertexBufferStride;
				SegmentGeometry.geometry.triangles.indexData = IndexBufferDeviceAddress;

				switch (Segment.VertexBufferElementType)
				{
				case VET_Float3:
				case VET_Float4:
					SegmentGeometry.geometry.triangles.vertexFormat = VK_FORMAT_R32G32B32_SFLOAT;
					break;
				default:
					checkNoEntry();
					break;
				}

				// No support for segment transform
				SegmentGeometry.geometry.triangles.transformData.deviceAddress = 0;
				SegmentGeometry.geometry.triangles.transformData.hostAddress = nullptr;

				if (IndexBufferRHI)
				{
					SegmentGeometry.geometry.triangles.indexType = (IndexStrideInBytes == 2) ? VK_INDEX_TYPE_UINT16 : VK_INDEX_TYPE_UINT32;
					// offset in bytes into the index buffer where primitive data for the current segment is defined
					PrimitiveOffset = Segment.FirstPrimitive * FVulkanRayTracingGeometry::IndicesPerPrimitive * IndexStrideInBytes;
				}
				else
				{
					SegmentGeometry.geometry.triangles.indexType = VK_INDEX_TYPE_NONE_KHR;
					// for non-indexed geometry, primitiveOffset is applied when reading from vertex buffer
					PrimitiveOffset = Segment.FirstPrimitive * FVulkanRayTracingGeometry::IndicesPerPrimitive * Segment.VertexBufferStride;
				}

				break;
			case RTGT_Procedural:
				checkf(Segment.VertexBufferStride >= (2 * sizeof(FVector3f)), TEXT("Procedural geometry vertex buffer must contain at least 2xFloat3 that defines 3D bounding boxes of primitives."));
				checkf(Segment.VertexBufferStride % 8 == 0, TEXT("Procedural geometry vertex buffer stride must be a multiple of 8."));

				SegmentGeometry.geometryType = VK_GEOMETRY_TYPE_AABBS_KHR;
				
				SegmentGeometry.geometry.aabbs.sType = VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_GEOMETRY_AABBS_DATA_KHR;
				SegmentGeometry.geometry.aabbs.data = VertexBufferDeviceAddress;
				SegmentGeometry.geometry.aabbs.stride = Segment.VertexBufferStride;

				break;

			case RTGT_LinearSweptSpheres:
				// no other application rather than hair strands for now.
				check(IndexStrideInBytes == 4);
				check(IndexBufferRHI != nullptr); // must have an index buffer in this case

				// there are 2 indices per primitive
				PrimitiveOffset = Segment.FirstPrimitive * 2 * IndexStrideInBytes;

				LSSSegmentGeometry.indexType = VK_INDEX_TYPE_UINT32;
				LSSSegmentGeometry.indexData = IndexBufferDeviceAddress;
				LSSSegmentGeometry.indexStride = sizeof(uint32);
				LSSSegmentGeometry.indexingMode = VkRayTracingLssIndexingModeNV::VK_RAY_TRACING_LSS_INDEXING_MODE_LIST_NV;
				
				LSSSegmentGeometry.vertexData = VertexBufferDeviceAddress;
				LSSSegmentGeometry.vertexFormat = VkFormat::VK_FORMAT_R32G32B32_SFLOAT;
				LSSSegmentGeometry.vertexStride = 4 * sizeof(float);
				
				LSSSegmentGeometry.radiusData = {.deviceAddress = VertexBufferDeviceAddress.deviceAddress + 3 * sizeof(float)};
				LSSSegmentGeometry.radiusFormat = VkFormat::VK_FORMAT_R32_SFLOAT;
				LSSSegmentGeometry.radiusStride = 4 * sizeof(float);

				LSSSegmentGeometry.endCapsMode = VkRayTracingLssPrimitiveEndCapsModeNV::VK_RAY_TRACING_LSS_PRIMITIVE_END_CAPS_MODE_NONE_NV;
				
				BuildData.LSSSegments.Add(LSSSegmentGeometry);

				SegmentGeometry.pNext = &BuildData.LSSSegments[SegmentIndex];
				SegmentGeometry.geometryType = VkGeometryTypeKHR::VK_GEOMETRY_TYPE_LINEAR_SWEPT_SPHERES_NV;

				break;
			default:
				checkf(false, TEXT("Unexpected ray tracing geometry type"));
				break;
		}

		BuildData.Segments.Add(SegmentGeometry);

		VkAccelerationStructureBuildRangeInfoKHR RangeInfo = {};
		RangeInfo.firstVertex = 0;

		// Disabled segments use an empty range. We still build them to keep the sbt valid.
		RangeInfo.primitiveCount = (Segment.bEnabled) ? Segment.NumPrimitives : 0;
		RangeInfo.primitiveOffset = PrimitiveOffset;
		RangeInfo.transformOffset = 0;

		BuildData.Ranges.Add(RangeInfo);

		PrimitiveCounts.Add(Segment.NumPrimitives);
	}

	BuildData.GeometryInfo.type = VK_ACCELERATION_STRUCTURE_TYPE_BOTTOM_LEVEL_KHR;
	BuildData.GeometryInfo.flags = (EnumHasAnyFlags(BuildFlags, ERayTracingAccelerationStructureFlags::FastBuild))
		? VK_BUILD_ACCELERATION_STRUCTURE_PREFER_FAST_BUILD_BIT_KHR 
		: VK_BUILD_ACCELERATION_STRUCTURE_PREFER_FAST_TRACE_BIT_KHR;
	if (EnumHasAnyFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate))
	{
		BuildData.GeometryInfo.flags |= VK_BUILD_ACCELERATION_STRUCTURE_ALLOW_UPDATE_BIT_KHR;
	}
	if (EnumHasAnyFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowCompaction))
	{
		BuildData.GeometryInfo.flags |= VK_BUILD_ACCELERATION_STRUCTURE_ALLOW_COMPACTION_BIT_KHR;
	}
#if VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH
	BuildData.GeometryInfo.flags |= VK_BUILD_ACCELERATION_STRUCTURE_ALLOW_DATA_ACCESS_KHR;
#endif

	BuildData.GeometryInfo.mode = (BuildMode == EAccelerationStructureBuildMode::Build) ? VK_BUILD_ACCELERATION_STRUCTURE_MODE_BUILD_KHR : VK_BUILD_ACCELERATION_STRUCTURE_MODE_UPDATE_KHR;
	BuildData.GeometryInfo.geometryCount = BuildData.Segments.Num();
	BuildData.GeometryInfo.pGeometries = BuildData.Segments.GetData();

	VulkanDynamicAPI::vkGetAccelerationStructureBuildSizesKHR(
		Device,
		VK_ACCELERATION_STRUCTURE_BUILD_TYPE_DEVICE_KHR,
		&BuildData.GeometryInfo,
		PrimitiveCounts.GetData(),
		&BuildData.SizesInfo);
}

// This structure is analogous to FHitGroupSystemParameters in D3D12 RHI.
// However, it only contains generic parameters that do not require a full shader binding table (i.e. no per-hit-group user data).
// It is designed to be used to access vertex and index buffers during inline ray tracing.
struct FVulkanRayTracingGeometryParameters
{
	union
	{
		struct
		{
			uint32 IndexStride : 8; // Can be just 1 bit to indicate 16 or 32 bit indices
			uint32 VertexStride : 8; // Can be just 2 bits to indicate float3, float2 or half2 format
			uint32 Unused : 16;
		} Config;
		uint32 ConfigBits = 0;
	};
	uint32 IndexBufferOffsetInBytes = 0;
	uint64 IndexBuffer = 0;
	uint64 VertexBuffer = 0;
};

FVulkanRayTracingGeometry::FVulkanRayTracingGeometry(ENoInit)
{}

FVulkanRayTracingGeometry::FVulkanRayTracingGeometry(FRHICommandListBase& RHICmdList, const FRayTracingGeometryInitializer& InInitializer, FVulkanDevice* InDevice)
	: FRHIRayTracingGeometry(InInitializer), Device(InDevice)
{
	INC_DWORD_STAT(STAT_VulkanRayTracingAllocatedBLAS);

	DebugName = !Initializer.DebugName.IsNone() ? Initializer.DebugName : FDebugName(FName("BLAS"));
	OwnerName = Initializer.OwnerName;

	uint32 IndexBufferStride = 0;
	if (Initializer.IndexBuffer)
	{
		// In case index buffer in initializer is not yet in valid state during streaming we assume the geometry is using UINT32 format.
		IndexBufferStride = Initializer.IndexBuffer->GetSize() > 0
			? Initializer.IndexBuffer->GetStride()
			: 4;
	}

	checkf(!Initializer.IndexBuffer || (IndexBufferStride == 2 || IndexBufferStride == 4), TEXT("Index buffer must be 16 or 32 bit if in use."));

	SizeInfo = RHICalcRayTracingGeometrySize(Initializer);

	// If this RayTracingGeometry going to be used as streaming destination 
	// we don't want to allocate its memory as it will be replaced later by streamed version
	// but we still need correct SizeInfo as it is used to estimate its memory requirements outside of RHI.
	if (Initializer.Type == ERayTracingGeometryInitializerType::StreamingDestination)
	{
		return;
	}

	FString DebugNameString = Initializer.DebugName.ToString();

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::Create(*DebugNameString, SizeInfo.ResultSize, 0, EBufferUsageFlags::AccelerationStructure)
		.SetInitialState(ERHIAccess::BVHWrite);

	AccelerationStructureBuffer = ResourceCast(RHICmdList.CreateBuffer(CreateDesc).GetReference());

	VkDevice NativeDevice = Device->GetHandle();

	VkAccelerationStructureCreateInfoKHR CreateInfo;
	ZeroVulkanStruct(CreateInfo, VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_CREATE_INFO_KHR);
	CreateInfo.buffer = AccelerationStructureBuffer->GetHandle();
	CreateInfo.offset = AccelerationStructureBuffer->GetOffset();
	CreateInfo.size = SizeInfo.ResultSize;
	CreateInfo.type = VK_ACCELERATION_STRUCTURE_TYPE_BOTTOM_LEVEL_KHR;
	VERIFYVULKANRESULT(VulkanDynamicAPI::vkCreateAccelerationStructureKHR(NativeDevice, &CreateInfo, VULKAN_CPU_ALLOCATOR, &Handle));
	VULKAN_SET_DEBUG_NAME(*Device, VK_OBJECT_TYPE_ACCELERATION_STRUCTURE_KHR, Handle, TEXT("%s"), *DebugName.ToString());

	INC_MEMORY_STAT_BY(STAT_VulkanRayTracingUsedVideoMemory, SizeInfo.ResultSize);
	INC_MEMORY_STAT_BY(STAT_VulkanRayTracingBLASMemory, SizeInfo.ResultSize);
	if (Initializer.bAllowUpdate)
	{
		INC_MEMORY_STAT_BY(STAT_VulkanRayTracingDynamicBLASMemory, SizeInfo.ResultSize);
	}
	else
	{
		INC_MEMORY_STAT_BY(STAT_VulkanRayTracingStaticBLASMemory, SizeInfo.ResultSize);
	}

	VkAccelerationStructureDeviceAddressInfoKHR DeviceAddressInfo;
	ZeroVulkanStruct(DeviceAddressInfo, VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_DEVICE_ADDRESS_INFO_KHR);
	DeviceAddressInfo.accelerationStructure = Handle;
	Address = VulkanDynamicAPI::vkGetAccelerationStructureDeviceAddressKHR(NativeDevice, &DeviceAddressInfo);

	INC_DWORD_STAT_BY(STAT_VulkanRayTracingTrianglesBLAS, Initializer.TotalPrimitiveCount);
}

FVulkanRayTracingGeometry::~FVulkanRayTracingGeometry()
{
	ReleaseBindlessHandles();

	DEC_DWORD_STAT(STAT_VulkanRayTracingAllocatedBLAS);
	DEC_DWORD_STAT_BY(STAT_VulkanRayTracingTrianglesBLAS, Initializer.TotalPrimitiveCount);
	if (Handle != VK_NULL_HANDLE)
	{
		DEC_MEMORY_STAT_BY(STAT_VulkanRayTracingUsedVideoMemory, AccelerationStructureBuffer->GetSize());
		DEC_MEMORY_STAT_BY(STAT_VulkanRayTracingBLASMemory, AccelerationStructureBuffer->GetSize());

		ERayTracingAccelerationStructureFlags BuildFlags = GetRayTracingAccelerationStructureBuildFlags(Initializer);
		if (EnumHasAllFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate))
		{
			DEC_MEMORY_STAT_BY(STAT_VulkanRayTracingDynamicBLASMemory, AccelerationStructureBuffer->GetSize());
		}
		else
		{
			DEC_MEMORY_STAT_BY(STAT_VulkanRayTracingStaticBLASMemory, AccelerationStructureBuffer->GetSize());
		}

		Device->GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::AccelerationStructure, Handle);
	}	

	RemoveCompactionRequest();
}

void FVulkanRayTracingGeometry::Swap(FVulkanRayTracingGeometry& Other)
{
	::Swap(Handle, Other.Handle);
	::Swap(Address, Other.Address);
	::Swap(AccelerationStructureCompactedSize, Other.AccelerationStructureCompactedSize);

	AccelerationStructureBuffer = Other.AccelerationStructureBuffer;

	Initializer = Other.Initializer;

	// TODO: Update HitGroup Parameters
}

void FVulkanRayTracingGeometry::RemoveCompactionRequest()
{
	if (bHasPendingCompactionRequests)
	{
		check(AccelerationStructureBuffer);
		bool bRequestFound = Device->GetRayTracingCompactionRequestHandler()->ReleaseRequest(this);
		check(bRequestFound);
		bHasPendingCompactionRequests = false;
	}
}

void FVulkanRayTracingGeometry::CompactAccelerationStructure(FVulkanCommandBuffer& CmdBuffer, uint64 InSizeAfterCompaction)
{
	check(bHasPendingCompactionRequests);
	bHasPendingCompactionRequests = false;

	ensureMsgf(InSizeAfterCompaction > 0, TEXT("Compacted acceleration structure size is expected to be non-zero. This error suggests that GPU readback synchronization is broken."));
	if (InSizeAfterCompaction == 0)
	{
		return;
	}

	DEC_MEMORY_STAT_BY(STAT_VulkanRayTracingUsedVideoMemory, AccelerationStructureBuffer->GetSize());
	DEC_MEMORY_STAT_BY(STAT_VulkanRayTracingBLASMemory, AccelerationStructureBuffer->GetSize());
	DEC_MEMORY_STAT_BY(STAT_VulkanRayTracingStaticBLASMemory, AccelerationStructureBuffer->GetSize());

	// Move old AS into this temporary variable which gets released when this function returns	
	TRefCountPtr<FVulkanBuffer> OldAccelerationStructure = AccelerationStructureBuffer;
	VkAccelerationStructureKHR OldHandle = Handle;

	const FString DebugNameString = Initializer.DebugName.ToString();
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::Create(*DebugNameString, InSizeAfterCompaction, 0, BUF_AccelerationStructure)
		.SetInitialState(ERHIAccess::BVHWrite);

	AccelerationStructureBuffer = new FVulkanBuffer(*Device, CreateDesc);

	INC_MEMORY_STAT_BY(STAT_VulkanRayTracingUsedVideoMemory, AccelerationStructureBuffer->GetSize());
	INC_MEMORY_STAT_BY(STAT_VulkanRayTracingBLASMemory, AccelerationStructureBuffer->GetSize());
	INC_MEMORY_STAT_BY(STAT_VulkanRayTracingStaticBLASMemory, AccelerationStructureBuffer->GetSize());

	VkDevice NativeDevice = Device->GetHandle();

	VkAccelerationStructureCreateInfoKHR CreateInfo;
	ZeroVulkanStruct(CreateInfo, VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_CREATE_INFO_KHR);
	CreateInfo.buffer = AccelerationStructureBuffer->GetHandle();
	CreateInfo.offset = AccelerationStructureBuffer->GetOffset();
	CreateInfo.size = InSizeAfterCompaction;
	CreateInfo.type = VK_ACCELERATION_STRUCTURE_TYPE_BOTTOM_LEVEL_KHR;
	VERIFYVULKANRESULT(VulkanDynamicAPI::vkCreateAccelerationStructureKHR(NativeDevice, &CreateInfo, VULKAN_CPU_ALLOCATOR, &Handle));
	VULKAN_SET_DEBUG_NAME(*Device, VK_OBJECT_TYPE_ACCELERATION_STRUCTURE_KHR, Handle, TEXT("%s (compact)"), *DebugName.ToString());
	VkAccelerationStructureDeviceAddressInfoKHR DeviceAddressInfo;
	ZeroVulkanStruct(DeviceAddressInfo, VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_DEVICE_ADDRESS_INFO_KHR);
	DeviceAddressInfo.accelerationStructure = Handle;
	Address = VulkanDynamicAPI::vkGetAccelerationStructureDeviceAddressKHR(NativeDevice, &DeviceAddressInfo);

	// Add a barrier to make sure acceleration structure are synchronized correctly for the copy command.
	AddAccelerationStructureBuildBarrier(CmdBuffer.GetHandle());

	VkCopyAccelerationStructureInfoKHR CopyInfo;
	ZeroVulkanStruct(CopyInfo, VK_STRUCTURE_TYPE_COPY_ACCELERATION_STRUCTURE_INFO_KHR);
	CopyInfo.src = OldHandle;
	CopyInfo.dst = Handle;
	CopyInfo.mode = VK_COPY_ACCELERATION_STRUCTURE_MODE_COMPACT_KHR;
	VulkanDynamicAPI::vkCmdCopyAccelerationStructureKHR(CmdBuffer.GetHandle(), &CopyInfo);

	AccelerationStructureCompactedSize = InSizeAfterCompaction;

	Device->GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::AccelerationStructure, OldHandle);
}


void FVulkanRayTracingGeometry::SetupHitGroupSystemParameters()
{
	const bool bIsTriangles = (Initializer.GeometryType == RTGT_Triangles);

	FVulkanBindlessDescriptorManager* BindlessDescriptorManager = Device->GetBindlessDescriptorManager();
	auto GetBindlessHandle = [BindlessDescriptorManager](FVulkanBuffer* Buffer, uint32 ExtraOffset)
	{
		if (Buffer)
		{
			FRHIDescriptorHandle BindlessHandle = BindlessDescriptorManager->AllocateDescriptor(ERHIDescriptorType::BufferUAV);
			check(ExtraOffset < Buffer->GetCurrentSize());
			BindlessDescriptorManager->UpdateDescriptor({}, BindlessHandle, Buffer->GetHandle(), Buffer->GetOffset() + ExtraOffset, Buffer->GetCurrentSize() - ExtraOffset, true);
			return BindlessHandle;
		}
		return FRHIDescriptorHandle();
	};

	ReleaseBindlessHandles();

	HitGroupSystemParameters.Reset(Initializer.Segments.Num());

	FVulkanBuffer* IndexBuffer = ResourceCast(Initializer.IndexBuffer.GetReference());
	const uint32 IndexStride = IndexBuffer ? IndexBuffer->GetStride() : 0;
	HitGroupSystemIndexView = GetBindlessHandle(IndexBuffer, 0);

	for (const FRayTracingGeometrySegment& Segment : Initializer.Segments)
	{
		FVulkanBuffer* VertexBuffer = ResourceCast(Segment.VertexBuffer.GetReference());
		const FRHIDescriptorHandle VBHandle = GetBindlessHandle(VertexBuffer, Segment.VertexBufferOffset);
		HitGroupSystemVertexViews.Add(VBHandle);


		FVulkanHitGroupSystemParameters& SystemParameters = HitGroupSystemParameters.AddZeroed_GetRef();
		SystemParameters.RootConstants.SetVertexAndIndexStride(Segment.VertexBufferStride, IndexStride);
		SystemParameters.BindlessHitGroupSystemVertexBuffer = VBHandle.GetIndex();

		if (bIsTriangles && (IndexBuffer != nullptr))
		{
			SystemParameters.BindlessHitGroupSystemIndexBuffer = HitGroupSystemIndexView.GetIndex();
			SystemParameters.RootConstants.IndexBufferOffsetInBytes = Initializer.IndexBufferOffset + IndexStride * Segment.FirstPrimitive * FVulkanRayTracingGeometry::IndicesPerPrimitive;
			SystemParameters.RootConstants.FirstPrimitive = Segment.FirstPrimitive;
		}
	}
}

void FVulkanRayTracingGeometry::ReleaseBindlessHandles()
{
	for (FRHIDescriptorHandle BindlesHandle : HitGroupSystemVertexViews)
	{
		Device->GetDeferredDeletionQueue().EnqueueBindlessHandle(BindlesHandle);
	}
	HitGroupSystemVertexViews.Reset(Initializer.Segments.Num());

	if (HitGroupSystemIndexView.IsValid())
	{
		Device->GetDeferredDeletionQueue().EnqueueBindlessHandle(HitGroupSystemIndexView);
		HitGroupSystemIndexView = FRHIDescriptorHandle();
	}
}

void FVulkanRayTracingGeometry::SetupInlineGeometryParameters(uint32 GeometrySegmentIndex, FVulkanRayTracingGeometryParameters& Parameters) const
{
	const FRayTracingGeometryInitializer& GeometryInitializer = GetInitializer();
	const FVulkanBuffer* IndexBuffer = ResourceCast(GeometryInitializer.IndexBuffer.GetReference());

	const uint32 IndexStride = IndexBuffer ? IndexBuffer->GetStride() : 0;
	const uint32 IndexOffsetInBytes = GeometryInitializer.IndexBufferOffset;
	const VkDeviceAddress IndexBufferAddress = IndexBuffer ? IndexBuffer->GetDeviceAddress() : VkDeviceAddress(0);

	const FRayTracingGeometrySegment& Segment = GeometryInitializer.Segments[GeometrySegmentIndex];

	const FVulkanBuffer* VertexBuffer = ResourceCast(Segment.VertexBuffer.GetReference());
	checkf(VertexBuffer, TEXT("All ray tracing geometry segments must have a valid vertex buffer"));
	const VkDeviceAddress VertexBufferAddress = VertexBuffer->GetDeviceAddress();

	Parameters.Config.IndexStride = IndexStride;
	Parameters.Config.VertexStride = Segment.VertexBufferStride;
	if (IndexStride)
	{
		Parameters.IndexBufferOffsetInBytes = IndexOffsetInBytes + IndexStride * Segment.FirstPrimitive * 3;
		Parameters.IndexBuffer = static_cast<uint64>(IndexBufferAddress);
	}
	else
	{
		Parameters.IndexBuffer = 0;
	}
	Parameters.VertexBuffer = static_cast<uint64>(VertexBufferAddress) + Segment.VertexBufferOffset;
}

static void GetTLASBuildData(
	const VkDevice Device,
	const uint32 NumInstances,
	const VkDeviceAddress InstanceBufferAddress,
	ERayTracingAccelerationStructureFlags BuildFlags,
	EAccelerationStructureBuildMode BuildMode,
	FVkRtTLASBuildData& BuildData)
{
	VkDeviceOrHostAddressConstKHR InstanceBufferDeviceAddress = {};
	InstanceBufferDeviceAddress.deviceAddress = InstanceBufferAddress;

	BuildData.Geometry.geometryType = VK_GEOMETRY_TYPE_INSTANCES_KHR;
	BuildData.Geometry.geometry.instances.sType = VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_GEOMETRY_INSTANCES_DATA_KHR;
	BuildData.Geometry.geometry.instances.arrayOfPointers = VK_FALSE;
	BuildData.Geometry.geometry.instances.data = InstanceBufferDeviceAddress;

	BuildData.GeometryInfo.type = VK_ACCELERATION_STRUCTURE_TYPE_TOP_LEVEL_KHR;
	BuildData.GeometryInfo.mode = BuildMode == EAccelerationStructureBuildMode::Build ? VK_BUILD_ACCELERATION_STRUCTURE_MODE_BUILD_KHR : VK_BUILD_ACCELERATION_STRUCTURE_MODE_UPDATE_KHR;
	BuildData.GeometryInfo.flags = TranslateRayTracingAccelerationStructureFlags(BuildFlags);
	BuildData.GeometryInfo.geometryCount = 1;
	BuildData.GeometryInfo.pGeometries = &BuildData.Geometry;

	VulkanDynamicAPI::vkGetAccelerationStructureBuildSizesKHR(
		Device,
		VK_ACCELERATION_STRUCTURE_BUILD_TYPE_DEVICE_KHR,
		&BuildData.GeometryInfo,
		&NumInstances,
		&BuildData.SizesInfo);
}

FVulkanRayTracingScene::FVulkanRayTracingScene(FRayTracingSceneInitializer InInitializer, FVulkanDevice& InDevice)
	: Device(InDevice)
	, Initializer(MoveTemp(InInitializer))
{
	INC_DWORD_STAT(STAT_VulkanRayTracingAllocatedTLAS);

	SizeInfo = RHICalcRayTracingSceneSize(Initializer);
}

FVulkanRayTracingScene::~FVulkanRayTracingScene()
{
	if (AccelerationStructureBuffer)
	{
		DEC_MEMORY_STAT_BY(STAT_VulkanRayTracingUsedVideoMemory, AccelerationStructureBuffer->GetSize());
		DEC_MEMORY_STAT_BY(STAT_VulkanRayTracingTLASMemory, AccelerationStructureBuffer->GetSize());
	}
	DEC_DWORD_STAT(STAT_VulkanRayTracingAllocatedTLAS);
}

void FVulkanRayTracingScene::BindBuffer(FRHIBuffer* InBuffer, uint32 InBufferOffset)
{
	check(IsInRHIThread() || !IsRunningRHIInSeparateThread());

	check(SizeInfo.ResultSize + InBufferOffset <= InBuffer->GetSize());
	
	if (AccelerationStructureBuffer)
	{
		DEC_MEMORY_STAT_BY(STAT_VulkanRayTracingUsedVideoMemory, AccelerationStructureBuffer->GetSize());
		DEC_MEMORY_STAT_BY(STAT_VulkanRayTracingTLASMemory, AccelerationStructureBuffer->GetSize());
	}

	AccelerationStructureBuffer = ResourceCast(InBuffer);

	INC_MEMORY_STAT_BY(STAT_VulkanRayTracingUsedVideoMemory, AccelerationStructureBuffer->GetSize());
	INC_MEMORY_STAT_BY(STAT_VulkanRayTracingTLASMemory, AccelerationStructureBuffer->GetSize());

	{
		check(InBufferOffset % GRHIRayTracingAccelerationStructureAlignment == 0);

		View = MakeUnique<FVulkanView>(Device, ERHIDescriptorType::AccelerationStructure);
		View->InitAsAccelerationStructureView(
			{},
			AccelerationStructureBuffer
			, InBufferOffset
			//, SizeInfo.ResultSize
			// TODO: Using whole remaining size instead of SizeInfo.ResultSize reintroduces a validation error but use of SizeInfo.ResultSize broke RT on Adreno.
			, InBuffer->GetSize() - InBufferOffset
		);

#if VULKAN_USE_DEBUG_NAMES
		if (const VkAccelerationStructureKHR NativeAccelerationStructureHandle = View->GetAccelerationStructureView().Handle)
		{
			FString DebugNameString = Initializer.DebugName.ToString();
			DebugNameString = (DebugNameString.IsEmpty()) ? TEXT("TLAS") : DebugNameString;
			VULKAN_SET_DEBUG_NAME(Device, VK_OBJECT_TYPE_ACCELERATION_STRUCTURE_KHR, NativeAccelerationStructureHandle, TEXT("%s"), *DebugNameString);
		}
#endif
	}
}

void BuildAccelerationStructure(
	FVulkanCommandListContext& CommandContext,
	FVulkanRayTracingScene& Scene,
	FVulkanBuffer* InScratchBuffer, uint32 InScratchOffset,
	FVulkanBuffer* InInstanceBuffer, uint32 InInstanceOffset,
	uint32 NumInstances,
	EAccelerationStructureBuildMode BuildMode)
{
	check(InInstanceBuffer != nullptr);
	checkf(NumInstances <= Scene.Initializer.MaxNumInstances, TEXT("NumInstances must be less or equal to MaxNumInstances"));

	checkf(Scene.AccelerationStructureBuffer.IsValid(), TEXT("A buffer must be bound to the ray tracing scene before it can be built."));
	checkf(Scene.View.IsValid(), TEXT("A buffer must be bound to the ray tracing scene before it can be built."));

	const bool bIsUpdate = BuildMode == EAccelerationStructureBuildMode::Update;

	if (bIsUpdate)
	{
		checkf(NumInstances == Scene.NumInstances, TEXT("Number of instances used to update TLAS must match the number used to build."));
	}
	else
	{
		Scene.NumInstances = NumInstances;
	}

	FBufferRHIRef ScratchBuffer;
	{
		TRHICommandList_RecursiveHazardous<FVulkanCommandListContext> RHICmdList(&CommandContext);

		if (InScratchBuffer == nullptr)
		{
			const uint64 ScratchBufferSize = bIsUpdate ? Scene.SizeInfo.UpdateScratchSize : Scene.SizeInfo.BuildScratchSize;

			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateStructured(TEXT("BuildScratchTLAS"), ScratchBufferSize, 0)
				.AddUsage(EBufferUsageFlags::RayTracingScratch)
				.SetInitialState(ERHIAccess::UAVCompute);

			ScratchBuffer = RHICmdList.CreateBuffer(CreateDesc);
			InScratchBuffer = ResourceCast(ScratchBuffer.GetReference());
			InScratchOffset = 0;
		}
	}

	if (bIsUpdate)
	{
		checkf(InScratchBuffer, TEXT("TLAS update requires scratch buffer of at least %lld bytes."), Scene.SizeInfo.UpdateScratchSize);
	}
	else
	{
		checkf(InScratchBuffer, TEXT("TLAS build requires scratch buffer of at least %lld bytes."), Scene.SizeInfo.BuildScratchSize);
	}

	FVkRtTLASBuildData BuildData;

	VkAccelerationStructureBuildRangeInfoKHR BuildRangeInfo;
	VkAccelerationStructureBuildRangeInfoKHR* pBuildRangeInfos;

	const VkDeviceAddress InstanceBufferAddress = InInstanceBuffer->GetDeviceAddress() + InInstanceOffset;

	{
		GetTLASBuildData(Scene.Device.GetHandle(), NumInstances, InstanceBufferAddress, Scene.Initializer.BuildFlags, BuildMode, BuildData);

		BuildData.GeometryInfo.dstAccelerationStructure = Scene.View->GetAccelerationStructureView().Handle;
		BuildData.GeometryInfo.srcAccelerationStructure = bIsUpdate ? Scene.View->GetAccelerationStructureView().Handle : nullptr;
		BuildData.GeometryInfo.scratchData.deviceAddress = InScratchBuffer->GetDeviceAddress() + InScratchOffset;

		BuildRangeInfo.primitiveCount = NumInstances;
		BuildRangeInfo.primitiveOffset = 0;
		BuildRangeInfo.transformOffset = 0;
		BuildRangeInfo.firstVertex = 0;

		pBuildRangeInfos = &BuildRangeInfo;

		if (bIsUpdate)
		{
			INC_DWORD_STAT(STAT_VulkanRayTracingUpdatedTLAS);
		}
		else
		{
			INC_DWORD_STAT(STAT_VulkanRayTracingBuiltTLAS);
		}
	}

	FVulkanCommandBuffer* const CmdBuffer = CommandContext.GetActiveCmdBuffer();

	// Force a memory barrier to make sure all previous builds ops are finished before building the TLAS
	AddAccelerationStructureBuildBarrier(CmdBuffer->GetHandle());

	VulkanDynamicAPI::vkCmdBuildAccelerationStructuresKHR(CmdBuffer->GetHandle(), 1, &BuildData.GeometryInfo, &pBuildRangeInfos);

	// Acceleration structure build barrier is used here to ensure that the acceleration structure build is complete before any rays are traced
	AddAccelerationStructureBuildBarrier(CmdBuffer->GetHandle());

	Scene.bBuilt = true;
}

FVulkanRayTracingShaderTable::FVulkanRayTracingShaderTable(FRHICommandListBase& RHICmdList, FVulkanDevice& InDevice, const FRayTracingShaderBindingTableInitializer& InInitializer)
	: FRHIShaderBindingTable(InInitializer)
	, Device(InDevice)
	, ShaderBindingMode(InInitializer.ShaderBindingMode)
	, HitGroupIndexingMode(InInitializer.HitGroupIndexingMode)
	, HandleSize(Device.GetOptionalExtensionProperties().RayTracingPipelineProps.shaderGroupHandleSize)
	, HandleSizeAligned(Align(HandleSize, Device.GetOptionalExtensionProperties().RayTracingPipelineProps.shaderGroupHandleAlignment))
{
	check(ShaderBindingMode != ERayTracingShaderBindingMode::Disabled);

	if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO))
	{
		auto InitAlloc = [HandleSizeAligned = HandleSizeAligned](FVulkanShaderTableAllocation& Alloc, uint32 InHandleCount, bool InUseLocalRecord)
		{
			Alloc.HandleCount = InHandleCount;
			Alloc.bUseLocalRecord = InUseLocalRecord;

			if (Alloc.HandleCount > 0)
			{
				Alloc.Region.stride = (Alloc.HandleCount > 1) ? (uint32)GVulkanRayTracingMaxShaderGroupStride : 0;
				Alloc.Region.size = Alloc.HandleCount * (uint32)GVulkanRayTracingMaxShaderGroupStride;

				// Host buffer
				Alloc.HostBuffer.SetNumZeroed(Alloc.Region.size);
			}
		};

		InitAlloc(Miss, Initializer.NumMissShaderSlots, true);
		InitAlloc(Callable, Initializer.NumCallableShaderSlots, true);

		uint32 NumHitGroupRecords = HitGroupIndexingMode == ERayTracingHitGroupIndexingMode::Allow ? Initializer.NumGeometrySegments * Initializer.NumShaderSlotsPerGeometrySegment : 1;
		InitAlloc(HitGroup, NumHitGroupRecords, true);
	}

	if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::Inline) && Initializer.NumGeometrySegments > 0)
	{
		// Doesn't make sense to have inline SBT without hitgroup indexing
		check(HitGroupIndexingMode == ERayTracingHitGroupIndexingMode::Allow);

		const uint32 ParameterBufferSize = Initializer.NumGeometrySegments * sizeof(FVulkanRayTracingGeometryParameters);
		InlineGeometryParameterData.SetNumUninitialized(ParameterBufferSize);
	}
}

FVulkanRayTracingShaderTable::~FVulkanRayTracingShaderTable()
{
	ReleaseLocalBuffers();
}

void FVulkanRayTracingShaderTable::ReleaseLocalBuffers()
{
	ReleaseLocalBuffer(Device, Miss);
	ReleaseLocalBuffer(Device, HitGroup);
	ReleaseLocalBuffer(Device, Callable);
}

void FVulkanRayTracingShaderTable::ReleaseLocalBuffer(FVulkanDevice& Device, FVulkanShaderTableAllocation& Alloc)
{
	if (Alloc.LocalBuffer != VK_NULL_HANDLE)
	{
		Device.GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::Buffer, Alloc.LocalBuffer);
		Alloc.LocalBuffer = VK_NULL_HANDLE;
	}

	if (Alloc.LocalAllocation.IsValid())
	{
		Device.GetMemoryManager().FreeVulkanAllocation(Alloc.LocalAllocation);
	}

	Alloc.Region.deviceAddress = 0;
}

FVulkanRayTracingShaderTable::FVulkanShaderTableAllocation& FVulkanRayTracingShaderTable::GetAlloc(EShaderFrequency Frequency)
{
	switch (Frequency)
	{
	case SF_RayMiss:      return Miss;
	case SF_RayHitGroup:  return HitGroup;
	case SF_RayCallable:  return Callable;

	case SF_RayGen:
		checkf(false, TEXT("RayGen have not ShaderTable allocation."));
		break;

	default:
		checkf(false, TEXT("Only usable with RayTracing shaders."));
		break;
	}

	static FVulkanShaderTableAllocation EmptyAlloc;
	return EmptyAlloc;
}

const VkStridedDeviceAddressRegionKHR* FVulkanRayTracingShaderTable::GetRegion(EShaderFrequency Frequency)
{
	const FVulkanShaderTableAllocation& Alloc = GetAlloc(Frequency);
	check(!Alloc.bIsDirty);
	return &Alloc.Region;
}

void FVulkanRayTracingShaderTable::SetSlot(EShaderFrequency Frequency, uint32 DstSlot, uint32 SrcHandleIndex, TConstArrayView<uint8> SrcHandleData)
{
	FVulkanShaderTableAllocation& Alloc = GetAlloc(Frequency);
	checkf((DstSlot == 0) || (Alloc.Region.stride != 0), TEXT("Attempting to index a record in a region without stride"));
	FMemory::Memcpy(&Alloc.HostBuffer[DstSlot * Alloc.Region.stride], &SrcHandleData[SrcHandleIndex * HandleSize], HandleSize);
	Alloc.bIsDirty = true;
}

VkStridedDeviceAddressRegionKHR FVulkanRayTracingShaderTable::CommitRayGenShader(FVulkanCommandListContext& Context, uint32 SrcHandleIndex, TConstArrayView<uint8> SrcHandleData)
{
	VkStridedDeviceAddressRegionKHR RayGenRegion;
	uint8* pMappedMemory = Device.GetTempBlockAllocator().Alloc(HandleSize, Context, &RayGenRegion);

	FMemory::Memcpy(pMappedMemory, &SrcHandleData[SrcHandleIndex * HandleSize], HandleSize);

	check(!Miss.bIsDirty);
	check(!HitGroup.bIsDirty);
	check(!Callable.bIsDirty);

	return RayGenRegion;
}

void FVulkanRayTracingShaderTable::SetLocalShaderParameters(EShaderFrequency Frequency, uint32 RecordIndex, uint32 OffsetWithinRecord, const void* InData, uint32 InDataSize)
{
	FVulkanShaderTableAllocation& Alloc = GetAlloc(Frequency);

	checkfSlow(OffsetWithinRecord % 4 == 0, TEXT("SBT record parameters must be written on DWORD-aligned boundary"));
	checkfSlow(InDataSize % 4 == 0, TEXT("SBT record parameters must be DWORD-aligned"));
	checkf((OffsetWithinRecord + InDataSize) <= (Alloc.Region.stride ? Alloc.Region.stride : Alloc.Region.size), TEXT("SBT record write request is out of bounds"));
	checkf((RecordIndex == 0) || (Alloc.Region.stride != 0), TEXT("Attempting to index a record in a region without stride"));

	const uint32 WriteOffset = HandleSizeAligned + (Alloc.Region.stride * RecordIndex) + OffsetWithinRecord;
	FMemory::Memcpy(&Alloc.HostBuffer[WriteOffset], InData, InDataSize);

	Alloc.bIsDirty = true;
}

void FVulkanRayTracingShaderTable::SetInlineGeometryParameters(uint32 SegmentIndex, const void* InData, uint32 InDataSize)
{
	const uint32 WriteOffset = InDataSize * SegmentIndex;
	FMemory::Memcpy(&InlineGeometryParameterData[WriteOffset], InData, InDataSize);
}

#if VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH

FRHISizeAndStride FVulkanRayTracingShaderTable::GetInlineBindingDataSizeAndStride() const
{	
	check(InlineGeometryParameterData.Num() == 0);
	return FRHISizeAndStride{0,0};
}


#else

FRHISizeAndStride FVulkanRayTracingShaderTable::GetInlineBindingDataSizeAndStride() const
{	
	return FRHISizeAndStride { (uint64)InlineGeometryParameterData.Num(), sizeof(FVulkanRayTracingGeometryParameters) };
}

#endif // VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH

void FVulkanRayTracingShaderTable::Commit(FVulkanCommandListContext& Context, FRHIBuffer* InlineBindingDataBuffer)
{
	FVulkanCommandBuffer& CommandBuffer = Context.GetCommandBuffer();

	VkMemoryBarrier BarrierBefore = { VK_STRUCTURE_TYPE_MEMORY_BARRIER, nullptr, VK_ACCESS_MEMORY_READ_BIT | VK_ACCESS_MEMORY_WRITE_BIT, VK_ACCESS_TRANSFER_READ_BIT | VK_ACCESS_TRANSFER_WRITE_BIT };
	VulkanRHI::vkCmdPipelineBarrier(CommandBuffer.GetHandle(), VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT, VK_PIPELINE_STAGE_TRANSFER_BIT, 0, 1, &BarrierBefore, 0, nullptr, 0, nullptr);

	auto CommitBuffer = [&Device = Device, &Context](FVulkanShaderTableAllocation& Alloc)
	{
		if (Alloc.bIsDirty)
		{
			if (!Alloc.HostBuffer.IsEmpty())
			{
				ReleaseLocalBuffer(Device, Alloc);

				const VkDevice DeviceHandle = Device.GetHandle();
				const VkPhysicalDeviceRayTracingPipelinePropertiesKHR& RayTracingPipelineProps = Device.GetOptionalExtensionProperties().RayTracingPipelineProps;

				// Fetch staging buffer and fill it
				VulkanRHI::FStagingBuffer* StagingBuffer = Device.GetStagingManager().AcquireBuffer(Alloc.Region.size);
				FMemory::Memcpy(StagingBuffer->GetMappedPointer(), Alloc.HostBuffer.GetData(), Alloc.Region.size);

				// Alloc a new Local buffer
				{
					const VkBufferUsageFlags BufferUsageFlags = VK_BUFFER_USAGE_TRANSFER_DST_BIT | VK_BUFFER_USAGE_SHADER_BINDING_TABLE_BIT_KHR | VK_BUFFER_USAGE_SHADER_DEVICE_ADDRESS_BIT;
					Alloc.LocalBuffer = Device.CreateBuffer(Alloc.Region.size, BufferUsageFlags);

					const VulkanRHI::EVulkanAllocationFlags AllocFlags = VulkanRHI::EVulkanAllocationFlags::AutoBind | VulkanRHI::EVulkanAllocationFlags::Dedicated;
					Device.GetMemoryManager().AllocateBufferMemory(Alloc.LocalAllocation, Alloc.LocalBuffer, AllocFlags, TEXT("LocalShaderTableAllocation"), RayTracingPipelineProps.shaderGroupBaseAlignment);

					VkBufferDeviceAddressInfoKHR DeviceAddressInfo;
					ZeroVulkanStruct(DeviceAddressInfo, VK_STRUCTURE_TYPE_BUFFER_DEVICE_ADDRESS_INFO);
					DeviceAddressInfo.buffer = Alloc.LocalBuffer;
					Alloc.Region.deviceAddress = VulkanRHI::vkGetBufferDeviceAddressKHR(DeviceHandle, &DeviceAddressInfo);
				}

				VkBufferCopy RegionInfo;
				RegionInfo.srcOffset = 0;
				RegionInfo.dstOffset = 0;
				RegionInfo.size = Alloc.Region.size;
				VulkanRHI::vkCmdCopyBuffer(Context.GetCommandBuffer().GetHandle(), StagingBuffer->GetHandle(), Alloc.LocalBuffer, 1, &RegionInfo);

				Device.GetStagingManager().ReleaseBuffer(&Context, StagingBuffer);
			}
			else
			{
				checkSlow(Alloc.LocalBuffer == VK_NULL_HANDLE);
			}

			Alloc.bIsDirty = false;
		}
	};

	CommitBuffer(Miss);
	CommitBuffer(HitGroup);
	CommitBuffer(Callable);

#if !VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH
	// Also copy geometry parameter data to the GPU buffer
	if (InlineBindingDataBuffer)
	{
		TRHICommandList_RecursiveHazardous<FVulkanCommandListContext> RHICmdList(&Context);

		const uint32 ParameterBufferSize = InlineGeometryParameterData.Num();
		FVulkanBuffer* VulkanBuffer = (FVulkanBuffer*)InlineBindingDataBuffer;
		void* MappedBuffer = VulkanBuffer->Lock(RHICmdList, RLM_WriteOnly, ParameterBufferSize, 0);
		FMemory::Memcpy(MappedBuffer, InlineGeometryParameterData.GetData(), ParameterBufferSize);
		VulkanBuffer->Unlock(RHICmdList);
	}
#endif // !VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH

	VkMemoryBarrier BarrierAfter = { VK_STRUCTURE_TYPE_MEMORY_BARRIER, nullptr, VK_ACCESS_TRANSFER_WRITE_BIT, VK_ACCESS_MEMORY_READ_BIT };  // :todo-jn: VK_ACCESS_2_SHADER_BINDING_TABLE_READ_BIT_KHR
	VulkanRHI::vkCmdPipelineBarrier(CommandBuffer.GetHandle(), VK_PIPELINE_STAGE_TRANSFER_BIT, VK_PIPELINE_STAGE_RAY_TRACING_SHADER_BIT_KHR, 0, 1, &BarrierAfter, 0, nullptr, 0, nullptr);
}

FRayTracingAccelerationStructureSize FVulkanDynamicRHI::RHICalcRayTracingSceneSize(const FRayTracingSceneInitializer& Initializer)
{
	FVkRtTLASBuildData BuildData;
	const VkDeviceAddress InstanceBufferAddress = 0; // No device address available when only querying TLAS size
	GetTLASBuildData(Device->GetHandle(), Initializer.MaxNumInstances, InstanceBufferAddress, Initializer.BuildFlags, EAccelerationStructureBuildMode::Build, BuildData);

	FRayTracingAccelerationStructureSize Result;
	Result.ResultSize = BuildData.SizesInfo.accelerationStructureSize;
	Result.BuildScratchSize = BuildData.SizesInfo.buildScratchSize;
	Result.UpdateScratchSize = BuildData.SizesInfo.updateScratchSize;

	return Result;
}

FRayTracingAccelerationStructureSize FVulkanDynamicRHI::RHICalcRayTracingGeometrySize(const FRayTracingGeometryInitializer& Initializer)
{	
	FVkRtBLASBuildData BuildData;
	GetBLASBuildData(
		Device->GetHandle(),
		MakeArrayView(Initializer.Segments),
		Initializer.GeometryType,
		Initializer.IndexBuffer,
		Initializer.IndexBufferOffset,
		GetRayTracingAccelerationStructureBuildFlags(Initializer),
		EAccelerationStructureBuildMode::Build,
		EBLASBuildDataUsage::Size,
		BuildData);

	FRayTracingAccelerationStructureSize Result;
	Result.ResultSize = Align(BuildData.SizesInfo.accelerationStructureSize, GRHIRayTracingAccelerationStructureAlignment);
	Result.BuildScratchSize = Align(BuildData.SizesInfo.buildScratchSize, GRHIRayTracingScratchBufferAlignment);
	Result.UpdateScratchSize = Align(BuildData.SizesInfo.updateScratchSize, GRHIRayTracingScratchBufferAlignment);
	
	return Result;
}

FRayTracingSceneRHIRef FVulkanDynamicRHI::RHICreateRayTracingScene(FRayTracingSceneInitializer Initializer)
{
	return new FVulkanRayTracingScene(MoveTemp(Initializer), *Device);
}

FRayTracingGeometryRHIRef FVulkanDynamicRHI::RHICreateRayTracingGeometry(FRHICommandListBase& RHICmdList, const FRayTracingGeometryInitializer& Initializer)
{
	return new FVulkanRayTracingGeometry(RHICmdList, Initializer, GetDevice());
}

FRayTracingPipelineStateRHIRef FVulkanDynamicRHI::RHICreateRayTracingPipelineState(const FRayTracingPipelineStateInitializer& Initializer)
{
	return new FVulkanRayTracingPipelineState(*Device, Initializer);
}

FShaderBindingTableRHIRef FVulkanDynamicRHI::RHICreateShaderBindingTable(FRHICommandListBase& RHICmdList, const FRayTracingShaderBindingTableInitializer& Initializer)
{
	return new FVulkanRayTracingShaderTable(RHICmdList, *Device, Initializer);
}

void FVulkanCommandListContext::RHICommitShaderBindingTable(FRHIShaderBindingTable* InSBT, FRHIBuffer* InlineBindingDataBuffer)
{
	FVulkanRayTracingShaderTable* SBT = ResourceCast(InSBT);
	SBT->Commit(*this, InlineBindingDataBuffer);
}

void FVulkanCommandListContext::RHIBindAccelerationStructureMemory(FRHIRayTracingScene* Scene, FRHIBuffer* Buffer, uint32 BufferOffset)
{
	ResourceCast(Scene)->BindBuffer(Buffer, BufferOffset);
}

// Todo: High level rhi call should have transitioned and verified vb and ib to read for each segment
void FVulkanCommandListContext::RHIBuildAccelerationStructures(TConstArrayView<FRayTracingGeometryBuildParams> Params, const FRHIBufferRange& ScratchBufferRange)
{
	checkf(ScratchBufferRange.Buffer != nullptr, TEXT("BuildAccelerationStructures requires valid scratch buffer"));

	// Update geometry vertex buffers
	for (const FRayTracingGeometryBuildParams& P : Params)
	{
		FVulkanRayTracingGeometry* const Geometry = ResourceCast(P.Geometry.GetReference());

		if (P.Segments.Num())
		{
			checkf(P.Segments.Num() == Geometry->Initializer.Segments.Num(),
				TEXT("If updated segments are provided, they must exactly match existing geometry segments. Only vertex buffer bindings may change."));

			for (int32 i = 0; i < P.Segments.Num(); ++i)
			{
				checkf(P.Segments[i].VertexBuffer != nullptr, TEXT("Segments used to build/update ray tracing geometry must have a valid VertexBuffer."));

				checkf(P.Segments[i].MaxVertices <= Geometry->Initializer.Segments[i].MaxVertices,
					TEXT("Maximum number of vertices in a segment (%u) must not be smaller than what was declared during FRHIRayTracingGeometry creation (%u), as this controls BLAS memory allocation."),
					P.Segments[i].MaxVertices, Geometry->Initializer.Segments[i].MaxVertices
				);

				Geometry->Initializer.Segments[i].VertexBuffer = P.Segments[i].VertexBuffer;
				Geometry->Initializer.Segments[i].VertexBufferElementType = P.Segments[i].VertexBufferElementType;
				Geometry->Initializer.Segments[i].VertexBufferStride = P.Segments[i].VertexBufferStride;
				Geometry->Initializer.Segments[i].VertexBufferOffset = P.Segments[i].VertexBufferOffset;
			}
		}
	}

	uint32 ScratchBufferSize = ScratchBufferRange.Size ? ScratchBufferRange.Size : ScratchBufferRange.Buffer->GetSize();

	checkf(ScratchBufferSize + ScratchBufferRange.Offset <= ScratchBufferRange.Buffer->GetSize(),
		TEXT("BLAS scratch buffer range size is %lld bytes with offset %lld, but the buffer only has %d bytes. "),
		ScratchBufferRange.Size, ScratchBufferRange.Offset, ScratchBufferRange.Buffer->GetSize());

	const uint64 ScratchAlignment = GRHIRayTracingScratchBufferAlignment;
	FVulkanBuffer* ScratchBuffer = ResourceCast(ScratchBufferRange.Buffer);
	uint32 ScratchBufferOffset = ScratchBufferRange.Offset;

	TArray<FVkRtBLASBuildData, TInlineAllocator<32>> TempBuildData;
	TArray<VkAccelerationStructureBuildGeometryInfoKHR, TInlineAllocator<32>> BuildGeometryInfos;
	TArray<VkAccelerationStructureBuildRangeInfoKHR*, TInlineAllocator<32>> BuildRangeInfos;
	TempBuildData.Reserve(Params.Num());
	BuildGeometryInfos.Reserve(Params.Num());
	BuildRangeInfos.Reserve(Params.Num());	

	for (const FRayTracingGeometryBuildParams& P : Params)
	{
		FVulkanRayTracingGeometry* const Geometry = ResourceCast(P.Geometry.GetReference());
		const bool bIsUpdate = P.BuildMode == EAccelerationStructureBuildMode::Update;

		if (bIsUpdate)
		{
			INC_DWORD_STAT(STAT_VulkanRayTracingUpdatedBLAS);
		}
		else
		{
			INC_DWORD_STAT(STAT_VulkanRayTracingBuiltBLAS);
		}

		uint64 ScratchBufferRequiredSize = bIsUpdate ? Geometry->SizeInfo.UpdateScratchSize : Geometry->SizeInfo.BuildScratchSize;
		checkf(ScratchBufferRequiredSize + ScratchBufferOffset <= ScratchBufferSize,
			TEXT("BLAS scratch buffer size is %ld bytes with offset %ld (%ld bytes available), but the build requires %lld bytes. "),
			ScratchBufferSize, ScratchBufferOffset, ScratchBufferSize - ScratchBufferOffset, ScratchBufferRequiredSize);

		FVkRtBLASBuildData& BuildData = TempBuildData.AddDefaulted_GetRef();
		GetBLASBuildData(
			Device.GetHandle(),
			MakeArrayView(Geometry->Initializer.Segments),
			Geometry->Initializer.GeometryType,
			Geometry->Initializer.IndexBuffer,
			Geometry->Initializer.IndexBufferOffset,
			GetRayTracingAccelerationStructureBuildFlags(Geometry->Initializer),
			P.BuildMode,
			EBLASBuildDataUsage::Rendering,
			BuildData);

		check(BuildData.SizesInfo.accelerationStructureSize <= Geometry->AccelerationStructureBuffer->GetSize());

		BuildData.GeometryInfo.dstAccelerationStructure = Geometry->Handle;
		BuildData.GeometryInfo.srcAccelerationStructure = bIsUpdate ? Geometry->Handle : VK_NULL_HANDLE;

		VkDeviceAddress ScratchBufferAddress = ScratchBuffer->GetDeviceAddress() + ScratchBufferOffset;
		ScratchBufferOffset += ScratchBufferRequiredSize;

		checkf(ScratchBufferAddress % GRHIRayTracingScratchBufferAlignment == 0,
			TEXT("BLAS scratch buffer (plus offset) must be aligned to %ld bytes."),
			GRHIRayTracingScratchBufferAlignment);

		BuildData.GeometryInfo.scratchData.deviceAddress = ScratchBufferAddress;

		VkAccelerationStructureBuildRangeInfoKHR* const pBuildRanges = BuildData.Ranges.GetData();

		BuildGeometryInfos.Add(BuildData.GeometryInfo);
		BuildRangeInfos.Add(pBuildRanges);

		Geometry->SetupHitGroupSystemParameters();
	}
	
	FVulkanCommandBuffer* const CmdBuffer = GetActiveCmdBuffer();
	VulkanDynamicAPI::vkCmdBuildAccelerationStructuresKHR(CmdBuffer->GetHandle(), Params.Num(), BuildGeometryInfos.GetData(), BuildRangeInfos.GetData());

	// Add an acceleration structure build barrier after each acceleration structure build batch.
	// This is required because there are currently no explicit read/write barriers
	// for acceleration structures, but we need to ensure that all commands
	// are complete before BLAS is used again on the GPU.
	AddAccelerationStructureBuildBarrier(CmdBuffer->GetHandle());

	for (const FRayTracingGeometryBuildParams& P : Params)
	{
		FVulkanRayTracingGeometry* const Geometry = ResourceCast(P.Geometry.GetReference());

		ERayTracingAccelerationStructureFlags GeometryBuildFlags = GetRayTracingAccelerationStructureBuildFlags(Geometry->Initializer);
		if (ShouldCompactAfterBuild(GeometryBuildFlags))
		{
			Device.GetRayTracingCompactionRequestHandler()->RequestCompact(Geometry);
			Geometry->bHasPendingCompactionRequests = true;
		}
	}
}

void FVulkanCommandListContext::RHIBuildAccelerationStructures(TConstArrayView<FRayTracingSceneBuildParams> Params)
{
	for (const FRayTracingSceneBuildParams& SceneBuildParams : Params)
	{
		FVulkanRayTracingScene* const Scene = ResourceCast(SceneBuildParams.Scene);
		FVulkanBuffer* const ScratchBuffer = ResourceCast(SceneBuildParams.ScratchBuffer);
		FVulkanBuffer* const InstanceBuffer = ResourceCast(SceneBuildParams.InstanceBuffer);

		Scene->SetReferencedResources(SceneBuildParams.ReferencedGeometries, SceneBuildParams.ReferencedBuffers);

		BuildAccelerationStructure(
			*this,
			*Scene,
			ScratchBuffer, SceneBuildParams.ScratchBufferOffset,
			InstanceBuffer, SceneBuildParams.InstanceBufferOffset,
			SceneBuildParams.NumInstances,
			SceneBuildParams.BuildMode);
	}
}

template<typename ShaderType>
static FRHIRayTracingShader* GetBuiltInRayTracingShader()
{
	const FGlobalShaderMap* const ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	auto Shader = ShaderMap->GetShader<ShaderType>();
	return static_cast<FRHIRayTracingShader*>(Shader.GetRayTracingShader());
}



class FVulkanRayTracingPipelineLibraryCache
{
public:

	static uint64 GetShaderHash64(FRHIRayTracingShader* ShaderRHI)
	{
		return ShaderRHI->GetHash().Hash;;
	}

	struct FKey
	{
		FKey(FVulkanRayTracingShader* InShader, uint32 InMaxAttributeSizeInBytes, uint32 InMaxPayloadSizeInBytes)
			: ShaderHash(GetShaderHash64(InShader))
			, ShaderHandle(InShader->GetOrCreateHandle(FVulkanRayTracingShader::MainModuleIdentifier)->GetVkShaderModule())
			, MaxAttributeSizeInBytes(InMaxAttributeSizeInBytes)
			, MaxPayloadSizeInBytes(InMaxPayloadSizeInBytes)
		{
		}

		const uint64 ShaderHash;
		const VkShaderModule ShaderHandle;
		const uint32 MaxAttributeSizeInBytes;
		const uint32 MaxPayloadSizeInBytes;

		bool operator == (const FKey& Other) const
		{
			return ShaderHash == Other.ShaderHash
				&& ShaderHandle == Other.ShaderHandle
				&& MaxAttributeSizeInBytes == Other.MaxAttributeSizeInBytes
				&& MaxPayloadSizeInBytes == Other.MaxPayloadSizeInBytes;
		}

		inline friend uint32 GetTypeHash(const FKey& Key)
		{
			return Key.ShaderHash;
		}
	};

	struct FPipelineLibrary
	{
		VkPipeline PipelineHandle = VK_NULL_HANDLE;
		FGraphEventRef GraphEvent;
	};

	FVulkanRayTracingPipelineLibraryCache(FVulkanDevice& InDevice)
		: Device(InDevice)
	{
	}

	~FVulkanRayTracingPipelineLibraryCache()
	{
		for (auto& Pair : PipelineLibraryMap)
		{
			VulkanRHI::vkDestroyPipeline(Device.GetHandle(), Pair.Value->PipelineHandle, VULKAN_CPU_ALLOCATOR);
			Pair.Value->PipelineHandle = VK_NULL_HANDLE;
		}
		PipelineLibraryMap.Empty();
	}

	FPipelineLibrary* GetOrAddLibrary(FVulkanRayTracingShader* Shader, const FRayTracingPipelineStateInitializer& Initializer)
	{
		FScopeLock Lock(&PipelineLibraryMapCS);

		const FKey Key(Shader, Initializer.MaxAttributeSizeInBytes, Initializer.MaxPayloadSizeInBytes);
		FPipelineLibrary** ExistingPipeline = PipelineLibraryMap.Find(Key);
		if (ExistingPipeline)
		{
			return *ExistingPipeline;
		}

		FPipelineLibrary* PipelineLibrary = new FPipelineLibrary();

		auto CreatePipelineLibraryTask = [this, PipelineLibrary, Shader, Key, bPartial=Initializer.bPartial]()
			{
				const EShaderFrequency Frequency = Shader->GetFrequency();

				ANSICHAR EntryPoint[MaxEntryPointNameLength];
				Shader->GetEntryPoint(EntryPoint, MaxEntryPointNameLength);

				uint32 ShaderCount = 0;
				TStaticArray<VkPipelineShaderStageCreateInfo, MaxHitGroupShaderCount> ShaderStages;
				ZeroVulkanStruct(ShaderStages[ShaderCount], VK_STRUCTURE_TYPE_PIPELINE_SHADER_STAGE_CREATE_INFO);
				ShaderStages[ShaderCount].module = Key.ShaderHandle;
				ShaderStages[ShaderCount].stage = UEFrequencyToVKStageBit(Frequency);  // will default to closestHitShader for SF_RayHitGroup
				ShaderStages[ShaderCount].pName = EntryPoint;

				VkRayTracingShaderGroupCreateInfoKHR ShaderGroup;
				ZeroVulkanStruct(ShaderGroup, VK_STRUCTURE_TYPE_RAY_TRACING_SHADER_GROUP_CREATE_INFO_KHR);

				if (Shader->GetFrequency() != SF_RayHitGroup)
				{
					ShaderGroup.type = VK_RAY_TRACING_SHADER_GROUP_TYPE_GENERAL_KHR;
					ShaderGroup.generalShader = ShaderCount++;
					ShaderGroup.closestHitShader = VK_SHADER_UNUSED_KHR;
					ShaderGroup.anyHitShader = VK_SHADER_UNUSED_KHR;
					ShaderGroup.intersectionShader = VK_SHADER_UNUSED_KHR;
				}
				else
				{
					ShaderGroup.type = VK_RAY_TRACING_SHADER_GROUP_TYPE_TRIANGLES_HIT_GROUP_KHR;
					ShaderGroup.generalShader = VK_SHADER_UNUSED_KHR;

					// Closest Hit, always present
					ShaderGroup.closestHitShader = ShaderCount++;

					// Any Hit, optional
					if (Shader->GetCodeHeader().RayGroupAnyHit != FVulkanShaderHeader::ERayHitGroupEntrypoint::NotPresent)
					{
						ZeroVulkanStruct(ShaderStages[ShaderCount], VK_STRUCTURE_TYPE_PIPELINE_SHADER_STAGE_CREATE_INFO);
						ShaderStages[ShaderCount].module = Shader->GetOrCreateHandle(FVulkanRayTracingShader::AnyHitModuleIdentifier)->GetVkShaderModule();
						ShaderStages[ShaderCount].stage = VK_SHADER_STAGE_ANY_HIT_BIT_KHR;
						ShaderStages[ShaderCount].pName = "main_00000000_00000000"; // :todo-jn: patch in the size_crc
						ShaderGroup.anyHitShader = ShaderCount++;
					}
					else
					{
						ShaderGroup.anyHitShader = VK_SHADER_UNUSED_KHR;
					}

					// Intersection, optional
					if (Shader->GetCodeHeader().RayGroupIntersection != FVulkanShaderHeader::ERayHitGroupEntrypoint::NotPresent)
					{
						ZeroVulkanStruct(ShaderStages[ShaderCount], VK_STRUCTURE_TYPE_PIPELINE_SHADER_STAGE_CREATE_INFO);
						ShaderStages[ShaderCount].module = Shader->GetOrCreateHandle(FVulkanRayTracingShader::IntersectionModuleIdentifier)->GetVkShaderModule();
						ShaderStages[ShaderCount].stage = VK_SHADER_STAGE_INTERSECTION_BIT_KHR;
						ShaderStages[ShaderCount].pName = "main_00000000_00000000"; // :todo-jn: patch in the size_crc
						ShaderGroup.intersectionShader = ShaderCount++;

						// Switch the shader group type given the presence of an intersection shader
						ShaderGroup.type = VK_RAY_TRACING_SHADER_GROUP_TYPE_PROCEDURAL_HIT_GROUP_KHR;
					}
					else
					{
						ShaderGroup.intersectionShader = VK_SHADER_UNUSED_KHR;
					}
				}

				const VkRayTracingPipelineInterfaceCreateInfoKHR RayTracingPipelineInterfaceCreateInfo = {
					.sType = VK_STRUCTURE_TYPE_RAY_TRACING_PIPELINE_INTERFACE_CREATE_INFO_KHR,
					.pNext = nullptr,
					.maxPipelineRayPayloadSize = Key.MaxPayloadSizeInBytes,
					.maxPipelineRayHitAttributeSize = Key.MaxAttributeSizeInBytes
				};

				const VkPipelineCreateFlags2CreateInfo Flags2CreateInfo = {
					.sType = VK_STRUCTURE_TYPE_PIPELINE_CREATE_FLAGS_2_CREATE_INFO,
					.pNext = nullptr,
					.flags = VK_PIPELINE_CREATE_2_LIBRARY_BIT_KHR |
							 Device.GetBindlessDescriptorManager()->GetPipelineCreateFlag() |
							 (GRHIGlobals.RayTracing.SupportsLinearSweptSpheres ? VK_PIPELINE_CREATE_2_RAY_TRACING_ALLOW_SPHERES_AND_LINEAR_SWEPT_SPHERES_BIT_NV : 0ull)
				};

				TStaticArray<VkShaderDescriptorSetAndBindingMappingInfoEXT, MaxHitGroupShaderCount> BindingMappingInfos;
				if (Device.GetBindlessDescriptorManager()->UseDescriptorHeaps())
				{
					TConstArrayView<VkDescriptorSetAndBindingMappingEXT> BindingMappings = Device.GetBindlessDescriptorManager()->GetBindingMappings(Frequency, false);
					for (uint32 StageIndex=0; StageIndex < ShaderCount; ++StageIndex)
					{
						BindingMappingInfos[StageIndex] = {
							.sType = VK_STRUCTURE_TYPE_SHADER_DESCRIPTOR_SET_AND_BINDING_MAPPING_INFO_EXT,
							.pNext = nullptr,
							.mappingCount = (uint32)BindingMappings.Num(),
							.pMappings = BindingMappings.GetData()
						};

						VkPipelineShaderStageCreateInfo& StageCreateInfo = ShaderStages[StageIndex];
						AddToPNext(StageCreateInfo, BindingMappingInfos[StageIndex]);
					}
				}

				VkRayTracingPipelineCreateInfoKHR RayTracingPipelineCreateInfo;
				ZeroVulkanStruct(RayTracingPipelineCreateInfo, VK_STRUCTURE_TYPE_RAY_TRACING_PIPELINE_CREATE_INFO_KHR);
				RayTracingPipelineCreateInfo.pLibraryInterface = &RayTracingPipelineInterfaceCreateInfo;
				RayTracingPipelineCreateInfo.stageCount = ShaderCount;
				RayTracingPipelineCreateInfo.pStages = ShaderStages.GetData();
				RayTracingPipelineCreateInfo.groupCount = 1;
				RayTracingPipelineCreateInfo.pGroups = &ShaderGroup;
				RayTracingPipelineCreateInfo.maxPipelineRayRecursionDepth = 1;
				RayTracingPipelineCreateInfo.layout = Device.GetBindlessDescriptorManager()->GetPipelineLayout();
				RayTracingPipelineCreateInfo.flags = 0; // using VkPipelineCreateFlags2CreateInfoKHR
				RayTracingPipelineCreateInfo.pNext = &Flags2CreateInfo;

				const VkResult Result = Device.GetPipelineStateCache()->CreateRayTracingPipeline(RayTracingPipelineCreateInfo, bPartial, PipelineLibrary->PipelineHandle);
				if (!bPartial)
				{
					VERIFYVULKANRESULT_EXPANDED(Result);
				}
			};

		PipelineLibrary->GraphEvent = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(CreatePipelineLibraryTask), QUICK_USE_CYCLE_STAT(FExecuteRHIThreadTask, STATGROUP_TaskGraphTasks));
		PipelineLibraryMap.Add(Key, PipelineLibrary);
		return PipelineLibrary;
	}

private:
	static const uint32 MaxEntryPointNameLength = 24;
	static const uint32 MaxHitGroupShaderCount = 3;

	FVulkanDevice& Device;

	FCriticalSection PipelineLibraryMapCS;
	TMap<FKey, FPipelineLibrary*> PipelineLibraryMap;
};




void FVulkanDevice::InitializeRayTracing()
{
	if (GRHISupportsRayTracingShaders)
	{
		const VkPhysicalDeviceRayTracingPipelinePropertiesKHR& RayTracingPipelineProps = GetOptionalExtensionProperties().RayTracingPipelineProps;
		if ((uint32)GVulkanRayTracingMaxShaderGroupStride > RayTracingPipelineProps.maxShaderGroupStride)
		{
			UE_LOGF(LogRHI, Warning, "Specified value for r.Vulkan.RayTracing.MaxShaderGroupStride is too large for this device! It will be capped.");
		}
		GVulkanRayTracingMaxShaderGroupStride = FMath::Min<VkDeviceSize>(RayTracingPipelineProps.maxShaderGroupStride, GVulkanRayTracingMaxShaderGroupStride);

		RayTracingPipelineLibraryCache = new FVulkanRayTracingPipelineLibraryCache(*this);
	}
}

// Temporary code to generate dummy UBs to bind when none is provided to prevent bindless code from crashing
// NOTE: Should currently only be used by InstanceCulling due to a binding that isn't stripped by DXC. See also USE_INSTANCE_CULLING_DATA for same issue in CS.
static FRWLock DummyUBLock;
static TMap<uint32, FUniformBufferRHIRef> DummyUBs;
static FVulkanUniformBuffer* GetDummyUB(FVulkanDevice& Device, uint32 UBLayoutHash)
{
	{
		FRWScopeLock ScopedReadLock(DummyUBLock, SLT_ReadOnly);
		FUniformBufferRHIRef* UBRef = DummyUBs.Find(UBLayoutHash);
		if (UBRef)
		{
			return ResourceCast(UBRef->GetReference());
		}
	}

	FRWScopeLock ScopedReadLock(DummyUBLock, SLT_Write);
	const FShaderParametersMetadata* DummyMetadata = FindUniformBufferStructByLayoutHash(UBLayoutHash);
	if (DummyMetadata && DummyMetadata->GetLayoutPtr())
	{
		const FRHIUniformBufferLayout* DummyLayout = DummyMetadata->GetLayoutPtr();
		TArray<uint8> DummyContent;
		DummyContent.SetNumZeroed(DummyLayout->ConstantBufferSize);
		FVulkanUniformBuffer* DummyUB = new FVulkanUniformBuffer(Device, DummyLayout, DummyContent.GetData(), UniformBuffer_MultiFrame, EUniformBufferValidation::None);
		DummyUBs.Add(UBLayoutHash, DummyUB);
		const FString& LayoutName = DummyLayout->GetDebugName();
		UE_LOGF(LogRHI, Warning, "Vulkan ray tracing using DummyUB for %ls.", LayoutName.IsEmpty() ? TEXT("<unknown>") : *LayoutName);
		return DummyUB;
	}
	return nullptr;
}

void FVulkanDevice::CleanUpRayTracing()
{
	if (RayTracingPipelineLibraryCache)
	{
		delete RayTracingPipelineLibraryCache;
		RayTracingPipelineLibraryCache = nullptr;
	}

	DummyUBs.Empty();
}


FVulkanRayTracingPipelineState::FVulkanRayTracingPipelineState(FVulkanDevice& InDevice, const FRayTracingPipelineStateInitializer& Initializer)
	: FRHIRayTracingPipelineState(Initializer)
	, Device(InDevice)
	, bIsPartialPipeline(Initializer.bPartial)
{
	checkf(InDevice.SupportsBindless(), TEXT("Vulkan ray tracing pipelines are only supported in bindless."));
	checkf(Initializer.MaxAttributeSizeInBytes <= InDevice.GetOptionalExtensionProperties().RayTracingPipelineProps.maxRayHitAttributeSize,
		TEXT("Required attribute size (%u) too large for current device (%u)."), 
		Initializer.MaxAttributeSizeInBytes, 
		InDevice.GetOptionalExtensionProperties().RayTracingPipelineProps.maxRayHitAttributeSize);

	TArrayView<FRHIRayTracingShader*> InitializerRayGenShaders = Initializer.GetRayGenTable();
	TArrayView<FRHIRayTracingShader*> InitializerMissShaders = Initializer.GetMissTable();
	TArrayView<FRHIRayTracingShader*> InitializerHitGroupShaders = Initializer.GetHitGroupTable();
	TArrayView<FRHIRayTracingShader*> InitializerCallableShaders = Initializer.GetCallableTable();

	FVulkanRayTracingPipelineLibraryCache* LibraryCache = InDevice.GetRayTracingPipelineLibraryCache();

	TArray<FVulkanRayTracingPipelineLibraryCache::FPipelineLibrary*> PipelineLibraries;
	FGraphEventArray PendingLibraryTasks;

	const uint32 NumShaders = InitializerRayGenShaders.Num() + InitializerMissShaders.Num() + InitializerHitGroupShaders.Num() + InitializerCallableShaders.Num();
	PipelineLibraries.Reserve(NumShaders);
	PendingLibraryTasks.Reserve(NumShaders);

	auto ProcessShaderArray = [&PipelineLibraries , &PendingLibraryTasks, LibraryCache, &Initializer](TArrayView<FRHIRayTracingShader*> InitializerShaders, TArray<TRefCountPtr<FVulkanRayTracingShader>>& OutShaders)
		{
			OutShaders.Reserve(InitializerShaders.Num());
			for (FRHIRayTracingShader* const ShaderRHI : InitializerShaders)
			{
				FVulkanRayTracingShader* const Shader = ResourceCast(ShaderRHI);
				FVulkanRayTracingPipelineLibraryCache::FPipelineLibrary* PipelineLibrary = LibraryCache->GetOrAddLibrary(Shader, Initializer);
				if (PipelineLibrary->GraphEvent.IsValid() && !PipelineLibrary->GraphEvent->IsComplete())
				{
					PendingLibraryTasks.Add(PipelineLibrary->GraphEvent);
				}
				
				PipelineLibraries.Add(PipelineLibrary);
				OutShaders.Add(Shader);
			}
		};

	ProcessShaderArray(InitializerRayGenShaders, RayGen.Shaders);
	ProcessShaderArray(InitializerMissShaders, Miss.Shaders);
	ProcessShaderArray(InitializerHitGroupShaders, HitGroup.Shaders);
	ProcessShaderArray(InitializerCallableShaders, Callable.Shaders);
	INC_DWORD_STAT_BY(STAT_VulkanRayTracingCompiledShaders, NumShaders);

	// No need to continue, partial pipelines are only used to kick off early shader compilation (they can't be used for rendering)
	if (bIsPartialPipeline)
	{
		return;
	}

	// Wait for all the libraries to be ready before fetching the handles
	FTaskGraphInterface::Get().WaitUntilTasksComplete(PendingLibraryTasks);

	// Pull out the pipeline library handles to use them to create the pipeline
	TArray<VkPipeline> Pipelines;
	for (FVulkanRayTracingPipelineLibraryCache::FPipelineLibrary* PipelineLibrary : PipelineLibraries)
	{
		checkSlow(!PipelineLibrary->GraphEvent.IsValid() || PipelineLibrary->GraphEvent->IsComplete());
		checkf(PipelineLibrary->PipelineHandle, TEXT("Invalid pipeline library handle while building ray tracing pipeline."));
		Pipelines.Add(PipelineLibrary->PipelineHandle);
	}

	const VkRayTracingPipelineInterfaceCreateInfoKHR RayTracingPipelineInterfaceCreateInfo = {
		.sType = VK_STRUCTURE_TYPE_RAY_TRACING_PIPELINE_INTERFACE_CREATE_INFO_KHR,
		.pNext = nullptr,
		.maxPipelineRayPayloadSize = Initializer.MaxPayloadSizeInBytes,
		.maxPipelineRayHitAttributeSize = Initializer.MaxAttributeSizeInBytes
	};

	const VkPipelineLibraryCreateInfoKHR PipelineLibraryCreateInfo = {
		.sType = VK_STRUCTURE_TYPE_PIPELINE_LIBRARY_CREATE_INFO_KHR,
		.pNext = nullptr,
		.libraryCount = (uint32)Pipelines.Num(),
		.pLibraries = Pipelines.GetData()
	};

	const VkPipelineCreateFlags2CreateInfo Flags2CreateInfo = {
		.sType = VK_STRUCTURE_TYPE_PIPELINE_CREATE_FLAGS_2_CREATE_INFO,
		.pNext = nullptr,
		.flags = Device.GetBindlessDescriptorManager()->GetPipelineCreateFlag() |
				 (GRHIGlobals.RayTracing.SupportsLinearSweptSpheres ? VK_PIPELINE_CREATE_2_RAY_TRACING_ALLOW_SPHERES_AND_LINEAR_SWEPT_SPHERES_BIT_NV : 0ull)
	};

	VkRayTracingPipelineCreateInfoKHR RayTracingPipelineCreateInfo;
	ZeroVulkanStruct(RayTracingPipelineCreateInfo, VK_STRUCTURE_TYPE_RAY_TRACING_PIPELINE_CREATE_INFO_KHR);
	RayTracingPipelineCreateInfo.pLibraryInfo = &PipelineLibraryCreateInfo;
	RayTracingPipelineCreateInfo.pLibraryInterface = &RayTracingPipelineInterfaceCreateInfo;
	RayTracingPipelineCreateInfo.maxPipelineRayRecursionDepth = 1;
	RayTracingPipelineCreateInfo.layout = InDevice.GetBindlessDescriptorManager()->GetPipelineLayout();
	RayTracingPipelineCreateInfo.flags = 0; // using VkPipelineCreateFlags2CreateInfo
	RayTracingPipelineCreateInfo.pNext = &Flags2CreateInfo;

	VERIFYVULKANRESULT_EXPANDED(InDevice.GetPipelineStateCache()->CreateRayTracingPipeline(RayTracingPipelineCreateInfo, bIsPartialPipeline, Pipeline));

	// Grab all shader handles for each stage
	{
		const VkPhysicalDeviceRayTracingPipelinePropertiesKHR& RayTracingPipelineProps = InDevice.GetOptionalExtensionProperties().RayTracingPipelineProps;
		const uint32 HandleSize = RayTracingPipelineProps.shaderGroupHandleSize;

		uint32 HandleOffset = 0;
		auto FetchShaderHandles = [&HandleOffset, HandleSize, &InDevice](VkPipeline RTPipeline, uint32 HandleCount) 
		{
			TArray<uint8> OutHandleStorage;

			if (HandleCount)
			{
				const uint32 ShaderHandleStorageSize = HandleCount * HandleSize;
				OutHandleStorage.AddUninitialized(ShaderHandleStorageSize);

				VERIFYVULKANRESULT(VulkanDynamicAPI::vkGetRayTracingShaderGroupHandlesKHR(InDevice.GetHandle(), RTPipeline, HandleOffset, HandleCount, ShaderHandleStorageSize, OutHandleStorage.GetData()));

				HandleOffset += HandleCount;
			}

			return OutHandleStorage;
		};

		// NOTE: Must be filled in the same order as created above
		RayGen.ShaderHandles = FetchShaderHandles(Pipeline, InitializerRayGenShaders.Num());
		Miss.ShaderHandles = FetchShaderHandles(Pipeline, InitializerMissShaders.Num());
		HitGroup.ShaderHandles = FetchShaderHandles(Pipeline, InitializerHitGroupShaders.Num());
		Callable.ShaderHandles = FetchShaderHandles(Pipeline, InitializerCallableShaders.Num());
	}

	INC_DWORD_STAT(STAT_VulkanRayTracingCreatedPipelines);
}

FVulkanRayTracingPipelineState::~FVulkanRayTracingPipelineState()
{
	if (Pipeline != VK_NULL_HANDLE)
	{
		VulkanRHI::vkDestroyPipeline(Device.GetHandle(), Pipeline, VULKAN_CPU_ALLOCATOR);
		Pipeline = VK_NULL_HANDLE;
	}
}

const FVulkanRayTracingPipelineState::ShaderData& FVulkanRayTracingPipelineState::GetShaderData(EShaderFrequency Frequency) const
{
	switch (Frequency)
	{
	case SF_RayGen:
		return RayGen;
	case SF_RayMiss:
		return Miss;
	case SF_RayHitGroup:
		return HitGroup;
	case SF_RayCallable:
		return Callable;

	default:
		checkf(false, TEXT("Only usable with RayTracing shaders."));
	};

	static ShaderData EmptyShaderData;
	return EmptyShaderData;
}

int32 FVulkanRayTracingPipelineState::GetShaderIndex(const FVulkanRayTracingShader* Shader) const
{
	const FShaderHash Hash = Shader->GetHash();

	const TArray<TRefCountPtr<FVulkanRayTracingShader>>& ShaderArray = GetShaderData(Shader->GetFrequency()).Shaders;
	for (int32 Index = 0; Index < ShaderArray.Num(); ++Index)
	{
		if (Hash == ShaderArray[Index]->GetHash())
		{
			return Index;
		}
	}

	checkf(false, TEXT("RayTracing shader is not present in the given ray tracing pipeline. "));
	return INDEX_NONE;
}

const FVulkanRayTracingShader* FVulkanRayTracingPipelineState::GetVulkanShader(EShaderFrequency Frequency, int32 ShaderIndex) const
{
	return GetShaderData(Frequency).Shaders[ShaderIndex].GetReference();
}

int32 FVulkanRayTracingPipelineState::GetVulkanShaderNum(EShaderFrequency Frequency) const
{
	return GetShaderData(Frequency).Shaders.Num();
}

const TArray<uint8>& FVulkanRayTracingPipelineState::GetShaderHandles(EShaderFrequency Frequency) const
{
	return GetShaderData(Frequency).ShaderHandles;
}

FVulkanRayTracingCompactedSizeQueryPool::FVulkanRayTracingCompactedSizeQueryPool(FVulkanDevice& InDevice, uint32 InMaxQueries)
	: FVulkanQueryPool(InDevice, InMaxQueries, EVulkanQueryPoolType::ASCompactedSize)
{
	QueryOutput.SetNumZeroed(InMaxQueries);
	if (bIsDirty)
	{
		VulkanRHI::vkResetQueryPoolEXT(InDevice.GetHandle(), QueryPool, 0, MaxQueries);
	}
}

void FVulkanRayTracingCompactedSizeQueryPool::EndBatch(FVulkanCommandListContext& CommandContext)
{
	SyncPoint = CommandContext.GetContextSyncPoint();
}

void FVulkanRayTracingCompactedSizeQueryPool::Reset()
{
	VulkanRHI::vkResetQueryPoolEXT(Device.GetHandle(), QueryPool, 0, MaxQueries);
	SyncPoint = nullptr;
	check(QueryOutput.Num() == MaxQueries);
	FMemory::Memzero(QueryOutput.GetData(), MaxQueries * sizeof(uint64));
}

bool FVulkanRayTracingCompactedSizeQueryPool::TryGetResults(uint32 NumResults)
{
	if (!SyncPoint.IsValid() || !SyncPoint->IsComplete())
	{
		return false;
	}

	VkResult Result = VulkanRHI::vkGetQueryPoolResults(Device.GetHandle(), QueryPool, 0, NumResults, NumResults * sizeof(uint64), QueryOutput.GetData(), sizeof(uint64), VK_QUERY_RESULT_64_BIT | VK_QUERY_RESULT_WAIT_BIT);
	if (Result == VK_SUCCESS)
	{
		return true;
	}
	return false;
}

FVulkanRayTracingCompactionRequestHandler::FVulkanRayTracingCompactionRequestHandler(FVulkanDevice& InDevice)
	: Device(InDevice)
{
	QueryPool = new FVulkanRayTracingCompactedSizeQueryPool(InDevice, GVulkanRayTracingMaxBatchedCompaction);

	ActiveRequests.Reserve(GVulkanRayTracingMaxBatchedCompaction);
	ActiveBLASes.Reserve(GVulkanRayTracingMaxBatchedCompaction);
}

FVulkanRayTracingCompactionRequestHandler::~FVulkanRayTracingCompactionRequestHandler()
{
	check(PendingRequests.IsEmpty());
	delete QueryPool;
}

void FVulkanRayTracingCompactionRequestHandler::RequestCompact(FVulkanRayTracingGeometry* InRTGeometry)
{
	check(InRTGeometry->AccelerationStructureBuffer);
	ERayTracingAccelerationStructureFlags GeometryBuildFlags = GetRayTracingAccelerationStructureBuildFlags(InRTGeometry->Initializer);
	check(EnumHasAllFlags(GeometryBuildFlags, ERayTracingAccelerationStructureFlags::AllowCompaction) &&
		EnumHasAllFlags(GeometryBuildFlags, ERayTracingAccelerationStructureFlags::FastTrace) &&
		!EnumHasAnyFlags(GeometryBuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate));

	FScopeLock Lock(&CS);
	PendingRequests.Add(InRTGeometry);
}

bool FVulkanRayTracingCompactionRequestHandler::ReleaseRequest(FVulkanRayTracingGeometry* InRTGeometry)
{
	FScopeLock Lock(&CS);

	// Remove from pending list, not found then try active requests
	if (PendingRequests.Remove(InRTGeometry) <= 0)
	{
		// If currently enqueued, then clear pointer to not handle the compaction request anymore			
		for (int32 BLASIndex = 0; BLASIndex < ActiveBLASes.Num(); ++BLASIndex)
		{
			if (ActiveRequests[BLASIndex] == InRTGeometry)
			{
				ActiveRequests[BLASIndex] = nullptr;
				return true;
			}
		}

		return false;
	}
	else
	{
		return true;
	}
}

void FVulkanRayTracingCompactionRequestHandler::Update(FVulkanCommandListContext& InCommandContext)
{
	LLM_SCOPE_BYNAME(TEXT("FVulkanRT/Compaction"));
	FScopeLock Lock(&CS);

	// If we have an active batch, wait on those queries and launch compaction when the complete
	if (ActiveBLASes.Num() > 0)
	{		
		if (QueryPool->TryGetResults(ActiveBLASes.Num()))
		{
			FVulkanCommandBuffer& CmdBuffer = InCommandContext.GetCommandBuffer();

			// Compact
			for (int32 BLASIndex = 0; BLASIndex < ActiveBLASes.Num(); ++BLASIndex)
			{
				if (ActiveRequests[BLASIndex] != nullptr)
				{
					ActiveRequests[BLASIndex]->CompactAccelerationStructure(CmdBuffer, QueryPool->QueryOutput[BLASIndex]);
				}
			}

			QueryPool->Reset();

			ActiveBLASes.Empty(GVulkanRayTracingMaxBatchedCompaction);

			ActiveRequestsSyncPoint = InCommandContext.GetContextSyncPoint();
		}

		// Only one active batch at a time (otherwise track the offset for when we launch queries)
		return;
	}
	// If we have an active batch, wait until the compaction went through to launch another batch
	else if (ActiveRequests.Num() > 0)
	{
		if (ActiveRequestsSyncPoint.IsValid())
		{
			if (!ActiveRequestsSyncPoint->IsComplete())
			{
				return;
			}

			ActiveRequestsSyncPoint = nullptr;
		}

		ActiveRequests.Empty(GVulkanRayTracingMaxBatchedCompaction);
	}

	check(ActiveBLASes.Num() == 0);
	check(ActiveRequests.Num() == 0);

	// build a new set of build requests to extract the build data	
	for (FVulkanRayTracingGeometry* RTGeometry : PendingRequests)
	{
		ActiveRequests.Add(RTGeometry);
		ActiveBLASes.Add(RTGeometry->Handle);

		// enqueued enough requests for this update round
		if (ActiveRequests.Num() >= GVulkanRayTracingMaxBatchedCompaction)
		{
			break;
		}
	}

	// Do we have requests?
	if (ActiveRequests.Num() > 0)
	{
		// clear out all of the pending requests, don't allow the array to shrink
		PendingRequests.RemoveAt(0, ActiveRequests.Num(), EAllowShrinking::No);

		// Barrier here is not stricly necessary as it is added after the build.
		// AddAccelerationStructureBuildBarrier(CmdBuffer->GetHandle());

		// Write compacted size info from the selected requests
		VulkanDynamicAPI::vkCmdWriteAccelerationStructuresPropertiesKHR(
			InCommandContext.GetCommandBuffer().GetHandle(),
			ActiveBLASes.Num(), ActiveBLASes.GetData(),
			VK_QUERY_TYPE_ACCELERATION_STRUCTURE_COMPACTED_SIZE_KHR,
			QueryPool->GetHandle(),
			0
		);

		QueryPool->EndBatch(InCommandContext);
	}
}


static void SetRayGenResources(FVulkanDevice& Device, FVulkanCommandListContext& Context, const FRayTracingShaderBindings& InGlobalResourceBindings)
{
	// Static uniform buffers
	Context.ApplyShaderBindingLayout();

	const uint32 MaxUniformBuffers = UE_ARRAY_COUNT(InGlobalResourceBindings.UniformBuffers);

	// Uniform buffers
	if (Device.GetBindlessDescriptorManager()->UseDescriptorHeaps())
	{
		TArray<uint32> PushData;
		PushData.Reserve(MaxUniformBuffers);

		uint64 GlobalAddressOrFirstBinding = MAX_uint64;
		for (uint32 UBIndex = 0; UBIndex < MaxUniformBuffers; ++UBIndex)
		{
			const FVulkanUniformBuffer* UniformBuffer = ResourceCast(InGlobalResourceBindings.UniformBuffers[UBIndex]);
			if (UniformBuffer)
			{
				check(UniformBuffer->GetBindlessHandle().IsValid());
				if (UBIndex == 0)
				{
					GlobalAddressOrFirstBinding = UniformBuffer->GetBindlessHandle().GetIndex();
				}
				else
				{
					PushData.SetNumZeroed(UBIndex);
					PushData[UBIndex-1] = UniformBuffer->GetBindlessHandle().GetIndex();
				}
			}
		}

		Context.GetCommandBuffer().SetBindingPushConstants(SF_RayGen, GlobalAddressOrFirstBinding, PushData);
	}
	else 
	{
		TArray<VkWriteDescriptorSet, TInlineAllocator<VulkanBindless::MaxUniformBuffersPerStage>> UBWrites;
		TArray<VkDescriptorBufferInfo, TInlineAllocator<VulkanBindless::MaxUniformBuffersPerStage>> UBBufferInfos;

		for (uint32 UBIndex = 0; UBIndex < MaxUniformBuffers; ++UBIndex)
		{
			const FVulkanUniformBuffer* UniformBuffer = ResourceCast(InGlobalResourceBindings.UniformBuffers[UBIndex]);
			if (UniformBuffer)
			{
				VkDescriptorBufferInfo& BufferInfo = UBBufferInfos.AddZeroed_GetRef();
				BufferInfo.buffer = UniformBuffer->GetBufferHandle();
				BufferInfo.offset = UniformBuffer->GetOffset();
				BufferInfo.range = UniformBuffer->GetSize();

				VkWriteDescriptorSet& Write = UBWrites.AddZeroed_GetRef();
				Write.sType = VK_STRUCTURE_TYPE_WRITE_DESCRIPTOR_SET;
				Write.dstBinding = UBIndex;
				Write.descriptorType = VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER;
				Write.descriptorCount = 1;
				Write.pBufferInfo = &BufferInfo;
			}
		}

		Device.GetBindlessDescriptorManager()->RegisterUniformBuffers(Context, VK_SHADER_STAGE_RAYGEN_BIT_KHR, UBWrites);
	}
}

void FVulkanCommandListContext::RHIRayTraceDispatch(
	FRHIRayTracingPipelineState* InRayTracingPipelineState, 
	FRHIRayTracingShader* InRayGenShader,
	FRHIShaderBindingTable* InSBT,
	const FRayTracingShaderBindings& InGlobalResourceBindings, // :todo-jn:
	uint32 InWidth, uint32 InHeight)
{
	const FVulkanRayTracingPipelineState* Pipeline = ResourceCast(InRayTracingPipelineState);
	FVulkanRayTracingShader* RayGenShader = ResourceCast(InRayGenShader);
	FVulkanRayTracingShaderTable* ShaderTable = ResourceCast(InSBT);
	
	Pipeline->FrameCounter.Set(GFrameNumberRenderThread);

	FVulkanCommandBuffer& CommandBuffer = GetCommandBuffer();
	checkf(!Pipeline->IsPartialPipeline(), TEXT("Attempting to bind partial pipeline, these can't be used for rendering."));
	VulkanRHI::vkCmdBindPipeline(CommandBuffer.GetHandle(), VK_PIPELINE_BIND_POINT_RAY_TRACING_KHR, Pipeline->GetPipeline());

	checkSlow(InRayGenShader->GetFrequency() == SF_RayGen);
	const VkStridedDeviceAddressRegionKHR RayGenRegion = ShaderTable->CommitRayGenShader(*this, Pipeline->GetShaderIndex(RayGenShader), Pipeline->GetShaderHandles(SF_RayGen));

	SetRayGenResources(Device, *this, InGlobalResourceBindings);

	VulkanRHI::vkCmdTraceRaysKHR(
		CommandBuffer.GetHandle(),
		&RayGenRegion,
		ShaderTable->GetRegion(SF_RayMiss),
		ShaderTable->GetRegion(SF_RayHitGroup),
		ShaderTable->GetRegion(SF_RayCallable),
		InWidth, InHeight, 1);
}

void FVulkanCommandListContext::RHIRayTraceDispatchIndirect(
	FRHIRayTracingPipelineState* InRayTracingPipelineState, 
	FRHIRayTracingShader* InRayGenShader,
	FRHIShaderBindingTable* InSBT,
	const FRayTracingShaderBindings& InGlobalResourceBindings, // :todo-jn:
	FRHIBuffer* InArgumentBuffer, uint32 InArgumentOffset)
{
	checkf(GRHISupportsRayTracingDispatchIndirect, TEXT("RHIRayTraceDispatchIndirect may not be used because it is not supported on this machine."));

	const FVulkanRayTracingPipelineState* Pipeline = ResourceCast(InRayTracingPipelineState);
	FVulkanRayTracingShader* RayGenShader = ResourceCast(InRayGenShader);
	FVulkanRayTracingShaderTable* ShaderTable = ResourceCast(InSBT);

	FVulkanCommandBuffer& CommandBuffer = GetCommandBuffer();
	checkf(!Pipeline->IsPartialPipeline(), TEXT("Attempting to bind partial pipeline, these can't be used for rendering."));
	VulkanRHI::vkCmdBindPipeline(CommandBuffer.GetHandle(), VK_PIPELINE_BIND_POINT_RAY_TRACING_KHR, Pipeline->GetPipeline());

	checkSlow(InRayGenShader->GetFrequency() == SF_RayGen);
	const VkStridedDeviceAddressRegionKHR RayGenRegion = ShaderTable->CommitRayGenShader(*this, Pipeline->GetShaderIndex(RayGenShader), Pipeline->GetShaderHandles(SF_RayGen));

	SetRayGenResources(Device, *this, InGlobalResourceBindings);

	FVulkanBuffer* ArgumentBuffer = ResourceCast(InArgumentBuffer);
	const VkDeviceAddress IndirectDeviceAddress = ArgumentBuffer->GetDeviceAddress() + InArgumentOffset;

	VulkanRHI::vkCmdTraceRaysIndirectKHR(
		CommandBuffer.GetHandle(),
		&RayGenRegion,
		ShaderTable->GetRegion(SF_RayMiss),
		ShaderTable->GetRegion(SF_RayHitGroup),
		ShaderTable->GetRegion(SF_RayCallable),
		IndirectDeviceAddress);
}



static void SetSystemParametersUB(FVulkanHitGroupSystemParameters& OutSystemParameters, FVulkanRayTracingShaderTable* ShaderTable, uint32 InNumUniformBuffers, FRHIUniformBuffer* const* InUniformBuffers, const FVulkanRayTracingShader* InShader)
{

	// Plug the shaders in the right slots using LayoutHash comparisons
	check(InShader->GetCodeHeader().UniformBufferInfos.Num() <= (int32)InNumUniformBuffers);
	for (int32 UBIndex = 0; UBIndex < InShader->GetCodeHeader().UniformBufferInfos.Num(); ++UBIndex)
	{
		FVulkanUniformBuffer* UniformBuffer = ResourceCast(InUniformBuffers[UBIndex]);

		const FVulkanShaderHeader::FUniformBufferInfo& UniformBufferInfo = InShader->GetCodeHeader().UniformBufferInfos[UBIndex];

		// :todo-jn: Hack to force in a DummyCullingBuffer in cases where it should have been culled from source (see SPIRV-Tools Issue 4902).
		if (!UniformBuffer)
		{
			UniformBuffer = GetDummyUB(ShaderTable->GetDevice(), UniformBufferInfo.LayoutHash);
		}

		check(UniformBuffer);
		check((UniformBufferInfo.LayoutHash == 0) || (UniformBufferInfo.LayoutHash == UniformBuffer->GetLayout().GetHash()));

		const FRHIDescriptorHandle BindlessHandle = UniformBuffer->GetBindlessHandle();
		check(BindlessHandle.IsValid());
		check(UniformBufferInfo.BindlessCBIndex < UE_ARRAY_COUNT(OutSystemParameters.BindlessUniformBuffers));
		OutSystemParameters.BindlessUniformBuffers[UniformBufferInfo.BindlessCBIndex] = BindlessHandle.GetIndex();

		ShaderTable->AddUBRef(UniformBuffer);
	}
}


static void SetRayTracingHitGroup(
	FVulkanRayTracingShaderTable* ShaderTable, uint32 RecordIndex,
	FVulkanRayTracingPipelineState* Pipeline, uint32 HitGroupIndex,
	const FVulkanRayTracingGeometry* Geometry, uint32 GeometrySegmentIndex,
	uint32 NumUniformBuffers, FRHIUniformBuffer* const* UniformBuffers,
	uint32 LooseParameterDataSize, const void* LooseParameterData,
	uint32 UserData,
	uint32 WorkerIndex)
{
#if DO_CHECK
	if (Geometry)
	{
		const uint32 NumGeometrySegments = Geometry->GetNumSegments();
		checkf(GeometrySegmentIndex < NumGeometrySegments, TEXT("Segment %d is out of range for ray tracing geometry '%s' that contains %d segments"),
			GeometrySegmentIndex, Geometry->DebugName.IsNone() ? TEXT("UNKNOWN") : *Geometry->DebugName.ToString(), NumGeometrySegments);
	}
#endif // DO_CHECK

	ERayTracingShaderBindingMode ShaderBindingMode = ShaderTable->GetShaderBindingMode();
	ERayTracingHitGroupIndexingMode HitGroupIndexingMode = ShaderTable->GetHitGroupIndexingMode();

	if (HitGroupIndexingMode == ERayTracingHitGroupIndexingMode::Allow && Geometry)
	{		
		if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO))
		{
			const FVulkanRayTracingShader* Shader = Pipeline->GetVulkanShader(SF_RayHitGroup, HitGroupIndex);

			FVulkanHitGroupSystemParameters SystemParameters = Geometry->HitGroupSystemParameters[GeometrySegmentIndex];
			SystemParameters.RootConstants.UserData = UserData;
			SetSystemParametersUB(SystemParameters, ShaderTable, NumUniformBuffers, UniformBuffers, Shader);

			ShaderTable->SetLocalShaderParameters(SF_RayHitGroup, RecordIndex, 0, SystemParameters);
			ShaderTable->SetLooseParameterData(SF_RayHitGroup, RecordIndex, LooseParameterData, LooseParameterDataSize);
		}

		if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::Inline))
		{			
			// Only care about shader slot 0 for inline geometry parameters
			uint32 NumShaderSlotsPerGeometrySegment = ShaderTable->GetInitializer().NumShaderSlotsPerGeometrySegment;
			if (RecordIndex % NumShaderSlotsPerGeometrySegment == 0)
			{
				// Setup the inline geometry parameters - can be cached on the geometry as well if needed
				FVulkanRayTracingGeometryParameters SegmentParameters;
				Geometry->SetupInlineGeometryParameters(GeometrySegmentIndex, SegmentParameters);

				// Recompute the geometry segment index from the record index
				uint32 SegmentIndex = RecordIndex / NumShaderSlotsPerGeometrySegment;
				ShaderTable->SetInlineGeometryParameters(SegmentIndex, &SegmentParameters, sizeof(FVulkanRayTracingGeometryParameters));
			}
		}
	}

	if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO))
	{
		ShaderTable->SetSlot(SF_RayHitGroup, RecordIndex, HitGroupIndex, Pipeline->GetShaderHandles(SF_RayHitGroup));
	}
}


static void SetGenericSystemParameters(
	FVulkanRayTracingShaderTable* ShaderTable, uint32 RecordIndex,
	FRHIRayTracingPipelineState* InPipeline, uint32 ShaderIndexInPipeline,
	uint32 NumUniformBuffers, FRHIUniformBuffer* const* UniformBuffers,
	uint32 LooseParameterDataSize, const void* LooseParameterData,
	uint32 UserData, const EShaderFrequency ShaderFrequency)
{
	FVulkanRayTracingPipelineState* Pipeline = ResourceCast(InPipeline);
	const FVulkanRayTracingShader* Shader = Pipeline->GetVulkanShader(ShaderFrequency, ShaderIndexInPipeline);

	FVulkanHitGroupSystemParameters SystemParameters;
	FMemory::Memzero(SystemParameters);
	SystemParameters.RootConstants.UserData = UserData;
	SetSystemParametersUB(SystemParameters, ShaderTable, NumUniformBuffers, UniformBuffers, Shader);
	ShaderTable->SetLocalShaderParameters(ShaderFrequency, RecordIndex, 0, SystemParameters);
	ShaderTable->SetLooseParameterData(ShaderFrequency, RecordIndex, LooseParameterData, LooseParameterDataSize);

	ShaderTable->SetSlot(ShaderFrequency, RecordIndex, ShaderIndexInPipeline, Pipeline->GetShaderHandles(ShaderFrequency));
}


void FVulkanCommandListContext::RHISetBindingsOnShaderBindingTable(FRHIShaderBindingTable* InSBT,
	FRHIRayTracingPipelineState* InPipeline,
	uint32 NumBindings, const FRayTracingLocalShaderBindings* Bindings,
	ERayTracingBindingType BindingType)
{
	FVulkanRayTracingPipelineState* Pipeline = ResourceCast(InPipeline);
	FVulkanRayTracingShaderTable* ShaderTable = ResourceCast(InSBT);

	FGraphEventArray TaskList;

	const uint32 NumWorkerThreads = FTaskGraphInterface::Get().GetNumWorkerThreads();
	const uint32 MaxTasks = FApp::ShouldUseThreadingForPerformance() ? FMath::Min<uint32>(NumWorkerThreads, FVulkanRayTracingShaderTable::MaxBindingWorkers) : 1;

	struct FTaskContext
	{
		uint32 WorkerIndex = 0;
	};

	TArray<FTaskContext, TInlineAllocator<FVulkanRayTracingShaderTable::MaxBindingWorkers>> TaskContexts;
	for (uint32 WorkerIndex = 0; WorkerIndex < MaxTasks; ++WorkerIndex)
	{
		TaskContexts.Add(FTaskContext{ WorkerIndex });
	}

	auto BindingTask = [this, Bindings, Pipeline, ShaderTable, BindingType](const FTaskContext& Context, int32 CurrentIndex)
	{
		const FRayTracingLocalShaderBindings& Binding = Bindings[CurrentIndex];

		if (BindingType == ERayTracingBindingType::HitGroup)
		{
			const FVulkanRayTracingGeometry* Geometry = ResourceCast(Binding.Geometry);

			if (Binding.BindingType != ERayTracingLocalShaderBindingType::Clear)
			{
				SetRayTracingHitGroup(
					ShaderTable, Binding.RecordIndex,
					Pipeline, Binding.ShaderIndexInPipeline,
					Geometry, Binding.SegmentIndex,
					Binding.NumUniformBuffers,
					Binding.UniformBuffers,
					Binding.LooseParameterDataSize,
					Binding.LooseParameterData,
					Binding.UserData,
					Context.WorkerIndex);
			}
			else
			{
				// Only transient SBT support for Vulkan right now (otherwise hit record data might need to be cleared)
				check(ShaderTable->GetInitializer().Lifetime == ERayTracingShaderBindingTableLifetime::Transient);
			}
		}
		else if (BindingType == ERayTracingBindingType::CallableShader)
		{
			SetGenericSystemParameters(
				ShaderTable, Binding.RecordIndex,
				Pipeline, Binding.ShaderIndexInPipeline,
				Binding.NumUniformBuffers, Binding.UniformBuffers,
				Binding.LooseParameterDataSize, Binding.LooseParameterData,
				Binding.UserData,
				SF_RayCallable);
		}
		else if (BindingType == ERayTracingBindingType::MissShader)
		{
			SetGenericSystemParameters(
				ShaderTable, Binding.RecordIndex,
				Pipeline, Binding.ShaderIndexInPipeline,
				Binding.NumUniformBuffers, Binding.UniformBuffers,
				Binding.LooseParameterDataSize, Binding.LooseParameterData,
				Binding.UserData,
				SF_RayMiss);
		}
		else
		{
			checkNoEntry();
		}
	};

	// One helper worker task will be created at most per this many work items, plus one worker for current thread (unless running on a task thread),
	// up to a hard maximum of FVulkanRayTracingShaderTable::MaxBindingWorkers.
	// Internally, parallel for tasks still subdivide the work into smaller chunks and perform fine-grained load-balancing.
	const int32 ItemsPerTask = 1024;

	ParallelForWithExistingTaskContext(TEXT("SetRayTracingBindings"), MakeArrayView(TaskContexts), NumBindings, ItemsPerTask, BindingTask);
}
