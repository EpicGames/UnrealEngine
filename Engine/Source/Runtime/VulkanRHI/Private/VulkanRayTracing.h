// Copyright Epic Games, Inc. All Rights Reserved..

#pragma once

#include "VulkanRHIPrivate.h"

#include "RayTracingBuiltInResources.h"
#include "VulkanPipeline.h"
#include "VulkanQuery.h"

class FVulkanCommandListContext;
class FVulkanBuffer;
struct FVulkanRayTracingGeometryParameters;

class FVulkanSyncPoint;
using FVulkanSyncPointRef = TRefCountPtr<FVulkanSyncPoint>;

class FVulkanRayTracingPlatform
{
public:
	static bool CheckVulkanInstanceFunctions(VkInstance inInstance);
};


// Built-in local root parameters that are always bound to all hit shaders
// :todo-jn: NOTE: Keep in sync with VulkanCommon.ush decl until it's put in a common header
struct FVulkanHitGroupSystemParameters
{
	FHitGroupSystemRootConstants RootConstants;

	uint32 BindlessHitGroupSystemIndexBuffer;
	uint32 BindlessHitGroupSystemVertexBuffer;

	uint32 BindlessUniformBuffers[32];

	// *** Globals start here ***
};


struct FVkRtTLASBuildData
{
	FVkRtTLASBuildData()
	{
		ZeroVulkanStruct(Geometry, VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_GEOMETRY_KHR);
		ZeroVulkanStruct(GeometryInfo, VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_BUILD_GEOMETRY_INFO_KHR);
		ZeroVulkanStruct(SizesInfo, VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_BUILD_SIZES_INFO_KHR);
	}

	VkAccelerationStructureGeometryKHR Geometry;
	VkAccelerationStructureBuildGeometryInfoKHR GeometryInfo;
	VkAccelerationStructureBuildSizesInfoKHR SizesInfo;
};

struct FVkRtBLASBuildData
{
	FVkRtBLASBuildData()
	{
		ZeroVulkanStruct(GeometryInfo, VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_BUILD_GEOMETRY_INFO_KHR);
		ZeroVulkanStruct(SizesInfo, VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_BUILD_SIZES_INFO_KHR);
	}

	TArray<VkAccelerationStructureGeometryKHR, TInlineAllocator<1>> Segments;
	TArray<VkAccelerationStructureGeometryLinearSweptSpheresDataNV, TInlineAllocator<1>> LSSSegments;
	TArray<VkAccelerationStructureBuildRangeInfoKHR, TInlineAllocator<1>> Ranges;
	VkAccelerationStructureBuildGeometryInfoKHR GeometryInfo;
	VkAccelerationStructureBuildSizesInfoKHR SizesInfo;
};

class FVulkanRayTracingShaderTable : public FRHIShaderBindingTable
{
public:
	FVulkanRayTracingShaderTable(FRHICommandListBase& RHICmdList, FVulkanDevice& InDevice, const FRayTracingShaderBindingTableInitializer& InInitializer);
	~FVulkanRayTracingShaderTable();

	void ReleaseLocalBuffers();

	const VkStridedDeviceAddressRegionKHR* GetRegion(EShaderFrequency Frequency);

	void SetSlot(EShaderFrequency Frequency, uint32 DstSlot, uint32 SrcHandleIndex, TConstArrayView<uint8> SrcHandleData);

	VkStridedDeviceAddressRegionKHR CommitRayGenShader(FVulkanCommandListContext& Context, uint32 SrcHandleIndex, TConstArrayView<uint8> SrcHandleData);

	template <typename T>
	void SetLocalShaderParameters(EShaderFrequency Frequency, uint32 RecordIndex, uint32 InOffsetWithinRootSignature, const T& Parameters)
	{
		SetLocalShaderParameters(Frequency, RecordIndex, InOffsetWithinRootSignature, &Parameters, sizeof(Parameters));
	}

	void SetLocalShaderParameters(EShaderFrequency Frequency, uint32 RecordIndex, uint32 OffsetWithinRecord, const void* InData, uint32 InDataSize);

	void SetLooseParameterData(EShaderFrequency Frequency, uint32 RecordIndex, const void* InData, uint32 InDataSize)
	{
		if (InData && InDataSize)
		{
			// Place the loose parameter data after the FVulkanHitGroupSystemParameters in the shader record
			const uint32 LooseParameterDataOffset = Align(sizeof(FVulkanHitGroupSystemParameters), 4);
			SetLocalShaderParameters(Frequency, RecordIndex, LooseParameterDataOffset, InData, InDataSize);
		}
	}

	void SetInlineGeometryParameters(uint32 SegmentIndex, const void* InData, uint32 InDataSize);

	void Commit(FVulkanCommandListContext& Context, FRHIBuffer* InlineBindingDataBuffer);

	virtual FRHISizeAndStride GetInlineBindingDataSizeAndStride() const override final;

	void AddUBRef(FRHIUniformBuffer* UB)
	{
		ReferencedUniformBuffers.AddUnique(UB);
	}

	TArrayView<TRefCountPtr<FRHIUniformBuffer>> GetUBRefs()
	{
		return ReferencedUniformBuffers;
	}

	ERayTracingHitGroupIndexingMode GetHitGroupIndexingMode() const
	{
		return HitGroupIndexingMode;
	}

	ERayTracingShaderBindingMode GetShaderBindingMode() const
	{
		return ShaderBindingMode;
	}

	FVulkanDevice& GetDevice()
	{
		return Device;
	}

	// Ray tracing shader bindings can be processed in parallel.
	// Each concurrent worker gets its own dedicated descriptor cache instance to avoid contention or locking.
	// Scaling beyond 5 total threads does not yield any speedup in practice (RHI thread + 4 parallel workers).
	static constexpr uint32 MaxBindingWorkers = 1; // :todo-jn:

private:

	struct FVulkanShaderTableAllocation
	{
		FVulkanShaderTableAllocation()
		{
			FMemory::Memzero(Region);
		}

		uint32 HandleCount = 0;
		bool bUseLocalRecord = false;

		// Host memory copy
		TArray<uint8> HostBuffer;

		// GPU Local memory copy
		VkBuffer LocalBuffer = VK_NULL_HANDLE;
		VulkanRHI::FVulkanAllocation LocalAllocation;
		VkStridedDeviceAddressRegionKHR Region;

		bool bIsDirty = true;
	};

	FVulkanShaderTableAllocation& GetAlloc(EShaderFrequency Frequency);
	static void ReleaseLocalBuffer(FVulkanDevice& Device, FVulkanShaderTableAllocation& Alloc);
	
	FVulkanDevice& Device;

	ERayTracingShaderBindingMode ShaderBindingMode = ERayTracingShaderBindingMode::Disabled;
	ERayTracingHitGroupIndexingMode HitGroupIndexingMode = ERayTracingHitGroupIndexingMode::Allow;

	FVulkanShaderTableAllocation Miss;
	FVulkanShaderTableAllocation HitGroup;
	FVulkanShaderTableAllocation Callable;
		
	// Buffer that contains per-hitrecord index and vertex buffer binding data
	TArray<uint8> InlineGeometryParameterData;

	TArray<TRefCountPtr<FRHIUniformBuffer>> ReferencedUniformBuffers;

	// Convenience
	const uint32 HandleSize;
	const uint32 HandleSizeAligned;
};

class FVulkanRayTracingGeometry : public FRHIRayTracingGeometry
{
public:
	static constexpr uint32 IndicesPerPrimitive = 3; // Only triangle meshes are supported

	FVulkanRayTracingGeometry(ENoInit);
	FVulkanRayTracingGeometry(FRHICommandListBase& RHICmdList, const FRayTracingGeometryInitializer& Initializer, FVulkanDevice* InDevice);
	~FVulkanRayTracingGeometry();

	virtual VkDeviceAddress GetDeviceAddress() const { return Address; }

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual FRayTracingAccelerationStructureAddress GetAccelerationStructureAddress(uint64 GPUIndex) const final override { return GetDeviceAddress(); }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	void Swap(FVulkanRayTracingGeometry& Other);

	using FRHIRayTracingGeometry::Initializer;
	using FRHIRayTracingGeometry::SizeInfo;
	
	void RemoveCompactionRequest();
	void CompactAccelerationStructure(FVulkanCommandBuffer& CmdBuffer, uint64 InSizeAfterCompaction);

	void SetupHitGroupSystemParameters();
	void ReleaseBindlessHandles();

	void SetupInlineGeometryParameters(uint32 GeometrySegmentIndex, FVulkanRayTracingGeometryParameters& Parameters) const;

	VkAccelerationStructureKHR Handle = VK_NULL_HANDLE;
	VkDeviceAddress Address = 0;
	TRefCountPtr<FVulkanBuffer> AccelerationStructureBuffer;
	bool bHasPendingCompactionRequests = false;
	uint64 AccelerationStructureCompactedSize = 0;

	FVulkanDevice* const Device = nullptr;

	TArray<FVulkanHitGroupSystemParameters> HitGroupSystemParameters;
	TArray<FRHIDescriptorHandle> HitGroupSystemVertexViews;  // :todo-jn: use views?
	FRHIDescriptorHandle HitGroupSystemIndexView;

	FDebugName DebugName;
	FName OwnerName;		// Store the path name of the owner object for resource tracking
};

class FVulkanRayTracingScene : public FRHIRayTracingScene
{
	friend FVulkanCommandListContext;

public:
	FVulkanRayTracingScene(FRayTracingSceneInitializer Initializer, FVulkanDevice& InDevice);
	~FVulkanRayTracingScene();

	const FRayTracingSceneInitializer& GetInitializer() const override final { return Initializer; }

	void BindBuffer(FRHIBuffer* InBuffer, uint32 InBufferOffset);

	inline bool IsBuilt() const
	{
		return bBuilt;
	}

	using FRHIRayTracingAccelerationStructure::SizeInfo;
	using FRHIRayTracingScene::ReferencedGeometries;
	using FRHIRayTracingScene::ReferencedBuffers;
	using FRHIRayTracingScene::SetReferencedResources;

	FVulkanDevice& Device;

	const FRayTracingSceneInitializer Initializer;

	// Native TLAS handles are owned by SRV objects in Vulkan RHI.
	// D3D12 and other RHIs allow creating TLAS SRVs from any GPU address at any point
	// and do not require them for operations such as build or update.
	// FVulkanRayTracingScene can't own the VkAccelerationStructureKHR directly because
	// we allow TLAS memory to be allocated using transient resource allocator and 
	// the lifetime of the scene object may be different from the lifetime of the buffer.
	// Many VkAccelerationStructureKHR-s may be created, pointing at the same buffer.

	TUniquePtr<FVulkanView> View;

	uint32 NumInstances = 0;
	
	TRefCountPtr<FVulkanBuffer> AccelerationStructureBuffer;
	
	bool bBuilt = false;

private:
	UE::FMutex Mutex;
};


class FVulkanRayTracingPipelineState : public FRHIRayTracingPipelineState
{
public:

	UE_NONCOPYABLE(FVulkanRayTracingPipelineState);
	FVulkanRayTracingPipelineState(FVulkanDevice& InDevice, const FRayTracingPipelineStateInitializer& Initializer);
	~FVulkanRayTracingPipelineState();

	VkPipeline GetPipeline() const
	{
		return Pipeline;
	}

	bool IsPartialPipeline() const
	{
		return bIsPartialPipeline;
	}

	int32 GetShaderIndex(const FVulkanRayTracingShader* Shader) const;
	const FVulkanRayTracingShader* GetVulkanShader(EShaderFrequency Frequency, int32 ShaderIndex) const;
	int32 GetVulkanShaderNum(EShaderFrequency Frequency) const;
	const TArray<uint8>& GetShaderHandles(EShaderFrequency Frequency) const;

private:
	FVulkanDevice& Device;

	struct ShaderData
	{
		TArray<TRefCountPtr<FVulkanRayTracingShader>> Shaders;
		TArray<uint8> ShaderHandles;
	};

	const ShaderData& GetShaderData(EShaderFrequency Frequency) const;

	ShaderData RayGen;
	ShaderData Miss;
	ShaderData HitGroup;
	ShaderData Callable;

	VkPipeline Pipeline = VK_NULL_HANDLE;

	const bool bIsPartialPipeline;

public:
	VulkanResourceFrameCounter FrameCounter;

	friend FVulkanCommandListContext;
	friend FVulkanRayTracingScene;
};

class FVulkanRayTracingCompactedSizeQueryPool : public FVulkanQueryPool
{
public:
	FVulkanRayTracingCompactedSizeQueryPool(FVulkanDevice& InDevice, uint32 InMaxQueries);

	void EndBatch(FVulkanCommandListContext& CommandContext);
	bool TryGetResults(uint32 NumResults);
	void Reset();

	TArray<uint64> QueryOutput;
	FVulkanSyncPointRef SyncPoint;
};

// Manages all the pending BLAS compaction requests
class FVulkanRayTracingCompactionRequestHandler
{
public:
	UE_NONCOPYABLE(FVulkanRayTracingCompactionRequestHandler)

	FVulkanRayTracingCompactionRequestHandler(FVulkanDevice& InDevice);
	~FVulkanRayTracingCompactionRequestHandler();

	void RequestCompact(FVulkanRayTracingGeometry* InRTGeometry);
	bool ReleaseRequest(FVulkanRayTracingGeometry* InRTGeometry);

	void Update(FVulkanCommandListContext& InCommandContext);

private:
	FVulkanDevice& Device;

	FCriticalSection CS;
	TArray<FVulkanRayTracingGeometry*> PendingRequests;
	TArray<VkAccelerationStructureKHR> ActiveBLASes;

	// Keep references on FVulkanRayTracingGeometry until lifetime issue is found (this prevents cancellation)
	TArray<TRefCountPtr<FVulkanRayTracingGeometry>> ActiveRequests;
	FVulkanSyncPointRef ActiveRequestsSyncPoint;

	FVulkanRayTracingCompactedSizeQueryPool* QueryPool = nullptr;
};

