// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanRenderTarget.cpp: Vulkan render target implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "ScreenRendering.h"
#include "VulkanPendingState.h"
#include "VulkanContext.h"
#include "VulkanSwapChain.h"
#include "SceneUtils.h"
#include "RHISurfaceDataConversion.h"
#include "VulkanRenderpass.h"

// Debug mode used as workaround when a DEVICE LOST occurs on alt+tab on some platforms
// This is a workaround and may end up causing some hitches on the rendering thread
static int32 GVulkanFlushOnMapStaging = 0;
static FAutoConsoleVariableRef CVarGVulkanFlushOnMapStaging(
	TEXT("r.Vulkan.FlushOnMapStaging"),
	GVulkanFlushOnMapStaging,
	TEXT("Flush GPU on MapStagingSurface calls without any fence.\n")
	TEXT(" 0: Do not Flush (default)\n")
	TEXT(" 1: Flush"),
	ECVF_Default
);

static int32 GIgnoreCPUReads = 0;
static FAutoConsoleVariableRef CVarVulkanIgnoreCPUReads(
	TEXT("r.Vulkan.IgnoreCPUReads"),
	GIgnoreCPUReads,
	TEXT("Debugging utility for GPU->CPU reads.\n")
	TEXT(" 0 will read from the GPU (default).\n")
	TEXT(" 1 will NOT read from the GPU and fill with zeros.\n"),
	ECVF_Default
	);

static FCriticalSection GStagingMapLock;
static TMap<FVulkanTexture*, VulkanRHI::FStagingBuffer*> GPendingLockedStagingBuffers;

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
TAutoConsoleVariable<int32> CVarVulkanDebugBarrier(
	TEXT("r.Vulkan.DebugBarrier"),
	0,
	TEXT("Forces a full barrier for debugging. This is a mask/bitfield (so add up the values)!\n")
	TEXT(" 0: Don't (default)\n")
	TEXT(" 1: Enable heavy barriers after EndRenderPass()\n")
	TEXT(" 2: Enable heavy barriers after every dispatch\n")
	TEXT(" 4: Enable heavy barriers after upload cmd buffers\n")
	TEXT(" 8: Enable heavy barriers after active cmd buffers\n")
	TEXT(" 16: Enable heavy buffer barrier after uploads\n")
	TEXT(" 32: Enable heavy buffer barrier between acquiring back buffer and blitting into swapchain\n"),
	ECVF_Default
);
#endif

FVulkanRenderPass* FVulkanCommandListContext::PrepareRenderPassForPSOCreation(const FGraphicsPipelineStateInitializer& Initializer)
{
	FVulkanRenderTargetLayout RTLayout(Initializer);
	return PrepareRenderPassForPSOCreation(RTLayout);
}

FVulkanRenderPass* FVulkanCommandListContext::PrepareRenderPassForPSOCreation(const FVulkanRenderTargetLayout& RTLayout)
{
	FVulkanRenderPass* RenderPass = nullptr;
	RenderPass = Device.GetRenderPassManager().GetOrCreateRenderPass(RTLayout);
	return RenderPass;
}

static void ConvertRawDataToFColor(VkFormat VulkanFormat, uint32 DestWidth, uint32 DestHeight, uint8* In, uint32 SrcPitch, FColor* Dest, const FReadSurfaceDataFlags& InFlags)
{
	const bool bLinearToGamma = InFlags.GetLinearToGamma();
	switch (VulkanFormat)
	{
	case VK_FORMAT_R32G32B32A32_SFLOAT:
		ConvertRawR32G32B32A32DataToFColor(DestWidth, DestHeight, In, SrcPitch, Dest, bLinearToGamma);
		break;

	case VK_FORMAT_R16G16B16A16_SFLOAT:
		ConvertRawR16G16B16A16FDataToFColor(DestWidth, DestHeight, In, SrcPitch, Dest, bLinearToGamma);
		break;

	case VK_FORMAT_B10G11R11_UFLOAT_PACK32:
		ConvertRawR11G11B10DataToFColor(DestWidth, DestHeight, In, SrcPitch, Dest, bLinearToGamma);
		break;

	case VK_FORMAT_A2B10G10R10_UNORM_PACK32:
		ConvertRawR10G10B10A2DataToFColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	case VK_FORMAT_R8G8B8A8_UNORM:
		ConvertRawR8G8B8A8DataToFColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	case VK_FORMAT_R16G16B16A16_UNORM:
		ConvertRawR16G16B16A16DataToFColor(DestWidth, DestHeight, In, SrcPitch, Dest, bLinearToGamma);
		break;

	case VK_FORMAT_B8G8R8A8_UNORM:
		ConvertRawB8G8R8A8DataToFColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	case VK_FORMAT_R8_UNORM:
		ConvertRawR8DataToFColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	case VK_FORMAT_R8G8_UNORM:
		ConvertRawR8G8DataToFColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	case VK_FORMAT_R16_UNORM:
		ConvertRawR16DataToFColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	case VK_FORMAT_R16G16_UNORM:
		ConvertRawR16G16DataToFColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	default:
		checkf(false, TEXT("Unsupported format [%d] for conversion to FColor!"), (uint32)VulkanFormat);
		break;
	}

}

static void ConvertRawDataToFLinearColor(VkFormat VulkanFormat, uint32 DestWidth, uint32 DestHeight, uint8* In, uint32 SrcPitch, FLinearColor* Dest, const FReadSurfaceDataFlags& InFlags)
{
	switch (VulkanFormat)
	{
	case VK_FORMAT_R32G32B32A32_SFLOAT:
		ConvertRawR32G32B32A32DataToFLinearColor(DestWidth, DestHeight, In, SrcPitch, Dest, InFlags);
		break;

	case VK_FORMAT_R16G16B16A16_SFLOAT:
		ConvertRawR16G16B16A16FDataToFLinearColor(DestWidth, DestHeight, In, SrcPitch, Dest, InFlags);
		break;

	case VK_FORMAT_R16_SFLOAT:
		ConvertRawR16FDataToFLinearColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	case VK_FORMAT_B10G11R11_UFLOAT_PACK32:
		ConvertRawR11G11B10FDataToFLinearColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	case VK_FORMAT_A2B10G10R10_UNORM_PACK32:
		ConvertRawA2B10G10R10DataToFLinearColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	case VK_FORMAT_R8G8B8A8_UNORM:
		ConvertRawR8G8B8A8DataToFLinearColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	case VK_FORMAT_R16G16B16A16_UNORM:
		ConvertRawR16G16B16A16DataToFLinearColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	case VK_FORMAT_B8G8R8A8_UNORM:
		ConvertRawB8G8R8A8DataToFLinearColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	case VK_FORMAT_R16G16_UNORM:
		ConvertRawR16G16DataToFLinearColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	case VK_FORMAT_R16_UNORM:
		ConvertRawR16UDataToFLinearColor(DestWidth, DestHeight, In, SrcPitch, Dest);
		break;

	default:
		checkf(false, TEXT("Unsupported format [%d] for conversion to FLinearColor!"), (uint32)VulkanFormat);
		break;
	}
}

template<typename ColorType>
static void ReadSurfaceData(FVulkanDevice* Device, FRHITexture* TextureRHI, FIntRect Rect, TArray<ColorType>& OutData, FReadSurfaceDataFlags InFlags)
{
	checkf((!TextureRHI->GetDesc().IsTextureCube()) || (InFlags.GetCubeFace() == CubeFace_MAX), TEXT("Cube faces not supported yet."));
	checkf(((Rect.Max.X > 0) && (Rect.Min.X >=0) && (Rect.Max.X >= Rect.Min.X)), TEXT("Width must be positive."));
	checkf(((Rect.Max.Y > 0) && (Rect.Min.Y >= 0) && (Rect.Max.Y >= Rect.Min.Y)), TEXT("Height must be positive."));

	const uint32 DestWidth = Rect.Max.X - Rect.Min.X;
	const uint32 DestHeight = Rect.Max.Y - Rect.Min.Y;
	const uint32 NumRequestedPixels = DestWidth * DestHeight;
	OutData.SetNumUninitialized(NumRequestedPixels);
	if (GIgnoreCPUReads)
	{
		// Debug: Fill with CPU
		FMemory::Memzero(OutData.GetData(), NumRequestedPixels * sizeof(ColorType));
		return;
	}

	const FRHITextureDesc& Desc = TextureRHI->GetDesc();
	switch (Desc.Dimension)
	{
	case ETextureDimension::Texture2D:
	case ETextureDimension::Texture2DArray:
		// In VR, the high level code calls this function on the viewport render target, without knowing that it's
		// actually a texture array created and managed by the VR runtime. In that case we'll just read the first
		// slice of the array, which corresponds to one of the eyes.
		break;

	default:
		// Just return black for texture types we don't support.
		FMemory::Memzero(OutData.GetData(), NumRequestedPixels * sizeof(ColorType));
		return;
	}

	FVulkanTexture& Surface = *ResourceCast(TextureRHI);


	// Figure out the size of the buffer required to hold the requested pixels
	const uint32 PixelByteSize = VulkanRHI::GetNumBitsPerPixel(Surface.StorageFormat) / 8;
	checkf(GPixelFormats[TextureRHI->GetFormat()].Supported && (PixelByteSize > 0), TEXT("Trying to read from unsupported format."));
	const uint32 BufferSize = NumRequestedPixels * PixelByteSize;

	// Validate that the Rect is within the texture
	const uint32 MipLevel = InFlags.GetMip();
	const uint32 MipSizeX = FMath::Max(Desc.Extent.X >> MipLevel, 1);
	const uint32 MipSizeY = FMath::Max(Desc.Extent.Y >> MipLevel, 1);
	checkf((Rect.Max.X <= (int32)MipSizeX) && (Rect.Max.Y <= (int32)MipSizeY), TEXT("The specified Rect [%dx%d] extends beyond this Mip [%dx%d]."), Rect.Max.X, Rect.Max.Y, MipSizeX, MipSizeY);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	VulkanRHI::FStagingBuffer* StagingBuffer = nullptr;
	const bool bCPUReadback = EnumHasAllFlags(Surface.GetDesc().Flags, TexCreate_CPUReadback);

	if (!bCPUReadback) //this function supports reading back arbitrary rendertargets, so if its not a cpu readback surface, we do a copy.
	{
		StagingBuffer = Device->GetStagingManager().AcquireBuffer(BufferSize, VK_BUFFER_USAGE_TRANSFER_DST_BIT, VK_MEMORY_PROPERTY_HOST_CACHED_BIT);
		ensure(StagingBuffer->GetSize() >= BufferSize);

		VkBufferImageCopy CopyRegion;
		FMemory::Memzero(CopyRegion);
		// Leave bufferRowLength/bufferImageHeight at 0 for tightly packed
		CopyRegion.imageSubresource.aspectMask = Surface.GetFullAspectMask();
		CopyRegion.imageSubresource.mipLevel = MipLevel;
		CopyRegion.imageSubresource.baseArrayLayer = InFlags.GetArrayIndex();
		CopyRegion.imageSubresource.layerCount = 1;
		CopyRegion.imageOffset.x = Rect.Min.X;
		CopyRegion.imageOffset.y = Rect.Min.Y;
		CopyRegion.imageExtent.width = DestWidth;
		CopyRegion.imageExtent.height = DestHeight;
		CopyRegion.imageExtent.depth = 1;

		RHICmdList.EnqueueLambda([CopyRegion, StagingBuffer, &Surface](FRHICommandListBase& ExecutingCmdList)
		{
			FVulkanCommandListContext& Context = FVulkanCommandListContext::Get(ExecutingCmdList);
			FVulkanCommandBuffer& CommandBuffer = Context.GetCommandBuffer();
			VulkanRHI::vkCmdCopyImageToBuffer(CommandBuffer.GetHandle(), Surface.Image, VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL, StagingBuffer->GetHandle(), 1, &CopyRegion);

			FVulkanPipelineBarrier AfterBarrier;
			AfterBarrier.AddMemoryBarrier(VK_ACCESS_TRANSFER_WRITE_BIT, VK_ACCESS_HOST_READ_BIT, VK_PIPELINE_STAGE_TRANSFER_BIT, VK_PIPELINE_STAGE_HOST_BIT);
			AfterBarrier.Execute(&CommandBuffer);
		});
	}

	// We need to execute the command list so we can read the data from the map below
	RHICmdList.SubmitAndBlockUntilGPUIdle();

	uint8* In;
	uint32 SrcPitch;
	if (bCPUReadback)
	{
		// If the text was bCPUReadback, then we have to deal with our Rect potentially being a subset of the total texture
		In = (uint8*)Surface.GetMappedPointer() + ((Rect.Min.Y * MipSizeX + Rect.Min.X) * PixelByteSize);
		SrcPitch = MipSizeX * PixelByteSize;
	}
	else
	{
		// If the text was NOT bCPUReadback, the buffer contains only the (tightly packed) Rect we requested
		StagingBuffer->InvalidateMappedMemory();
		In = (uint8*)StagingBuffer->GetMappedPointer();
		SrcPitch = DestWidth * PixelByteSize;
	}

	ColorType* Dest = OutData.GetData();
	if constexpr (std::is_same_v<FLinearColor, ColorType>)
	{
		ConvertRawDataToFLinearColor(Surface.StorageFormat, DestWidth, DestHeight, In, SrcPitch, Dest, InFlags);
	}
	else if constexpr (std::is_same_v<FColor, ColorType>)
	{
		ConvertRawDataToFColor(Surface.StorageFormat, DestWidth, DestHeight, In, SrcPitch, Dest, InFlags);
	}
	else
	{
		// set output to black for invalid ColorType's
		FMemory::Memzero(OutData.GetData(), NumRequestedPixels * sizeof(ColorType));
		checkNoEntry();
	}

	if (!bCPUReadback)
	{
		Device->GetStagingManager().ReleaseBuffer(nullptr, StagingBuffer);
	}
}

void FVulkanDynamicRHI::RHIReadSurfaceData(FRHITexture* TextureRHI, FIntRect Rect, TArray<FColor>& OutData, FReadSurfaceDataFlags InFlags)
{
	ReadSurfaceData(Device, TextureRHI, Rect, OutData, InFlags);
}

void FVulkanDynamicRHI::RHIReadSurfaceData(FRHITexture* TextureRHI, FIntRect Rect, TArray<FLinearColor>& OutData, FReadSurfaceDataFlags InFlags)
{
	ReadSurfaceData(Device, TextureRHI, Rect, OutData, InFlags);
}

void FVulkanDynamicRHI::RHIMapStagingSurface(FRHITexture* TextureRHI, FRHIGPUFence* FenceRHI, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex)
{
	FVulkanTexture* Texture = ResourceCast(TextureRHI);

	if (FenceRHI && !FenceRHI->Poll())
	{
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);

		// SubmitAndBlockUntilGPUIdle might update fence state if it was tied to a previously submitted command buffer.
		// Its state will have been updated from Submitted to NeedReset, and would assert in WaitForCmdBuffer (which is not needed in such a case)
		FenceRHI->Wait(RHICmdList, FRHIGPUMask::All());
	}
	else
	{
		if (GVulkanFlushOnMapStaging)
		{
			FRHICommandListImmediate::Get().ImmediateFlush(EImmediateFlushType::FlushRHIThread);
			Device->WaitUntilIdle();
		}
	}


	check(EnumHasAllFlags(Texture->GetDesc().Flags, TexCreate_CPUReadback));
	OutData = Texture->GetMappedPointer();
	Texture->InvalidateMappedMemory();
	OutWidth = Texture->GetSizeX();
	OutHeight = Texture->GetSizeY();
}

void FVulkanDynamicRHI::RHIUnmapStagingSurface(FRHITexture* TextureRHI, uint32 GPUIndex)
{
}

void FVulkanDynamicRHI::RHIReadSurfaceFloatData(FRHITexture* TextureRHI, FIntRect Rect, TArray<FFloat16Color>& OutData, ECubeFace CubeFace, int32 ArrayIndex, int32 MipIndex)
{
	auto DoCopyFloat = [](FVulkanDevice* InDevice, const FVulkanTexture& VulkanTexture, uint32 InMipIndex, uint32 SrcBaseArrayLayer, FIntRect InRect, TArray<FFloat16Color>& OutputData)
	{
		ensure(VulkanTexture.StorageFormat == VK_FORMAT_R16G16B16A16_SFLOAT);

		const FRHITextureDesc& Desc = VulkanTexture.GetDesc();

		const uint32 NumPixels = (Desc.Extent.X >> InMipIndex) * (Desc.Extent.Y >> InMipIndex);
		const uint32 Size = NumPixels * sizeof(FFloat16Color);
		VulkanRHI::FStagingBuffer* StagingBuffer = InDevice->GetStagingManager().AcquireBuffer(Size, VK_BUFFER_USAGE_TRANSFER_DST_BIT, VK_MEMORY_PROPERTY_HOST_CACHED_BIT);

		// the staging buffer size may be bigger then the size due to alignment, etc. but it must not be smaller!
		ensure(StagingBuffer->GetSize() >= Size);

		VkBufferImageCopy CopyRegion;
		FMemory::Memzero(CopyRegion);
		//Region.bufferOffset = 0;
		CopyRegion.bufferRowLength = FMath::Max(1, Desc.Extent.X >> InMipIndex);
		CopyRegion.bufferImageHeight = FMath::Max(1, Desc.Extent.Y >> InMipIndex);
		CopyRegion.imageSubresource.aspectMask = VulkanTexture.GetFullAspectMask();
		CopyRegion.imageSubresource.mipLevel = InMipIndex;
		CopyRegion.imageSubresource.baseArrayLayer = SrcBaseArrayLayer;
		CopyRegion.imageSubresource.layerCount = 1;
		CopyRegion.imageExtent.width = FMath::Max(1, Desc.Extent.X >> InMipIndex);
		CopyRegion.imageExtent.height = FMath::Max(1, Desc.Extent.Y >> InMipIndex);
		CopyRegion.imageExtent.depth = 1;

		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
		RHICmdList.EnqueueLambda([CopyRegion, StagingBuffer, &VulkanTexture](FRHICommandListBase& ExecutingCmdList)
		{
			FVulkanCommandListContext& Context = FVulkanCommandListContext::Get(ExecutingCmdList);
			FVulkanCommandBuffer& CommandBuffer = Context.GetCommandBuffer();
			VulkanRHI::vkCmdCopyImageToBuffer(CommandBuffer.GetHandle(), VulkanTexture.Image, VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL, StagingBuffer->GetHandle(), 1, &CopyRegion);

			FVulkanPipelineBarrier AfterBarrier;
			AfterBarrier.AddMemoryBarrier(VK_ACCESS_TRANSFER_WRITE_BIT, VK_ACCESS_HOST_READ_BIT, VK_PIPELINE_STAGE_TRANSFER_BIT, VK_PIPELINE_STAGE_HOST_BIT);
			AfterBarrier.Execute(&CommandBuffer);
		});

		// We need to execute the command list so we can read the data from the map below
		RHICmdList.SubmitAndBlockUntilGPUIdle();

		StagingBuffer->InvalidateMappedMemory();

		uint32 OutWidth = InRect.Max.X - InRect.Min.X;
		uint32 OutHeight= InRect.Max.Y - InRect.Min.Y;
		OutputData.SetNumUninitialized(OutWidth * OutHeight);
		uint32 OutIndex = 0;
		FFloat16Color* Dest = OutputData.GetData();
		void* MappedPointer = StagingBuffer->GetMappedPointer();
		for (int32 Row = InRect.Min.Y; Row < InRect.Max.Y; ++Row)
		{
			FFloat16Color* Src = (FFloat16Color*)MappedPointer + Row * (Desc.Extent.X >> InMipIndex) + InRect.Min.X;
			for (int32 Col = InRect.Min.X; Col < InRect.Max.X; ++Col)
			{
				OutputData[OutIndex++] = *Src++;
			}
		}

		InDevice->GetStagingManager().ReleaseBuffer(nullptr, StagingBuffer);
	};

	FVulkanTexture& Surface = *ResourceCast(TextureRHI);
	const FRHITextureDesc& Desc = Surface.GetDesc();

	if (GIgnoreCPUReads)
	{
		// Debug: Fill with CPU
		uint32 NumPixels = 0;
		switch(Desc.Dimension)
		{
		case ETextureDimension::TextureCubeArray:
		case ETextureDimension::TextureCube:
			NumPixels = (Desc.Extent.X >> MipIndex) * (Desc.Extent.Y >> MipIndex);
			break;

		case ETextureDimension::Texture2DArray:
		case ETextureDimension::Texture2D:
			NumPixels = (Desc.Extent.X >> MipIndex) * (Desc.Extent.Y >> MipIndex);
			break;

		default:
			checkNoEntry();
			break;
		}

		OutData.Empty(0);
		OutData.AddZeroed(NumPixels);
	}
	else
	{
		switch (TextureRHI->GetDesc().Dimension)
		{
			case ETextureDimension::TextureCubeArray:
			case ETextureDimension::TextureCube:
				DoCopyFloat(Device, Surface, MipIndex, CubeFace + 6 * ArrayIndex, Rect, OutData);
				break;

			case ETextureDimension::Texture2DArray:
			case ETextureDimension::Texture2D:
				DoCopyFloat(Device, Surface, MipIndex, ArrayIndex, Rect, OutData);
				break;

			default:
				checkNoEntry();
				break;
		}
	}
}

void FVulkanDynamicRHI::RHIRead3DSurfaceFloatData(FRHITexture* TextureRHI, FIntRect InRect, FIntPoint ZMinMax, TArray<FFloat16Color>& OutData)
{
	FVulkanTexture& Surface = *ResourceCast(TextureRHI);
	const FRHITextureDesc& Desc = Surface.GetDesc();

	const uint32 SizeX = InRect.Width();
	const uint32 SizeY = InRect.Height();
	const uint32 SizeZ = ZMinMax.Y - ZMinMax.X;
	const uint32 NumPixels = SizeX * SizeY * SizeZ;
	const uint32 Size = NumPixels * sizeof(FFloat16Color);

	// Allocate the output buffer.
	OutData.SetNumUninitialized(Size);

	if (GIgnoreCPUReads)
	{
		// Debug: Fill with CPU
		FMemory::Memzero(OutData.GetData(), Size * sizeof(FFloat16Color));
		return;
	}

	ensure(Surface.StorageFormat == VK_FORMAT_R16G16B16A16_SFLOAT);

	VulkanRHI::FStagingBuffer* StagingBuffer = Device->GetStagingManager().AcquireBuffer(Size, VK_BUFFER_USAGE_TRANSFER_DST_BIT, VK_MEMORY_PROPERTY_HOST_CACHED_BIT);
	// the staging buffer size may be bigger then the size due to alignment, etc. but it must not be smaller!
	ensure(StagingBuffer->GetSize() >= Size);

	VkBufferImageCopy CopyRegion;
	FMemory::Memzero(CopyRegion);
	//Region.bufferOffset = 0;
	CopyRegion.bufferRowLength = Desc.Extent.X;
	CopyRegion.bufferImageHeight = Desc.Extent.Y;
	CopyRegion.imageSubresource.aspectMask = Surface.GetFullAspectMask();
	//CopyRegion.imageSubresource.mipLevel = 0;
	//CopyRegion.imageSubresource.baseArrayLayer = 0;
	CopyRegion.imageSubresource.layerCount = 1;
	CopyRegion.imageOffset.x = InRect.Min.X;
	CopyRegion.imageOffset.y = InRect.Min.Y;
	CopyRegion.imageOffset.z = ZMinMax.X;
	CopyRegion.imageExtent.width = SizeX;
	CopyRegion.imageExtent.height = SizeY;
	CopyRegion.imageExtent.depth = SizeZ;

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	RHICmdList.EnqueueLambda([CopyRegion, StagingBuffer, &Surface](FRHICommandListBase& ExecutingCmdList)
	{
		FVulkanCommandListContext& Context = FVulkanCommandListContext::Get(ExecutingCmdList);
		FVulkanCommandBuffer& CommandBuffer = Context.GetCommandBuffer();
		VulkanRHI::vkCmdCopyImageToBuffer(CommandBuffer.GetHandle(), Surface.Image, VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL, StagingBuffer->GetHandle(), 1, &CopyRegion);

		FVulkanPipelineBarrier AfterBarrier;
		AfterBarrier.AddMemoryBarrier(VK_ACCESS_TRANSFER_WRITE_BIT, VK_ACCESS_HOST_READ_BIT, VK_PIPELINE_STAGE_TRANSFER_BIT, VK_PIPELINE_STAGE_HOST_BIT);
		AfterBarrier.Execute(&CommandBuffer);
	});

	// We need to execute the command list so we can read the data from the map below
	RHICmdList.SubmitAndBlockUntilGPUIdle();

	StagingBuffer->InvalidateMappedMemory();

	FFloat16Color* Dest = OutData.GetData();
	for (int32 Layer = ZMinMax.X; Layer < ZMinMax.Y; ++Layer)
	{
		for (int32 Row = InRect.Min.Y; Row < InRect.Max.Y; ++Row)
		{
			FFloat16Color* Src = (FFloat16Color*)StagingBuffer->GetMappedPointer() + Layer * SizeX * SizeY + Row * Desc.Extent.X + InRect.Min.X;
			for (int32 Col = InRect.Min.X; Col < InRect.Max.X; ++Col)
			{
				*Dest++ = *Src++;
			}
		}
	}
	FFloat16Color* End = OutData.GetData() + OutData.Num();
	checkf(Dest <= End, TEXT("Memory overwrite! Calculated total size %d: SizeX %d SizeY %d SizeZ %d; InRect(%d, %d, %d, %d) InZ(%d, %d)"),
		Size, SizeX, SizeY, SizeZ, InRect.Min.X, InRect.Min.Y, InRect.Max.X, InRect.Max.Y, ZMinMax.X, ZMinMax.Y);
	Device->GetStagingManager().ReleaseBuffer(nullptr, StagingBuffer);
}

FVulkanSwapChain* FVulkanCommandListContext::GetSwapChain() const
{
	TArray<FVulkanViewport*>& viewports = FVulkanDynamicRHI::Get().GetViewports();
	uint32 numViewports = viewports.Num();

	if (viewports.Num() == 0)
	{
		return nullptr;
	}

	return viewports[0]->GetSwapChain();
}

bool FVulkanCommandListContext::IsSwapchainImage(FRHITexture* InTexture) const
{
	TArray<FVulkanViewport*>& Viewports = FVulkanDynamicRHI::Get().GetViewports();
	uint32 NumViewports = Viewports.Num();

	for (uint32 i = 0; i < NumViewports; i++)
	{
		VkImage Image = ResourceCast(InTexture)->Image;
		uint32 BackBufferImageCount = Viewports[i]->GetBackBufferImageCount();

		for (uint32 SwapchainImageIdx = 0; SwapchainImageIdx < BackBufferImageCount; SwapchainImageIdx++)
		{
			if (Image == Viewports[i]->GetBackBufferImage(SwapchainImageIdx))
			{
				return true;
			}
		}
	}
	return false;
}

void FVulkanCommandListContext::ExtractDepthStencilLayouts(const FRHIRenderPassInfo& InInfo, VkImageLayout& OutDepthLayout, VkImageLayout& OutStencilLayout)
{
	FRHITexture* DSTexture = InInfo.DepthStencilRenderTarget.DepthStencilTarget;
	if (DSTexture)
	{
		const bool bNeedsAllPlanes = Device.NeedsAllPlanes();

		FVulkanTexture& VulkanTexture = *ResourceCast(DSTexture);
		const VkImageAspectFlags AspectFlags = VulkanTexture.GetFullAspectMask();

		const FExclusiveDepthStencil ExclusiveDepthStencil = InInfo.DepthStencilRenderTarget.ExclusiveDepthStencil;
		if (VKHasAnyFlags(AspectFlags, VK_IMAGE_ASPECT_DEPTH_BIT))
		{
			if (ExclusiveDepthStencil.IsDepthWrite())
			{
				OutDepthLayout = VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL;
			}
			else if (ExclusiveDepthStencil.IsDepthRead())
			{
				OutDepthLayout = VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL;
			}
			else if (bNeedsAllPlanes)
			{
				OutDepthLayout = FVulkanPipelineBarrier::GetDepthOrStencilLayout(VulkanTexture.AllPlanesTrackedAccess[0]);
			}
		}

		if (VKHasAnyFlags(AspectFlags, VK_IMAGE_ASPECT_STENCIL_BIT))
		{
			if (ExclusiveDepthStencil.IsStencilWrite())
			{
				OutStencilLayout = VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL;
			}
			else if (ExclusiveDepthStencil.IsStencilRead())
			{
				OutStencilLayout = VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL;
			}
			else if (bNeedsAllPlanes)
			{
				OutStencilLayout = FVulkanPipelineBarrier::GetDepthOrStencilLayout(VulkanTexture.AllPlanesTrackedAccess[1]);
			}
		}
	}
}

void FVulkanCommandListContext::RHIBeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* InName)
{
	RenderPassInfo = InInfo;

	if (InInfo.NumOcclusionQueries > 0)
	{
		BeginOcclusionQueryBatch(InInfo.NumOcclusionQueries, InInfo.MultiViewCount);
	}

	// Acquire a descriptor pool set on a first render pass
	AcquirePoolSetContainer();

	const bool bIsParallelRenderPass = (CurrentParallelRenderPassInfo != nullptr);
	if (Device.GetOptionalExtensions().HasKHRDynamicRendering)
	{
		check(!CurrentRenderPass && !CurrentFramebuffer);

		VkRect2D RenderArea;
		if (bIsParallelRenderPass)
		{
			check(CurrentParallelRenderPassInfo->DynamicRenderingInfo);
			GetCommandBuffer().BeginDynamicRendering(CurrentParallelRenderPassInfo->DynamicRenderingInfo->RenderingInfo, nullptr);
			RenderArea = CurrentParallelRenderPassInfo->DynamicRenderingInfo->RenderingInfo.renderArea;
		}
		else
		{
			check(CurrentDynamicRenderingInfo);
			FillDynamicRenderingInfo(InInfo, *CurrentDynamicRenderingInfo);
			GetCommandBuffer().BeginDynamicRendering(CurrentDynamicRenderingInfo->RenderingInfo, nullptr);
			RenderArea = CurrentDynamicRenderingInfo->RenderingInfo.renderArea;
		}

		GetPendingGfxState()->SetViewport(RenderArea.offset.x, RenderArea.offset.y, 0, RenderArea.extent.width, RenderArea.extent.height, 1);
	}
	else
	{
		VkImageLayout CurrentDepthLayout = VK_IMAGE_LAYOUT_UNDEFINED;
		VkImageLayout CurrentStencilLayout = VK_IMAGE_LAYOUT_UNDEFINED;
		ExtractDepthStencilLayouts(InInfo, CurrentDepthLayout, CurrentStencilLayout);

		FVulkanRenderTargetLayout RTLayout(Device, InInfo, CurrentDepthLayout, CurrentStencilLayout);
		check(RTLayout.GetExtent2D().width != 0 && RTLayout.GetExtent2D().height != 0);

		FVulkanRenderPass* RenderPass = Device.GetRenderPassManager().GetOrCreateRenderPass(RTLayout);
		FRHISetRenderTargetsInfo RTInfo;
		InInfo.ConvertToRenderTargetsInfo(RTInfo);

		FVulkanFramebuffer* Framebuffer = Device.GetRenderPassManager().GetOrCreateFramebuffer(RTInfo, RTLayout, *RenderPass);
		checkf(RenderPass != nullptr && Framebuffer != nullptr, TEXT("RenderPass not started! Bad combination of values? Depth %p #Color %d Color0 %p"), (void*)InInfo.DepthStencilRenderTarget.DepthStencilTarget, InInfo.GetNumColorRenderTargets(), (void*)InInfo.ColorRenderTargets[0].RenderTarget);

		FVulkanBeginRenderPassInfo BeginRenderPassInfo{ *RenderPass, *Framebuffer, bIsParallelRenderPass };
		Device.GetRenderPassManager().BeginRenderPass(*this, InInfo, RTLayout, BeginRenderPassInfo);

		check(!CurrentRenderPass);
		CurrentRenderPass = RenderPass;
		CurrentFramebuffer = Framebuffer;
	}
}

void FVulkanCommandListContext::RHIEndRenderPass()
{
	if (Device.GetOptionalExtensions().HasKHRDynamicRendering)
	{
		GetCommandBuffer().EndDynamicRendering(nullptr);
	}
	else
	{
		Device.GetRenderPassManager().EndRenderPass(*this);

		check(CurrentRenderPass);
		CurrentRenderPass = nullptr;
	}
}

void FVulkanCommandListContext::RHINextSubpass()
{
	if (Device.GetOptionalExtensions().HasKHRDynamicRendering)
	{
		// :todo: Need to properly handle local_read (VK_IMAGE_LAYOUT_RENDERING_LOCAL_READ)
		VkMemoryBarrier Barrier = { VK_STRUCTURE_TYPE_MEMORY_BARRIER, nullptr, VK_ACCESS_SHADER_WRITE_BIT, VK_ACCESS_SHADER_READ_BIT };
		VulkanRHI::vkCmdPipelineBarrier(GetCommandBuffer().GetHandle(), 
			VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT, VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT, 
			VK_DEPENDENCY_BY_REGION_BIT, 1, &Barrier, 0, nullptr, 0, nullptr);
	}
	else
	{
		check(CurrentRenderPass);
		FVulkanCommandBuffer& CommandBuffer = GetCommandBuffer();
		VkCommandBuffer CommandBufferHandle = CommandBuffer.GetHandle();
		VulkanRHI::vkCmdNextSubpass(CommandBufferHandle, VK_SUBPASS_CONTENTS_INLINE);
	}
}
