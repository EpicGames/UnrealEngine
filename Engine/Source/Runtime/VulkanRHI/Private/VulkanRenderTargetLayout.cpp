// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanRenderTargetLayout.h"
#include "VulkanDevice.h"

// Need a separate struct so we can memzero/remove dependencies on reference counts
struct FRenderPassCompatibleHashableStruct
{
	FRenderPassCompatibleHashableStruct()
	{
		FMemory::Memzero(*this);
	}

	uint8							NumAttachments;
	uint8							MultiViewCount;
	uint8							NumSamples;
	uint8							SubpassHint;
	// +1 for Depth, +1 for Stencil, +1 for Fragment Density
	VkFormat						Formats[MaxSimultaneousRenderTargets + 3];
	uint16							AttachmentsToResolve;
};

// Need a separate struct so we can memzero/remove dependencies on reference counts
struct FRenderPassFullHashableStruct
{
	FRenderPassFullHashableStruct()
	{
		FMemory::Memzero(*this);
	}

	// +1 for Depth, +1 for Stencil, +1 for Fragment Density
	TEnumAsByte<VkAttachmentLoadOp>		LoadOps[MaxSimultaneousRenderTargets + 3];
	TEnumAsByte<VkAttachmentStoreOp>	StoreOps[MaxSimultaneousRenderTargets + 3];
	// If the initial != final we need to add FinalLayout and potentially RefLayout
	VkImageLayout						InitialLayout[MaxSimultaneousRenderTargets + 3];
	//VkImageLayout						FinalLayout[MaxSimultaneousRenderTargets + 3];
	//VkImageLayout						RefLayout[MaxSimultaneousRenderTargets + 3];
};

VkImageLayout FVulkanRenderTargetLayout::GetVRSImageLayout() const
{
	if (ValidateShadingRateDataType())
	{
		if (GRHIVariableRateShadingImageDataType == VRSImage_Palette)
		{
			return VK_IMAGE_LAYOUT_FRAGMENT_SHADING_RATE_ATTACHMENT_OPTIMAL_KHR;
		}
		if (GRHIVariableRateShadingImageDataType == VRSImage_Fractional)
		{
			return VK_IMAGE_LAYOUT_FRAGMENT_DENSITY_MAP_OPTIMAL_EXT;
		}
	}

	return VK_IMAGE_LAYOUT_UNDEFINED;
}

FVulkanRenderTargetLayout::FVulkanRenderTargetLayout(FVulkanDevice& InDevice, const FRHISetRenderTargetsInfo& RTInfo)
	: NumAttachmentDescriptions(0)
	, NumColorAttachments(0)
	, bHasDepthStencil(false)
	, bHasResolveAttachments(false)
	, bHasDepthStencilResolve(false)
	, bHasFragmentDensityAttachment(false)
	, NumSamples(0)
	, NumUsedClearValues(0)
	, MultiViewCount(0)
{
	ResetAttachments();

	FRenderPassCompatibleHashableStruct CompatibleHashInfo;
	FRenderPassFullHashableStruct FullHashInfo;

	bool bSetExtent = false;
	bool bFoundClearOp = false;
	for (int32 Index = 0; Index < RTInfo.NumColorRenderTargets; ++Index)
	{
		const FRHIRenderTargetView& RTView = RTInfo.ColorRenderTarget[Index];
		if (RTView.Texture)
		{
			FVulkanTexture* Texture = ResourceCast(RTView.Texture);
			check(Texture);
			const FRHITextureDesc& TextureDesc = Texture->GetDesc();

			if (bSetExtent)
			{
				ensure(Extent.Extent3D.width == FMath::Max(1, TextureDesc.Extent.X >> RTView.MipIndex));
				ensure(Extent.Extent3D.height == FMath::Max(1, TextureDesc.Extent.Y >> RTView.MipIndex));
				ensure(Extent.Extent3D.depth == TextureDesc.Depth);
			}
			else
			{
				bSetExtent = true;
				Extent.Extent3D.width = FMath::Max(1, TextureDesc.Extent.X >> RTView.MipIndex);
				Extent.Extent3D.height = FMath::Max(1, TextureDesc.Extent.Y >> RTView.MipIndex);
				Extent.Extent3D.depth = TextureDesc.Depth;
			}

			ensure(!NumSamples || NumSamples == Texture->GetNumSamples());
			NumSamples = Texture->GetNumSamples();

			VkAttachmentDescription& CurrDesc = Desc[NumAttachmentDescriptions];
			CurrDesc.samples = static_cast<VkSampleCountFlagBits>(NumSamples);
			CurrDesc.format = UEToVkTextureFormat(RTView.Texture->GetFormat(), EnumHasAllFlags(TextureDesc.Flags, TexCreate_SRGB));
			CurrDesc.loadOp = RenderTargetLoadActionToVulkan(RTView.LoadAction);
			bFoundClearOp = bFoundClearOp || (CurrDesc.loadOp == VK_ATTACHMENT_LOAD_OP_CLEAR);
			CurrDesc.storeOp = RenderTargetStoreActionToVulkan(RTView.StoreAction);
			CurrDesc.stencilLoadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
			CurrDesc.stencilStoreOp = VK_ATTACHMENT_STORE_OP_DONT_CARE;

			// Removed this temporarily as we need a way to determine if the target is actually memoryless
			/*if (EnumHasAllFlags(Texture->UEFlags, TexCreate_Memoryless))
			{
				ensure(CurrDesc.storeOp == VK_ATTACHMENT_STORE_OP_DONT_CARE);
			}*/

			// If the initial != final we need to change the FullHashInfo and use FinalLayout
			CurrDesc.initialLayout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;
			CurrDesc.finalLayout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;

			ColorReferences[NumColorAttachments].attachment = NumAttachmentDescriptions;
			ColorReferences[NumColorAttachments].layout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;

			const bool bHasValidResolveAttachment = RTInfo.bHasResolveAttachments && RTInfo.ColorResolveRenderTarget[Index].Texture;
			if (CurrDesc.samples > VK_SAMPLE_COUNT_1_BIT && bHasValidResolveAttachment)
			{
				Desc[NumAttachmentDescriptions + 1] = Desc[NumAttachmentDescriptions];
				Desc[NumAttachmentDescriptions + 1].samples = VK_SAMPLE_COUNT_1_BIT;
				Desc[NumAttachmentDescriptions + 1].loadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
				Desc[NumAttachmentDescriptions + 1].storeOp = VK_ATTACHMENT_STORE_OP_STORE;
				ResolveReferences[NumColorAttachments].attachment = NumAttachmentDescriptions + 1;
				ResolveReferences[NumColorAttachments].layout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;
				CompatibleHashInfo.AttachmentsToResolve |= (uint16)(1 << NumColorAttachments);
				++NumAttachmentDescriptions;
				bHasResolveAttachments = true;
			}

			CompatibleHashInfo.Formats[NumColorAttachments] = CurrDesc.format;
			FullHashInfo.LoadOps[NumColorAttachments] = CurrDesc.loadOp;
			FullHashInfo.StoreOps[NumColorAttachments] = CurrDesc.storeOp;
			FullHashInfo.InitialLayout[NumColorAttachments] = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;
			++CompatibleHashInfo.NumAttachments;

			++NumAttachmentDescriptions;
			++NumColorAttachments;
		}
	}

	if (RTInfo.DepthStencilRenderTarget.Texture)
	{
		VkAttachmentDescription& CurrDesc = Desc[NumAttachmentDescriptions];
		FMemory::Memzero(CurrDesc);
		FVulkanTexture* Texture = ResourceCast(RTInfo.DepthStencilRenderTarget.Texture);
		check(Texture);
		const FRHITextureDesc& TextureDesc = Texture->GetDesc();

		ensure(!NumSamples || NumSamples == Texture->GetNumSamples());
		NumSamples = TextureDesc.NumSamples;

		CurrDesc.samples = static_cast<VkSampleCountFlagBits>(NumSamples);
		CurrDesc.format = UEToVkTextureFormat(RTInfo.DepthStencilRenderTarget.Texture->GetFormat(), false);
		CurrDesc.loadOp = RenderTargetLoadActionToVulkan(RTInfo.DepthStencilRenderTarget.DepthLoadAction);
		CurrDesc.stencilLoadOp = RenderTargetLoadActionToVulkan(RTInfo.DepthStencilRenderTarget.StencilLoadAction);
		bFoundClearOp = bFoundClearOp || (CurrDesc.loadOp == VK_ATTACHMENT_LOAD_OP_CLEAR || CurrDesc.stencilLoadOp == VK_ATTACHMENT_LOAD_OP_CLEAR);
		CurrDesc.storeOp = RenderTargetStoreActionToVulkan(RTInfo.DepthStencilRenderTarget.DepthStoreAction);
		CurrDesc.stencilStoreOp = RenderTargetStoreActionToVulkan(RTInfo.DepthStencilRenderTarget.GetStencilStoreAction());

		// Removed this temporarily as we need a way to determine if the target is actually memoryless
		/*if (EnumHasAllFlags(Texture->UEFlags, TexCreate_Memoryless))
		{
			ensure(CurrDesc.storeOp == VK_ATTACHMENT_STORE_OP_DONT_CARE);
			ensure(CurrDesc.stencilStoreOp == VK_ATTACHMENT_STORE_OP_DONT_CARE);
		}*/

		const VkImageLayout DepthLayout = RTInfo.DepthStencilRenderTarget.GetDepthStencilAccess().IsDepthWrite() ? VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL : VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL;
		const VkImageLayout StencilLayout = RTInfo.DepthStencilRenderTarget.GetDepthStencilAccess().IsStencilWrite() ? VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL : VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL;

		// If the initial != final we need to change the FullHashInfo and use FinalLayout
		CurrDesc.initialLayout = DepthLayout;
		CurrDesc.finalLayout = DepthLayout;
		StencilDesc.stencilInitialLayout = StencilLayout;
		StencilDesc.stencilFinalLayout = StencilLayout;

		DepthReference.attachment = NumAttachmentDescriptions;
		DepthReference.layout = DepthLayout;
		StencilReference.stencilLayout = StencilLayout;

		// Use depth/stencil resolve target only if we're MSAA
		const bool bDepthStencilResolve = (RTInfo.DepthStencilRenderTarget.DepthStoreAction == ERenderTargetStoreAction::EMultisampleResolve) || (RTInfo.DepthStencilRenderTarget.GetStencilStoreAction() == ERenderTargetStoreAction::EMultisampleResolve);
		if (GRHISupportsDepthStencilResolve && bDepthStencilResolve && CurrDesc.samples > VK_SAMPLE_COUNT_1_BIT && RTInfo.DepthStencilResolveRenderTarget.Texture)
		{
			Desc[NumAttachmentDescriptions + 1] = Desc[NumAttachmentDescriptions];
			Desc[NumAttachmentDescriptions + 1].samples = VK_SAMPLE_COUNT_1_BIT;
			Desc[NumAttachmentDescriptions + 1].loadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
			Desc[NumAttachmentDescriptions + 1].storeOp = VK_ATTACHMENT_STORE_OP_STORE;
			Desc[NumAttachmentDescriptions + 1].stencilLoadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
			Desc[NumAttachmentDescriptions + 1].stencilStoreOp = VK_ATTACHMENT_STORE_OP_STORE;
			DepthStencilResolveReference.attachment = NumAttachmentDescriptions + 1;
			DepthStencilResolveReference.layout = DepthLayout;
			// NumColorAttachments was incremented after the last color attachment
			ensureMsgf(NumColorAttachments < 16, TEXT("Must have room for depth resolve bit"));
			CompatibleHashInfo.AttachmentsToResolve |= (uint16)(1 << NumColorAttachments);
			++NumAttachmentDescriptions;
			bHasDepthStencilResolve = true;
		}

		FullHashInfo.LoadOps[MaxSimultaneousRenderTargets] = CurrDesc.loadOp;
		FullHashInfo.LoadOps[MaxSimultaneousRenderTargets + 1] = CurrDesc.stencilLoadOp;
		FullHashInfo.StoreOps[MaxSimultaneousRenderTargets] = CurrDesc.storeOp;
		FullHashInfo.StoreOps[MaxSimultaneousRenderTargets + 1] = CurrDesc.stencilStoreOp;
		FullHashInfo.InitialLayout[MaxSimultaneousRenderTargets] = DepthLayout;
		FullHashInfo.InitialLayout[MaxSimultaneousRenderTargets + 1] = StencilLayout;
		CompatibleHashInfo.Formats[MaxSimultaneousRenderTargets] = CurrDesc.format;

		++NumAttachmentDescriptions;

		bHasDepthStencil = true;

		if (bSetExtent)
		{
			// Depth can be greater or equal to color. Clamp to the smaller size.
			Extent.Extent3D.width = FMath::Min<uint32>(Extent.Extent3D.width, TextureDesc.Extent.X);
			Extent.Extent3D.height = FMath::Min<uint32>(Extent.Extent3D.height, TextureDesc.Extent.Y);
		}
		else
		{
			bSetExtent = true;
			Extent.Extent3D.width = TextureDesc.Extent.X;
			Extent.Extent3D.height = TextureDesc.Extent.Y;
			Extent.Extent3D.depth = Texture->GetNumberOfArrayLevels();
		}
	}

	if (GRHISupportsAttachmentVariableRateShading && RTInfo.ShadingRateTexture)
	{
		FVulkanTexture* Texture = ResourceCast(RTInfo.ShadingRateTexture);
		check(Texture->GetFormat() == GRHIVariableRateShadingImageFormat);

		VkAttachmentDescription& CurrDesc = Desc[NumAttachmentDescriptions];
		FMemory::Memzero(CurrDesc);

		const VkImageLayout VRSLayout = GetVRSImageLayout();

		CurrDesc.flags = 0;
		CurrDesc.format = UEToVkTextureFormat(RTInfo.ShadingRateTexture->GetFormat(), false);
		CurrDesc.samples = static_cast<VkSampleCountFlagBits>(RTInfo.ShadingRateTexture->GetNumSamples());
		CurrDesc.loadOp = VK_ATTACHMENT_LOAD_OP_LOAD;
		CurrDesc.storeOp = VK_ATTACHMENT_STORE_OP_DONT_CARE;
		CurrDesc.stencilLoadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
		CurrDesc.stencilStoreOp = VK_ATTACHMENT_STORE_OP_DONT_CARE;
		CurrDesc.initialLayout = VRSLayout;
		CurrDesc.finalLayout = VRSLayout;

		FragmentDensityReference.attachment = NumAttachmentDescriptions;
		FragmentDensityReference.layout = VRSLayout;

		FullHashInfo.LoadOps[MaxSimultaneousRenderTargets + 2] = CurrDesc.stencilLoadOp;
		FullHashInfo.StoreOps[MaxSimultaneousRenderTargets + 2] = CurrDesc.stencilStoreOp;
		FullHashInfo.InitialLayout[MaxSimultaneousRenderTargets + 2] = VRSLayout;
		CompatibleHashInfo.Formats[MaxSimultaneousRenderTargets + 1] = CurrDesc.format;

		++NumAttachmentDescriptions;
		bHasFragmentDensityAttachment = true;
	}

	SubpassHint = ESubpassHint::None;
	CompatibleHashInfo.SubpassHint = 0;

	CompatibleHashInfo.NumSamples = NumSamples;
	CompatibleHashInfo.MultiViewCount = MultiViewCount;

	RenderPassCompatibleHash = FCrc::MemCrc32(&CompatibleHashInfo, sizeof(CompatibleHashInfo));
	RenderPassFullHash = FCrc::MemCrc32(&FullHashInfo, sizeof(FullHashInfo), RenderPassCompatibleHash);
	NumUsedClearValues = bFoundClearOp ? NumAttachmentDescriptions : 0;
	bCalculatedHash = true;
}


FVulkanRenderTargetLayout::FVulkanRenderTargetLayout(FVulkanDevice& InDevice, const FRHIRenderPassInfo& RPInfo, VkImageLayout CurrentDepthLayout, VkImageLayout CurrentStencilLayout)
	: NumAttachmentDescriptions(0)
	, NumColorAttachments(0)
	, bHasDepthStencil(false)
	, bHasResolveAttachments(false)
	, bHasDepthStencilResolve(false)
	, bHasFragmentDensityAttachment(false)
	, NumSamples(0)
	, NumUsedClearValues(0)
	, MultiViewCount(RPInfo.MultiViewCount)
{
	ResetAttachments();

	FRenderPassCompatibleHashableStruct CompatibleHashInfo;
	FRenderPassFullHashableStruct FullHashInfo;

	bool bSetExtent = false;
	bool bFoundClearOp = false;
	bool bMultiviewRenderTargets = false;

	const int32 NumColorRenderTargets = RPInfo.GetNumColorRenderTargets();
	for (int32 Index = 0; Index < NumColorRenderTargets; ++Index)
	{
		const FRHIRenderPassInfo::FColorEntry& ColorEntry = RPInfo.ColorRenderTargets[Index];
		FVulkanTexture* Texture = ResourceCast(ColorEntry.RenderTarget);
		check(Texture);
		const FRHITextureDesc& TextureDesc = Texture->GetDesc();

		if (bSetExtent)
		{
			ensure(Extent.Extent3D.width == FMath::Max(1, TextureDesc.Extent.X >> ColorEntry.MipIndex));
			ensure(Extent.Extent3D.height == FMath::Max(1, TextureDesc.Extent.Y >> ColorEntry.MipIndex));
			ensure(Extent.Extent3D.depth == TextureDesc.Depth);
		}
		else
		{
			bSetExtent = true;
			Extent.Extent3D.width = FMath::Max(1, TextureDesc.Extent.X >> ColorEntry.MipIndex);
			Extent.Extent3D.height = FMath::Max(1, TextureDesc.Extent.Y >> ColorEntry.MipIndex);
			Extent.Extent3D.depth = TextureDesc.Depth;
		}

		// CustomResolveSubpass can have targets with a different NumSamples
		ensure(!NumSamples || NumSamples == ColorEntry.RenderTarget->GetNumSamples() || RPInfo.SubpassHint == ESubpassHint::CustomResolveSubpass);
		NumSamples = ColorEntry.RenderTarget->GetNumSamples();

		ensure(!GetIsMultiView() || !bMultiviewRenderTargets || Texture->GetNumberOfArrayLevels() > 1);
		bMultiviewRenderTargets = Texture->GetNumberOfArrayLevels() > 1;
		
		// With a CustomResolveSubpass last color attachment is a resolve target
		bool bCustomResolveAttachment = (Index == (NumColorRenderTargets - 1)) && RPInfo.SubpassHint == ESubpassHint::CustomResolveSubpass;

		VkAttachmentDescription& CurrDesc = Desc[NumAttachmentDescriptions];
		CurrDesc.samples = bCustomResolveAttachment ? VK_SAMPLE_COUNT_1_BIT : static_cast<VkSampleCountFlagBits>(NumSamples);
		CurrDesc.format = UEToVkTextureFormat(ColorEntry.RenderTarget->GetFormat(), EnumHasAllFlags(Texture->GetDesc().Flags, TexCreate_SRGB));
		CurrDesc.loadOp = RenderTargetLoadActionToVulkan(GetLoadAction(ColorEntry.Action));
		bFoundClearOp = bFoundClearOp || (CurrDesc.loadOp == VK_ATTACHMENT_LOAD_OP_CLEAR);
		CurrDesc.storeOp = RenderTargetStoreActionToVulkan(GetStoreAction(ColorEntry.Action));
		CurrDesc.stencilLoadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
		CurrDesc.stencilStoreOp = VK_ATTACHMENT_STORE_OP_DONT_CARE;

		if (EnumHasAnyFlags(Texture->GetDesc().Flags, TexCreate_Memoryless))
		{
			ensure(CurrDesc.storeOp == VK_ATTACHMENT_STORE_OP_DONT_CARE);
		}

		CurrDesc.initialLayout = (CurrDesc.loadOp == VK_ATTACHMENT_LOAD_OP_CLEAR) ? VK_IMAGE_LAYOUT_UNDEFINED : VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;
		CurrDesc.finalLayout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;

		ColorReferences[NumColorAttachments].attachment = NumAttachmentDescriptions;
		ColorReferences[NumColorAttachments].layout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;

		if (CurrDesc.samples > VK_SAMPLE_COUNT_1_BIT && ColorEntry.ResolveTarget)
		{
			Desc[NumAttachmentDescriptions + 1] = Desc[NumAttachmentDescriptions];
			Desc[NumAttachmentDescriptions + 1].samples = VK_SAMPLE_COUNT_1_BIT;
			Desc[NumAttachmentDescriptions + 1].loadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
			Desc[NumAttachmentDescriptions + 1].storeOp = VK_ATTACHMENT_STORE_OP_STORE;
			ResolveReferences[NumColorAttachments].attachment = NumAttachmentDescriptions + 1;
			ResolveReferences[NumColorAttachments].layout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;
			CompatibleHashInfo.AttachmentsToResolve |= (uint16)(1 << NumColorAttachments);
			++NumAttachmentDescriptions;
			bHasResolveAttachments = true;
		}

		CompatibleHashInfo.Formats[NumColorAttachments] = CurrDesc.format;
		FullHashInfo.LoadOps[NumColorAttachments] = CurrDesc.loadOp;
		FullHashInfo.InitialLayout[NumColorAttachments] = CurrDesc.initialLayout;
		FullHashInfo.StoreOps[NumColorAttachments] = CurrDesc.storeOp;
		++CompatibleHashInfo.NumAttachments;

		++NumAttachmentDescriptions;
		++NumColorAttachments;
	}
	bool bMultiViewDepthStencil = false;
	if (RPInfo.DepthStencilRenderTarget.DepthStencilTarget)
	{
		VkAttachmentDescription& CurrDesc = Desc[NumAttachmentDescriptions];
		FMemory::Memzero(CurrDesc);
		FVulkanTexture* Texture = ResourceCast(RPInfo.DepthStencilRenderTarget.DepthStencilTarget);
		check(Texture);
		const FRHITextureDesc& TextureDesc = Texture->GetDesc();
		bMultiViewDepthStencil = (Texture->GetNumberOfArrayLevels() > 1) && !Texture->GetDesc().IsTextureCube();
		CurrDesc.samples = static_cast<VkSampleCountFlagBits>(RPInfo.DepthStencilRenderTarget.DepthStencilTarget->GetNumSamples());
		// CustomResolveSubpass can have targets with a different NumSamples
		ensure(!NumSamples || CurrDesc.samples == NumSamples || RPInfo.SubpassHint == ESubpassHint::CustomResolveSubpass);
		NumSamples = CurrDesc.samples;
		CurrDesc.format = UEToVkTextureFormat(RPInfo.DepthStencilRenderTarget.DepthStencilTarget->GetFormat(), false);
		CurrDesc.loadOp = RenderTargetLoadActionToVulkan(GetLoadAction(GetDepthActions(RPInfo.DepthStencilRenderTarget.Action)));
		CurrDesc.stencilLoadOp = RenderTargetLoadActionToVulkan(GetLoadAction(GetStencilActions(RPInfo.DepthStencilRenderTarget.Action)));
		bFoundClearOp = bFoundClearOp || (CurrDesc.loadOp == VK_ATTACHMENT_LOAD_OP_CLEAR || CurrDesc.stencilLoadOp == VK_ATTACHMENT_LOAD_OP_CLEAR);

		CurrDesc.storeOp = RenderTargetStoreActionToVulkan(GetStoreAction(GetDepthActions(RPInfo.DepthStencilRenderTarget.Action)));
		CurrDesc.stencilStoreOp = RenderTargetStoreActionToVulkan(GetStoreAction(GetStencilActions(RPInfo.DepthStencilRenderTarget.Action)));

		if (EnumHasAnyFlags(TextureDesc.Flags, TexCreate_Memoryless))
		{
			ensure(CurrDesc.storeOp == VK_ATTACHMENT_STORE_OP_DONT_CARE);
			ensure(CurrDesc.stencilStoreOp == VK_ATTACHMENT_STORE_OP_DONT_CARE);
		}

		if (FVulkanPlatform::RequiresDepthStencilFullWrite() &&
			Texture->GetFullAspectMask() == (VK_IMAGE_ASPECT_DEPTH_BIT | VK_IMAGE_ASPECT_STENCIL_BIT) &&
			(CurrDesc.storeOp == VK_ATTACHMENT_STORE_OP_STORE || CurrDesc.stencilStoreOp == VK_ATTACHMENT_STORE_OP_STORE))
		{
			// Workaround for old mali drivers: writing not all of the image aspects to compressed render-target could cause gpu-hang
			CurrDesc.storeOp = VK_ATTACHMENT_STORE_OP_STORE;
			CurrDesc.stencilStoreOp = VK_ATTACHMENT_STORE_OP_STORE;
		}

		// If the initial != final we need to change the FullHashInfo and use FinalLayout
		CurrDesc.initialLayout = CurrentDepthLayout;
		CurrDesc.finalLayout = CurrentDepthLayout;
		StencilDesc.stencilInitialLayout = CurrentStencilLayout;
		StencilDesc.stencilFinalLayout = CurrentStencilLayout;

		// We can't have the final layout be UNDEFINED, but it's possible that we get here from a transient texture
		// where the stencil was never used yet.  We can set the layout to whatever we want, the next transition will
		// happen from UNDEFINED anyhow.
		if (CurrentDepthLayout == VK_IMAGE_LAYOUT_UNDEFINED)
		{
			// Unused image aspects with a LoadOp but undefined layout should just remain untouched
			if (!RPInfo.DepthStencilRenderTarget.ExclusiveDepthStencil.IsUsingDepth() &&
				InDevice.GetOptionalExtensions().HasEXTLoadStoreOpNone &&
				(CurrDesc.loadOp == VK_ATTACHMENT_LOAD_OP_LOAD))
			{
				CurrDesc.loadOp = VK_ATTACHMENT_LOAD_OP_NONE_KHR;
			}

			check(CurrDesc.storeOp == VK_ATTACHMENT_STORE_OP_DONT_CARE);
			CurrDesc.finalLayout = VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL;
		}
		if (CurrentStencilLayout == VK_IMAGE_LAYOUT_UNDEFINED)
		{
			// Unused image aspects with a LoadOp but undefined layout should just remain untouched
			if (!RPInfo.DepthStencilRenderTarget.ExclusiveDepthStencil.IsUsingStencil() &&
				InDevice.GetOptionalExtensions().HasEXTLoadStoreOpNone &&
				(CurrDesc.stencilLoadOp == VK_ATTACHMENT_LOAD_OP_LOAD))
			{
				CurrDesc.stencilLoadOp = VK_ATTACHMENT_LOAD_OP_NONE_KHR;
			}

			check(CurrDesc.stencilStoreOp == VK_ATTACHMENT_STORE_OP_DONT_CARE);
			StencilDesc.stencilFinalLayout = VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL;
		}

		DepthReference.attachment = NumAttachmentDescriptions;
		DepthReference.layout = CurrentDepthLayout;
		StencilReference.stencilLayout = CurrentStencilLayout;

		if (GRHISupportsDepthStencilResolve && CurrDesc.samples > VK_SAMPLE_COUNT_1_BIT && RPInfo.DepthStencilRenderTarget.ResolveTarget)
		{
			Desc[NumAttachmentDescriptions + 1] = Desc[NumAttachmentDescriptions];
			Desc[NumAttachmentDescriptions + 1].samples = VK_SAMPLE_COUNT_1_BIT;
			Desc[NumAttachmentDescriptions + 1].loadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
			Desc[NumAttachmentDescriptions + 1].storeOp = VK_ATTACHMENT_STORE_OP_STORE;
			Desc[NumAttachmentDescriptions + 1].stencilLoadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
			Desc[NumAttachmentDescriptions + 1].stencilStoreOp = VK_ATTACHMENT_STORE_OP_STORE;
			DepthStencilResolveReference.attachment = NumAttachmentDescriptions + 1;
			DepthStencilResolveReference.layout = CurrentDepthLayout;
			// NumColorAttachments was incremented after the last color attachment
			ensureMsgf(NumColorAttachments < 16, TEXT("Must have room for depth resolve bit"));
			CompatibleHashInfo.AttachmentsToResolve |= (uint16)(1 << NumColorAttachments);
			++NumAttachmentDescriptions;
			bHasDepthStencilResolve = true;
		}

		FullHashInfo.LoadOps[MaxSimultaneousRenderTargets] = CurrDesc.loadOp;
		FullHashInfo.LoadOps[MaxSimultaneousRenderTargets + 1] = CurrDesc.stencilLoadOp;
		FullHashInfo.StoreOps[MaxSimultaneousRenderTargets] = CurrDesc.storeOp;
		FullHashInfo.StoreOps[MaxSimultaneousRenderTargets + 1] = CurrDesc.stencilStoreOp;
		FullHashInfo.InitialLayout[MaxSimultaneousRenderTargets] = CurrentDepthLayout;
		FullHashInfo.InitialLayout[MaxSimultaneousRenderTargets + 1] = CurrentStencilLayout;
		CompatibleHashInfo.Formats[MaxSimultaneousRenderTargets] = CurrDesc.format;

		++NumAttachmentDescriptions;

		bHasDepthStencil = true;

		if (bSetExtent)
		{
			// Depth can be greater or equal to color. Clamp to the smaller size.
			Extent.Extent3D.width = FMath::Min<uint32>(Extent.Extent3D.width, TextureDesc.Extent.X);
			Extent.Extent3D.height = FMath::Min<uint32>(Extent.Extent3D.height, TextureDesc.Extent.Y);
		}
		else
		{
			bSetExtent = true;
			Extent.Extent3D.width = TextureDesc.Extent.X;
			Extent.Extent3D.height = TextureDesc.Extent.Y;
			Extent.Extent3D.depth = TextureDesc.Depth;
		}
	}
	else if (NumColorRenderTargets == 0)
	{
		// No Depth and no color, it's a raster-only pass so make sure the renderArea will be set up properly
		checkf(RPInfo.ResolveRect.IsValid(), TEXT("For raster-only passes without render targets, ResolveRect has to contain the render area"));
		bSetExtent = true;
		Offset.Offset3D.x = RPInfo.ResolveRect.X1;
		Offset.Offset3D.y = RPInfo.ResolveRect.Y1;
		Offset.Offset3D.z = 0;
		Extent.Extent3D.width = RPInfo.ResolveRect.X2 - RPInfo.ResolveRect.X1;
		Extent.Extent3D.height = RPInfo.ResolveRect.Y2 - RPInfo.ResolveRect.Y1;
		Extent.Extent3D.depth = 1;
	}

	if (GRHISupportsAttachmentVariableRateShading && RPInfo.ShadingRateTexture)
	{
		FVulkanTexture* Texture = ResourceCast(RPInfo.ShadingRateTexture);
		check(Texture->GetFormat() == GRHIVariableRateShadingImageFormat);

		VkAttachmentDescription& CurrDesc = Desc[NumAttachmentDescriptions];
		FMemory::Memzero(CurrDesc);

		const VkImageLayout VRSLayout = GetVRSImageLayout();

		CurrDesc.flags = 0;
		CurrDesc.format = UEToVkTextureFormat(RPInfo.ShadingRateTexture->GetFormat(), false);
		CurrDesc.samples = static_cast<VkSampleCountFlagBits>(RPInfo.ShadingRateTexture->GetNumSamples());
		CurrDesc.loadOp = VK_ATTACHMENT_LOAD_OP_LOAD;
		CurrDesc.storeOp = VK_ATTACHMENT_STORE_OP_DONT_CARE;
		CurrDesc.stencilLoadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
		CurrDesc.stencilStoreOp = VK_ATTACHMENT_STORE_OP_DONT_CARE;
		CurrDesc.initialLayout = VRSLayout;
		CurrDesc.finalLayout = VRSLayout;

		FragmentDensityReference.attachment = NumAttachmentDescriptions;
		FragmentDensityReference.layout = VRSLayout;

		FullHashInfo.LoadOps[MaxSimultaneousRenderTargets + 2] = CurrDesc.stencilLoadOp;
		FullHashInfo.StoreOps[MaxSimultaneousRenderTargets + 2] = CurrDesc.stencilStoreOp;
		FullHashInfo.InitialLayout[MaxSimultaneousRenderTargets + 2] = VRSLayout;
		CompatibleHashInfo.Formats[MaxSimultaneousRenderTargets + 1] = CurrDesc.format;

		++NumAttachmentDescriptions;
		bHasFragmentDensityAttachment = true;
	}

	SubpassHint = RPInfo.SubpassHint;
	CompatibleHashInfo.SubpassHint = (uint8)RPInfo.SubpassHint;

	CompatibleHashInfo.NumSamples = NumSamples;
	CompatibleHashInfo.MultiViewCount = MultiViewCount;
	// Depth prepass has no color RTs but has a depth attachment that must be multiview
	if (MultiViewCount > 1 && !bMultiviewRenderTargets && !(NumColorRenderTargets == 0 && bMultiViewDepthStencil))
	{
		UE_LOGF(LogVulkan, Error, "Non multiview textures on a multiview layout!");
	}

	RenderPassCompatibleHash = FCrc::MemCrc32(&CompatibleHashInfo, sizeof(CompatibleHashInfo));
	RenderPassFullHash = FCrc::MemCrc32(&FullHashInfo, sizeof(FullHashInfo), RenderPassCompatibleHash);
	NumUsedClearValues = bFoundClearOp ? NumAttachmentDescriptions : 0;
	bCalculatedHash = true;
}

FVulkanRenderTargetLayout::FVulkanRenderTargetLayout(const FGraphicsPipelineStateInitializer& Initializer)
	: NumAttachmentDescriptions(0)
	, NumColorAttachments(0)
	, bHasDepthStencil(false)
	, bHasResolveAttachments(false)
	, bHasDepthStencilResolve(false)
	, bHasFragmentDensityAttachment(false)
	, NumSamples(0)
	, NumUsedClearValues(0)
	, MultiViewCount(0)
{
	ResetAttachments();

	FRenderPassCompatibleHashableStruct CompatibleHashInfo;
	FRenderPassFullHashableStruct FullHashInfo;

	bool bFoundClearOp = false;
	MultiViewCount = Initializer.MultiViewCount;
	NumSamples = Initializer.NumSamples;
	for (uint32 Index = 0; Index < Initializer.RenderTargetsEnabled; ++Index)
	{
		EPixelFormat UEFormat = (EPixelFormat)Initializer.RenderTargetFormats[Index];
		if (UEFormat != PF_Unknown)
		{
			// With a CustomResolveSubpass last color attachment is a resolve target
			bool bCustomResolveAttachment = (Index == (Initializer.RenderTargetsEnabled - 1)) && Initializer.SubpassHint == ESubpassHint::CustomResolveSubpass;

			VkAttachmentDescription& CurrDesc = Desc[NumAttachmentDescriptions];
			CurrDesc.samples = bCustomResolveAttachment ? VK_SAMPLE_COUNT_1_BIT : static_cast<VkSampleCountFlagBits>(NumSamples);
			CurrDesc.format = UEToVkTextureFormat(UEFormat, EnumHasAllFlags(Initializer.RenderTargetFlags[Index], TexCreate_SRGB));
			CurrDesc.loadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
			CurrDesc.storeOp = VK_ATTACHMENT_STORE_OP_DONT_CARE;
			CurrDesc.stencilLoadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
			CurrDesc.stencilStoreOp = VK_ATTACHMENT_STORE_OP_DONT_CARE;

			// If the initial != final we need to change the FullHashInfo and use FinalLayout
			CurrDesc.initialLayout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;
			CurrDesc.finalLayout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;

			ColorReferences[NumColorAttachments].attachment = NumAttachmentDescriptions;
			ColorReferences[NumColorAttachments].layout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;

			if (CurrDesc.samples > VK_SAMPLE_COUNT_1_BIT)
			{
				Desc[NumAttachmentDescriptions + 1] = Desc[NumAttachmentDescriptions];
				Desc[NumAttachmentDescriptions + 1].samples = VK_SAMPLE_COUNT_1_BIT;
				Desc[NumAttachmentDescriptions + 1].loadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
				Desc[NumAttachmentDescriptions + 1].storeOp = VK_ATTACHMENT_STORE_OP_STORE;
				ResolveReferences[NumColorAttachments].attachment = NumAttachmentDescriptions + 1;
				ResolveReferences[NumColorAttachments].layout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;
				CompatibleHashInfo.AttachmentsToResolve |= (uint16)(1 << NumColorAttachments);
				++NumAttachmentDescriptions;
				bHasResolveAttachments = true;
			}

			CompatibleHashInfo.Formats[NumColorAttachments] = CurrDesc.format;
			FullHashInfo.LoadOps[NumColorAttachments] = CurrDesc.loadOp;
			FullHashInfo.StoreOps[NumColorAttachments] = CurrDesc.storeOp;
			FullHashInfo.InitialLayout[NumColorAttachments] = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;
			++CompatibleHashInfo.NumAttachments;

			++NumAttachmentDescriptions;
			++NumColorAttachments;
		}
	}

	if (Initializer.DepthStencilTargetFormat != PF_Unknown)
	{
		VkAttachmentDescription& CurrDesc = Desc[NumAttachmentDescriptions];
		FMemory::Memzero(CurrDesc);

		CurrDesc.samples = static_cast<VkSampleCountFlagBits>(NumSamples);
		CurrDesc.format = UEToVkTextureFormat(Initializer.DepthStencilTargetFormat, false);
		CurrDesc.loadOp = RenderTargetLoadActionToVulkan(Initializer.DepthTargetLoadAction);
		CurrDesc.stencilLoadOp = RenderTargetLoadActionToVulkan(Initializer.StencilTargetLoadAction);
		if (CurrDesc.loadOp == VK_ATTACHMENT_LOAD_OP_CLEAR || CurrDesc.stencilLoadOp == VK_ATTACHMENT_LOAD_OP_CLEAR)
		{
			bFoundClearOp = true;
		}
		CurrDesc.storeOp = RenderTargetStoreActionToVulkan(Initializer.DepthTargetStoreAction);
		CurrDesc.stencilStoreOp = RenderTargetStoreActionToVulkan(Initializer.StencilTargetStoreAction);

		const VkImageLayout DepthLayout = Initializer.DepthStencilAccess.IsDepthWrite() ? VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL : VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL;
		const VkImageLayout StencilLayout = Initializer.DepthStencilAccess.IsStencilWrite() ? VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL : VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL;

		// If the initial != final we need to change the FullHashInfo and use FinalLayout
		CurrDesc.initialLayout = DepthLayout;
		CurrDesc.finalLayout = DepthLayout;
		StencilDesc.stencilInitialLayout = StencilLayout;
		StencilDesc.stencilFinalLayout = StencilLayout;

		DepthReference.attachment = NumAttachmentDescriptions;
		DepthReference.layout = DepthLayout;
		StencilReference.stencilLayout = StencilLayout;

		const bool bDepthStencilResolve = (Initializer.DepthTargetStoreAction == ERenderTargetStoreAction::EMultisampleResolve) || (Initializer.StencilTargetStoreAction == ERenderTargetStoreAction::EMultisampleResolve);
		if (bDepthStencilResolve && GRHISupportsDepthStencilResolve && CurrDesc.samples > VK_SAMPLE_COUNT_1_BIT)
		{
			Desc[NumAttachmentDescriptions + 1] = Desc[NumAttachmentDescriptions];
			Desc[NumAttachmentDescriptions + 1].samples = VK_SAMPLE_COUNT_1_BIT;
			Desc[NumAttachmentDescriptions + 1].loadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
			Desc[NumAttachmentDescriptions + 1].storeOp = VK_ATTACHMENT_STORE_OP_STORE;
			Desc[NumAttachmentDescriptions + 1].stencilLoadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
			Desc[NumAttachmentDescriptions + 1].stencilStoreOp = VK_ATTACHMENT_STORE_OP_STORE;
			DepthStencilResolveReference.attachment = NumAttachmentDescriptions + 1;
			DepthStencilResolveReference.layout = VK_IMAGE_LAYOUT_DEPTH_STENCIL_ATTACHMENT_OPTIMAL;
			// NumColorAttachments was incremented after the last color attachment
			ensureMsgf(NumColorAttachments < 16, TEXT("Must have room for depth resolve bit"));
			CompatibleHashInfo.AttachmentsToResolve |= (uint16)(1 << NumColorAttachments);
			++NumAttachmentDescriptions;
			bHasDepthStencilResolve = true;
		}

		FullHashInfo.LoadOps[MaxSimultaneousRenderTargets] = CurrDesc.loadOp;
		FullHashInfo.LoadOps[MaxSimultaneousRenderTargets + 1] = CurrDesc.stencilLoadOp;
		FullHashInfo.StoreOps[MaxSimultaneousRenderTargets] = CurrDesc.storeOp;
		FullHashInfo.StoreOps[MaxSimultaneousRenderTargets + 1] = CurrDesc.stencilStoreOp;
		FullHashInfo.InitialLayout[MaxSimultaneousRenderTargets] = DepthLayout;
		FullHashInfo.InitialLayout[MaxSimultaneousRenderTargets + 1] = StencilLayout;
		CompatibleHashInfo.Formats[MaxSimultaneousRenderTargets] = CurrDesc.format;

		++NumAttachmentDescriptions;
		bHasDepthStencil = true;
	}

	if (Initializer.bHasFragmentDensityAttachment)
	{
		VkAttachmentDescription& CurrDesc = Desc[NumAttachmentDescriptions];
		FMemory::Memzero(CurrDesc);

		const VkImageLayout VRSLayout = GetVRSImageLayout();

		check(GRHIVariableRateShadingImageFormat != PF_Unknown);

		CurrDesc.flags = 0;
		CurrDesc.format = UEToVkTextureFormat(GRHIVariableRateShadingImageFormat, false);
		CurrDesc.samples = VK_SAMPLE_COUNT_1_BIT;
		CurrDesc.loadOp = VK_ATTACHMENT_LOAD_OP_LOAD;
		CurrDesc.storeOp = VK_ATTACHMENT_STORE_OP_DONT_CARE;
		CurrDesc.stencilLoadOp = VK_ATTACHMENT_LOAD_OP_DONT_CARE;
		CurrDesc.stencilStoreOp = VK_ATTACHMENT_STORE_OP_DONT_CARE;
		CurrDesc.initialLayout = VRSLayout;
		CurrDesc.finalLayout = VRSLayout;

		FragmentDensityReference.attachment = NumAttachmentDescriptions;
		FragmentDensityReference.layout = VRSLayout;

		FullHashInfo.LoadOps[MaxSimultaneousRenderTargets + 2] = CurrDesc.stencilLoadOp;
		FullHashInfo.StoreOps[MaxSimultaneousRenderTargets + 2] = CurrDesc.stencilStoreOp;
		FullHashInfo.InitialLayout[MaxSimultaneousRenderTargets + 2] = VRSLayout;
		CompatibleHashInfo.Formats[MaxSimultaneousRenderTargets + 1] = CurrDesc.format;

		++NumAttachmentDescriptions;
		bHasFragmentDensityAttachment = true;
	}

	SubpassHint = Initializer.SubpassHint;
	CompatibleHashInfo.SubpassHint = (uint8)Initializer.SubpassHint;

	CompatibleHashInfo.NumSamples = NumSamples;
	CompatibleHashInfo.MultiViewCount = MultiViewCount;

	RenderPassCompatibleHash = FCrc::MemCrc32(&CompatibleHashInfo, sizeof(CompatibleHashInfo));
	RenderPassFullHash = FCrc::MemCrc32(&FullHashInfo, sizeof(FullHashInfo), RenderPassCompatibleHash);
	NumUsedClearValues = bFoundClearOp ? NumAttachmentDescriptions : 0;
	bCalculatedHash = true;
}

