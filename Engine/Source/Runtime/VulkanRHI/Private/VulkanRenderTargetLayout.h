// Copyright Epic Games, Inc. All Rights Reserved..

#pragma once

#include "VulkanRHIPrivate.h"

class FVulkanRenderTargetLayout
{
public:
	FVulkanRenderTargetLayout(const FGraphicsPipelineStateInitializer& Initializer);
	FVulkanRenderTargetLayout(FVulkanDevice& InDevice, const FRHISetRenderTargetsInfo& RTInfo);
	FVulkanRenderTargetLayout(FVulkanDevice& InDevice, const FRHIRenderPassInfo& RPInfo, VkImageLayout CurrentDepthLayout, VkImageLayout CurrentStencilLayout);

	inline uint32 GetRenderPassCompatibleHash() const
	{
		check(bCalculatedHash);
		return RenderPassCompatibleHash;
	}
	inline uint32 GetRenderPassFullHash() const
	{
		check(bCalculatedHash);
		return RenderPassFullHash;
	}
	inline const VkOffset2D& GetOffset2D() const { return Offset.Offset2D; }
	inline const VkOffset3D& GetOffset3D() const { return Offset.Offset3D; }
	inline const VkExtent2D& GetExtent2D() const { return Extent.Extent2D; }
	inline const VkExtent3D& GetExtent3D() const { return Extent.Extent3D; }
	inline const VkAttachmentDescription* GetAttachmentDescriptions() const { return Desc; }
	inline uint32 GetNumColorAttachments() const { return NumColorAttachments; }
	inline bool GetHasDepthStencil() const { return bHasDepthStencil != 0; }
	inline bool GetHasResolveAttachments() const { return bHasResolveAttachments != 0; }
	inline bool GetHasDepthStencilResolve() const { return bHasDepthStencilResolve != 0; }
	inline bool GetHasFragmentDensityAttachment() const { return bHasFragmentDensityAttachment != 0; }
	inline uint32 GetNumAttachmentDescriptions() const { return NumAttachmentDescriptions; }
	inline uint32 GetNumSamples() const { return NumSamples; }
	inline uint32 GetNumUsedClearValues() const { return NumUsedClearValues; }
	inline bool GetIsMultiView() const { return MultiViewCount != 0; }
	inline uint32 GetMultiViewCount() const { return MultiViewCount; }


	inline const VkAttachmentReference* GetColorAttachmentReferences() const { return NumColorAttachments > 0 ? ColorReferences : nullptr; }
	inline const VkAttachmentReference* GetResolveAttachmentReferences() const { return bHasResolveAttachments ? ResolveReferences : nullptr; }
	inline const VkAttachmentReference* GetDepthAttachmentReference() const { return bHasDepthStencil ? &DepthReference : nullptr; }
	inline const VkAttachmentReferenceStencilLayout* GetStencilAttachmentReference() const { return bHasDepthStencil ? &StencilReference : nullptr; }
	inline const VkAttachmentReference* GetDepthStencilResolveAttachmentReference() const { return bHasDepthStencilResolve ? &DepthStencilResolveReference : nullptr; }
	inline const VkAttachmentReference* GetFragmentDensityAttachmentReference() const { return bHasFragmentDensityAttachment ? &FragmentDensityReference : nullptr; }

	inline const VkAttachmentDescriptionStencilLayout* GetStencilDesc() const { return bHasDepthStencil ? &StencilDesc : nullptr; }

	inline const ESubpassHint GetSubpassHint() const { return SubpassHint; }

protected:
	VkImageLayout GetVRSImageLayout() const;

protected:
	VkAttachmentReference ColorReferences[MaxSimultaneousRenderTargets];
	VkAttachmentReference DepthReference;
	VkAttachmentReferenceStencilLayout StencilReference;
	VkAttachmentReference FragmentDensityReference;
	VkAttachmentReference ResolveReferences[MaxSimultaneousRenderTargets];
	VkAttachmentReference DepthStencilResolveReference;

	// Depth goes in the "+1" slot, Depth resolve goes in the "+2 slot", and the Shading Rate texture goes in the "+3" slot.
	VkAttachmentDescription Desc[MaxSimultaneousRenderTargets * 2 + 3];
	VkAttachmentDescriptionStencilLayout StencilDesc;

	uint8 NumAttachmentDescriptions;
	uint8 NumColorAttachments;
	uint8 NumInputAttachments = 0;
	uint8 bHasDepthStencil;
	uint8 bHasResolveAttachments;
	uint8 bHasDepthStencilResolve;
	uint8 bHasFragmentDensityAttachment;
	uint8 NumSamples;
	uint8 NumUsedClearValues;
	ESubpassHint SubpassHint = ESubpassHint::None;
	uint8 MultiViewCount;

	// Hash for a compatible RenderPass
	uint32 RenderPassCompatibleHash = 0;
	// Hash for the render pass including the load/store operations
	uint32 RenderPassFullHash = 0;

	union
	{
		VkOffset3D Offset3D;
		VkOffset2D Offset2D;
	} Offset;

	union
	{
		VkExtent3D	Extent3D;
		VkExtent2D	Extent2D;
	} Extent;

	inline void ResetAttachments()
	{
		FMemory::Memzero(ColorReferences);
		FMemory::Memzero(DepthReference);
		FMemory::Memzero(FragmentDensityReference);
		FMemory::Memzero(ResolveReferences);
		FMemory::Memzero(DepthStencilResolveReference);
		FMemory::Memzero(Desc);
		FMemory::Memzero(Offset);
		FMemory::Memzero(Extent);

		ZeroVulkanStruct(StencilReference, VK_STRUCTURE_TYPE_ATTACHMENT_REFERENCE_STENCIL_LAYOUT);
		ZeroVulkanStruct(StencilDesc, VK_STRUCTURE_TYPE_ATTACHMENT_DESCRIPTION_STENCIL_LAYOUT);
	}

	FVulkanRenderTargetLayout()
	{
		NumAttachmentDescriptions = 0;
		NumColorAttachments = 0;
		bHasDepthStencil = 0;
		bHasResolveAttachments = 0;
		bHasDepthStencilResolve = 0;
		bHasFragmentDensityAttachment = 0;
		NumSamples = 0;
		NumUsedClearValues = 0;
		MultiViewCount = 0;

		ResetAttachments();
	}

	bool bCalculatedHash = false;

	friend class FVulkanPipelineStateCacheManager;
	friend struct FGfxPipelineDesc;
};

