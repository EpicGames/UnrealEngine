// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanRenderpass.h"
#include "VulkanFramebuffer.h"
#include "VulkanRenderpass.h"
#include "VulkanContext.h"

VkRenderPass CreateVulkanRenderPass(FVulkanDevice& InDevice, const FVulkanRenderTargetLayout& RTLayout)
{
	VkRenderPass OutRenderpass;

	if (InDevice.GetOptionalExtensions().HasKHRRenderPass2)
	{
		FVulkanRenderPassBuilder<FVulkanSubpassDescription<VkSubpassDescription2>, FVulkanSubpassDependency<VkSubpassDependency2>, FVulkanAttachmentReference<VkAttachmentReference2>, FVulkanAttachmentDescription<VkAttachmentDescription2>, FVulkanRenderPassCreateInfo<VkRenderPassCreateInfo2>> Creator(InDevice);
		OutRenderpass = Creator.Create(RTLayout);
	}
	else
	{
		FVulkanRenderPassBuilder<FVulkanSubpassDescription<VkSubpassDescription>, FVulkanSubpassDependency<VkSubpassDependency>, FVulkanAttachmentReference<VkAttachmentReference>, FVulkanAttachmentDescription<VkAttachmentDescription>, FVulkanRenderPassCreateInfo<VkRenderPassCreateInfo>> Creator(InDevice);
		OutRenderpass = Creator.Create(RTLayout);
	}

	return OutRenderpass;
}



FVulkanRenderPassManager::~FVulkanRenderPassManager()
{
	check(!GIsRHIInitialized);

	for (auto& Pair : RenderPasses)
	{
		delete Pair.Value;
	}

	for (auto& Pair : Framebuffers)
	{
		FFramebufferList* List = Pair.Value;
		for (int32 Index = List->Framebuffer.Num() - 1; Index >= 0; --Index)
		{
			List->Framebuffer[Index]->Destroy(Device);
			delete List->Framebuffer[Index];
		}
		delete List;
	}

	RenderPasses.Reset();
	Framebuffers.Reset();
}

FVulkanFramebuffer* FVulkanRenderPassManager::GetOrCreateFramebuffer(const FRHISetRenderTargetsInfo& RenderTargetsInfo, const FVulkanRenderTargetLayout& RTLayout, FVulkanRenderPass& RenderPass)
{
	uint32 RTLayoutHash = RTLayout.GetRenderPassCompatibleHash();

	uint64 MipsAndSlicesValues[MaxSimultaneousRenderTargets];
	for (int32 Index = 0; Index < MaxSimultaneousRenderTargets; ++Index)
	{
		MipsAndSlicesValues[Index] = ((uint64)RenderTargetsInfo.ColorRenderTarget[Index].ArraySliceIndex << (uint64)32) | (uint64)RenderTargetsInfo.ColorRenderTarget[Index].MipIndex;
	}
	RTLayoutHash = FCrc::MemCrc32(MipsAndSlicesValues, sizeof(MipsAndSlicesValues), RTLayoutHash);

	auto FindFramebufferInList = [&](FFramebufferList* InFramebufferList)
	{
		FVulkanFramebuffer* OutFramebuffer = nullptr;

		for (int32 Index = 0; Index < InFramebufferList->Framebuffer.Num(); ++Index)
		{
			const VkRect2D RenderArea = InFramebufferList->Framebuffer[Index]->GetRenderArea();

			if (InFramebufferList->Framebuffer[Index]->Matches(RenderTargetsInfo) &&
				((RTLayout.GetExtent2D().width == RenderArea.extent.width) && (RTLayout.GetExtent2D().height == RenderArea.extent.height) &&
					(RTLayout.GetOffset2D().x == RenderArea.offset.x) && (RTLayout.GetOffset2D().y == RenderArea.offset.y)))
			{
				OutFramebuffer = InFramebufferList->Framebuffer[Index];
				break;
			}
		}

		return OutFramebuffer;
	};

	FFramebufferList** FoundFramebufferList = nullptr;
	FFramebufferList* FramebufferList = nullptr;

	{
		FRWScopeLock ScopedReadLock(FramebuffersLock, SLT_ReadOnly);

		FoundFramebufferList = Framebuffers.Find(RTLayoutHash);
		if (FoundFramebufferList)
		{
			FramebufferList = *FoundFramebufferList;

			FVulkanFramebuffer* ExistingFramebuffer = FindFramebufferInList(FramebufferList);
			if (ExistingFramebuffer)
			{
				return ExistingFramebuffer;
			}
		}
	}

	FRWScopeLock ScopedWriteLock(FramebuffersLock, SLT_Write);
	FoundFramebufferList = Framebuffers.Find(RTLayoutHash);
	if (!FoundFramebufferList)
	{
		FramebufferList = new FFramebufferList;
		Framebuffers.Add(RTLayoutHash, FramebufferList);
	}
	else
	{
		FramebufferList = *FoundFramebufferList;
		FVulkanFramebuffer* ExistingFramebuffer = FindFramebufferInList(FramebufferList);
		if (ExistingFramebuffer)
		{
			return ExistingFramebuffer;
		}
	}

	FVulkanFramebuffer* Framebuffer = new FVulkanFramebuffer(Device, RenderTargetsInfo, RTLayout, RenderPass);
	FramebufferList->Framebuffer.Add(Framebuffer);
	return Framebuffer;
}

void FVulkanRenderPassManager::BeginRenderPass(FVulkanCommandListContext& Context, const FRHIRenderPassInfo& RPInfo, const FVulkanRenderTargetLayout& RTLayout, const FVulkanBeginRenderPassInfo& BeginRenderPassInfo)
{
	// (NumRT + 1 [Depth] ) * 2 [surface + resolve]
	VkClearValue ClearValues[(MaxSimultaneousRenderTargets + 1) * 2];
	uint32 ClearValueIndex = 0;
	bool bNeedsClearValues = BeginRenderPassInfo.RenderPass.GetNumUsedClearValues() > 0;
	FMemory::Memzero(ClearValues);

	int32 NumColorTargets = RPInfo.GetNumColorRenderTargets();
	int32 Index = 0;

	FVulkanPipelineBarrier Barrier;

	for (Index = 0; Index < NumColorTargets; ++Index)
	{
		const FRHIRenderPassInfo::FColorEntry& ColorEntry = RPInfo.ColorRenderTargets[Index];

		FRHITexture* ColorTexture = ColorEntry.RenderTarget;
		CA_ASSUME(ColorTexture);
		FVulkanTexture& ColorSurface = *ResourceCast(ColorTexture);
		const bool bPassPerformsResolve = ColorSurface.GetNumSamples() > 1 && ColorEntry.ResolveTarget;

		if (GetLoadAction(ColorEntry.Action) == ERenderTargetLoadAction::ELoad)
		{
			// Insert a barrier if we're loading from any color targets, to make sure the passes aren't reordered and we end up running before
			// the pass we're supposed to read from.
			const VkAccessFlags AccessMask = VK_ACCESS_COLOR_ATTACHMENT_WRITE_BIT;
			const VkPipelineStageFlags StageMask = VK_PIPELINE_STAGE_COLOR_ATTACHMENT_OUTPUT_BIT;
			Barrier.AddMemoryBarrier(AccessMask, AccessMask, StageMask, StageMask);
		}

		if (bNeedsClearValues)
		{
			const FLinearColor& ClearColor = ColorTexture->HasClearValue() ? ColorTexture->GetClearColor() : FLinearColor::Black;
			ClearValues[ClearValueIndex].color.float32[0] = ClearColor.R;
			ClearValues[ClearValueIndex].color.float32[1] = ClearColor.G;
			ClearValues[ClearValueIndex].color.float32[2] = ClearColor.B;
			ClearValues[ClearValueIndex].color.float32[3] = ClearColor.A;
			++ClearValueIndex;
			if (bPassPerformsResolve)
			{
				++ClearValueIndex;
			}
		}
	}

	FRHITexture* DSTexture = RPInfo.DepthStencilRenderTarget.DepthStencilTarget;
	if (DSTexture)
	{
		const FExclusiveDepthStencil RequestedDSAccess = RPInfo.DepthStencilRenderTarget.ExclusiveDepthStencil;
		if (RequestedDSAccess.IsDepthRead() || RequestedDSAccess.IsStencilRead())
		{
			// If the depth-stencil state doesn't change between passes, the high level code won't perform any transitions.
			// Make sure we have a barrier in case we're loading depth or stencil, to prevent rearranging passes.
			const VkAccessFlags AccessMask = VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_READ_BIT | VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_WRITE_BIT;
			const VkPipelineStageFlags StageMask = VK_PIPELINE_STAGE_EARLY_FRAGMENT_TESTS_BIT | VK_PIPELINE_STAGE_LATE_FRAGMENT_TESTS_BIT;
			Barrier.AddMemoryBarrier(AccessMask, AccessMask, StageMask, StageMask);
		}

		if (DSTexture->HasClearValue() && bNeedsClearValues)
		{
			float Depth = 0;
			uint32 Stencil = 0;
			DSTexture->GetDepthStencilClearValue(Depth, Stencil);
			ClearValues[ClearValueIndex].depthStencil.depth = Depth;
			ClearValues[ClearValueIndex].depthStencil.stencil = Stencil;
			++ClearValueIndex;
		}
	}

	FRHITexture* ShadingRateTexture = RPInfo.ShadingRateTexture;
	if (ShadingRateTexture)
	{
		ValidateShadingRateDataType();
	}

	ensure(ClearValueIndex <= BeginRenderPassInfo.RenderPass.GetNumUsedClearValues());

	FVulkanCommandBuffer& CommandBuffer = Context.GetCommandBuffer();

	Barrier.Execute(&CommandBuffer);

	CommandBuffer.BeginRenderPass(BeginRenderPassInfo, ClearValues);

	{
		const VkExtent3D& Extents = RTLayout.GetExtent3D();
		Context.GetPendingGfxState()->SetViewport(0, 0, 0, Extents.width, Extents.height, 1);
	}
}

void FVulkanRenderPassManager::EndRenderPass(FVulkanCommandListContext& Context)
{
	FVulkanCommandBuffer& CommandBuffer = Context.GetCommandBuffer();
	CommandBuffer.EndRenderPass();

	VulkanRHI::DebugHeavyWeightBarrier(CommandBuffer.GetHandle(), 1);
}

void FVulkanRenderPassManager::NotifyDeletedRenderTarget(VkImage Image)
{
	for (auto It = Framebuffers.CreateIterator(); It; ++It)
	{
		FFramebufferList* List = It->Value;
		for (int32 Index = List->Framebuffer.Num() - 1; Index >= 0; --Index)
		{
			FVulkanFramebuffer* Framebuffer = List->Framebuffer[Index];
			if (Framebuffer->ContainsRenderTarget(Image))
			{
				List->Framebuffer.RemoveAtSwap(Index, EAllowShrinking::No);
				Framebuffer->Destroy(Device);
				delete Framebuffer;
			}
		}

		if (List->Framebuffer.Num() == 0)
		{
			delete List;
			It.RemoveCurrent();
		}
	}
}
