// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanState.h: Vulkan state definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "VulkanRHIPrivate.h"
#include "VulkanRenderTargetLayout.h"
#include "VulkanResources.h"
#include "VulkanPendingState.h"

class FVulkanCommandListContext;

template <typename TAttachmentReferenceType>
struct FVulkanAttachmentReference
	: public TAttachmentReferenceType
{
	FVulkanAttachmentReference()
	{
		ZeroStruct();
	}

	FVulkanAttachmentReference(const VkAttachmentReference& AttachmentReferenceIn, VkImageAspectFlags AspectMask)
	{
		SetAttachment(AttachmentReferenceIn, AspectMask);
	}

	inline void SetAttachment(const VkAttachmentReference& AttachmentReferenceIn, VkImageAspectFlags AspectMask) { checkNoEntry(); }
	inline void SetAttachment(const FVulkanAttachmentReference<TAttachmentReferenceType>& AttachmentReferenceIn, VkImageAspectFlags AspectMask) { *this = AttachmentReferenceIn; }
	inline void SetDepthStencilAttachment(const VkAttachmentReference& AttachmentReferenceIn, const VkAttachmentReferenceStencilLayout* StencilReference, VkImageAspectFlags AspectMask, bool bSupportsParallelRendering) { checkNoEntry(); }
	inline void ZeroStruct() {}
	inline void SetAspect(uint32 Aspect) {}
};

template <>
inline void FVulkanAttachmentReference<VkAttachmentReference>::SetAttachment(const VkAttachmentReference& AttachmentReferenceIn, VkImageAspectFlags AspectMask)
{
	attachment = AttachmentReferenceIn.attachment;
	layout = AttachmentReferenceIn.layout;
}

template <>
inline void FVulkanAttachmentReference<VkAttachmentReference2>::SetAttachment(const VkAttachmentReference& AttachmentReferenceIn, VkImageAspectFlags AspectMask)
{
	sType = VK_STRUCTURE_TYPE_ATTACHMENT_REFERENCE_2;
	pNext = nullptr;
	attachment = AttachmentReferenceIn.attachment;
	layout = AttachmentReferenceIn.layout;
	aspectMask = AspectMask;
}

template<>
inline void FVulkanAttachmentReference<VkAttachmentReference2>::SetAttachment(const FVulkanAttachmentReference<VkAttachmentReference2>& AttachmentReferenceIn, VkImageAspectFlags AspectMask)
{
	sType = VK_STRUCTURE_TYPE_ATTACHMENT_REFERENCE_2;
	pNext = nullptr;
	attachment = AttachmentReferenceIn.attachment;
	layout = AttachmentReferenceIn.layout;
	aspectMask = AspectMask;
}

template <>
inline void FVulkanAttachmentReference<VkAttachmentReference>::SetDepthStencilAttachment(const VkAttachmentReference& AttachmentReferenceIn, 
	const VkAttachmentReferenceStencilLayout* StencilReference, VkImageAspectFlags AspectMask, bool bSupportsParallelRendering)
{
	attachment = AttachmentReferenceIn.attachment;
	const VkImageLayout StencilLayout = StencilReference ? StencilReference->stencilLayout : VK_IMAGE_LAYOUT_UNDEFINED;
	layout = VulkanRHI::GetMergedDepthStencilLayout(AttachmentReferenceIn.layout, StencilLayout);
}

template <>
inline void FVulkanAttachmentReference<VkAttachmentReference2>::SetDepthStencilAttachment(const VkAttachmentReference& AttachmentReferenceIn, 
	const VkAttachmentReferenceStencilLayout* StencilReference, VkImageAspectFlags AspectMask, bool bSupportsParallelRendering)
{
	if (FVulkanPlatform::RequiresMergedDepthStencilLayout())
	{
		bSupportsParallelRendering = false;
	}

	sType = VK_STRUCTURE_TYPE_ATTACHMENT_REFERENCE_2;
	attachment = AttachmentReferenceIn.attachment;
	aspectMask = AspectMask;

	if (bSupportsParallelRendering)
	{
		const bool bHasStencilReference = StencilReference && (StencilReference->stencilLayout != VK_IMAGE_LAYOUT_UNDEFINED);
		pNext = bHasStencilReference ? StencilReference : nullptr;
		layout = AttachmentReferenceIn.layout;
	}
	else
	{
		pNext = nullptr;
		const VkImageLayout StencilLayout = StencilReference ? StencilReference->stencilLayout : VK_IMAGE_LAYOUT_UNDEFINED;
		layout = VulkanRHI::GetMergedDepthStencilLayout(AttachmentReferenceIn.layout, StencilLayout);
	}
}


template<>
inline void FVulkanAttachmentReference<VkAttachmentReference>::ZeroStruct()
{
	attachment = 0;
	layout = VK_IMAGE_LAYOUT_UNDEFINED;
}

template<>
inline void FVulkanAttachmentReference<VkAttachmentReference2>::ZeroStruct()
{
	sType = VK_STRUCTURE_TYPE_ATTACHMENT_REFERENCE_2;
	pNext = nullptr;
	attachment = 0;
	layout = VK_IMAGE_LAYOUT_UNDEFINED;
	aspectMask = 0;
}

template<>
inline void FVulkanAttachmentReference<VkAttachmentReference2>::SetAspect(uint32 Aspect)
{
	aspectMask = Aspect;
}

template <typename TSubpassDescriptionType>
class FVulkanSubpassDescription
{
};

template<>
struct FVulkanSubpassDescription<VkSubpassDescription>
	: public VkSubpassDescription
{
	FVulkanSubpassDescription()
	{
		FMemory::Memzero(this, sizeof(VkSubpassDescription));
		pipelineBindPoint = VK_PIPELINE_BIND_POINT_GRAPHICS;
	}

	void SetColorAttachments(const TArray<FVulkanAttachmentReference<VkAttachmentReference>>& ColorAttachmentReferences, int OverrideCount = -1)
	{
		colorAttachmentCount = (OverrideCount == -1) ? ColorAttachmentReferences.Num() : OverrideCount;
		pColorAttachments = ColorAttachmentReferences.GetData();
	}

	void SetResolveAttachments(const TArrayView<FVulkanAttachmentReference<VkAttachmentReference>>& ResolveAttachmentReferences)
	{
		if (ResolveAttachmentReferences.Num() > 0)
		{
			check(colorAttachmentCount == ResolveAttachmentReferences.Num());
			pResolveAttachments = ResolveAttachmentReferences.GetData();
		}
	}

	void SetDepthStencilAttachment(FVulkanAttachmentReference<VkAttachmentReference>* DepthStencilAttachmentReference)
	{
		pDepthStencilAttachment = static_cast<VkAttachmentReference*>(DepthStencilAttachmentReference);
	}

	void SetInputAttachments(FVulkanAttachmentReference<VkAttachmentReference>* InputAttachmentReferences, uint32 NumInputAttachmentReferences)
	{
		pInputAttachments = static_cast<VkAttachmentReference*>(InputAttachmentReferences);
		inputAttachmentCount = NumInputAttachmentReferences;
	}

	void SetDepthStencilResolveAttachment(VkSubpassDescriptionDepthStencilResolveKHR* DepthStencilResolveAttachmentDesc)
	{
		// No-op without VK_KHR_create_renderpass2
	}

	void SetShadingRateAttachment(VkFragmentShadingRateAttachmentInfoKHR* /* ShadingRateAttachmentInfo */)
	{
		// No-op without VK_KHR_create_renderpass2
	}

	void SetMultiViewMask(uint32_t Mask)
	{
		// No-op without VK_KHR_create_renderpass2
	}
};

template<>
struct FVulkanSubpassDescription<VkSubpassDescription2>
	: public VkSubpassDescription2
{
	FVulkanSubpassDescription()
	{
		ZeroVulkanStruct(*this, VK_STRUCTURE_TYPE_SUBPASS_DESCRIPTION_2);
		pipelineBindPoint = VK_PIPELINE_BIND_POINT_GRAPHICS;
		viewMask = 0;
	}

	void SetColorAttachments(const TArray<FVulkanAttachmentReference<VkAttachmentReference2>>& ColorAttachmentReferences, int OverrideCount = -1)
	{
		colorAttachmentCount = OverrideCount == -1 ? ColorAttachmentReferences.Num() : OverrideCount;
		pColorAttachments = ColorAttachmentReferences.GetData();
	}

	void SetResolveAttachments(const TArrayView<FVulkanAttachmentReference<VkAttachmentReference2>>& ResolveAttachmentReferences)
	{
		if (ResolveAttachmentReferences.Num() > 0)
		{
			check(colorAttachmentCount == ResolveAttachmentReferences.Num());
			pResolveAttachments = ResolveAttachmentReferences.GetData();
		}
	}

	void SetDepthStencilAttachment(FVulkanAttachmentReference<VkAttachmentReference2>* DepthStencilAttachmentReference)
	{
		pDepthStencilAttachment = static_cast<VkAttachmentReference2*>(DepthStencilAttachmentReference);
	}

	void SetInputAttachments(FVulkanAttachmentReference<VkAttachmentReference2>* InputAttachmentReferences, uint32 NumInputAttachmentReferences)
	{
		pInputAttachments = static_cast<VkAttachmentReference2*>(InputAttachmentReferences);
		inputAttachmentCount = NumInputAttachmentReferences;
	}

	void SetDepthStencilResolveAttachment(VkSubpassDescriptionDepthStencilResolveKHR* DepthStencilResolveAttachmentDesc)
	{
		const void* Next = pNext;
		pNext = DepthStencilResolveAttachmentDesc;
		DepthStencilResolveAttachmentDesc->pNext = Next;
	}

	void SetShadingRateAttachment(VkFragmentShadingRateAttachmentInfoKHR* ShadingRateAttachmentInfo)
	{
		const void* Next = pNext;
		pNext = ShadingRateAttachmentInfo;
		ShadingRateAttachmentInfo->pNext = Next;
	}

	void SetMultiViewMask(uint32_t Mask)
	{
		viewMask = Mask;
	}
};

template <typename TSubpassDependencyType>
struct FVulkanSubpassDependency
	: public TSubpassDependencyType
{
};

template<>
struct FVulkanSubpassDependency<VkSubpassDependency>
	: public VkSubpassDependency
{
	FVulkanSubpassDependency()
	{
		FMemory::Memzero(this, sizeof(VkSubpassDependency));
	}
};

template<>
struct FVulkanSubpassDependency<VkSubpassDependency2>
	: public VkSubpassDependency2
{
	FVulkanSubpassDependency()
	{
		ZeroVulkanStruct(*this, VK_STRUCTURE_TYPE_SUBPASS_DEPENDENCY_2);
		viewOffset = 0;		// According to the Vulkan spec: "If dependencyFlags does not include VK_DEPENDENCY_VIEW_LOCAL_BIT, viewOffset must be 0"
	}
};

template<typename TAttachmentDescriptionType>
struct FVulkanAttachmentDescription
{
};

template<>
struct FVulkanAttachmentDescription<VkAttachmentDescription>
	: public VkAttachmentDescription
{
	FVulkanAttachmentDescription()
	{
		FMemory::Memzero(this, sizeof(VkAttachmentDescription));
	}

	FVulkanAttachmentDescription(const VkAttachmentDescription& InDesc)
	{
		flags = InDesc.flags;
		format = InDesc.format;
		samples = InDesc.samples;
		loadOp = InDesc.loadOp;
		storeOp = InDesc.storeOp;
		stencilLoadOp = InDesc.stencilLoadOp;
		stencilStoreOp = InDesc.stencilStoreOp;
		initialLayout = InDesc.initialLayout;
		finalLayout = InDesc.finalLayout;
	}

	FVulkanAttachmentDescription(const VkAttachmentDescription& InDesc, const VkAttachmentDescriptionStencilLayout* InStencilDesc, bool bSupportsParallelRendering)
	{
		flags = InDesc.flags;
		format = InDesc.format;
		samples = InDesc.samples;
		loadOp = InDesc.loadOp;
		storeOp = InDesc.storeOp;
		stencilLoadOp = InDesc.stencilLoadOp;
		stencilStoreOp = InDesc.stencilStoreOp;

		const bool bHasStencilLayout = VulkanRHI::VulkanFormatHasStencil(InDesc.format) && (InStencilDesc != nullptr);
		const VkImageLayout StencilInitialLayout = bHasStencilLayout ? InStencilDesc->stencilInitialLayout : VK_IMAGE_LAYOUT_UNDEFINED;
		initialLayout = VulkanRHI::GetMergedDepthStencilLayout(InDesc.initialLayout, StencilInitialLayout);
		const VkImageLayout StencilFinalLayout = bHasStencilLayout ? InStencilDesc->stencilFinalLayout : VK_IMAGE_LAYOUT_UNDEFINED;
		finalLayout = VulkanRHI::GetMergedDepthStencilLayout(InDesc.finalLayout, StencilFinalLayout);
	}
};

template<>
struct FVulkanAttachmentDescription<VkAttachmentDescription2>
	: public VkAttachmentDescription2
{
	FVulkanAttachmentDescription()
	{
		ZeroVulkanStruct(*this, VK_STRUCTURE_TYPE_ATTACHMENT_DESCRIPTION_2);
	}

	FVulkanAttachmentDescription(const VkAttachmentDescription& InDesc)
	{
		sType = VK_STRUCTURE_TYPE_ATTACHMENT_DESCRIPTION_2;
		pNext = nullptr;
		flags = InDesc.flags;
		format = InDesc.format;
		samples = InDesc.samples;
		loadOp = InDesc.loadOp;
		storeOp = InDesc.storeOp;
		stencilLoadOp = InDesc.stencilLoadOp;
		stencilStoreOp = InDesc.stencilStoreOp;
		initialLayout = InDesc.initialLayout;
		finalLayout = InDesc.finalLayout;
	}

	FVulkanAttachmentDescription(const VkAttachmentDescription& InDesc, const VkAttachmentDescriptionStencilLayout* InStencilDesc, bool bSupportsParallelRendering)
	{
		if (FVulkanPlatform::RequiresMergedDepthStencilLayout())
		{
			bSupportsParallelRendering = false;
		}
		const bool bHasStencilLayout = bSupportsParallelRendering && VulkanRHI::VulkanFormatHasStencil(InDesc.format) && (InStencilDesc != nullptr);

		sType = VK_STRUCTURE_TYPE_ATTACHMENT_DESCRIPTION_2;
		pNext = (bHasStencilLayout && (InStencilDesc->stencilFinalLayout != VK_IMAGE_LAYOUT_UNDEFINED)) ? InStencilDesc : nullptr;
		flags = InDesc.flags;
		format = InDesc.format;
		samples = InDesc.samples;
		loadOp = InDesc.loadOp;
		storeOp = InDesc.storeOp;
		stencilLoadOp = InDesc.stencilLoadOp;
		stencilStoreOp = InDesc.stencilStoreOp;
		if (bSupportsParallelRendering)
		{
			initialLayout = InDesc.initialLayout;
			finalLayout = InDesc.finalLayout;
		}
		else
		{
			const VkImageLayout StencilInitialLayout = InStencilDesc ? InStencilDesc->stencilInitialLayout : VK_IMAGE_LAYOUT_UNDEFINED;
			initialLayout = VulkanRHI::GetMergedDepthStencilLayout(InDesc.initialLayout, StencilInitialLayout);
			const VkImageLayout StencilFinalLayout = InStencilDesc ? InStencilDesc->stencilFinalLayout : VK_IMAGE_LAYOUT_UNDEFINED;
			finalLayout = VulkanRHI::GetMergedDepthStencilLayout(InDesc.finalLayout, StencilFinalLayout);
		}
	}
};

template <typename T>
struct FVulkanRenderPassCreateInfo
{};

template<>
struct FVulkanRenderPassCreateInfo<VkRenderPassCreateInfo>
	: public VkRenderPassCreateInfo
{
	FVulkanRenderPassCreateInfo()
	{
		ZeroVulkanStruct(*this, VK_STRUCTURE_TYPE_RENDER_PASS_CREATE_INFO);
	}

	void SetCorrelationMask(const uint32_t* MaskPtr)
	{
		// No-op without VK_KHR_create_renderpass2
	}

	VkRenderPass Create(FVulkanDevice& Device)
	{
		VkRenderPass Handle = VK_NULL_HANDLE;
		VERIFYVULKANRESULT_EXPANDED(VulkanRHI::vkCreateRenderPass(Device.GetHandle(), this, VULKAN_CPU_ALLOCATOR, &Handle));
		return Handle;
	}
};

struct FVulkanRenderPassFragmentDensityMapCreateInfoEXT
	: public VkRenderPassFragmentDensityMapCreateInfoEXT
{
	FVulkanRenderPassFragmentDensityMapCreateInfoEXT()
	{
		ZeroVulkanStruct(*this, VK_STRUCTURE_TYPE_RENDER_PASS_FRAGMENT_DENSITY_MAP_CREATE_INFO_EXT);
	}
};

struct FVulkanRenderPassMultiviewCreateInfo
	: public VkRenderPassMultiviewCreateInfo
{
	FVulkanRenderPassMultiviewCreateInfo()
	{
		ZeroVulkanStruct(*this, VK_STRUCTURE_TYPE_RENDER_PASS_MULTIVIEW_CREATE_INFO);
	}
};

template<>
struct FVulkanRenderPassCreateInfo<VkRenderPassCreateInfo2>
	: public VkRenderPassCreateInfo2
{
	FVulkanRenderPassCreateInfo()
	{
		ZeroVulkanStruct(*this, VK_STRUCTURE_TYPE_RENDER_PASS_CREATE_INFO_2);
	}

	void SetCorrelationMask(const uint32_t* MaskPtr)
	{
		correlatedViewMaskCount = 1;
		pCorrelatedViewMasks = MaskPtr;
	}

	VkRenderPass Create(FVulkanDevice& Device)
	{
		VkRenderPass Handle = VK_NULL_HANDLE;
		VERIFYVULKANRESULT_EXPANDED(VulkanRHI::vkCreateRenderPass2KHR(Device.GetHandle(), this, VULKAN_CPU_ALLOCATOR, &Handle));
		return Handle;
	}
};

struct FVulkanFragmentShadingRateAttachmentInfo
	: public VkFragmentShadingRateAttachmentInfoKHR
{
	FVulkanFragmentShadingRateAttachmentInfo()
	{
		ZeroVulkanStruct(*this, VK_STRUCTURE_TYPE_FRAGMENT_SHADING_RATE_ATTACHMENT_INFO_KHR);
		// For now, just use the smallest tile-size available. TODO: Add a setting to allow prioritizing either higher resolution/larger shading rate attachment targets 
		// or lower-resolution/smaller attachments.
		shadingRateAttachmentTexelSize = { (uint32)GRHIVariableRateShadingImageTileMinWidth, (uint32)GRHIVariableRateShadingImageTileMinHeight };
	}

	void SetReference(FVulkanAttachmentReference<VkAttachmentReference2>* AttachmentReference)
	{
		pFragmentShadingRateAttachment = AttachmentReference;
	}
};

struct FVulkanDepthStencilResolveSubpassDesc
	: public VkSubpassDescriptionDepthStencilResolveKHR
{
	FVulkanDepthStencilResolveSubpassDesc()
	{
		ZeroVulkanStruct(*this, VK_STRUCTURE_TYPE_SUBPASS_DESCRIPTION_DEPTH_STENCIL_RESOLVE);
	}

	void SetResolveModes(VkResolveModeFlagBits DepthMode, VkResolveModeFlagBits StencilMode)
	{
		depthResolveMode = DepthMode;
		stencilResolveMode = StencilMode;
	}

	void SetReference(FVulkanAttachmentReference<VkAttachmentReference2>* AttachmentReference)
	{
		pDepthStencilResolveAttachment = AttachmentReference;
	}
};

class FVulkanRenderPass
{
public:
	inline const FVulkanRenderTargetLayout& GetLayout() const
	{
		return Layout;
	}

	inline VkRenderPass GetHandle() const
	{
		return RenderPass;
	}

	inline uint32 GetNumUsedClearValues() const
	{
		return NumUsedClearValues;
	}

private:
	friend class FVulkanRenderPassManager;
	friend class FVulkanPipelineStateCacheManager;

	FVulkanRenderPass(FVulkanDevice& Device, const FVulkanRenderTargetLayout& RTLayout);
	~FVulkanRenderPass();

private:
	FVulkanRenderTargetLayout	Layout;
	VkRenderPass				RenderPass;
	uint32						NumUsedClearValues;
	FVulkanDevice& Device;
};

extern int32 GVulkanInputAttachmentShaderRead;

template <typename TSubpassDescriptionClass, typename TSubpassDependencyClass, typename TAttachmentReferenceClass, typename TAttachmentDescriptionClass, typename TRenderPassCreateInfoClass>
class FVulkanRenderPassBuilder
{
public:
	FVulkanRenderPassBuilder(FVulkanDevice& InDevice)
		: Device(InDevice)
		, CorrelationMask(0)
	{}

	void BuildCreateInfo(const FVulkanRenderTargetLayout& RTLayout)
	{
		uint32 NumSubpasses = 0;
		uint32 NumDependencies = 0;

		//0b11 for 2, 0b1111 for 4, and so on
		uint32 MultiviewMask = (0b1 << RTLayout.GetMultiViewCount()) - 1;

		const bool bDeferredShadingSubpass = RTLayout.GetSubpassHint() == ESubpassHint::DeferredShadingSubpass;
		const bool bApplyFragmentShadingRate =  GRHISupportsAttachmentVariableRateShading
												&& RTLayout.GetFragmentDensityAttachmentReference() != nullptr
												&& Device.GetOptionalExtensions().HasKHRFragmentShadingRate 
												&& Device.GetOptionalExtensionProperties().FragmentShadingRateFeatures.attachmentFragmentShadingRate == VK_TRUE;
		const bool bResolveDepth =  GRHISupportsDepthStencilResolve &&
									Device.GetOptionalExtensions().HasKHRDepthStencilResolve &&
									RTLayout.GetHasDepthStencilResolve();

		const bool bCustomResolveSubpass = RTLayout.GetSubpassHint() == ESubpassHint::CustomResolveSubpass;
		const bool bDepthReadSubpass = bCustomResolveSubpass || (RTLayout.GetSubpassHint() == ESubpassHint::DepthReadSubpass);
		const bool bHasDepthStencilAttachmentReference = (RTLayout.GetDepthAttachmentReference() != nullptr);

		if (bApplyFragmentShadingRate)
		{
			ShadingRateAttachmentReference.SetAttachment(*RTLayout.GetFragmentDensityAttachmentReference(), VkImageAspectFlagBits::VK_IMAGE_ASPECT_COLOR_BIT);
			FragmentShadingRateAttachmentInfo.SetReference(&ShadingRateAttachmentReference);
		}

		if (bResolveDepth)
		{
			DepthStencilResolveAttachmentReference.SetAttachment(*RTLayout.GetDepthStencilResolveAttachmentReference(), VkImageAspectFlagBits::VK_IMAGE_ASPECT_NONE);
			// Using zero bit because it is always supported if the extension is supported, from spec: "The VK_RESOLVE_MODE_SAMPLE_ZERO_BIT mode
			// is the only mode that is required of all implementations (that support the extension or support Vulkan 1.2 or higher)."
			DepthStencilResolveSubpassDesc.SetResolveModes(VK_RESOLVE_MODE_SAMPLE_ZERO_BIT, VK_RESOLVE_MODE_SAMPLE_ZERO_BIT);
			DepthStencilResolveSubpassDesc.SetReference(&DepthStencilResolveAttachmentReference);
		}

		// Grab (and optionally convert) attachment references.
		uint32 NumColorAttachments = RTLayout.GetNumColorAttachments();
		for (uint32 ColorAttachment = 0; ColorAttachment < NumColorAttachments; ++ColorAttachment)
		{
			ColorAttachmentReferences.Add(TAttachmentReferenceClass(RTLayout.GetColorAttachmentReferences()[ColorAttachment], 0));
			if (RTLayout.GetResolveAttachmentReferences() != nullptr)
			{
				ResolveAttachmentReferences.Add(TAttachmentReferenceClass(RTLayout.GetResolveAttachmentReferences()[ColorAttachment], 0));
			}
		}

		// CustomResolveSubpass has an additional color attachment that should not be used by main and depth subpasses
		if (bCustomResolveSubpass && (NumColorAttachments > 1))
		{
			NumColorAttachments--;
		}

		uint32_t DepthInputAttachment = VK_ATTACHMENT_UNUSED;
		VkImageLayout DepthInputAttachmentLayout = VK_IMAGE_LAYOUT_UNDEFINED;
		VkImageAspectFlags DepthInputAspectMask = 0;
		if (bHasDepthStencilAttachmentReference)
		{
			DepthStencilAttachmentReference.SetDepthStencilAttachment(*RTLayout.GetDepthAttachmentReference(), RTLayout.GetStencilAttachmentReference(), 0, Device.SupportsParallelRendering());

			if (bDepthReadSubpass || bDeferredShadingSubpass)
			{
				DepthStencilAttachment.attachment = RTLayout.GetDepthAttachmentReference()->attachment;
				DepthStencilAttachment.SetAspect(VK_IMAGE_ASPECT_DEPTH_BIT);	// @todo?

				// FIXME: checking a Depth layout is not correct in all cases
				// PSO cache can create a PSO for subpass 1 or 2 first, where depth is read-only but that does not mean depth pre-pass is enabled
				if (false && RTLayout.GetDepthAttachmentReference()->layout == VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL)
				{
					// Depth is read only and is expected to be sampled as a regular texture
					DepthStencilAttachment.layout = VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL;
				}
				else
				{
					// lights write to stencil for culling, so stencil is expected to be writebale while depth is read-only
					DepthStencilAttachment.layout = VK_IMAGE_LAYOUT_DEPTH_READ_ONLY_STENCIL_ATTACHMENT_OPTIMAL;
					DepthInputAttachment = DepthStencilAttachment.attachment;
					DepthInputAttachmentLayout = DepthStencilAttachment.layout;
					DepthInputAspectMask = VK_IMAGE_ASPECT_DEPTH_BIT;
				}
			}
		}

		// main sub-pass
		{
			TSubpassDescriptionClass& SubpassDesc = SubpassDescriptions[NumSubpasses++];

			SubpassDesc.SetColorAttachments(ColorAttachmentReferences, NumColorAttachments);

			if (bHasDepthStencilAttachmentReference)
			{
				SubpassDesc.SetDepthStencilAttachment(&DepthStencilAttachmentReference);
			}

			if (!bDepthReadSubpass && bResolveDepth)
			{
				SubpassDesc.SetDepthStencilResolveAttachment(&DepthStencilResolveSubpassDesc);
			}

			if (bApplyFragmentShadingRate)
			{
				SubpassDesc.SetShadingRateAttachment(&FragmentShadingRateAttachmentInfo);
			}
			SubpassDesc.SetMultiViewMask(MultiviewMask);
		}

		// Color write and depth read sub-pass
		if (bDepthReadSubpass)
		{
			TSubpassDescriptionClass& SubpassDesc = SubpassDescriptions[NumSubpasses++];

			SubpassDesc.SetColorAttachments(ColorAttachmentReferences, 1);

			check(RTLayout.GetDepthAttachmentReference());

			// Depth as Input0
			InputAttachments1[0].attachment = DepthInputAttachment;
			InputAttachments1[0].layout = DepthInputAttachmentLayout;
			InputAttachments1[0].SetAspect(DepthInputAspectMask);
			SubpassDesc.SetInputAttachments(InputAttachments1, InputAttachment1Count);
			// depth attachment is same as input attachment
			SubpassDesc.SetDepthStencilAttachment(&DepthStencilAttachment);

			if (bResolveDepth && !bCustomResolveSubpass)
			{
				SubpassDesc.SetDepthStencilResolveAttachment(&DepthStencilResolveSubpassDesc);
			}

			if (bApplyFragmentShadingRate)
			{
				SubpassDesc.SetShadingRateAttachment(&FragmentShadingRateAttachmentInfo);
			}
			SubpassDesc.SetMultiViewMask(MultiviewMask);

			TSubpassDependencyClass& SubpassDep = SubpassDependencies[NumDependencies++];
			SubpassDep.srcSubpass = 0;
			SubpassDep.dstSubpass = 1;
			SubpassDep.srcStageMask = VK_PIPELINE_STAGE_EARLY_FRAGMENT_TESTS_BIT | VK_PIPELINE_STAGE_LATE_FRAGMENT_TESTS_BIT;
			SubpassDep.dstStageMask = VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT;
			SubpassDep.srcAccessMask = VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_WRITE_BIT;
			SubpassDep.dstAccessMask = VK_ACCESS_INPUT_ATTACHMENT_READ_BIT;
			SubpassDep.dependencyFlags = VK_DEPENDENCY_BY_REGION_BIT;
		}

		// Two subpasses for deferred shading
		if (bDeferredShadingSubpass)
		{
			//const VkAttachmentReference* ColorRef = RTLayout.GetColorAttachmentReferences();
			//uint32 NumColorAttachments = RTLayout.GetNumColorAttachments();
			//check(RTLayout.GetNumColorAttachments() == 5); //current layout is SceneColor, GBufferA/B/C/D

			// 1. Write to SceneColor and GBuffer, input DepthStencil
			{
				TSubpassDescriptionClass& SubpassDesc = SubpassDescriptions[NumSubpasses++];
				SubpassDesc.SetColorAttachments(ColorAttachmentReferences);
				SubpassDesc.SetDepthStencilAttachment(&DepthStencilAttachment);
				InputAttachments1[0].attachment = DepthInputAttachment;
				InputAttachments1[0].layout = DepthInputAttachmentLayout;
				InputAttachments1[0].SetAspect(DepthInputAspectMask);
				SubpassDesc.SetInputAttachments(InputAttachments1, InputAttachment1Count);

				if (bApplyFragmentShadingRate)
				{
					SubpassDesc.SetShadingRateAttachment(&FragmentShadingRateAttachmentInfo);
				}
				SubpassDesc.SetMultiViewMask(MultiviewMask);

				// Depth as Input0
				TSubpassDependencyClass& SubpassDep = SubpassDependencies[NumDependencies++];
				SubpassDep.srcSubpass = 0;
				SubpassDep.dstSubpass = 1;
				SubpassDep.srcStageMask = VK_PIPELINE_STAGE_COLOR_ATTACHMENT_OUTPUT_BIT | VK_PIPELINE_STAGE_EARLY_FRAGMENT_TESTS_BIT | VK_PIPELINE_STAGE_LATE_FRAGMENT_TESTS_BIT;
				SubpassDep.dstStageMask = VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT;
				SubpassDep.srcAccessMask = VK_ACCESS_COLOR_ATTACHMENT_WRITE_BIT | VK_ACCESS_DEPTH_STENCIL_ATTACHMENT_WRITE_BIT;
				SubpassDep.dstAccessMask = VK_ACCESS_INPUT_ATTACHMENT_READ_BIT;
				SubpassDep.dependencyFlags = VK_DEPENDENCY_BY_REGION_BIT;
			}

			// 2. Write to SceneColor, input GBuffer and DepthStencil
			{
				TSubpassDescriptionClass& SubpassDesc = SubpassDescriptions[NumSubpasses++];
				SubpassDesc.SetColorAttachments(ColorAttachmentReferences, 1); // SceneColor only
				SubpassDesc.SetDepthStencilAttachment(&DepthStencilAttachment);

				// Depth as Input0
				InputAttachments2[0].attachment = DepthInputAttachment;
				InputAttachments2[0].layout = DepthInputAttachmentLayout;
				InputAttachments2[0].SetAspect(DepthInputAspectMask);

				// SceneColor write only
				InputAttachments2[1].attachment = VK_ATTACHMENT_UNUSED;
				InputAttachments2[1].layout = VK_IMAGE_LAYOUT_UNDEFINED;
				InputAttachments2[1].SetAspect(0);

				// GBufferA/B/C/D as Input2/3/4/5
				int32 NumColorInputs = ColorAttachmentReferences.Num() - 1;
				for (int32 i = 2; i < (NumColorInputs + 2); ++i)
				{
					InputAttachments2[i].attachment = ColorAttachmentReferences[i - 1].attachment;
					InputAttachments2[i].layout = VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL;
					InputAttachments2[i].SetAspect(VK_IMAGE_ASPECT_COLOR_BIT);
				}

				SubpassDesc.SetInputAttachments(InputAttachments2, NumColorInputs + 2);
				if (bApplyFragmentShadingRate)
				{
					SubpassDesc.SetShadingRateAttachment(&FragmentShadingRateAttachmentInfo);
				}
				SubpassDesc.SetMultiViewMask(MultiviewMask);

				TSubpassDependencyClass& SubpassDep = SubpassDependencies[NumDependencies++];
				SubpassDep.srcSubpass = 1;
				SubpassDep.dstSubpass = 2;
				SubpassDep.srcStageMask = VK_PIPELINE_STAGE_COLOR_ATTACHMENT_OUTPUT_BIT;
				SubpassDep.dstStageMask = VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT;
				SubpassDep.srcAccessMask = VK_ACCESS_COLOR_ATTACHMENT_WRITE_BIT;
				SubpassDep.dstAccessMask = VK_ACCESS_INPUT_ATTACHMENT_READ_BIT;
				if (GVulkanInputAttachmentShaderRead == 1)
				{
					// this is not required, but might flicker on some devices without
					SubpassDep.dstAccessMask |= VK_ACCESS_SHADER_READ_BIT;
				}
				SubpassDep.dependencyFlags = VK_DEPENDENCY_BY_REGION_BIT;
			}
		}

		// Custom resolve subpass
		if (bCustomResolveSubpass)
		{
			TSubpassDescriptionClass& SubpassDesc = SubpassDescriptions[NumSubpasses++];
			ColorAttachments3[0].attachment = ColorAttachmentReferences[1].attachment;
			ColorAttachments3[0].layout = VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;
			ColorAttachments3[0].SetAspect(VK_IMAGE_ASPECT_COLOR_BIT);
			
			InputAttachments3[0].attachment = VK_ATTACHMENT_UNUSED; // The subpass fetch logic expects depth in first attachment.
			InputAttachments3[0].layout = VK_IMAGE_LAYOUT_UNDEFINED;
			InputAttachments3[0].SetAspect(0);
			
			InputAttachments3[1].attachment = ColorAttachmentReferences[0].attachment; // SceneColor as input
			InputAttachments3[1].layout = VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL;
			InputAttachments3[1].SetAspect(VK_IMAGE_ASPECT_COLOR_BIT);

			SubpassDesc.SetDepthStencilAttachment(&DepthStencilAttachment);

			if (bResolveDepth)
			{
				SubpassDesc.SetDepthStencilResolveAttachment(&DepthStencilResolveSubpassDesc);
			}
		
			SubpassDesc.SetInputAttachments(InputAttachments3, 2);
			SubpassDesc.colorAttachmentCount = 1;
			SubpassDesc.pColorAttachments = ColorAttachments3;

			SubpassDesc.SetMultiViewMask(MultiviewMask);

			TSubpassDependencyClass& SubpassDep = SubpassDependencies[NumDependencies++];
			SubpassDep.srcSubpass = 1;
			SubpassDep.dstSubpass = 2;
			SubpassDep.srcStageMask = VK_PIPELINE_STAGE_COLOR_ATTACHMENT_OUTPUT_BIT;
			SubpassDep.dstStageMask = VK_PIPELINE_STAGE_FRAGMENT_SHADER_BIT;
			SubpassDep.srcAccessMask = VK_ACCESS_COLOR_ATTACHMENT_WRITE_BIT;
			SubpassDep.dstAccessMask = VK_ACCESS_INPUT_ATTACHMENT_READ_BIT;
			SubpassDep.dependencyFlags = VK_DEPENDENCY_BY_REGION_BIT;
		}

		// CustomResolveSubpass does a custom resolve into second color target and does not need resolve attachments
		if (!bCustomResolveSubpass)
		{
			if (bDepthReadSubpass && ResolveAttachmentReferences.Num() > 1)
			{
				// Handle SceneDepthAux resolve:
				// The depth read subpass has only a single color attachment (using more would not be compatible dual source blending), so make SceneDepthAux a resolve attachment of the first subpass instead
				ResolveAttachmentReferences.Add(TAttachmentReferenceClass{});
				ResolveAttachmentReferences.Last().attachment = VK_ATTACHMENT_UNUSED;
				Swap(ResolveAttachmentReferences.Last(), ResolveAttachmentReferences[0]);

				SubpassDescriptions[0].SetResolveAttachments(TArrayView<TAttachmentReferenceClass>(ResolveAttachmentReferences.GetData(), ResolveAttachmentReferences.Num() - 1));
				SubpassDescriptions[NumSubpasses - 1].SetResolveAttachments(TArrayView<TAttachmentReferenceClass>(&ResolveAttachmentReferences.Last(), 1));
			}
			else
			{
				// Only set resolve attachment on the last subpass
				SubpassDescriptions[NumSubpasses - 1].SetResolveAttachments(ResolveAttachmentReferences);
			}
		}

		for (uint32 Attachment = 0; Attachment < RTLayout.GetNumAttachmentDescriptions(); ++Attachment)
		{
			if (bHasDepthStencilAttachmentReference && (Attachment == DepthStencilAttachmentReference.attachment))
			{
				AttachmentDescriptions.Add(TAttachmentDescriptionClass(RTLayout.GetAttachmentDescriptions()[Attachment], RTLayout.GetStencilDesc(), Device.SupportsParallelRendering()));
			}
			else
			{
				AttachmentDescriptions.Add(TAttachmentDescriptionClass(RTLayout.GetAttachmentDescriptions()[Attachment]));
			}
		}

		CreateInfo.attachmentCount = AttachmentDescriptions.Num();
		CreateInfo.pAttachments = AttachmentDescriptions.GetData();
		CreateInfo.subpassCount = NumSubpasses;
		CreateInfo.pSubpasses = SubpassDescriptions;
		CreateInfo.dependencyCount = NumDependencies;
		CreateInfo.pDependencies = SubpassDependencies;

		/*
		Bit mask that specifies which view rendering is broadcast to
		0011 = Broadcast to first and second view (layer)
		*/
		ViewMask[0] = MultiviewMask;
		ViewMask[1] = MultiviewMask;

		/*
		Bit mask that specifices correlation between views
		An implementation may use this for optimizations (concurrent render)
		*/
		CorrelationMask = MultiviewMask;

		if (RTLayout.GetIsMultiView())
		{
			if (Device.GetOptionalExtensions().HasKHRRenderPass2)
			{
				CreateInfo.SetCorrelationMask(&CorrelationMask);
			}
			else
			{
				checkf(Device.GetOptionalExtensions().HasKHRMultiview, TEXT("Layout is multiview but extension is not supported!"));
				MultiviewInfo.subpassCount = NumSubpasses;
				MultiviewInfo.pViewMasks = ViewMask;
				MultiviewInfo.dependencyCount = 0;
				MultiviewInfo.pViewOffsets = nullptr;
				MultiviewInfo.correlationMaskCount = 1;
				MultiviewInfo.pCorrelationMasks = &CorrelationMask;

				MultiviewInfo.pNext = CreateInfo.pNext;
				CreateInfo.pNext = &MultiviewInfo;
			}
		}

		if (Device.GetOptionalExtensions().HasEXTFragmentDensityMap && RTLayout.GetHasFragmentDensityAttachment())
		{
			FragDensityCreateInfo.fragmentDensityMapAttachment = *RTLayout.GetFragmentDensityAttachmentReference();

			// Chain fragment density info onto create info and the rest of the pNexts
			// onto the fragment density info
			FragDensityCreateInfo.pNext = CreateInfo.pNext;
			CreateInfo.pNext = &FragDensityCreateInfo;
		}
	}

	VkRenderPass Create(const FVulkanRenderTargetLayout& RTLayout)
	{
		BuildCreateInfo(RTLayout);

		return CreateInfo.Create(Device);
	}

	TRenderPassCreateInfoClass& GetCreateInfo()
	{
		return CreateInfo;
	}

private:
	TSubpassDescriptionClass SubpassDescriptions[8];
	TSubpassDependencyClass SubpassDependencies[8];

	TArray<TAttachmentReferenceClass> ColorAttachmentReferences;
	TArray<TAttachmentReferenceClass> ResolveAttachmentReferences;

	// Color write and depth read sub-pass
	static const uint32 InputAttachment1Count = 1;
	TAttachmentReferenceClass InputAttachments1[InputAttachment1Count];

	// Two subpasses for deferred shading
	TAttachmentReferenceClass InputAttachments2[MaxSimultaneousRenderTargets + 1];
	TAttachmentReferenceClass DepthStencilAttachment;

	TAttachmentReferenceClass DepthStencilAttachmentReference;
	TArray<TAttachmentDescriptionClass> AttachmentDescriptions;

	// Tonemap subpass
	TAttachmentReferenceClass InputAttachments3[MaxSimultaneousRenderTargets + 1];
	TAttachmentReferenceClass ColorAttachments3[MaxSimultaneousRenderTargets + 1];

	FVulkanAttachmentReference<VkAttachmentReference2> ShadingRateAttachmentReference;
	FVulkanFragmentShadingRateAttachmentInfo FragmentShadingRateAttachmentInfo;

	// Depth stencil resolve
	FVulkanAttachmentReference<VkAttachmentReference2> DepthStencilResolveAttachmentReference;
	FVulkanDepthStencilResolveSubpassDesc DepthStencilResolveSubpassDesc;

	FVulkanRenderPassFragmentDensityMapCreateInfoEXT FragDensityCreateInfo;
	FVulkanRenderPassMultiviewCreateInfo MultiviewInfo;

	TRenderPassCreateInfoClass CreateInfo;
	FVulkanDevice& Device;

	uint32 ViewMask[2];
	uint32 CorrelationMask;
};

VkRenderPass CreateVulkanRenderPass(FVulkanDevice& Device, const FVulkanRenderTargetLayout& RTLayout);

struct FVulkanBeginRenderPassInfo
{
	FVulkanRenderPass& RenderPass;
	FVulkanFramebuffer& Framebuffer;
	bool bIsParallelRenderPass;
};

class FVulkanRenderPassManager
{
public:
	FVulkanRenderPassManager(FVulkanDevice& InDevice) : Device(InDevice) {}
	~FVulkanRenderPassManager();

	FVulkanFramebuffer* GetOrCreateFramebuffer(const FRHISetRenderTargetsInfo& RenderTargetsInfo, const FVulkanRenderTargetLayout& RTLayout, FVulkanRenderPass& RenderPass);
	FVulkanRenderPass* GetOrCreateRenderPass(const FVulkanRenderTargetLayout& RTLayout)
	{
		const uint32 RenderPassHash = RTLayout.GetRenderPassFullHash();

		{
			FRWScopeLock ScopedReadLock(RenderPassesLock, SLT_ReadOnly);
			FVulkanRenderPass** FoundRenderPass = RenderPasses.Find(RenderPassHash);
			if (FoundRenderPass)
			{
				return *FoundRenderPass;
			}
		}

		FVulkanRenderPass* RenderPass = new FVulkanRenderPass(Device, RTLayout);
		{
			FRWScopeLock ScopedWriteLock(RenderPassesLock, SLT_Write);
			FVulkanRenderPass** FoundRenderPass = RenderPasses.Find(RenderPassHash);
			if (FoundRenderPass)
			{
				delete RenderPass;
				return *FoundRenderPass;
			}
			RenderPasses.Add(RenderPassHash, RenderPass);
		}
		return RenderPass;
	}

	void BeginRenderPass(FVulkanCommandListContext& Context, const FRHIRenderPassInfo& RPInfo, const FVulkanRenderTargetLayout& RTLayout, const FVulkanBeginRenderPassInfo& BeginRenderPassInfo);
	void EndRenderPass(FVulkanCommandListContext& Context);

	FRWLock RenderPassesLock;
	FRWLock FramebuffersLock;

	void NotifyDeletedRenderTarget(VkImage Image);

private:

	TMap<uint32, FVulkanRenderPass*> RenderPasses;

	struct FFramebufferList
	{
		TArray<FVulkanFramebuffer*> Framebuffer;
	};
	TMap<uint32, FFramebufferList*> Framebuffers;

	FVulkanDevice& Device;
};
