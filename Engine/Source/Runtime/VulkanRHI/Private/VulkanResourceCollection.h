// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHICoreResourceCollection.h"
#include "VulkanResources.h"

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
using FVulkanResourceCollection = UE::RHICore::FGenericResourceCollection;

template<>
struct TVulkanResourceTraits<FRHIResourceCollection>
{
	using TConcreteType = FVulkanResourceCollection;
};
#endif