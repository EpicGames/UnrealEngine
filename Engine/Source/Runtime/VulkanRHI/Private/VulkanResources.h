// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanResources.h: Vulkan resource RHI definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHICommandList.h"
#include "VulkanPlatform.h"
#include "VulkanConfiguration.h"
#include "VulkanState.h"
#include "VulkanUtil.h"
#include "BoundShaderStateCache.h"
#include "VulkanShaderResources.h"
#include "VulkanMemory.h"
#include "Misc/ScopeRWLock.h"
#include "IVulkanDynamicRHI.h"

class FVulkanDevice;
class FVulkanQueue;
class FVulkanContextCommon;
class FVulkanCommandBuffer;
class FVulkanTexture;
class FVulkanBuffer;
class FVulkanLayout;
class FVulkanOcclusionQuery;
struct FRHITransientHeapAllocation;
struct FVulkanPendingBufferLock;
class FVulkanView;
class FVulkanViewableResource;
class FVulkanShaderResourceView;
class FVulkanUnorderedAccessView;
class FVulkanRenderQuery;
class FVulkanDynamicRHI;
struct FVulkanContextArray;

class FVulkanSyncPoint;
using FVulkanSyncPointRef = TRefCountPtr<FVulkanSyncPoint>;

namespace VulkanRHI
{
	class FDeviceMemoryAllocation;
}

enum
{
	NUM_OCCLUSION_QUERIES_PER_POOL = 4096,

	NUM_TIMESTAMP_QUERIES_PER_POOL = 1024,
};

// Mirror GPixelFormats with format information for buffers
extern VkFormat GVulkanBufferFormat[PF_MAX];

// Converts the internal texture dimension to Vulkan view type
inline VkImageViewType UETextureDimensionToVkImageViewType(ETextureDimension Dimension)
{
	switch (Dimension)
	{
	case ETextureDimension::Texture2D: return VK_IMAGE_VIEW_TYPE_2D;
	case ETextureDimension::Texture2DArray: return VK_IMAGE_VIEW_TYPE_2D_ARRAY;
	case ETextureDimension::Texture3D: return VK_IMAGE_VIEW_TYPE_3D;
	case ETextureDimension::TextureCube: return VK_IMAGE_VIEW_TYPE_CUBE;
	case ETextureDimension::TextureCubeArray: return VK_IMAGE_VIEW_TYPE_CUBE_ARRAY;
	default: checkNoEntry(); return VK_IMAGE_VIEW_TYPE_MAX_ENUM;
	}
}

/** This represents a vertex declaration that hasn't been combined with a specific shader to create a bound shader. */
class FVulkanVertexDeclaration : public FRHIVertexDeclaration
{
public:
	FVertexDeclarationElementList Elements;
	uint32 Hash;
	uint32 HashNoStrides;

	FVulkanVertexDeclaration(const FVertexDeclarationElementList& InElements, uint32 InHash, uint32 InHashNoStrides);

	virtual bool GetInitializer(FVertexDeclarationElementList& Out) final override
	{
		Out = Elements;
		return true;
	}

	static void EmptyCache();

	virtual uint32 GetPrecachePSOHash() const final override { return HashNoStrides; }
};

struct FGfxPipelineDesc;

class FVulkanShaderModule : public FRefCountedObject
{
	static FVulkanDevice* Device;
	VkShaderModule ActualShaderModule;
public:
	FVulkanShaderModule(FVulkanDevice* DeviceIn, VkShaderModule ShaderModuleIn) : ActualShaderModule(ShaderModuleIn) 
	{
		check(DeviceIn && (Device == DeviceIn || !Device));
		Device = DeviceIn;
	}
	virtual ~FVulkanShaderModule();
	VkShaderModule& GetVkShaderModule() { return ActualShaderModule; }
};

class FVulkanShader : public IRefCountedObject
{
protected:

	static FCriticalSection VulkanShaderModulesMapCS;

public:
	virtual ~FVulkanShader();

	void PurgeShaderModules();

	TRefCountPtr<FVulkanShaderModule> GetOrCreateHandle();

	TRefCountPtr<FVulkanShaderModule> GetOrCreateHandle(const FVulkanLayout* Layout, uint32 LayoutHash)
	{
		FScopeLock Lock(&VulkanShaderModulesMapCS);
		TRefCountPtr<FVulkanShaderModule>* Found = ShaderModules.Find(LayoutHash);
		if (Found)
		{
			return *Found;
		}

		return CreateHandle(Layout, LayoutHash);
	}
	
	TRefCountPtr<FVulkanShaderModule> GetOrCreateHandle(const FGfxPipelineDesc& Desc, const FVulkanLayout* Layout, uint32 LayoutHash)
	{
		FScopeLock Lock(&VulkanShaderModulesMapCS);
		if (NeedsSpirvInputAttachmentPatching(Desc))
		{
			LayoutHash = HashCombine(LayoutHash, 1);
		}
		
		TRefCountPtr<FVulkanShaderModule>* Found = ShaderModules.Find(LayoutHash);
		if (Found)
		{
			return *Found;
		}

		return CreateHandle(Desc, Layout, LayoutHash);
	}

	const FString& GetDebugName() const
	{
		return CodeHeader.DebugName;
	}

	// Name should be pointing to "main_"
	void GetEntryPoint(ANSICHAR* Name, int32 NameLength) const
	{
		FCStringAnsi::Snprintf(Name, NameLength, "main_%0.8x_%0.8x", SpirvContainer.GetSizeBytes(), CodeHeader.SpirvCRC);
	}

	FORCEINLINE const FVulkanShaderHeader& GetCodeHeader() const
	{
		return CodeHeader;
	}

	uint64 GetShaderKey() const
	{
		return ShaderKey;
	}

	// This provides a view of the raw spirv bytecode.
	// If it is stored compressed then the result of GetSpirvCode will contain the decompressed spirv.
	class FSpirvCode
	{
		friend class FVulkanShader;
		explicit FSpirvCode(TArray<uint32>&& UncompressedCodeIn) : UncompressedCode(MoveTemp(UncompressedCodeIn))
		{
			CodeView = UncompressedCode;
		}
		explicit FSpirvCode(TArrayView<uint32> UncompressedCodeView) : CodeView(UncompressedCodeView)	{	}
		TArrayView<uint32> CodeView;
		TArray<uint32> UncompressedCode;
	public:
		TArrayView<uint32> GetCodeView() {return CodeView;}
		int32 GetCodeSizeInBytes() {return CodeView.Num() * sizeof(uint32);}
	};

	inline FSpirvCode GetSpirvCode() const
	{
		return GetSpirvCode(SpirvContainer);
	}
	
	FSpirvCode GetPatchedSpirvCode(const FGfxPipelineDesc& Desc, const FVulkanLayout* Layout);

	TArray<FUniformBufferStaticSlot>& StaticSlots;

	void SetUsesBindless(bool bValue)
	{
		bUsesBindless = bValue;
	}

	bool UsesBindless() const
	{
		return bUsesBindless;
	}

	void WaitForAsyncCompile()
	{
		if (AsyncCompileTask.IsValid() && !AsyncCompileTask->IsComplete())
		{
			AsyncCompileTask->Wait();
		}
	}

	VkShaderEXT GetShaderObject() const
	{
		return ShaderObject;
	}

protected:

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
	FString							DebugEntryPoint;
#endif
	uint64							ShaderKey;

	/** External bindings for this shader. */
	FVulkanShaderHeader				CodeHeader;
	TMap<uint32, TRefCountPtr<FVulkanShaderModule>>	ShaderModules;
	const EShaderFrequency			Frequency;
	bool							bUsesBindless = false;

	VkPipeline						PipelineLibrary = VK_NULL_HANDLE;
	VkShaderEXT						ShaderObject = VK_NULL_HANDLE;
	FGraphEventRef					AsyncCompileTask;

protected:
	class FSpirvContainer
	{
		friend class FVulkanShader;
		TArray<uint8>	SpirvCode;
		int32 UncompressedSizeBytes = -1;
	public:
		bool IsCompressed() const {	return UncompressedSizeBytes != -1;	}
		int32 GetSizeBytes() const { return UncompressedSizeBytes >= 0 ? UncompressedSizeBytes : SpirvCode.Num(); }
		friend FArchive& operator<<(FArchive& Ar, class FVulkanShader::FSpirvContainer& SpirvContainer);
	} SpirvContainer;

	friend FArchive& operator<<(FArchive& Ar, class FVulkanShader::FSpirvContainer& SpirvContainer);
	static FSpirvCode PatchSpirvInputAttachments(FSpirvCode& SpirvCode);

	static FSpirvCode GetSpirvCode(const FSpirvContainer& Container);

protected:
	FVulkanDevice*					Device;

	TRefCountPtr<FVulkanShaderModule> CreateHandle(const FVulkanLayout* Layout, uint32 LayoutHash);
	TRefCountPtr<FVulkanShaderModule> CreateHandle(const FGfxPipelineDesc& Desc, const FVulkanLayout* Layout, uint32 LayoutHash);

	bool NeedsSpirvInputAttachmentPatching(const FGfxPipelineDesc& Desc) const;

	FVulkanShader(FVulkanDevice* InDevice, EShaderFrequency InFrequency, FVulkanShaderHeader&& InCodeHeader, FSpirvContainer&& InSpirvContainer, uint64 InShaderKey, TArray<FUniformBufferStaticSlot>& InStaticSlots);

	friend class FVulkanCommandListContext;
	friend class FVulkanPipelineStateCacheManager;
	friend class FVulkanComputeShaderState;
	friend class FVulkanComputePipeline;
	friend class FVulkanShaderFactory;
	friend class FVulkanShaderObjectManager;
	friend class FVulkanGenericPlatform;
};

/** This represents a vertex shader that hasn't been combined with a specific declaration to create a bound shader. */
template<typename BaseResourceType, EShaderFrequency ShaderType>
class TVulkanBaseShader : public BaseResourceType, public FVulkanShader
{
private:
	TVulkanBaseShader(FVulkanDevice* InDevice, FShaderResourceTable&& InSRT, FVulkanShaderHeader&& InCodeHeader, FSpirvContainer&& InSpirvContainer, uint64 InShaderKey)
		: BaseResourceType()
		, FVulkanShader(InDevice, ShaderType, MoveTemp(InCodeHeader), MoveTemp(InSpirvContainer), InShaderKey, BaseResourceType::StaticSlots)
	{
		BaseResourceType::ShaderResourceTable = MoveTemp(InSRT);
	}
	friend class FVulkanShaderFactory;
public:
	enum { StaticFrequency = ShaderType };

	// IRefCountedObject interface.
	virtual void AddRef() const override final
	{
		FRHIResource::AddRef();
	}
	virtual FReturnedRefCountValue Release() const override final
	{
		return FReturnedRefCountValue{ FRHIResource::Release() };
	}
	virtual FReturnedRefCountValue GetRefCount() const override final
	{
		return FReturnedRefCountValue{ FRHIResource::GetRefCount() };
	}
};

typedef TVulkanBaseShader<FRHIVertexShader, SF_Vertex>				FVulkanVertexShader;
typedef TVulkanBaseShader<FRHIPixelShader, SF_Pixel>				FVulkanPixelShader;
typedef TVulkanBaseShader<FRHIComputeShader, SF_Compute>			FVulkanComputeShader;
typedef TVulkanBaseShader<FRHIGeometryShader, SF_Geometry>			FVulkanGeometryShader;
typedef TVulkanBaseShader<FRHIMeshShader, SF_Mesh>					FVulkanMeshShader;
typedef TVulkanBaseShader<FRHIAmplificationShader, SF_Amplification> FVulkanTaskShader;

class FVulkanRayTracingShader : public FRHIRayTracingShader, public FVulkanShader
{
private:
	FVulkanRayTracingShader(FVulkanDevice* InDevice, EShaderFrequency InFrequency, FShaderResourceTable&& InSRT, FVulkanShaderHeader&& InCodeHeader, FSpirvContainer&& InSpirvContainer, uint64 InShaderKey)
		: FRHIRayTracingShader(InFrequency)
		, FVulkanShader(InDevice, InFrequency, MoveTemp(InCodeHeader), MoveTemp(InSpirvContainer), InShaderKey, FRHIRayTracingShader::StaticSlots)
	{
		ShaderResourceTable = MoveTemp(InSRT);
	}

	FSpirvContainer AnyHitSpirvContainer;
	FSpirvContainer IntersectionSpirvContainer;

	friend class FVulkanShaderFactory;

public:
	static const uint32 MainModuleIdentifier = 0;
	static const uint32 ClosestHitModuleIdentifier = MainModuleIdentifier;
	static const uint32 AnyHitModuleIdentifier = 1;
	static const uint32 IntersectionModuleIdentifier = 2;

	TRefCountPtr<FVulkanShaderModule> GetOrCreateHandle(uint32 ModuleIdentifier);

	// IRefCountedObject interface.
	virtual void AddRef() const override final
	{
		FRHIResource::AddRef();
	}
	virtual FReturnedRefCountValue Release() const override final
	{
		return FReturnedRefCountValue{ FRHIResource::Release() };
	}
	virtual FReturnedRefCountValue GetRefCount() const override final
	{
		return FReturnedRefCountValue{ FRHIResource::GetRefCount() };
	}
};

class FVulkanShaderFactory
{
public:
	~FVulkanShaderFactory();

	template <typename ShaderType>
	ShaderType* CreateShader(TArrayView<const uint8> Code, FVulkanDevice* Device);

	template <typename ShaderType>
	ShaderType* LookupShader(uint64 ShaderKey) const
	{
		if (ShaderKey)
		{
			FRWScopeLock ScopedLock(RWLock[ShaderType::StaticFrequency], SLT_ReadOnly);
			FVulkanShader* const * FoundShaderPtr = ShaderMap[ShaderType::StaticFrequency].Find(ShaderKey);
			if (FoundShaderPtr)
			{
				return static_cast<ShaderType*>(*FoundShaderPtr);
			}
		}
		return nullptr;
	}

	template <EShaderFrequency ShaderFrequency>
	FVulkanRayTracingShader* CreateRayTracingShader(TArrayView<const uint8> Code, FVulkanDevice* Device);

	void LookupGfxShaders(const uint64 InShaderKeys[ShaderStage::NumGraphicsStages], FVulkanShader* OutShaders[ShaderStage::NumGraphicsStages]) const;

	void OnDeleteShader(const FVulkanShader& Shader);

private:
	mutable FRWLock RWLock[SF_NumFrequencies];
	TMap<uint64, FVulkanShader*> ShaderMap[SF_NumFrequencies];
};

class FVulkanBoundShaderState : public FRHIBoundShaderState
{
public:
	FVulkanBoundShaderState(
		FRHIVertexDeclaration* InVertexDeclarationRHI,
		FRHIVertexShader* InVertexShaderRHI,
		FRHIPixelShader* InPixelShaderRHI,
		FRHIGeometryShader* InGeometryShaderRHI
	);

	virtual ~FVulkanBoundShaderState();

	FORCEINLINE FVulkanVertexShader*   GetVertexShader() const { return (FVulkanVertexShader*)CacheLink.GetVertexShader(); }
	FORCEINLINE FVulkanPixelShader*    GetPixelShader() const { return (FVulkanPixelShader*)CacheLink.GetPixelShader(); }
	FORCEINLINE FVulkanMeshShader*     GetMeshShader() const { return (FVulkanMeshShader*)CacheLink.GetMeshShader(); }
	FORCEINLINE FVulkanTaskShader*     GetTaskShader() const { return (FVulkanTaskShader*)CacheLink.GetAmplificationShader(); }
	FORCEINLINE FVulkanGeometryShader* GetGeometryShader() const { return (FVulkanGeometryShader*)CacheLink.GetGeometryShader(); }

	const FVulkanShader* GetShader(ShaderStage::EStage Stage) const
	{
		switch (Stage)
		{
		case ShaderStage::Vertex:		return GetVertexShader();
		case ShaderStage::Pixel:		return GetPixelShader();
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case ShaderStage::Mesh:			return GetMeshShader();
		case ShaderStage::Task:			return GetTaskShader();
#endif
#if VULKAN_SUPPORTS_GEOMETRY_SHADERS
		case ShaderStage::Geometry:	return GetGeometryShader();
#endif
		default: break;
		}
		checkf(0, TEXT("Invalid Shader Frequency %d"), (int32)Stage);
		return nullptr;
	}

private:
	FCachedBoundShaderStateLink_Threadsafe CacheLink;
};

struct FVulkanCpuReadbackBuffer
{
	VkBuffer Buffer;
	uint32 MipOffsets[MAX_TEXTURE_MIP_COUNT];
};


struct FVulkanTextureViewDesc
{
	VkImageViewType ViewType = VK_IMAGE_VIEW_TYPE_MAX_ENUM;
	VkImageAspectFlags AspectFlags = VK_IMAGE_ASPECT_NONE;
	EPixelFormat UEFormat = PF_Unknown;
	VkFormat Format = VK_FORMAT_UNDEFINED;
	uint32 FirstMip = 0;
	uint32 NumMips = VK_REMAINING_MIP_LEVELS;
	uint32 ArraySliceIndex = 0;
	uint32 NumArraySlices = VK_REMAINING_ARRAY_LAYERS;
	VkImageUsageFlags ImageUsageFlags = 0;
	VkSamplerYcbcrConversion SamplerYcbcrConversion = VK_NULL_HANDLE;
	bool bUseIdentitySwizzle = false;

	uint32 GetHash() const;
};

class FVulkanView
{
public:

	struct FInvalidatedState
	{
		bool bInitialized = false;
	};

	struct FTypedBufferView
	{
		VkBufferView View      = VK_NULL_HANDLE;
		uint32       ViewId    = 0;
		bool         bVolatile = false; // Whether source buffer is volatile
	};

	struct FStructuredBufferView
	{
		VkBuffer Buffer = VK_NULL_HANDLE;
		uint32 HandleId = 0;
		uint32 Offset   = 0;
		uint32 Size     = 0;
	};

	struct FAccelerationStructureView
	{
		VkAccelerationStructureKHR Handle = VK_NULL_HANDLE;
	};

	struct FTextureView
	{
		VkImageView View   = VK_NULL_HANDLE;
		VkImage     Image  = VK_NULL_HANDLE;
		uint32      ViewId = 0;
	};

	typedef TVariant<
		  FInvalidatedState
		, FTypedBufferView
		, FTextureView
		, FStructuredBufferView
		, FAccelerationStructureView
	> TStorage;

	enum EType
	{
		Null                  = TStorage::IndexOfType<FInvalidatedState     >(),
		TypedBuffer           = TStorage::IndexOfType<FTypedBufferView      >(),
		Texture               = TStorage::IndexOfType<FTextureView          >(),
		StructuredBuffer      = TStorage::IndexOfType<FStructuredBufferView >(),
		AccelerationStructure = TStorage::IndexOfType<FAccelerationStructureView>(),
	};

	FVulkanView(FVulkanDevice& InDevice, ERHIDescriptorType InDescriptorType);

	~FVulkanView();

	void Invalidate();

	EType GetViewType() const
	{
		return EType(Storage.GetIndex());
	}

	bool IsInitialized() const
	{
		return (GetViewType() != Null) || Storage.Get<FInvalidatedState>().bInitialized;
	}

	FTypedBufferView           const& GetTypedBufferView          () const { return Storage.Get<FTypedBufferView          >(); }
	FTextureView               const& GetTextureView              () const { return Storage.Get<FTextureView              >(); }
	FStructuredBufferView      const& GetStructuredBufferView     () const { return Storage.Get<FStructuredBufferView     >(); }
	FAccelerationStructureView const& GetAccelerationStructureView() const { return Storage.Get<FAccelerationStructureView>(); }

	void CreateTextureView(VkImage InImage, const FVulkanTextureViewDesc& ViewDesc);
	void UpdateTextureView(const FVulkanContextArray& Contexts, VkImage InImage, const FVulkanTextureViewDesc& ViewDesc);

	// NOTE: The InOffset applies to the FVulkanBuffer (it does not include any internal Allocation offsets that may exist)
	void InitAsTypedBufferView(const FVulkanContextArray& Contexts, FVulkanBuffer* Buffer, EPixelFormat Format, uint32 InOffset, uint32 InSize);

	// NOTE: The InOffset applies to the FVulkanBuffer (it does not include any internal Allocation offsets that may exist)
	void InitAsStructuredBufferView(const FVulkanContextArray& Contexts, FVulkanBuffer* Buffer, uint32 InOffset, uint32 InSize);
	void InitAsAccelerationStructureView(const FVulkanContextArray& Contexts, FVulkanBuffer* Buffer, uint32 Offset, uint32 Size);

	// No moving or copying
	FVulkanView(FVulkanView     &&) = delete;
	FVulkanView(FVulkanView const&) = delete;
	FVulkanView& operator = (FVulkanView     &&) = delete;
	FVulkanView& operator = (FVulkanView const&) = delete;

	FRHIDescriptorHandle GetBindlessHandle() const
	{
		return BindlessHandle;
	}

protected:
	void InitAsTextureView(const FVulkanContextArray& Contexts, VkImage InImage, const FVulkanTextureViewDesc& ViewDesc);

private:
	FVulkanDevice& Device;
	FRHIDescriptorHandle BindlessHandle;
	const ERHIDescriptorType DescriptorType;
	TStorage Storage;
};

class FVulkanLinkedView : public FVulkanView, public TIntrusiveLinkedList<FVulkanLinkedView>
{
protected:
	FVulkanLinkedView(FVulkanDevice& Device, ERHIDescriptorType DescriptorType)
		: FVulkanView(Device, DescriptorType)
	{}

	~FVulkanLinkedView()
	{
		Unlink();
	}

public:
	virtual void UpdateView(const FVulkanContextArray& Contexts) = 0;
};

class FVulkanViewableResource
{
public:
	virtual ~FVulkanViewableResource()
	{
		checkf(!HasLinkedViews(), TEXT("All linked views must have been removed before the underlying resource can be deleted."));
	}

	bool HasLinkedViews() const
	{
		return LinkedViews != nullptr;
	}

	// @todo convert views owned by the texture into proper
	// FVulkanView instances, then remove 'virtual' from this class
	virtual void UpdateLinkedViews(const FVulkanContextArray& Contexts);

private:
	friend FVulkanShaderResourceView;
	friend FVulkanUnorderedAccessView;
	FVulkanLinkedView* LinkedViews = nullptr;
};

struct FVulkanReservedResourceData
{
	struct FSparseAllocation
	{
		VulkanRHI::FVulkanAllocation Allocation;
		uint64 ResourceOffset = 0;
	};
	TArray<FSparseAllocation> Allocations;
	VkMemoryRequirements MemoryRequirements;
	uint64 LastBlockSize = 0;
	uint32 BlockCount = 0;
	uint32 NumCommittedTiles = 0;
};


enum class EImageOwnerType : uint8
{
	None,
	LocalOwner,
	ExternalOwner,
	Aliased
};

class FVulkanTexture : public FRHITexture, public FVulkanEvictable, public FVulkanViewableResource
{
public:
	// Constructor without command list: targets will not be cleared, cannot contain initial data, will be left in VK_IMAGE_LAYOUT_UNDEFINED.
	// Layout changes and initial data upload should be handled by FinalizeCreateTextureInternal
	FVulkanTexture(FVulkanDevice& InDevice, const FRHITextureCreateDesc& InCreateDesc, const FRHITransientHeapAllocation* InTransientHeapAllocation);

	// Construct from external resource.
	FVulkanTexture(FVulkanDevice& InDevice, const FRHITextureCreateDesc& InCreateDesc, VkImage InImage, const FVulkanRHIExternalImageDeleteCallbackInfo& InExternalImageDeleteCallbackInfo);

#if PLATFORM_ANDROID
	// Construct from Android Hardware buffer. HardwareBuffer_Desc could be extracted from HardwareBuffer, but adding it here makes the function signature unambigious
	FVulkanTexture(FVulkanDevice& InDevice, const FRHITextureCreateDesc& InCreateDesc, const AHardwareBuffer_Desc& HardwareBufferDesc, AHardwareBuffer* HardwareBuffer);
#endif

	// Aliasing constructor.
	FVulkanTexture(FVulkanDevice& InDevice, const FRHITextureCreateDesc& InCreateDesc, FTextureRHIRef& SrcTextureRHI);

	virtual ~FVulkanTexture();

	void AliasTextureResources(FTextureRHIRef& SrcTextureRHI);

	void UploadInitialData(FRHICommandListBase& RHICmdList, VulkanRHI::FStagingBuffer* UploadBuffer);
	void HostUploadInitialData(FRHICommandListBase& RHICmdList, const uint8* BulkData, uint32 BulkDataSize);

	// View with all mips/layers
	FVulkanView* DefaultView = nullptr;

	// View with all mips/layers, but if it's a Depth/Stencil, only the Depth view
	FVulkanView* PartialView = nullptr;

	FVulkanView* FindOrAddInternalView(const FVulkanTextureViewDesc& ViewDesc);

	FTextureRHIRef AliasedTexture;

	template<typename T>
	void DumpMemory(T Callback)
	{
		const FIntVector SizeXYZ = GetSizeXYZ();
		Callback(TEXT("FVulkanTexture"), GetName(), this, static_cast<FRHIResource*>(this), SizeXYZ.X, SizeXYZ.Y, SizeXYZ.Z, StorageFormat);
	}

	// FVulkanEvictable interface.
	bool CanMove() const override { return false; }
	bool CanEvict() const override { return false; }
	void Evict(FVulkanDevice& Device, const FVulkanContextArray& Contexts) override; ///evict to system memory
	void Move(FVulkanDevice& Device, const FVulkanContextArray& Contexts, VulkanRHI::FVulkanAllocation& NewAllocation) override; //move to a full new allocation
	FVulkanTexture* GetEvictableTexture() override { return CanEvict() ? this : nullptr; }

	bool GetTextureResourceInfo(FRHIResourceInfo& OutResourceInfo) const;

	void* GetNativeResource() const override final { return (void*)Image; }
	void* GetTextureBaseRHI() override final { return this; }

	virtual FRHIDescriptorHandle GetDefaultBindlessHandle() const override final
	{
		check(PartialView);
		return PartialView->GetBindlessHandle();
	}

	struct FImageCreateInfo
	{
		VkImageCreateInfo ImageCreateInfo;
		//only used when HasImageFormatListKHR is supported. Otherise VK_IMAGE_CREATE_MUTABLE_FORMAT_BIT is used.
		VkImageFormatListCreateInfoKHR ImageFormatListCreateInfo;
		//used when TexCreate_External is given
		VkExternalMemoryImageCreateInfoKHR ExternalMemImageCreateInfo;
		// Array of formats used for mutable formats
		TArray<VkFormat, TInlineAllocator<2>> FormatsUsed;
		VkImageCompressionFixedRateFlagsEXT CompressionFixedRateFlags;
		VkImageCompressionControlEXT CompressionControl;
	};

	// Separate method for creating VkImageCreateInfo
	static void GenerateImageCreateInfo(
		FImageCreateInfo& OutImageCreateInfo,
		FVulkanDevice& InDevice,
		const FRHITextureDesc& InDesc,
		VkFormat* OutStorageFormat = nullptr,
		VkFormat* OutViewFormat = nullptr,
		bool bForceLinearTexture = false);

	void DestroySurface();
	void InvalidateMappedMemory();
	void* GetMappedPointer();
		
	// FRHIResource overrides 
#if RHI_ENABLE_RESOURCE_INFO
	bool GetResourceInfo(FRHIResourceInfo& OutResourceInfo) const override;
#endif

	/**
	 * Returns how much memory is used by the surface
	 */
	uint32 GetMemorySize() const
	{
		return MemoryRequirements.size;
	}

	/**
	 * Returns one of the texture's mip-maps stride.
	 */
	void GetMipStride(uint32 MipIndex, uint32& Stride);

	/**
	* Returns how much memory a single mip uses.
	*/
	void GetMipSize(uint32 MipIndex, uint64& MipBytes);

	VkImageViewType GetViewType() const
	{
		return UETextureDimensionToVkImageViewType(GetDesc().Dimension);
	}

	VkImageTiling GetTiling() const
	{
		return Tiling;
	}

	uint32 GetNumberOfArrayLevels() const
	{
		switch (GetViewType())
		{
		case VK_IMAGE_VIEW_TYPE_1D:
		case VK_IMAGE_VIEW_TYPE_2D:
		case VK_IMAGE_VIEW_TYPE_3D:
			return 1;
		case VK_IMAGE_VIEW_TYPE_2D_ARRAY:
			return GetDesc().ArraySize;
		case VK_IMAGE_VIEW_TYPE_CUBE:
			return 6;
		case VK_IMAGE_VIEW_TYPE_CUBE_ARRAY:
			return 6 * GetDesc().ArraySize;
		default:
			ErrorInvalidViewType();
			return 1;
		}
	}
	VULKANRHI_API void ErrorInvalidViewType() const;

	// Full includes Depth+Stencil
	VkImageAspectFlags GetFullAspectMask() const
	{
		return FullAspectMask;
	}

	// Only Depth or Stencil
	VkImageAspectFlags GetPartialAspectMask() const
	{
		return PartialAspectMask;
	}

	bool IsDepthOrStencilAspect() const
	{
		return (FullAspectMask & (VK_IMAGE_ASPECT_DEPTH_BIT | VK_IMAGE_ASPECT_STENCIL_BIT)) != 0;
	}

	bool IsImageOwner() const
	{
		return (ImageOwnerType == EImageOwnerType::LocalOwner);
	}

	bool SupportsSampling() const
	{
		return EnumHasAllFlags(GPixelFormats[GetDesc().Format].Capabilities, EPixelFormatCapabilities::TextureSample) &&
			((ImageUsageFlags & VK_IMAGE_USAGE_SAMPLED_BIT) != 0);
	}

	VULKANRHI_API VkDeviceMemory GetAllocationHandle() const;
	VULKANRHI_API uint64 GetAllocationOffset() const;

	static void InternalLockWrite(FVulkanContextCommon& Context, FVulkanTexture* Surface, const VkBufferImageCopy& Region, VulkanRHI::FStagingBuffer* StagingBuffer, bool bNeedsTransitions);

	const FVulkanCpuReadbackBuffer* GetCpuReadbackBuffer() const { return CpuReadbackBuffer; }

	virtual void UpdateLinkedViews(const FVulkanContextArray& Contexts) override;

	FVulkanDevice* const Device;
	VkImage Image = VK_NULL_HANDLE;
	VkImageUsageFlags ImageUsageFlags = 0;
	
	// Removes SRGB if requested, used to upload data
	VkFormat StorageFormat = VK_FORMAT_UNDEFINED;

	// Format for SRVs, render targets
	VkFormat ViewFormat = VK_FORMAT_UNDEFINED;

	VkMemoryPropertyFlags MemProps = VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT;
	VkMemoryRequirements MemoryRequirements;

	FVulkanRHIExternalImageDeleteCallbackInfo ExternalImageDeleteCallbackInfo;

	// Only used when HasSeparateDepthStencilLayouts == false.
	TStaticArray<ERHIAccess, 2> AllPlanesTrackedAccess{ InPlace, ERHIAccess::Unknown };

	void SetInitialImageState(FRHICommandListBase& RHICmdList, VkImageLayout InitialLayout, bool bClear, const FClearValueBinding& ClearValueBinding, bool bIsTransientResource);

	TArray<VkSparseMemoryBind> CommitReservedResource(uint64 RequiredCommitSizeInBytes);

private:
	void InternalMoveSurface(FVulkanDevice& InDevice, FVulkanCommandListContext& Context, VulkanRHI::FVulkanAllocation& DestAllocation);

	static const VkImageUsageFlags DefaultImageUsageFlags = 0;
	void InitViews(VkImageUsageFlags ViewUsageFlags, VkSamplerYcbcrConversion SamplerYcbcrConversion = VK_NULL_HANDLE);
	void UpdateViews(const FVulkanContextArray& Contexts, VkImageUsageFlags ViewUsageFlags, VkSamplerYcbcrConversion SamplerYcbcrConversion = VK_NULL_HANDLE);

	VkImageTiling Tiling = VK_IMAGE_TILING_MAX_ENUM;
	VulkanRHI::FVulkanAllocation Allocation;
	VkImageAspectFlags FullAspectMask = 0;
	VkImageAspectFlags PartialAspectMask = 0;
	FVulkanCpuReadbackBuffer* CpuReadbackBuffer = nullptr;

	// Views with lifetime tied to this image
	TMap<uint32, FVulkanView*> InternalViews;

	TUniquePtr<FVulkanReservedResourceData> ReservedResourceData;

protected:
	EImageOwnerType ImageOwnerType;
};


class FVulkanBuffer : public FRHIBuffer, public FVulkanViewableResource
{
public:
	FVulkanBuffer(FVulkanDevice& InDevice, const FRHIBufferCreateDesc& CreateDesc, const FRHITransientHeapAllocation* InTransientHeapAllocation = nullptr);
	virtual ~FVulkanBuffer();

	inline const VulkanRHI::FVulkanAllocation& GetCurrentAllocation() const
	{
		return CurrentBufferAlloc.Alloc;
	}

	inline VkBuffer GetHandle() const
	{
		return (VkBuffer)GetCurrentAllocation().VulkanHandle;
	}

	inline bool IsVolatile() const
	{
		return EnumHasAnyFlags(GetUsage(), BUF_Volatile);
	}

	// Offset used for Binding a VkBuffer
	inline uint32 GetOffset() const
	{
		return GetCurrentAllocation().Offset;
	}

	// Remaining size from the current offset
	inline uint64 GetCurrentSize() const
	{
		return GetCurrentAllocation().Size;
	}

	inline VkDeviceAddress GetDeviceAddress() const
	{
		return CurrentBufferAlloc.DeviceAddress;
	}

	inline VkBufferUsageFlags GetBufferUsageFlags() const
	{
		return BufferUsageFlags;
	}

	inline VkIndexType GetIndexType() const
	{
		return (GetStride() == 4)? VK_INDEX_TYPE_UINT32 : VK_INDEX_TYPE_UINT16;
	}

	void* Lock(FRHICommandListBase& RHICmdList, EResourceLockMode LockMode, uint32 Size, uint32 Offset);
	void Unlock(FRHICommandListBase& RHICmdList);

	void TakeOwnership(FVulkanBuffer& Other);
	void ReleaseOwnership();

	template<typename T>
	void DumpMemory(T Callback)
	{
		Callback(TEXT("FVulkanBuffer"), FName(), this, 0, GetCurrentSize(), 1, 1, VK_FORMAT_UNDEFINED);
	}

	static VkBufferUsageFlags UEToVKBufferUsageFlags(FVulkanDevice& InDevice, EBufferUsageFlags InUEUsage, bool bZeroSize);

	struct FBufferAlloc
	{
		VulkanRHI::FVulkanAllocation Alloc;
		void* HostPtr = nullptr;
		VkDeviceAddress DeviceAddress = 0;
	};

	void* GetCurrentHostPointer()
	{
		return CurrentBufferAlloc.HostPtr;
	}

	void IncrementLockCounter()
	{
		LockCounter++;
	}

	// FRHIResource overrides
#if RHI_ENABLE_RESOURCE_INFO
	bool GetResourceInfo(FRHIResourceInfo& OutResourceInfo) const override
	{
		OutResourceInfo = FRHIResourceInfo{};
		OutResourceInfo.Name = GetName();
		OutResourceInfo.Type = GetType();
		OutResourceInfo.VRamAllocation.AllocationSize = GetCurrentSize();
		OutResourceInfo.IsTransient = CurrentBufferAlloc.Alloc.bTransient;

		return true;
	}
#endif

	TArray<VkSparseMemoryBind> CommitReservedResource(uint64 RequiredCommitSizeInBytes);

protected:

	// Will return a new allocation that can be used for this buffer (proper memory type and size)
	void AllocateMemory(FBufferAlloc& OutAlloc);

	FVulkanDevice& Device;

	VkBufferUsageFlags BufferUsageFlags;

	enum class ELockStatus : uint8
	{
		Unlocked,
		Locked,
		PersistentMapping,
	} LockStatus = ELockStatus::Unlocked;

	FBufferAlloc CurrentBufferAlloc;
	uint32 LockCounter = 0;

	TUniquePtr<FVulkanReservedResourceData> ReservedResourceData;

	friend class FVulkanCommandListContext;
	friend struct FRHICommandMultiBufferUnlock;
};

class FVulkanUniformBuffer : public FRHIUniformBuffer
{
public:
	FVulkanUniformBuffer(FVulkanDevice& Device, const FRHIUniformBufferLayout* InLayout, const void* Contents, EUniformBufferUsage InUsage, EUniformBufferValidation Validation);
	virtual ~FVulkanUniformBuffer();

	const TArray<TRefCountPtr<FRHIResource>>& GetResourceTable() const { return ResourceTable; }

	void UpdateResourceTable(const FRHIUniformBufferLayout& InLayout, const void* Contents, int32 ResourceNum);
	void UpdateResourceTable(TArrayView<FRHIResource*> Resources);

	VkBuffer GetBufferHandle() const
	{
		return Allocation.GetBufferHandle();
	}

	uint32 GetOffset() const
	{
		return Allocation.Offset;
	}

	const VulkanRHI::FVulkanAllocation& GetAllocation() const
	{
		return Allocation;
	}

	bool IsUniformView() const
	{
		return UniformViewSRV != nullptr;
	}

	FRHIDescriptorHandle GetBindlessHandle() const override
	{
		return BindlessHandle;
	}
	
	void UpdateBindlessHandle(const FVulkanContextArray& Contexts, bool bImmediate);
	VkDeviceAddress GetDeviceAddress() const;

	void SetupUniformBufferView();

	void AllocateMemory(const void* InitialData);

public:
	const EUniformBufferUsage Usage;

protected:

	FVulkanDevice& Device;
	VulkanRHI::FVulkanAllocation Allocation;
	FRHIDescriptorHandle BindlessHandle;
	FRHIShaderResourceView* UniformViewSRV = nullptr;
};

class FVulkanUnorderedAccessView final : public FRHIUnorderedAccessView, public FVulkanLinkedView
{
public:
	FVulkanUnorderedAccessView(FRHICommandListBase& RHICmdList, FVulkanDevice& InDevice, FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc);

	FVulkanViewableResource* GetBaseResource() const;
	void UpdateView(const FVulkanContextArray& Contexts) override;

	virtual FRHIDescriptorHandle GetBindlessHandle() const override
	{
		return FVulkanLinkedView::GetBindlessHandle();
	}

	void Clear(TRHICommandList_RecursiveHazardous<FVulkanCommandListContext>& RHICmdList, const void* ClearValue, bool bFloat);
};


class FVulkanShaderResourceView final : public FRHIShaderResourceView, public FVulkanLinkedView
{
public:
	FVulkanShaderResourceView(FRHICommandListBase& RHICmdList, FVulkanDevice& InDevice, FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc);

	FVulkanViewableResource* GetBaseResource() const;
	void UpdateView(const FVulkanContextArray& Contexts) override;

	virtual FRHIDescriptorHandle GetBindlessHandle() const override
	{
		return FVulkanLinkedView::GetBindlessHandle();
	}
};

class FVulkanVertexInputStateInfo
{
public:
	FVulkanVertexInputStateInfo();
	~FVulkanVertexInputStateInfo();

	void Generate(FVulkanVertexDeclaration* VertexDeclaration, uint32 VertexHeaderInOutAttributeMask);

	inline uint32 GetHash() const
	{
		check(Info.sType == VK_STRUCTURE_TYPE_PIPELINE_VERTEX_INPUT_STATE_CREATE_INFO);
		return Hash;
	}

	inline const VkPipelineVertexInputStateCreateInfo& GetInfo() const
	{
		return Info;
	}

	bool operator ==(const FVulkanVertexInputStateInfo& Other);

protected:
	VkPipelineVertexInputStateCreateInfo Info;
	uint32 Hash;

	uint32 BindingsNum;
	uint32 BindingsMask;

	//#todo-rco: Remove these TMaps
	TMap<uint32, uint32> BindingToStream;
	TMap<uint32, uint32> StreamToBinding;
	VkVertexInputBindingDescription Bindings[MaxVertexElementCount];

	uint32 AttributesNum;
	VkVertexInputAttributeDescription Attributes[MaxVertexElementCount];

	friend class FVulkanPendingGfxState;
	friend class FVulkanPipelineStateCacheManager;
};

// This class holds the staging area for packed global uniform buffers for a given shader
class FPackedUniformBuffers
{
public:
	// One buffer is a chunk of bytes
	typedef TArray<uint8> FPackedBuffer;

	void Init(const FVulkanShaderHeader& InCodeHeader, uint32& OutPackedUniformBufferStagingMask)
	{
		if (InCodeHeader.PackedGlobalsSize > 0)
		{
			check(PackedUniformBuffers.Num() == 0);
			PackedUniformBuffers.AddUninitialized(InCodeHeader.PackedGlobalsSize);
			OutPackedUniformBufferStagingMask = 1;
		}
		else
		{
			OutPackedUniformBufferStagingMask = 0;
		}
	}

	inline void SetPackedGlobalParameter(uint32 ByteOffset, uint32 NumBytes, const void* RESTRICT NewValue, uint32& InOutPackedUniformBufferStagingDirty)
	{
		check(ByteOffset + NumBytes <= (uint32)PackedUniformBuffers.Num());
		check((NumBytes & 3) == 0 && (ByteOffset & 3) == 0);
		uint32* RESTRICT RawDst = (uint32*)(PackedUniformBuffers.GetData() + ByteOffset);
		uint32* RESTRICT RawSrc = (uint32*)NewValue;
		uint32* RESTRICT RawSrcEnd = RawSrc + (NumBytes >> 2);

		bool bChanged = false;
		while (RawSrc != RawSrcEnd)
		{
			bChanged |= CopyAndReturnNotEqual(*RawDst++, *RawSrc++);
		}

		if (bChanged)
		{
			InOutPackedUniformBufferStagingDirty = 1;
		}
	}

	inline const FPackedBuffer& GetBuffer() const
	{
		return PackedUniformBuffers;
	}

protected:
	FPackedBuffer PackedUniformBuffers;
};

class FVulkanStagingBuffer : public FRHIStagingBuffer
{
	friend class FVulkanCommandListContext;
public:
	FVulkanStagingBuffer()
		: FRHIStagingBuffer()
	{
		check(!bIsLocked);
	}

	virtual ~FVulkanStagingBuffer();

	virtual void* Lock(uint32 Offset, uint32 NumBytes) final override;
	virtual void Unlock() final override;

	uint64 GetGPUSizeBytes() const override { return StagingBuffer ? StagingBuffer->GetSize() : 0; }

private:
	VulkanRHI::FStagingBuffer* StagingBuffer = nullptr;
	uint32 QueuedNumBytes = 0;
	// The staging buffer was allocated from this device.
	FVulkanDevice* Device;
};

class FVulkanGPUFence final : public FRHIGPUFence
{
public:
	FVulkanGPUFence(FVulkanDevice& InDevice, FName InName);
	virtual ~FVulkanGPUFence();

	virtual void Clear() override;
	virtual bool Poll() const override;
	virtual void Wait(FRHICommandListImmediate& RHICmdList, FRHIGPUMask GPUMask) const override;

protected:
	FVulkanDevice& Device;
	VkEvent Event = VK_NULL_HANDLE;
	FGraphEventRef SubmittedSyncPoint;
	FVulkanSyncPointRef CompletedSyncPoint;

	friend class FVulkanDynamicRHI;
};

template<class T>
struct TVulkanResourceTraits
{
};
template<>
struct TVulkanResourceTraits<FRHIVertexDeclaration>
{
	typedef FVulkanVertexDeclaration TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIVertexShader>
{
	typedef FVulkanVertexShader TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIMeshShader>
{
	typedef FVulkanMeshShader TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIAmplificationShader>
{
	typedef FVulkanTaskShader TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIGeometryShader>
{
	typedef FVulkanGeometryShader TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIPixelShader>
{
	typedef FVulkanPixelShader TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIComputeShader>
{
	typedef FVulkanComputeShader TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIRenderQuery>
{
	typedef FVulkanRenderQuery TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIUniformBuffer>
{
	typedef FVulkanUniformBuffer TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIBuffer>
{
	typedef FVulkanBuffer TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIShaderResourceView>
{
	typedef FVulkanShaderResourceView TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIUnorderedAccessView>
{
	typedef FVulkanUnorderedAccessView TConcreteType;
};

template<>
struct TVulkanResourceTraits<FRHISamplerState>
{
	typedef FVulkanSamplerState TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIRasterizerState>
{
	typedef FVulkanRasterizerState TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIDepthStencilState>
{
	typedef FVulkanDepthStencilState TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIBlendState>
{
	typedef FVulkanBlendState TConcreteType;
};

template<>
struct TVulkanResourceTraits<FRHIBoundShaderState>
{
	typedef FVulkanBoundShaderState TConcreteType;
};

template<>
struct TVulkanResourceTraits<FRHIStagingBuffer>
{
	typedef FVulkanStagingBuffer TConcreteType;
};

template<>
struct TVulkanResourceTraits<FRHIGPUFence>
{
	typedef FVulkanGPUFence TConcreteType;
};

template<typename TRHIType>
static FORCEINLINE typename TVulkanResourceTraits<TRHIType>::TConcreteType* ResourceCast(TRHIType* Resource)
{
	return static_cast<typename TVulkanResourceTraits<TRHIType>::TConcreteType*>(Resource);
}

template<typename TRHIType>
static FORCEINLINE const typename TVulkanResourceTraits<TRHIType>::TConcreteType* ResourceCast(const TRHIType* Resource)
{
	return static_cast<const typename TVulkanResourceTraits<TRHIType>::TConcreteType*>(Resource);
}

static FORCEINLINE FVulkanTexture* ResourceCast(FRHITexture* Texture)
{
	return static_cast<FVulkanTexture*>(Texture->GetTextureBaseRHI());
}

class FVulkanRayTracingScene;
class FVulkanRayTracingGeometry;
class FVulkanRayTracingShaderTable;
class FVulkanRayTracingPipelineState;
template<>
struct TVulkanResourceTraits<FRHIRayTracingScene>
{
	typedef FVulkanRayTracingScene TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIRayTracingGeometry>
{
	typedef FVulkanRayTracingGeometry TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIShaderBindingTable>
{
	typedef FVulkanRayTracingShaderTable TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIRayTracingPipelineState>
{
	typedef FVulkanRayTracingPipelineState TConcreteType;
};
template<>
struct TVulkanResourceTraits<FRHIRayTracingShader>
{
	typedef FVulkanRayTracingShader TConcreteType;
};
