// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanRHIPrivate.h"
#include "VulkanContext.h"

static VkImageViewType GetVkImageViewTypeForDimensionSRV(FRHIViewDesc::EDimension DescDimension, VkImageViewType TextureViewType)
{
	switch (DescDimension)
	{
	case FRHIViewDesc::EDimension::Texture2D:        return VK_IMAGE_VIEW_TYPE_2D;
	case FRHIViewDesc::EDimension::Texture2DArray:   return VK_IMAGE_VIEW_TYPE_2D_ARRAY;
	case FRHIViewDesc::EDimension::Texture3D:        return VK_IMAGE_VIEW_TYPE_3D;
	case FRHIViewDesc::EDimension::TextureCube:      return VK_IMAGE_VIEW_TYPE_CUBE;
	case FRHIViewDesc::EDimension::TextureCubeArray: return VK_IMAGE_VIEW_TYPE_CUBE_ARRAY;
	case FRHIViewDesc::EDimension::Unknown:          return TextureViewType;
	default: break;
	}

	checkf(false, TEXT("Unknown texture dimension value!"));
	return VK_IMAGE_VIEW_TYPE_MAX_ENUM;
}

inline ERHIDescriptorType GetSRVDescriptorType(const FRHIViewDesc& ViewDesc)
{
	if (ViewDesc.IsTexture())
	{
		// Sampled images aren't supported in R64, shadercompiler patches them to storage image
		if (ViewDesc.Texture.SRV.Format == PF_R64_UINT)
		{
			return ERHIDescriptorType::TextureUAV;
		}

		return ERHIDescriptorType::TextureSRV;
	}

	switch (ViewDesc.Buffer.SRV.BufferType)
	{
	case FRHIViewDesc::EBufferType::Raw:
	case FRHIViewDesc::EBufferType::Structured:
		return ERHIDescriptorType::BufferUAV;

	case FRHIViewDesc::EBufferType::Typed:
		return ERHIDescriptorType::TypedBufferSRV;

	case FRHIViewDesc::EBufferType::AccelerationStructure:
		return ERHIDescriptorType::AccelerationStructure;

	default:
		checkNoEntry();
		return ERHIDescriptorType::Invalid;
	}
}

FVulkanShaderResourceView::FVulkanShaderResourceView(FRHICommandListBase& RHICmdList, FVulkanDevice& InDevice, FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc)
	: FRHIShaderResourceView(InResource, InViewDesc)
	, FVulkanLinkedView(InDevice, GetSRVDescriptorType(InViewDesc))
{
	RHICmdList.EnqueueLambda([this](FRHICommandListBase&)
	{
		LinkHead(GetBaseResource()->LinkedViews);
		UpdateView({});
	});

	RHICmdList.RHIThreadFence(true);
}

FVulkanViewableResource* FVulkanShaderResourceView::GetBaseResource() const
{
	if (IsBuffer())
	{
		return ResourceCast(GetBuffer());
	}

	return ResourceCast(GetTexture());
}

void FVulkanShaderResourceView::UpdateView(const FVulkanContextArray& Contexts)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanSRVUpdateTime);
#endif

	Invalidate();

	if (IsBuffer())
	{
		FVulkanBuffer* Buffer = ResourceCast(GetBuffer());
		auto const Info = ViewDesc.Buffer.SRV.GetViewInfo(Buffer);

		if (!Info.bNullView)
		{
			switch (Info.BufferType)
			{
			case FRHIViewDesc::EBufferType::Raw:
			case FRHIViewDesc::EBufferType::Structured:
				InitAsStructuredBufferView(Contexts, Buffer, Info.OffsetInBytes, Info.SizeInBytes);
				break;

			case FRHIViewDesc::EBufferType::Typed:
				check(VKHasAllFlags(Buffer->GetBufferUsageFlags(), VK_BUFFER_USAGE_UNIFORM_TEXEL_BUFFER_BIT));
				InitAsTypedBufferView(Contexts, Buffer, Info.Format, Info.OffsetInBytes, Info.SizeInBytes);
				break;

			case FRHIViewDesc::EBufferType::AccelerationStructure:
				InitAsAccelerationStructureView(Contexts, Buffer, Info.OffsetInBytes, Info.SizeInBytes);
				break;

			default:
				checkNoEntry();
				break;
			}
		}
	}
	else
	{
		FVulkanTexture* Texture = ResourceCast(GetTexture());
		auto const Info = ViewDesc.Texture.SRV.GetViewInfo(Texture);

		uint32 ArrayFirst = Info.ArrayRange.First;
		uint32 ArrayNum = Info.ArrayRange.Num;
		if (Info.Dimension == FRHIViewDesc::EDimension::TextureCube || Info.Dimension == FRHIViewDesc::EDimension::TextureCubeArray)
		{
			ArrayFirst *= 6;
			ArrayNum *= 6;
			checkf((ArrayFirst + ArrayNum) <= Texture->GetNumberOfArrayLevels(), TEXT("View extends beyond original cube texture level count!"));
		}

		// Remove storage bit from usage flags for SRVs
		const VkImageUsageFlags ImageUsageFlags = Texture->ImageUsageFlags & ~VK_IMAGE_USAGE_STORAGE_BIT;

		FVulkanTextureViewDesc VulkanViewDesc;
		VulkanViewDesc.ViewType = GetVkImageViewTypeForDimensionSRV(Info.Dimension, Texture->GetViewType());
		VulkanViewDesc.AspectFlags = Texture->GetPartialAspectMask();
		VulkanViewDesc.UEFormat = Info.Format;
		VulkanViewDesc.Format = UEToVkTextureFormat(Info.Format, Info.bSRGB);
		VulkanViewDesc.FirstMip = Info.MipRange.First;
		VulkanViewDesc.NumMips = Info.MipRange.Num;
		VulkanViewDesc.ArraySliceIndex = ArrayFirst;
		VulkanViewDesc.NumArraySlices = ArrayNum;
		VulkanViewDesc.bUseIdentitySwizzle = false;
		VulkanViewDesc.ImageUsageFlags = ImageUsageFlags;
		InitAsTextureView(Contexts, Texture->Image, VulkanViewDesc);
	}
}

FShaderResourceViewRHIRef  FVulkanDynamicRHI::RHICreateShaderResourceView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
{
	return new FVulkanShaderResourceView(RHICmdList, *Device, Resource, ViewDesc);
}
