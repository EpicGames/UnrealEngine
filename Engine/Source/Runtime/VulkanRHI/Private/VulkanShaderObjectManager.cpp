// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanRHIPrivate.h"
#include "VulkanShaderObjectManager.h"
#include "VulkanResources.h"
#include "VulkanDevice.h"
#include "VulkanBindlessDescriptorManager.h"


static int32 GAllowAsyncShaderObjectCompilation = 1;
FAutoConsoleVariableRef CVarVulkanAllowAsyncShaderObjectCompilation(
	TEXT("r.Vulkan.AsyncShaderObjectCompilation"),
	GAllowAsyncShaderObjectCompilation,
	TEXT("If enabled, ShaderObjects will be compiled in async tasks. (default: 1)"),
	ECVF_ReadOnly
);



FVulkanShaderObjectManager::FVulkanShaderObjectManager(FVulkanDevice& InDevice)
	: Device(InDevice)
{
}

FVulkanShaderObjectManager::~FVulkanShaderObjectManager()
{
	// Move pending async jobs into an array we won't need to lock, and wait for them to complete
	TArray<TRefCountPtr<FVulkanShader>> ShaderArray;
	{
		FScopeLock ScopeLock(&AsyncTaskLock);
		ShaderArray = AsyncTasks.Array();
		AsyncTasks.Empty();
	}
	for (TRefCountPtr<FVulkanShader>& ShaderRef : ShaderArray)
	{
		if (ShaderRef->AsyncCompileTask.IsValid() && !ShaderRef->AsyncCompileTask->IsComplete())
		{
			ShaderRef->AsyncCompileTask->Wait();
		}
	}
}

VkShaderStageFlags FVulkanShaderObjectManager::GetPossibleNextStages(VkShaderStageFlagBits Stage)
{
	switch (Stage)
	{
	case VK_SHADER_STAGE_VERTEX_BIT:
		return Device.GetPhysicalDeviceFeatures().Core_1_0.geometryShader ?
			VK_SHADER_STAGE_GEOMETRY_BIT | VK_SHADER_STAGE_FRAGMENT_BIT : VK_SHADER_STAGE_FRAGMENT_BIT;

	case VK_SHADER_STAGE_GEOMETRY_BIT: 
	case VK_SHADER_STAGE_MESH_BIT_EXT:
		return VK_SHADER_STAGE_FRAGMENT_BIT;

	case VK_SHADER_STAGE_TASK_BIT_EXT:
		return VK_SHADER_STAGE_MESH_BIT_EXT;

	case VK_SHADER_STAGE_FRAGMENT_BIT:
	case VK_SHADER_STAGE_COMPUTE_BIT:
		return 0;
	}

	checkNoEntry();
	return 0;
}

void FVulkanShaderObjectManager::CreateShaderObject(FVulkanShader& VulkanShader)
{
	if (GAllowAsyncShaderObjectCompilation)
	{
		FScopeLock ScopeLock(&AsyncTaskLock);
		VulkanShader.AsyncCompileTask = FFunctionGraphTask::CreateAndDispatchWhenReady([ShaderObjectManager = this, &VulkanShader]()
		{
			ShaderObjectManager->InternalCreateShaderObject(VulkanShader);
		});
		AsyncTasks.Emplace(&VulkanShader);
	}
	else
	{
		InternalCreateShaderObject(VulkanShader);
	}
}

void FVulkanShaderObjectManager::InternalCreateShaderObject(FVulkanShader& VulkanShader)
{
	checkf(VulkanShader.UsesBindless(), TEXT("Shader Objects currently only supported in bindless."));

	const VkShaderStageFlagBits Stage = UEFrequencyToVKStageBit(VulkanShader.Frequency);
	FVulkanShader::FSpirvCode SpirvCode = VulkanShader.GetSpirvCode();

	ANSICHAR EntryPoint[24];
	VulkanShader.GetEntryPoint(EntryPoint, 24);

	TConstArrayView<VkDescriptorSetLayout> DescriptorSetLayout = Device.GetBindlessDescriptorManager()->GetDescriptorSetLayouts();
	TConstArrayView<VkPushConstantRange> PushConstantRanges = Device.GetBindlessDescriptorManager()->GetPushConstantRanges();

	VkShaderCreateInfoEXT ShaderCreateInfo = {
		.sType = VK_STRUCTURE_TYPE_SHADER_CREATE_INFO_EXT,
		.pNext = nullptr,
		.flags = 0,
		.stage = Stage,
		.nextStage = GetPossibleNextStages(Stage),
		.codeType = VK_SHADER_CODE_TYPE_SPIRV_EXT,
		.codeSize = (size_t)SpirvCode.GetCodeSizeInBytes(),
		.pCode = SpirvCode.GetCodeView().GetData(),
		.pName = EntryPoint,
		.setLayoutCount = (uint32)DescriptorSetLayout.Num(),
		.pSetLayouts = DescriptorSetLayout.GetData(),
		.pushConstantRangeCount = (uint32)PushConstantRanges.Num(),
		.pPushConstantRanges = PushConstantRanges.GetData(),
		.pSpecializationInfo = nullptr
	};

	VkShaderDescriptorSetAndBindingMappingInfoEXT BindingMappingInfo;
	if (Device.GetBindlessDescriptorManager()->UseDescriptorHeaps())
	{
		ShaderCreateInfo.flags |= VK_SHADER_CREATE_DESCRIPTOR_HEAP_BIT_EXT;

		TConstArrayView<VkDescriptorSetAndBindingMappingEXT> BindingMappings = Device.GetBindlessDescriptorManager()->GetBindingMappings(VulkanShader.Frequency, (VulkanShader.GetCodeHeader().PackedGlobalsSize > 0));
		BindingMappingInfo = {
			.sType = VK_STRUCTURE_TYPE_SHADER_DESCRIPTOR_SET_AND_BINDING_MAPPING_INFO_EXT,
			.pNext = nullptr,
			.mappingCount = (uint32)BindingMappings.Num(),
			.pMappings = BindingMappings.GetData()
		};
		AddToPNext(ShaderCreateInfo, BindingMappingInfo);
	}

	const uint32 WaveSize = VulkanShader.GetCodeHeader().WaveSize;
	VkPipelineShaderStageRequiredSubgroupSizeCreateInfo RequiredSubgroupSizeCreateInfo;
	if ((WaveSize > 0) && Device.GetOptionalExtensions().HasEXTSubgroupSizeControl)
	{
		const VkPhysicalDeviceSubgroupSizeControlPropertiesEXT& SubgroupSizeControlProperties = Device.GetOptionalExtensionProperties().SubgroupSizeControlProperties;
		const bool bSupportedStage = (VKHasAllFlags(SubgroupSizeControlProperties.requiredSubgroupSizeStages, Stage));
		const bool bSupportedSize = ((WaveSize >= SubgroupSizeControlProperties.minSubgroupSize) && (WaveSize <= SubgroupSizeControlProperties.maxSubgroupSize));
		if (bSupportedStage && bSupportedSize)
		{
			RequiredSubgroupSizeCreateInfo = {
				.sType = VK_STRUCTURE_TYPE_PIPELINE_SHADER_STAGE_REQUIRED_SUBGROUP_SIZE_CREATE_INFO,
				.pNext = nullptr,
				.requiredSubgroupSize = WaveSize
			};

			AddToPNext(ShaderCreateInfo, RequiredSubgroupSizeCreateInfo);
		}
	}

	// :todo-jn: Task shaders never used at the moment
	if (Stage == VK_SHADER_STAGE_MESH_BIT_EXT)
	{
		ShaderCreateInfo.flags |= VK_SHADER_CREATE_NO_TASK_SHADER_BIT_EXT;
	}

	// :todo-jn: Could have a penalty, should narrow when it's applied
	if (GRHISupportsAttachmentVariableRateShading && (Stage == VK_SHADER_STAGE_FRAGMENT_BIT))
	{
		if (ValidateShadingRateDataType())
		{
			if (GRHIVariableRateShadingImageDataType == VRSImage_Palette)
			{
				ShaderCreateInfo.flags |= VK_SHADER_CREATE_FRAGMENT_SHADING_RATE_ATTACHMENT_BIT_EXT;
			}
			if (GRHIVariableRateShadingImageDataType == VRSImage_Fractional)
			{
				ShaderCreateInfo.flags |= VK_SHADER_CREATE_FRAGMENT_DENSITY_MAP_ATTACHMENT_BIT_EXT;
			}
		}
	}

	check(VulkanShader.ShaderObject == VK_NULL_HANDLE);
	VERIFYVULKANRESULT(VulkanRHI::vkCreateShadersEXT(Device.GetHandle(), 1, &ShaderCreateInfo, VULKAN_CPU_ALLOCATOR, &VulkanShader.ShaderObject));

	if (!VulkanShader.GetCodeHeader().DebugName.IsEmpty())
	{
		VULKAN_SET_DEBUG_NAME(Device, VK_OBJECT_TYPE_SHADER_EXT, VulkanShader.ShaderObject, TEXT("%s"), *VulkanShader.GetCodeHeader().DebugName);
	}

	{
		FScopeLock ScopeLock(&AsyncTaskLock);
		AsyncTasks.Remove(&VulkanShader);
	}
}

bool FVulkanShaderObjectManager::CreateLinkedShaderObjects(FVulkanGraphicsPipelineState* PSO, FVulkanShader* Shaders[ShaderStage::NumGraphicsStages], FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType PSOCompileType)
{
	// :todo-jn: Create new linked binaries from SPIRV with VK_SHADER_CREATE_LINK_STAGE_BIT_EXT
	// :todo-jn: Calls to bind will intercept and replace with linked binaries when present.
	// :todo-jn: These are the binaries will be cached.
	return true;
}

