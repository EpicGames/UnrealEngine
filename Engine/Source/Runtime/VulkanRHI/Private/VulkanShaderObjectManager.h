// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VulkanPlatform.h"
#include "VulkanCommon.h"
#include "RHIResources.h"

class FVulkanDevice;
class FVulkanShader;
class FVulkanGraphicsPipelineState;


class FVulkanShaderObjectManager
{
public:
	FVulkanShaderObjectManager(FVulkanDevice& InDevice);
	~FVulkanShaderObjectManager();

	void CreateShaderObject(FVulkanShader& VulkanShader);
	bool CreateLinkedShaderObjects(FVulkanGraphicsPipelineState* PSO, FVulkanShader* Shaders[ShaderStage::NumGraphicsStages], FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType PSOCompileType);

private:
	VkShaderStageFlags GetPossibleNextStages(VkShaderStageFlagBits Stage);
	void InternalCreateShaderObject(FVulkanShader& VulkanShader);

	FVulkanDevice& Device;

	FCriticalSection AsyncTaskLock;
	TSet<TRefCountPtr<FVulkanShader>> AsyncTasks;
};