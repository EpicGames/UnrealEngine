// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanShaders.cpp: Vulkan shader RHI implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanPendingState.h"
#include "VulkanContext.h"
#include "GlobalShader.h"
#include "Serialization/MemoryReader.h"
#include "VulkanLLM.h"
#include "VulkanDescriptorSets.h"
#include "RHICoreShader.h"
#include "VulkanBindlessDescriptorManager.h"
#include "VulkanShaderObjectManager.h"


extern int32 GVulkanGraphicPipelineLibraryLinkingMode;


static int32 GVulkanCompressSPIRV = 0;
static FAutoConsoleVariableRef GVulkanCompressSPIRVCVar(
	TEXT("r.Vulkan.CompressSPIRV"),
	GVulkanCompressSPIRV,
	TEXT("0 SPIRV source is stored in RAM as-is. (default)\n")
	TEXT("1 SPIRV source is compressed on load and decompressed as when needed, this saves RAM but can introduce hitching when creating shaders."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);
FCriticalSection FVulkanShader::VulkanShaderModulesMapCS;

FVulkanShaderFactory::~FVulkanShaderFactory()
{
	for (auto& Map : ShaderMap)
	{
		Map.Empty();
	}
}

template <typename ShaderType>
static void ReadShaderOptionalData(FShaderCodeReader& ShaderCode, ShaderType* RHIShader)
{
	const FShaderCodePackedResourceCounts* PackedResourceCounts = ShaderCode.FindOptionalData<FShaderCodePackedResourceCounts>();
	if (PackedResourceCounts)
	{
		if (RHIShader->GetFrequency() == SF_Compute)
		{
			RHIShader->SetNoDerivativeOps(EnumHasAnyFlags(PackedResourceCounts->UsageFlags, EShaderResourceUsageFlags::NoDerivativeOps));
		}
		RHIShader->SetShaderBundleUsage(EnumHasAnyFlags(PackedResourceCounts->UsageFlags, EShaderResourceUsageFlags::ShaderBundle));
		RHIShader->SetUsesBindless(EnumHasAnyFlags(PackedResourceCounts->UsageFlags, EShaderResourceUsageFlags::BindlessSamplers | EShaderResourceUsageFlags::BindlessResources));
	}

#if RHI_INCLUDE_SHADER_DEBUG_DATA
	RHIShader->Debug.ShaderName = ShaderCode.FindOptionalData(FShaderCodeName::Key);
	UE::RHICore::SetupShaderCodeValidationData(RHIShader, ShaderCode);
#endif
}

static VkPipeline CreatePreRasterShaderLibrary(FVulkanDevice* Device, FVulkanShader* Shader, VkShaderStageFlagBits StageFlag)
{
	if (StageFlag == VK_SHADER_STAGE_VERTEX_BIT)
	{
		INC_DWORD_STAT(STAT_VulkanNumVSLibs);
	}
	else if (StageFlag == VK_SHADER_STAGE_MESH_BIT_EXT)
	{
		INC_DWORD_STAT(STAT_VulkanNumMSLibs);
	}
	else
	{
		checkNoEntry();
	}

#if VULKAN_ENABLE_AGGRESSIVE_STATS
	TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(VulkanVSLibTime, (StageFlag == VK_SHADER_STAGE_VERTEX_BIT));
	TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(VulkanMSLibTime, (StageFlag == VK_SHADER_STAGE_MESH_BIT_EXT));
#endif

	const VkGraphicsPipelineLibraryCreateInfoEXT LibraryInfo = {
		.sType = VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_LIBRARY_CREATE_INFO_EXT,
		.pNext = nullptr,
		.flags = VK_GRAPHICS_PIPELINE_LIBRARY_PRE_RASTERIZATION_SHADERS_BIT_EXT
	};

	FVulkanShader::FSpirvCode SpirvCode = Shader->GetSpirvCode();
	const VkShaderModuleCreateInfo ModuleCreateInfo = {
		.sType = VK_STRUCTURE_TYPE_SHADER_MODULE_CREATE_INFO,
		.pNext = nullptr,
		.flags = 0,
		.codeSize = (size_t)SpirvCode.GetCodeSizeInBytes(),
		.pCode = SpirvCode.GetCodeView().GetData()
	};

	ANSICHAR EntryPoint[24];
	Shader->GetEntryPoint(EntryPoint, 24);
	const VkPipelineShaderStageCreateInfo ShaderStageCreateInfo = {
		.sType = VK_STRUCTURE_TYPE_PIPELINE_SHADER_STAGE_CREATE_INFO,
		.pNext = &ModuleCreateInfo,
		.flags = 0,
		.stage = StageFlag,
		.module = VK_NULL_HANDLE, // specified in pNext
		.pName = EntryPoint,
		.pSpecializationInfo = nullptr // :todo: create every specialization
	};

	// See table for pre-raster: https://docs.vulkan.org/guide/latest/dynamic_state_map.html
	const VkDynamicState PreRasterDynamicStates[] = {
		VK_DYNAMIC_STATE_DEPTH_BIAS,
		VK_DYNAMIC_STATE_CULL_MODE, // ext1
		VK_DYNAMIC_STATE_VIEWPORT_WITH_COUNT_EXT, // ext1
		VK_DYNAMIC_STATE_SCISSOR_WITH_COUNT_EXT, // ext1
		VK_DYNAMIC_STATE_DEPTH_BIAS_ENABLE,  // ext2
		VK_DYNAMIC_STATE_POLYGON_MODE_EXT  // ext3
	};

	const VkPipelineDynamicStateCreateInfo DynamicStateCreateInfo = {
		.sType = VK_STRUCTURE_TYPE_PIPELINE_DYNAMIC_STATE_CREATE_INFO,
		.pNext = nullptr,
		.flags = 0,
		.dynamicStateCount = UE_ARRAY_COUNT(PreRasterDynamicStates),
		.pDynamicStates = PreRasterDynamicStates
	};

	// These struct's important fields are dynamic, set the rest to the default
	VkPipelineRasterizationStateCreateInfo RasterizerState;
	FVulkanRasterizerState::ResetCreateInfo(RasterizerState);

	const VkPipelineCreateFlags2CreateInfo Flags2CreateInfo = {
		.sType = VK_STRUCTURE_TYPE_PIPELINE_CREATE_FLAGS_2_CREATE_INFO,
		.pNext = &LibraryInfo,
		.flags = VK_PIPELINE_CREATE_2_LIBRARY_BIT_KHR | 
				 Device->GetBindlessDescriptorManager()->GetPipelineCreateFlag() |
				 (GVulkanGraphicPipelineLibraryLinkingMode == 1 ? VK_PIPELINE_CREATE_2_RETAIN_LINK_TIME_OPTIMIZATION_INFO_BIT_EXT : 0)
	};

	VkGraphicsPipelineCreateInfo GraphicsPipelineCreateInfo;
	ZeroVulkanStruct(GraphicsPipelineCreateInfo, VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_CREATE_INFO);
	GraphicsPipelineCreateInfo.sType = VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_CREATE_INFO;
	GraphicsPipelineCreateInfo.pNext = &Flags2CreateInfo;
	GraphicsPipelineCreateInfo.flags = 0; // using VkPipelineCreateFlags2CreateInfo
	GraphicsPipelineCreateInfo.stageCount = 1;
	GraphicsPipelineCreateInfo.pStages = &ShaderStageCreateInfo;
	GraphicsPipelineCreateInfo.layout = Device->GetBindlessDescriptorManager()->GetPipelineLayout();
	GraphicsPipelineCreateInfo.pDynamicState = &DynamicStateCreateInfo;
	GraphicsPipelineCreateInfo.pRasterizationState = &RasterizerState;

	VkPipeline PipelineLibrary = VK_NULL_HANDLE;
	VERIFYVULKANRESULT(VulkanRHI::vkCreateGraphicsPipelines(Device->GetHandle(), VK_NULL_HANDLE, 1, &GraphicsPipelineCreateInfo, VULKAN_CPU_ALLOCATOR, &PipelineLibrary));
	return PipelineLibrary;
}

static VkPipeline CreatePixelShaderLibrary(FVulkanDevice* Device, FVulkanShader* PixelShader)
{
	INC_DWORD_STAT(STAT_VulkanNumPSLibs);

#if VULKAN_ENABLE_AGGRESSIVE_STATS
	TRACE_CPUPROFILER_EVENT_SCOPE(VulkanPSLibTime);
#endif

	const VkPipelineRenderingCreateInfo RenderingCreateInfo = {
		.sType = VK_STRUCTURE_TYPE_PIPELINE_RENDERING_CREATE_INFO,
		.pNext = nullptr,
		.viewMask = 0,  // :todo-jn: fill viewMask for multiview
		.colorAttachmentCount = 0, // not used
		.pColorAttachmentFormats = nullptr, // not used
		.depthAttachmentFormat = VK_FORMAT_UNDEFINED, // not used
		.stencilAttachmentFormat = VK_FORMAT_UNDEFINED // not used
	};

	const VkGraphicsPipelineLibraryCreateInfoEXT LibraryInfo = {
		.sType = VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_LIBRARY_CREATE_INFO_EXT,
		.pNext = &RenderingCreateInfo,
		.flags = VK_GRAPHICS_PIPELINE_LIBRARY_FRAGMENT_SHADER_BIT_EXT
	};

	FVulkanShader::FSpirvCode SpirvCode = PixelShader->GetSpirvCode();
	const VkShaderModuleCreateInfo ModuleCreateInfo = {
		.sType = VK_STRUCTURE_TYPE_SHADER_MODULE_CREATE_INFO,
		.pNext = nullptr,
		.flags = 0,
		.codeSize = (size_t)SpirvCode.GetCodeSizeInBytes(),
		.pCode = SpirvCode.GetCodeView().GetData()
	};

	ANSICHAR EntryPoint[24];
	PixelShader->GetEntryPoint(EntryPoint, 24);
	const VkPipelineShaderStageCreateInfo ShaderStageCreateInfo = {
		.sType = VK_STRUCTURE_TYPE_PIPELINE_SHADER_STAGE_CREATE_INFO,
		.pNext = &ModuleCreateInfo,
		.flags = 0,
		.stage = VK_SHADER_STAGE_FRAGMENT_BIT,
		.module = VK_NULL_HANDLE,  // specified in pNext chain
		.pName = EntryPoint,
		.pSpecializationInfo = nullptr // :todo: create every specialization
	};

	// See table for fragment shader: https://docs.vulkan.org/guide/latest/dynamic_state_map.html
	const VkDynamicState FragmentShaderDynamicStates[] = {
		VK_DYNAMIC_STATE_STENCIL_REFERENCE,
		VK_DYNAMIC_STATE_DEPTH_BOUNDS,
		VK_DYNAMIC_STATE_STENCIL_TEST_ENABLE, // ext1
		VK_DYNAMIC_STATE_STENCIL_OP, // ext1
		VK_DYNAMIC_STATE_DEPTH_COMPARE_OP, // ext1
		VK_DYNAMIC_STATE_DEPTH_TEST_ENABLE, // ext1
		VK_DYNAMIC_STATE_DEPTH_BOUNDS_TEST_ENABLE, // ext1
		VK_DYNAMIC_STATE_DEPTH_WRITE_ENABLE, // ext1
		VK_DYNAMIC_STATE_RASTERIZATION_SAMPLES_EXT, // ext3
		VK_DYNAMIC_STATE_ALPHA_TO_COVERAGE_ENABLE_EXT // ext3
	};

	const VkPipelineDynamicStateCreateInfo DynamicStateCreateInfo = {
		.sType = VK_STRUCTURE_TYPE_PIPELINE_DYNAMIC_STATE_CREATE_INFO,
		.pNext = nullptr,
		.flags = 0,
		.dynamicStateCount = UE_ARRAY_COUNT(FragmentShaderDynamicStates),
		.pDynamicStates = FragmentShaderDynamicStates
	};

	// These struct's important fields are dynamic, set the rest to the default
	VkPipelineMultisampleStateCreateInfo MultisampleStateCreateInfo;
	ZeroVulkanStruct(MultisampleStateCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_MULTISAMPLE_STATE_CREATE_INFO);
	VkPipelineDepthStencilStateCreateInfo DepthStencilStateCreateInfo;
	ZeroVulkanStruct(DepthStencilStateCreateInfo, VK_STRUCTURE_TYPE_PIPELINE_DEPTH_STENCIL_STATE_CREATE_INFO);

	const VkPipelineCreateFlags2CreateInfo Flags2CreateInfo = {
		.sType = VK_STRUCTURE_TYPE_PIPELINE_CREATE_FLAGS_2_CREATE_INFO,
		.pNext = &LibraryInfo,
		.flags = VK_PIPELINE_CREATE_2_LIBRARY_BIT_KHR |
				 Device->GetBindlessDescriptorManager()->GetPipelineCreateFlag() |
				 (GVulkanGraphicPipelineLibraryLinkingMode == 1 ? VK_PIPELINE_CREATE_2_RETAIN_LINK_TIME_OPTIMIZATION_INFO_BIT_EXT : 0)
	};

	VkGraphicsPipelineCreateInfo GraphicsPipelineCreateInfo;
	ZeroVulkanStruct(GraphicsPipelineCreateInfo, VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_CREATE_INFO);
	GraphicsPipelineCreateInfo.sType = VK_STRUCTURE_TYPE_GRAPHICS_PIPELINE_CREATE_INFO;
	GraphicsPipelineCreateInfo.pNext = &Flags2CreateInfo;
	GraphicsPipelineCreateInfo.flags = 0; // using VkPipelineCreateFlags2CreateInfo
	GraphicsPipelineCreateInfo.stageCount = 1;
	GraphicsPipelineCreateInfo.pStages = &ShaderStageCreateInfo;
	GraphicsPipelineCreateInfo.pMultisampleState = &MultisampleStateCreateInfo;
	GraphicsPipelineCreateInfo.pDepthStencilState = &DepthStencilStateCreateInfo;
	GraphicsPipelineCreateInfo.layout = Device->GetBindlessDescriptorManager()->GetPipelineLayout();
	GraphicsPipelineCreateInfo.pDynamicState = &DynamicStateCreateInfo;

	VkPipeline PipelineLibrary = VK_NULL_HANDLE;
	VERIFYVULKANRESULT(VulkanRHI::vkCreateGraphicsPipelines(Device->GetHandle(), VK_NULL_HANDLE, 1, &GraphicsPipelineCreateInfo, VULKAN_CPU_ALLOCATOR, &PipelineLibrary));
	return PipelineLibrary;
}

template <typename ShaderType> 
ShaderType* FVulkanShaderFactory::CreateShader(TArrayView<const uint8> Code, FVulkanDevice* Device)
{
	static_assert(ShaderType::StaticFrequency != SF_RayCallable && ShaderType::StaticFrequency != SF_RayGen && ShaderType::StaticFrequency != SF_RayHitGroup && ShaderType::StaticFrequency != SF_RayMiss);

	const uint32 ShaderCodeLen = Code.Num();
	const uint32 ShaderCodeCRC = FCrc::MemCrc32(Code.GetData(), Code.Num());
	const uint64 ShaderKey = ((uint64)ShaderCodeLen | ((uint64)ShaderCodeCRC << 32));

	ShaderType* RetShader = LookupShader<ShaderType>(ShaderKey);

	if (RetShader == nullptr)
	{
		// Do serialize outside of lock
		FMemoryReaderView Ar(Code, true);
		FVulkanShaderHeader CodeHeader;
		Ar << CodeHeader;
		FShaderResourceTable SerializedSRT;
		Ar << SerializedSRT;
		FVulkanShader::FSpirvContainer SpirvContainer;
		Ar << SpirvContainer;

		{
			FRWScopeLock ScopedLock(RWLock[ShaderType::StaticFrequency], SLT_Write);
			FVulkanShader* const* FoundShaderPtr = ShaderMap[ShaderType::StaticFrequency].Find(ShaderKey);
			if (FoundShaderPtr)
			{
				RetShader = static_cast<ShaderType*>(*FoundShaderPtr);
			}
			else
			{
				RetShader = new ShaderType(Device, MoveTemp(SerializedSRT), MoveTemp(CodeHeader), MoveTemp(SpirvContainer), ShaderKey);

				ShaderMap[ShaderType::StaticFrequency].Add(ShaderKey, RetShader);

				FShaderCodeReader ShaderCode(Code);
				ReadShaderOptionalData(ShaderCode, RetShader);

				if (FDataDrivenShaderPlatformInfo::GetSupportsSpecializationConstants(GMaxRHIShaderPlatform))
				{
					RetShader->SetSpecializationConstants(RetShader->GetCodeHeader().SpecializationConstantData.Values.Num() > 0);
					RetShader->SetSpecializationExclusions(RetShader->GetCodeHeader().SpecializationConstantData.GetExclusionMap());
				}

				if (Device->SupportsShaderObjects())
				{
					Device->GetShaderObjectManager()->CreateShaderObject(*RetShader);
				}
				else if (Device->SupportsGraphicPipelineLibraries())
				{
					if constexpr (std::is_same_v<ShaderType, FVulkanVertexShader>)
					{
						RetShader->PipelineLibrary = CreatePreRasterShaderLibrary(Device, RetShader, VK_SHADER_STAGE_VERTEX_BIT);
					}
					if constexpr (std::is_same_v<ShaderType, FVulkanMeshShader>)
					{
						RetShader->PipelineLibrary = CreatePreRasterShaderLibrary(Device, RetShader, VK_SHADER_STAGE_MESH_BIT_EXT);
					}
					if constexpr (std::is_same_v<ShaderType, FVulkanPixelShader>)
					{
						RetShader->PipelineLibrary = CreatePixelShaderLibrary(Device, RetShader);
					}
				}
			}
		}
	}

	return RetShader;
}

template <EShaderFrequency ShaderFrequency>
FVulkanRayTracingShader* FVulkanShaderFactory::CreateRayTracingShader(TArrayView<const uint8> Code, FVulkanDevice* Device)
{
	static_assert(ShaderFrequency == SF_RayCallable || ShaderFrequency == SF_RayGen || ShaderFrequency == SF_RayHitGroup || ShaderFrequency == SF_RayMiss);

	auto LookupRayTracingShader = [this](uint64 ShaderKey)
	{
		FVulkanRayTracingShader* RTShader = nullptr;
		if (ShaderKey)
		{
			FRWScopeLock ScopedLock(RWLock[ShaderFrequency], SLT_ReadOnly);
			FVulkanShader* const* FoundShaderPtr = ShaderMap[ShaderFrequency].Find(ShaderKey);
			if (FoundShaderPtr)
			{
				RTShader = static_cast<FVulkanRayTracingShader*>(*FoundShaderPtr);
			}
		}
		return RTShader;
	};

	const uint32 ShaderCodeLen = Code.Num();
	const uint32 ShaderCodeCRC = FCrc::MemCrc32(Code.GetData(), Code.Num());
	const uint64 ShaderKey = ((uint64)ShaderCodeLen | ((uint64)ShaderCodeCRC << 32));

	FVulkanRayTracingShader* RetShader = LookupRayTracingShader(ShaderKey);

	if (RetShader == nullptr)
	{
		// Do serialize outside of lock
		FMemoryReaderView Ar(Code, true);
		FVulkanShaderHeader CodeHeader;
		Ar << CodeHeader;
		FShaderResourceTable SerializedSRT;
		Ar << SerializedSRT;
		FVulkanShader::FSpirvContainer SpirvContainer;
		Ar << SpirvContainer;

		const bool bIsHitGroup = (ShaderFrequency == SF_RayHitGroup);
		FVulkanShader::FSpirvContainer AnyHitSpirvContainer;
		FVulkanShader::FSpirvContainer IntersectionSpirvContainer;
		if (bIsHitGroup)
		{
			if (CodeHeader.RayGroupAnyHit == FVulkanShaderHeader::ERayHitGroupEntrypoint::SeparateBlob)
			{
				Ar << AnyHitSpirvContainer;
			}
			if (CodeHeader.RayGroupIntersection == FVulkanShaderHeader::ERayHitGroupEntrypoint::SeparateBlob)
			{
				Ar << IntersectionSpirvContainer;
			}
		}

		{
			FRWScopeLock ScopedLock(RWLock[ShaderFrequency], SLT_Write);
			FVulkanShader* const* FoundShaderPtr = ShaderMap[ShaderFrequency].Find(ShaderKey);
			if (FoundShaderPtr)
			{
				RetShader = static_cast<FVulkanRayTracingShader*>(*FoundShaderPtr);
			}
			else
			{
				RetShader = new FVulkanRayTracingShader(Device, ShaderFrequency, MoveTemp(SerializedSRT), MoveTemp(CodeHeader), MoveTemp(SpirvContainer), ShaderKey);

				if (bIsHitGroup)
				{
					RetShader->AnyHitSpirvContainer = MoveTemp(AnyHitSpirvContainer);
					RetShader->IntersectionSpirvContainer = MoveTemp(IntersectionSpirvContainer);
				}
				RetShader->RayTracingPayloadType = RetShader->CodeHeader.RayTracingPayloadType;
				RetShader->RayTracingPayloadSize = RetShader->CodeHeader.RayTracingPayloadSize;

				ShaderMap[ShaderFrequency].Add(ShaderKey, RetShader);

				FShaderCodeReader ShaderCode(Code);
				ReadShaderOptionalData(ShaderCode, RetShader);
			}
		}
	}

	return RetShader;
}

void FVulkanShaderFactory::LookupGfxShaders(const uint64 InShaderKeys[ShaderStage::NumGraphicsStages], FVulkanShader* OutShaders[ShaderStage::NumGraphicsStages]) const
{
	for (int32 Idx = 0; Idx < ShaderStage::NumGraphicsStages; ++Idx)
	{
		uint64 ShaderKey = InShaderKeys[Idx];
		if (ShaderKey)
		{
			EShaderFrequency ShaderFrequency = ShaderStage::GetFrequencyForGfxStage((ShaderStage::EStage)Idx);
			FRWScopeLock ScopedLock(RWLock[ShaderFrequency], SLT_ReadOnly);
			FVulkanShader* const * FoundShaderPtr = ShaderMap[ShaderFrequency].Find(ShaderKey);
			if (FoundShaderPtr)
			{
				OutShaders[Idx] = *FoundShaderPtr;
			}
		}
	}
}

void FVulkanShaderFactory::OnDeleteShader(const FVulkanShader& Shader)
{
	const uint64 ShaderKey = Shader.GetShaderKey(); 
	FRWScopeLock ScopedLock(RWLock[Shader.Frequency], SLT_Write);
	ShaderMap[Shader.Frequency].Remove(ShaderKey);
}

FArchive& operator<<(FArchive& Ar, FVulkanShader::FSpirvContainer& SpirvContainer)
{
	uint32 SpirvCodeSizeInBytes;
	Ar << SpirvCodeSizeInBytes;
	check(SpirvCodeSizeInBytes);
	check(Ar.IsLoading());

	TArray<uint8>& SpirvCode = SpirvContainer.SpirvCode;

	if (!GVulkanCompressSPIRV)
	{
		SpirvCode.Reserve(SpirvCodeSizeInBytes);
		SpirvCode.SetNumUninitialized(SpirvCodeSizeInBytes);
		Ar.Serialize(SpirvCode.GetData(), SpirvCodeSizeInBytes);
	}
	else
	{
		const int32 CompressedUpperBound = FCompression::CompressMemoryBound(NAME_Oodle, SpirvCodeSizeInBytes);
		SpirvCode.Reserve(CompressedUpperBound);
		SpirvCode.SetNumUninitialized(CompressedUpperBound);

		TArray<uint8> UncompressedSpirv;
		UncompressedSpirv.SetNumUninitialized(SpirvCodeSizeInBytes);
		Ar.Serialize(UncompressedSpirv.GetData(), SpirvCodeSizeInBytes);

		int32 CompressedSizeBytes = CompressedUpperBound;
		if (FCompression::CompressMemory(NAME_Oodle, SpirvCode.GetData(), CompressedSizeBytes, UncompressedSpirv.GetData(), UncompressedSpirv.GetTypeSize() * UncompressedSpirv.Num(), ECompressionFlags::COMPRESS_BiasSpeed))
		{
			SpirvContainer.UncompressedSizeBytes = SpirvCodeSizeInBytes;
			SpirvCode.SetNumUninitialized(CompressedSizeBytes);
		}
		else
		{
			SpirvCode = MoveTemp(UncompressedSpirv);
		}
	}

	return Ar;
}

FVulkanDevice* FVulkanShaderModule::Device = nullptr;

FVulkanShaderModule::~FVulkanShaderModule()
{
	DEC_DWORD_STAT(STAT_VulkanNumShaderModule);
	Device->GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::ShaderModule, ActualShaderModule);
}

FVulkanShader::FSpirvCode FVulkanShader::GetSpirvCode(const FSpirvContainer& Container)
{
	if (Container.IsCompressed())
	{
		TArray<uint32> UncompressedSpirv;
		const size_t ElementSize = UncompressedSpirv.GetTypeSize();
		UncompressedSpirv.Reserve(Container.GetSizeBytes() / ElementSize);
		UncompressedSpirv.SetNumUninitialized(Container.GetSizeBytes() / ElementSize);
		FCompression::UncompressMemory(NAME_Oodle, UncompressedSpirv.GetData(), Container.GetSizeBytes(), Container.SpirvCode.GetData(), Container.SpirvCode.Num());

		return FSpirvCode(MoveTemp(UncompressedSpirv));
	}
	else
	{
		return FSpirvCode(TArrayView<uint32>((uint32*)Container.SpirvCode.GetData(), Container.SpirvCode.Num() / sizeof(uint32)));
	}
}


FVulkanShader::FVulkanShader(FVulkanDevice* InDevice, EShaderFrequency InFrequency, FVulkanShaderHeader&& InCodeHeader, FSpirvContainer&& InSpirvContainer, uint64 InShaderKey, TArray<FUniformBufferStaticSlot>& InStaticSlots)
	: StaticSlots(InStaticSlots)
	, ShaderKey(InShaderKey)
	, CodeHeader(MoveTemp(InCodeHeader))
	, Frequency(InFrequency)
	, SpirvContainer(MoveTemp(InSpirvContainer))
	, Device(InDevice)
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanShaders);
	check(Device);

	checkf(SpirvContainer.GetSizeBytes() != 0, TEXT("Empty SPIR-V! %s"), *CodeHeader.DebugName);

	const int32 NumGlobalPackedBuffer = (CodeHeader.PackedGlobalsSize > 0) ? 1 : 0;
	if (CodeHeader.UniformBufferInfos.Num() > NumGlobalPackedBuffer)
	{
		StaticSlots.Reserve(CodeHeader.UniformBufferInfos.Num());

		for (const FVulkanShaderHeader::FUniformBufferInfo& UBInfo : CodeHeader.UniformBufferInfos)
		{
			if (const FShaderParametersMetadata* Metadata = FindUniformBufferStructByLayoutHash(UBInfo.LayoutHash))
			{
				StaticSlots.Add(Metadata->GetLayout().StaticSlot);
			}
			else
			{
				StaticSlots.Add(MAX_UNIFORM_BUFFER_STATIC_SLOTS);
			}
		}
	}

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
	// main_00000000_00000000
	ANSICHAR EntryPoint[24];
	GetEntryPoint(EntryPoint, 24);
	DebugEntryPoint = EntryPoint;
#endif
}

static TRefCountPtr<FVulkanShaderModule> CreateShaderModule(FVulkanDevice* Device, FVulkanShader::FSpirvCode& SpirvCode)
{
	INC_DWORD_STAT(STAT_VulkanNumShaderModule);

#if VULKAN_ENABLE_AGGRESSIVE_STATS
	TRACE_CPUPROFILER_EVENT_SCOPE(VulkanShaderModuleTime);
#endif

	const TArrayView<uint32> Spirv = SpirvCode.GetCodeView();
	VkShaderModule ShaderModule;
	VkShaderModuleCreateInfo ModuleCreateInfo;
	ZeroVulkanStruct(ModuleCreateInfo, VK_STRUCTURE_TYPE_SHADER_MODULE_CREATE_INFO);
	//ModuleCreateInfo.flags = 0;

	ModuleCreateInfo.codeSize = Spirv.Num() * sizeof(uint32);
	ModuleCreateInfo.pCode = Spirv.GetData();

#if VULKAN_SUPPORTS_VALIDATION_CACHE
	VkShaderModuleValidationCacheCreateInfoEXT ValidationInfo;
	if (Device->GetOptionalExtensions().HasEXTValidationCache)
	{
		ZeroVulkanStruct(ValidationInfo, VK_STRUCTURE_TYPE_SHADER_MODULE_VALIDATION_CACHE_CREATE_INFO_EXT);
		ValidationInfo.validationCache = Device->GetValidationCache();
		ModuleCreateInfo.pNext = &ValidationInfo;
	}
#endif

	VERIFYVULKANRESULT(VulkanRHI::vkCreateShaderModule(Device->GetHandle(), &ModuleCreateInfo, VULKAN_CPU_ALLOCATOR, &ShaderModule));
	
	TRefCountPtr<FVulkanShaderModule> ReturnPtr = TRefCountPtr<FVulkanShaderModule>(new FVulkanShaderModule(Device, ShaderModule));

	return ReturnPtr;
}

/*
 *  Replace all subpassInput declarations with subpassInputMS
 *  Replace all subpassLoad(Input) with subpassLoad(Input, 0)
 */
FVulkanShader::FSpirvCode FVulkanShader::PatchSpirvInputAttachments(FVulkanShader::FSpirvCode& SpirvCode)
{
	TArrayView<uint32> InSpirv = SpirvCode.GetCodeView();
	const uint32 kHeaderLength = 5;
	const uint32 kOpTypeImage = 25;
	const uint32 kDimSubpassData = 6;
	const uint32 kOpImageRead = 98;
	const uint32 kOpLoad = 61;
	const uint32 kOpConstant = 43;
	const uint32 kOpTypeInt = 21;

	const uint32 Len = InSpirv.Num();
	// Make sure we at least have a header
	if (Len < kHeaderLength)
	{
		return SpirvCode;
	}

	TArray<uint32> OutSpirv;
	OutSpirv.Reserve(Len + 2);
	// Copy header
	OutSpirv.Append(&InSpirv[0], kHeaderLength);

	uint32 IntegerType = 0;
	uint32 Constant0 = 0;
	TArray<uint32, TInlineAllocator<4>> SubpassDataImages;
	
	for (uint32 Pos = kHeaderLength; Pos < Len;)
	{
		uint32* SpirvData = &InSpirv[Pos];
		const uint32 InstLen =	SpirvData[0] >> 16;
		const uint32 Opcode =	SpirvData[0] & 0x0000ffffu;
		bool bSkip = false;

		if (Opcode == kOpTypeInt && SpirvData[3] == 1)
		{
			// signed int
			IntegerType = SpirvData[1];
		}
		else if (Opcode == kOpConstant && SpirvData[1] == IntegerType && SpirvData[3] == 0)
		{
			// const signed int == 0
			Constant0 = SpirvData[2];
		}
		else if (Opcode == kOpTypeImage && SpirvData[3] == kDimSubpassData)
		{
			SpirvData[6] = 1; // mark as multisampled
			SubpassDataImages.Add(SpirvData[1]);
		}
		else if (Opcode == kOpLoad && SubpassDataImages.Contains(SpirvData[1]))
		{
			// pointers to our image
			SubpassDataImages.Add(SpirvData[2]);
		}
		else if (Opcode == kOpImageRead && SubpassDataImages.Contains(SpirvData[3]))
		{
			// const int 0, must be present as it's used for coord operand in image sampling
			check(Constant0 != 0);

			OutSpirv.Add((7u << 16) | kOpImageRead); // new instruction with 7 operands
			OutSpirv.Append(&SpirvData[1], 4); // copy existing operands
			OutSpirv.Add(0x40);			// Sample operand
			OutSpirv.Add(Constant0);	// Sample number
			bSkip = true;
		}

		if (!bSkip)
		{
			OutSpirv.Append(&SpirvData[0], InstLen);
		}
		Pos += InstLen;
	}
	return FVulkanShader::FSpirvCode(MoveTemp(OutSpirv));
}

bool FVulkanShader::NeedsSpirvInputAttachmentPatching(const FGfxPipelineDesc& Desc) const
{
	return (Desc.RasterizationSamples > 1 && CodeHeader.InputAttachmentsMask != 0);
}

TRefCountPtr<FVulkanShaderModule> FVulkanShader::CreateHandle(const FGfxPipelineDesc& Desc, const FVulkanLayout* Layout, uint32 LayoutHash)
{
	FScopeLock Lock(&VulkanShaderModulesMapCS);
	FSpirvCode Spirv = GetPatchedSpirvCode(Desc, Layout);

	TRefCountPtr<FVulkanShaderModule> Module = CreateShaderModule(Device, Spirv);
	ShaderModules.Add(LayoutHash, Module);
	return Module;
}

FVulkanShader::FSpirvCode FVulkanShader::GetPatchedSpirvCode(const FGfxPipelineDesc& Desc, const FVulkanLayout* Layout)
{
	FSpirvCode Spirv = GetSpirvCode();

	if (NeedsSpirvInputAttachmentPatching(Desc))
	{
		Spirv = PatchSpirvInputAttachments(Spirv);
	}

	return Spirv;
}

// Bindless variant of function that does not require layout for patching
TRefCountPtr<FVulkanShaderModule> FVulkanShader::GetOrCreateHandle()
{
	check(Device->SupportsBindless());
	FScopeLock Lock(&VulkanShaderModulesMapCS);

	const uint32 MainModuleIndex = 0;
	TRefCountPtr<FVulkanShaderModule>* Found = ShaderModules.Find(MainModuleIndex);
	if (Found)
	{
		return *Found;
	}

	FSpirvCode Spirv = GetSpirvCode();

	TRefCountPtr<FVulkanShaderModule> Module = CreateShaderModule(Device, Spirv);
	ShaderModules.Add(MainModuleIndex, Module);
	if (!CodeHeader.DebugName.IsEmpty())
	{
		VULKAN_SET_DEBUG_NAME((*Device), VK_OBJECT_TYPE_SHADER_MODULE, Module->GetVkShaderModule(), TEXT("%s : (FVulkanShader*)0x%p"), *CodeHeader.DebugName, this);
	}
	return Module;
}

TRefCountPtr<FVulkanShaderModule> FVulkanRayTracingShader::GetOrCreateHandle(uint32 ModuleIdentifier)
{
	check(Device->SupportsBindless());

	const bool bIsAnyHitModuleIdentifier = (ModuleIdentifier == AnyHitModuleIdentifier);
	const bool bIsIntersectionModuleIdentifier = (ModuleIdentifier == IntersectionModuleIdentifier);

	// If we're using a single blob with multiple entry points, forward everything to the main module
	if ((bIsAnyHitModuleIdentifier && (GetCodeHeader().RayGroupAnyHit == FVulkanShaderHeader::ERayHitGroupEntrypoint::CommonBlob)) ||
		(bIsIntersectionModuleIdentifier && (GetCodeHeader().RayGroupIntersection == FVulkanShaderHeader::ERayHitGroupEntrypoint::CommonBlob)))
	{
		return GetOrCreateHandle(MainModuleIdentifier);
	}

	FScopeLock Lock(&VulkanShaderModulesMapCS);

	TRefCountPtr<FVulkanShaderModule>* Found = ShaderModules.Find(ModuleIdentifier);
	if (Found)
	{
		return *Found;
	}

	auto CreateHitGroupHandle = [&](const FSpirvContainer& Container)
	{
		FSpirvCode Spirv = GetSpirvCode(Container);
		TRefCountPtr<FVulkanShaderModule> Module = CreateShaderModule(Device, Spirv);
		ShaderModules.Add(ModuleIdentifier, Module);
		return Module;
	};

	TRefCountPtr<FVulkanShaderModule> Module;
	if (bIsAnyHitModuleIdentifier)
	{
		check(GetFrequency() == SF_RayHitGroup);
		if (GetCodeHeader().RayGroupAnyHit == FVulkanShaderHeader::ERayHitGroupEntrypoint::SeparateBlob)
		{
			Module = CreateHitGroupHandle(AnyHitSpirvContainer);
		}
		else
		{
			return TRefCountPtr<FVulkanShaderModule>();
		}
	}
	else if (bIsIntersectionModuleIdentifier)
	{
		check(GetFrequency() == SF_RayHitGroup);
		if (GetCodeHeader().RayGroupIntersection == FVulkanShaderHeader::ERayHitGroupEntrypoint::SeparateBlob)
		{
			Module = CreateHitGroupHandle(IntersectionSpirvContainer);
		}
		else
		{
			return TRefCountPtr<FVulkanShaderModule>();
		}
	}
	else
	{
		Module = CreateHitGroupHandle(SpirvContainer);
	}

	if (!CodeHeader.DebugName.IsEmpty())
	{
		VULKAN_SET_DEBUG_NAME((*Device), VK_OBJECT_TYPE_SHADER_MODULE, Module->GetVkShaderModule(), TEXT("%s : (FVulkanShader*)0x%p"), *CodeHeader.DebugName, this);
	}

	return Module;
}


TRefCountPtr<FVulkanShaderModule> FVulkanShader::CreateHandle(const FVulkanLayout* Layout, uint32 LayoutHash)
{
	FScopeLock Lock(&VulkanShaderModulesMapCS);
	FSpirvCode Spirv = GetSpirvCode();

	TRefCountPtr<FVulkanShaderModule> Module = CreateShaderModule(Device, Spirv);
	ShaderModules.Add(LayoutHash, Module);
	if (!CodeHeader.DebugName.IsEmpty())
	{
		VULKAN_SET_DEBUG_NAME((*Device), VK_OBJECT_TYPE_SHADER_MODULE, Module->GetVkShaderModule(), TEXT("%s : (FVulkanShader*)0x%p"), *CodeHeader.DebugName, this);
	}
	return Module;
}

FVulkanShader::~FVulkanShader()
{
	PurgeShaderModules();
	Device->GetShaderFactory().OnDeleteShader(*this);

	if (AsyncCompileTask.IsValid() && !AsyncCompileTask->IsComplete())
	{
		AsyncCompileTask->Wait();
	}

	if (ShaderObject)
	{
		Device->GetDeferredDeletionQueue().EnqueueFunc([Handle=ShaderObject](FVulkanDevice& Device) 
		{
			VulkanRHI::vkDestroyShaderEXT(Device.GetHandle(), Handle, VULKAN_CPU_ALLOCATOR);
		});
		ShaderObject = VK_NULL_HANDLE;
	}

	if (PipelineLibrary)
	{
		Device->GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::Pipeline, PipelineLibrary);
		PipelineLibrary = VK_NULL_HANDLE;

		if (Frequency == SF_Vertex)
		{
			DEC_DWORD_STAT(STAT_VulkanNumVSLibs);
		}
		else if (Frequency == SF_Mesh)
		{
			DEC_DWORD_STAT(STAT_VulkanNumMSLibs);
		}
		else if (Frequency == SF_Pixel)
		{
			DEC_DWORD_STAT(STAT_VulkanNumPSLibs);
		}
	}
}

void FVulkanShader::PurgeShaderModules()
{
	FScopeLock Lock(&VulkanShaderModulesMapCS);
	ShaderModules.Empty(0);
}

FVertexShaderRHIRef FVulkanDynamicRHI::RHICreateVertexShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	return Device->GetShaderFactory().CreateShader<FVulkanVertexShader>(CreateShaderDesc.Code, Device);
}

FPixelShaderRHIRef FVulkanDynamicRHI::RHICreatePixelShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	return Device->GetShaderFactory().CreateShader<FVulkanPixelShader>(CreateShaderDesc.Code, Device);
}

FMeshShaderRHIRef FVulkanDynamicRHI::RHICreateMeshShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	return Device->GetShaderFactory().CreateShader<FVulkanMeshShader>(CreateShaderDesc.Code, Device);
}

FAmplificationShaderRHIRef FVulkanDynamicRHI::RHICreateAmplificationShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	return Device->GetShaderFactory().CreateShader<FVulkanTaskShader>(CreateShaderDesc.Code, Device);
}

FGeometryShaderRHIRef FVulkanDynamicRHI::RHICreateGeometryShader(const FRHICreateShaderDesc& CreateShaderDesc)
{ 
	return Device->GetShaderFactory().CreateShader<FVulkanGeometryShader>(CreateShaderDesc.Code, Device);
}

FComputeShaderRHIRef FVulkanDynamicRHI::RHICreateComputeShader(const FRHICreateShaderDesc& CreateShaderDesc)
{ 
	return Device->GetShaderFactory().CreateShader<FVulkanComputeShader>(CreateShaderDesc.Code, Device);
}

FRayTracingShaderRHIRef FVulkanDynamicRHI::RHICreateRayTracingShader(const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency ShaderFrequency)
{
	switch (ShaderFrequency)
	{
	case EShaderFrequency::SF_RayGen:
		 return Device->GetShaderFactory().CreateRayTracingShader<SF_RayGen>(CreateShaderDesc.Code, Device);

	case EShaderFrequency::SF_RayMiss:
		return Device->GetShaderFactory().CreateRayTracingShader<SF_RayMiss>(CreateShaderDesc.Code, Device);

	case EShaderFrequency::SF_RayCallable:
		return Device->GetShaderFactory().CreateRayTracingShader<SF_RayCallable>(CreateShaderDesc.Code, Device);

	case EShaderFrequency::SF_RayHitGroup:
		return Device->GetShaderFactory().CreateRayTracingShader<SF_RayHitGroup>(CreateShaderDesc.Code, Device);

	default:
		check(false);
		return nullptr;
	}
}

FVulkanBoundShaderState::FVulkanBoundShaderState(FRHIVertexDeclaration* InVertexDeclarationRHI, FRHIVertexShader* InVertexShaderRHI,
	FRHIPixelShader* InPixelShaderRHI, FRHIGeometryShader* InGeometryShaderRHI)
	: CacheLink(InVertexDeclarationRHI, InVertexShaderRHI, InPixelShaderRHI, InGeometryShaderRHI, this)
{
	CacheLink.AddToCache();
}

FVulkanBoundShaderState::~FVulkanBoundShaderState()
{
	CacheLink.RemoveFromCache();
}

FBoundShaderStateRHIRef FVulkanDynamicRHI::RHICreateBoundShaderState(
	FRHIVertexDeclaration* VertexDeclarationRHI,
	FRHIVertexShader* VertexShaderRHI,
	FRHIPixelShader* PixelShaderRHI,
	FRHIGeometryShader* GeometryShaderRHI
	)
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanShaders);
	FBoundShaderStateRHIRef CachedBoundShaderState = GetCachedBoundShaderState_Threadsafe(
		VertexDeclarationRHI,
		VertexShaderRHI,
		PixelShaderRHI,
		GeometryShaderRHI
	);
	if (CachedBoundShaderState.GetReference())
	{
		// If we've already created a bound shader state with these parameters, reuse it.
		return CachedBoundShaderState;
	}

	return new FVulkanBoundShaderState(VertexDeclarationRHI, VertexShaderRHI, PixelShaderRHI, GeometryShaderRHI);
}
