// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanState.cpp: Vulkan state implementation.
=============================================================================*/

#include "VulkanState.h"
#include "VulkanBindlessDescriptorManager.h"
#include "VulkanPendingState.h"
#include "VulkanDescriptorSets.h"
#include "RHICore.h"
#include "RHIUtilities.h"

static FCriticalSection GSamplerHashLock;

inline VkSamplerMipmapMode TranslateMipFilterMode(ESamplerFilter InFilter)
{
	switch (InFilter)
	{
		case SF_Point:				return VK_SAMPLER_MIPMAP_MODE_NEAREST;
		case SF_Bilinear:			return VK_SAMPLER_MIPMAP_MODE_NEAREST;
		case SF_Trilinear:			return VK_SAMPLER_MIPMAP_MODE_LINEAR;
		case SF_AnisotropicPoint:	return VK_SAMPLER_MIPMAP_MODE_NEAREST;
		case SF_AnisotropicLinear:	return VK_SAMPLER_MIPMAP_MODE_LINEAR;
		default:
			break;
	}

	checkf(0, TEXT("Unknown Mip ESamplerFilter %d"), (uint32)InFilter);
	return VK_SAMPLER_MIPMAP_MODE_MAX_ENUM;
}

inline VkFilter TranslateMinMagFilterMode(ESamplerFilter InFilter)
{
	switch (InFilter)
	{
		case SF_Point:				return VK_FILTER_NEAREST;
		case SF_Bilinear:			return VK_FILTER_LINEAR;
		case SF_Trilinear:			return VK_FILTER_LINEAR;
		case SF_AnisotropicPoint:
		case SF_AnisotropicLinear:	return VK_FILTER_LINEAR;
		default:
			break;
	}

	checkf(0, TEXT("Unknown ESamplerFilter %d"), (uint32)InFilter);
	return VK_FILTER_MAX_ENUM;
}

inline VkSamplerAddressMode TranslateWrapMode(ESamplerAddressMode InAddressMode)
{
	switch (InAddressMode)
	{
		case AM_Wrap:		return VK_SAMPLER_ADDRESS_MODE_REPEAT;
		case AM_Clamp:		return VK_SAMPLER_ADDRESS_MODE_CLAMP_TO_EDGE;
		case AM_Mirror:		return VK_SAMPLER_ADDRESS_MODE_MIRRORED_REPEAT;
		case AM_Border:		return VK_SAMPLER_ADDRESS_MODE_CLAMP_TO_BORDER;
		default:
			break;
	}

	checkf(0, TEXT("Unknown Wrap ESamplerAddressMode %d"), (uint32)InAddressMode);
	return VK_SAMPLER_ADDRESS_MODE_MAX_ENUM;
}

inline VkCompareOp TranslateSamplerCompareFunction(ESamplerCompareFunction InSamplerComparisonFunction)
{
	switch (InSamplerComparisonFunction)
	{
		case SCF_Less:	return VK_COMPARE_OP_LESS;
		case SCF_Never:	return VK_COMPARE_OP_NEVER;
		default:
			break;
	};

	checkf(0, TEXT("Unknown ESamplerCompareFunction %d"), (uint32)InSamplerComparisonFunction);
	return VK_COMPARE_OP_MAX_ENUM;
}

static inline VkBlendOp BlendOpToVulkan(EBlendOperation InOp)
{
	switch (InOp)
	{
		case BO_Add:				return VK_BLEND_OP_ADD;
		case BO_Subtract:			return VK_BLEND_OP_SUBTRACT;
		case BO_Min:				return VK_BLEND_OP_MIN;
		case BO_Max:				return VK_BLEND_OP_MAX;
		case BO_ReverseSubtract:	return VK_BLEND_OP_REVERSE_SUBTRACT;
		default:
			break;
	}

	checkf(0, TEXT("Unknown EBlendOperation %d"), (uint32)InOp);
	return VK_BLEND_OP_MAX_ENUM;
}

static inline VkBlendFactor BlendFactorToVulkan(EBlendFactor InFactor)
{
	switch (InFactor)
	{
		case BF_Zero:						return VK_BLEND_FACTOR_ZERO;
		case BF_One:						return VK_BLEND_FACTOR_ONE;
		case BF_SourceColor:				return VK_BLEND_FACTOR_SRC_COLOR;
		case BF_InverseSourceColor:			return VK_BLEND_FACTOR_ONE_MINUS_SRC_COLOR;
		case BF_SourceAlpha:				return VK_BLEND_FACTOR_SRC_ALPHA;
		case BF_InverseSourceAlpha:			return VK_BLEND_FACTOR_ONE_MINUS_SRC_ALPHA;
		case BF_DestAlpha:					return VK_BLEND_FACTOR_DST_ALPHA;
		case BF_InverseDestAlpha:			return VK_BLEND_FACTOR_ONE_MINUS_DST_ALPHA;
		case BF_DestColor:					return VK_BLEND_FACTOR_DST_COLOR;
		case BF_InverseDestColor:			return VK_BLEND_FACTOR_ONE_MINUS_DST_COLOR;
		case BF_ConstantBlendFactor:		return VK_BLEND_FACTOR_CONSTANT_COLOR;
		case BF_InverseConstantBlendFactor:	return VK_BLEND_FACTOR_ONE_MINUS_CONSTANT_COLOR;
		case BF_Source1Color:				return VK_BLEND_FACTOR_SRC1_COLOR;
		case BF_InverseSource1Color:		return VK_BLEND_FACTOR_ONE_MINUS_SRC1_COLOR;
		case BF_Source1Alpha:				return VK_BLEND_FACTOR_SRC1_ALPHA;
		case BF_InverseSource1Alpha:		return VK_BLEND_FACTOR_ONE_MINUS_SRC1_ALPHA;
		default:
			break;
	}

	checkf(0, TEXT("Unknown EBlendFactor %d"), (uint32)InFactor);
	return VK_BLEND_FACTOR_MAX_ENUM;
}

static inline VkCompareOp CompareOpToVulkan(ECompareFunction InOp)
{
	switch (InOp)
	{
		case CF_Less:			return VK_COMPARE_OP_LESS;
		case CF_LessEqual:		return VK_COMPARE_OP_LESS_OR_EQUAL;
		case CF_Greater:		return VK_COMPARE_OP_GREATER;
		case CF_GreaterEqual:	return VK_COMPARE_OP_GREATER_OR_EQUAL;
		case CF_Equal:			return VK_COMPARE_OP_EQUAL;
		case CF_NotEqual:		return VK_COMPARE_OP_NOT_EQUAL;
		case CF_Never:			return VK_COMPARE_OP_NEVER;
		case CF_Always:			return VK_COMPARE_OP_ALWAYS;
		default:
			break;
	}

	checkf(0, TEXT("Unknown ECompareFunction %d"), (uint32)InOp);
	return VK_COMPARE_OP_MAX_ENUM;
}

static inline VkStencilOp StencilOpToVulkan(EStencilOp InOp)
{
	VkStencilOp OutOp = VK_STENCIL_OP_MAX_ENUM;

	switch (InOp)
	{
		case SO_Keep:					return VK_STENCIL_OP_KEEP;
		case SO_Zero:					return VK_STENCIL_OP_ZERO;
		case SO_Replace:				return VK_STENCIL_OP_REPLACE;
		case SO_SaturatedIncrement:		return VK_STENCIL_OP_INCREMENT_AND_CLAMP;
		case SO_SaturatedDecrement:		return VK_STENCIL_OP_DECREMENT_AND_CLAMP;
		case SO_Invert:					return VK_STENCIL_OP_INVERT;
		case SO_Increment:				return VK_STENCIL_OP_INCREMENT_AND_WRAP;
		case SO_Decrement:				return VK_STENCIL_OP_DECREMENT_AND_WRAP;
		default:
			break;
	}

	checkf(0, TEXT("Unknown EStencilOp %d"), (uint32)InOp);
	return VK_STENCIL_OP_MAX_ENUM;
}

static inline VkPolygonMode RasterizerFillModeToVulkan(ERasterizerFillMode InFillMode)
{
	switch (InFillMode)
	{
		case FM_Point:			return VK_POLYGON_MODE_POINT;
		case FM_Wireframe:		return VK_POLYGON_MODE_LINE;
		case FM_Solid:			return VK_POLYGON_MODE_FILL;
		default:
			break;
	}

	checkf(0, TEXT("Unknown ERasterizerFillMode %d"), (uint32)InFillMode);
	return VK_POLYGON_MODE_MAX_ENUM;
}

static inline VkCullModeFlags RasterizerCullModeToVulkan(ERasterizerCullMode InCullMode)
{
	switch (InCullMode)
	{
		case CM_None:	return VK_CULL_MODE_NONE;
		case CM_CW:		return VK_CULL_MODE_FRONT_BIT;
		case CM_CCW:	return VK_CULL_MODE_BACK_BIT;
		default:		break;
	}

	checkf(0, TEXT("Unknown ERasterizerCullMode %d"), (uint32)InCullMode);
	return VK_CULL_MODE_NONE;
}


void FVulkanSamplerState::SetupSamplerCreateInfo(const FSamplerStateInitializerRHI& Initializer, FVulkanDevice& InDevice, VkSamplerCreateInfo& OutSamplerInfo)
{
	ZeroVulkanStruct(OutSamplerInfo, VK_STRUCTURE_TYPE_SAMPLER_CREATE_INFO);

	OutSamplerInfo.magFilter = TranslateMinMagFilterMode(Initializer.Filter);
	OutSamplerInfo.minFilter = TranslateMinMagFilterMode(Initializer.Filter);
	OutSamplerInfo.mipmapMode = TranslateMipFilterMode(Initializer.Filter);
	OutSamplerInfo.addressModeU = TranslateWrapMode(Initializer.AddressU);
	OutSamplerInfo.addressModeV = TranslateWrapMode(Initializer.AddressV);
	OutSamplerInfo.addressModeW = TranslateWrapMode(Initializer.AddressW);

	OutSamplerInfo.mipLodBias = Initializer.MipBias;
	
	OutSamplerInfo.maxAnisotropy = 1.0f;
	if (Initializer.Filter == SF_AnisotropicLinear || Initializer.Filter == SF_AnisotropicPoint)
	{
		OutSamplerInfo.maxAnisotropy = FMath::Clamp((float)ComputeAnisotropyRT(Initializer.MaxAnisotropy), 1.0f, InDevice.GetLimits().maxSamplerAnisotropy);
	}
	OutSamplerInfo.anisotropyEnable = OutSamplerInfo.maxAnisotropy > 1.0f;

	OutSamplerInfo.compareEnable = Initializer.SamplerComparisonFunction != SCF_Never ? VK_TRUE : VK_FALSE;
	OutSamplerInfo.compareOp = TranslateSamplerCompareFunction(Initializer.SamplerComparisonFunction);
	OutSamplerInfo.minLod = Initializer.MinMipLevel;
	OutSamplerInfo.maxLod = Initializer.MaxMipLevel;
	OutSamplerInfo.borderColor = Initializer.BorderColor == 0 ? VK_BORDER_COLOR_FLOAT_TRANSPARENT_BLACK : VK_BORDER_COLOR_FLOAT_OPAQUE_WHITE;
}


FVulkanSamplerState::FVulkanSamplerState(const VkSamplerCreateInfo& InInfo, FVulkanDevice& InDevice, const bool bInIsImmutable)
	: Sampler(VK_NULL_HANDLE)
	, SamplerId(0)
	, BindlessHandle()
	, bIsImmutable(bInIsImmutable)
{
	FVulkanBindlessDescriptorManager* BindlessDescriptorManager = InDevice.GetBindlessDescriptorManager();
	checkf(BindlessDescriptorManager, TEXT("SamplerState is being created before creation of the vulkan bindless manager."));

	BindlessHandle = BindlessDescriptorManager->AllocateDescriptor(ERHIDescriptorType::Sampler);

	if (BindlessDescriptorManager->UseDescriptorHeaps())
	{
		BindlessDescriptorManager->UpdateDescriptor(BindlessHandle, InInfo);
	}
	else
	{
		VERIFYVULKANRESULT(VulkanRHI::vkCreateSampler(InDevice.GetHandle(), &InInfo, VULKAN_CPU_ALLOCATOR, &Sampler));

		if (BindlessDescriptorManager->UseDescriptorBuffers())
		{
			BindlessDescriptorManager->UpdateDescriptor(BindlessHandle, Sampler);
		}
	}

	if (UseVulkanDescriptorCache())
	{
		SamplerId = ++GVulkanSamplerHandleIdCounter;
	}
}

FVulkanRasterizerState::FVulkanRasterizerState(const FRasterizerStateInitializerRHI& InInitializer)
	: Initializer(InInitializer)
{
	FVulkanRasterizerState::ResetCreateInfo(RasterizerState);

	// @todo vulkan: I'm assuming that Solid and Wireframe wouldn't ever be mixed within the same BoundShaderState, so we are ignoring the fill mode as a unique identifier
	//checkf(Initializer.FillMode == FM_Solid, TEXT("PIPELINE KEY: Only FM_Solid is supported fill mode [got %d]"), (int32)Initializer.FillMode);

	RasterizerState.polygonMode = RasterizerFillModeToVulkan(Initializer.FillMode);
	RasterizerState.cullMode = RasterizerCullModeToVulkan(Initializer.CullMode);

	RasterizerState.depthClampEnable = InInitializer.DepthClipMode == ERasterizerDepthClipMode::DepthClamp ? VK_TRUE : VK_FALSE;
	RasterizerState.depthBiasEnable = Initializer.DepthBias != 0.0f ? VK_TRUE : VK_FALSE;
	//RasterizerState.rasterizerDiscardEnable = VK_FALSE;

	RasterizerState.depthBiasSlopeFactor = Initializer.SlopeScaleDepthBias;
	RasterizerState.depthBiasConstantFactor = Initializer.DepthBias;

	//RasterizerState.lineWidth = 1.0f;
}

void FVulkanDepthStencilState::SetupCreateInfo(const FGraphicsPipelineStateInitializer& GfxPSOInit, VkPipelineDepthStencilStateCreateInfo& OutDepthStencilState)
{
	ZeroVulkanStruct(OutDepthStencilState, VK_STRUCTURE_TYPE_PIPELINE_DEPTH_STENCIL_STATE_CREATE_INFO);

	OutDepthStencilState.depthTestEnable = (Initializer.DepthTest != CF_Always || Initializer.bEnableDepthWrite) ? VK_TRUE : VK_FALSE;
	OutDepthStencilState.depthCompareOp = CompareOpToVulkan(Initializer.DepthTest);
	OutDepthStencilState.depthWriteEnable = Initializer.bEnableDepthWrite ? VK_TRUE : VK_FALSE;

	{
		// This will be filled in from the PSO
		OutDepthStencilState.depthBoundsTestEnable = GfxPSOInit.bDepthBounds;
		OutDepthStencilState.minDepthBounds = 0.0f;
		OutDepthStencilState.maxDepthBounds = 1.0f;
	}

	OutDepthStencilState.stencilTestEnable = (Initializer.bEnableFrontFaceStencil || Initializer.bEnableBackFaceStencil) ? VK_TRUE : VK_FALSE;

	// Front
	OutDepthStencilState.back.failOp = StencilOpToVulkan(Initializer.FrontFaceStencilFailStencilOp);
	OutDepthStencilState.back.passOp = StencilOpToVulkan(Initializer.FrontFacePassStencilOp);
	OutDepthStencilState.back.depthFailOp = StencilOpToVulkan(Initializer.FrontFaceDepthFailStencilOp);
	OutDepthStencilState.back.compareOp = CompareOpToVulkan(Initializer.FrontFaceStencilTest);
	OutDepthStencilState.back.compareMask = Initializer.StencilReadMask;
	OutDepthStencilState.back.writeMask = Initializer.StencilWriteMask;
	OutDepthStencilState.back.reference = 0;

	if (Initializer.bEnableBackFaceStencil)
	{
		// Back
		OutDepthStencilState.front.failOp = StencilOpToVulkan(Initializer.BackFaceStencilFailStencilOp);
		OutDepthStencilState.front.passOp = StencilOpToVulkan(Initializer.BackFacePassStencilOp);
		OutDepthStencilState.front.depthFailOp = StencilOpToVulkan(Initializer.BackFaceDepthFailStencilOp);
		OutDepthStencilState.front.compareOp = CompareOpToVulkan(Initializer.BackFaceStencilTest);
		OutDepthStencilState.front.compareMask = Initializer.StencilReadMask;
		OutDepthStencilState.front.writeMask = Initializer.StencilWriteMask;
		OutDepthStencilState.front.reference = 0;
	}
	else
	{
		OutDepthStencilState.front = OutDepthStencilState.back;
	}
}

FVulkanBlendState::FVulkanBlendState(const FBlendStateInitializerRHI& InInitializer)
{
	Initializer = InInitializer;
	for (uint32 Index = 0; Index < MaxSimultaneousRenderTargets; ++Index)
	{
		const FBlendStateInitializerRHI::FRenderTarget& ColorTarget = Initializer.RenderTargets[Index];
		VkPipelineColorBlendAttachmentState& BlendState = BlendStates[Index];
		FMemory::Memzero(BlendState);

		BlendState.colorBlendOp = BlendOpToVulkan(ColorTarget.ColorBlendOp);
		BlendState.alphaBlendOp = BlendOpToVulkan(ColorTarget.AlphaBlendOp);

		BlendState.dstColorBlendFactor = BlendFactorToVulkan(ColorTarget.ColorDestBlend);
		BlendState.dstAlphaBlendFactor = BlendFactorToVulkan(ColorTarget.AlphaDestBlend);

		BlendState.srcColorBlendFactor = BlendFactorToVulkan(ColorTarget.ColorSrcBlend);
		BlendState.srcAlphaBlendFactor = BlendFactorToVulkan(ColorTarget.AlphaSrcBlend);

		BlendState.blendEnable =
			(ColorTarget.ColorBlendOp != BO_Add || ColorTarget.ColorDestBlend != BF_Zero || ColorTarget.ColorSrcBlend != BF_One ||
			ColorTarget.AlphaBlendOp != BO_Add || ColorTarget.AlphaDestBlend != BF_Zero || ColorTarget.AlphaSrcBlend != BF_One) ? VK_TRUE : VK_FALSE;

		BlendState.colorWriteMask = (ColorTarget.ColorWriteMask & CW_RED) ? VK_COLOR_COMPONENT_R_BIT : 0;
		BlendState.colorWriteMask |= (ColorTarget.ColorWriteMask & CW_GREEN) ? VK_COLOR_COMPONENT_G_BIT : 0;
		BlendState.colorWriteMask |= (ColorTarget.ColorWriteMask & CW_BLUE) ? VK_COLOR_COMPONENT_B_BIT : 0;
		BlendState.colorWriteMask |= (ColorTarget.ColorWriteMask & CW_ALPHA) ? VK_COLOR_COMPONENT_A_BIT : 0;
	}
}

FSamplerStateRHIRef FVulkanDynamicRHI::RHICreateSamplerState(const FSamplerStateInitializerRHI& Initializer)
{
	VkSamplerCreateInfo SamplerInfo;
	FVulkanSamplerState::SetupSamplerCreateInfo(Initializer, *Device, SamplerInfo);

	uint32 CRC = FCrc::MemCrc32(&SamplerInfo, sizeof(SamplerInfo));

	{
		FScopeLock ScopeLock(&GSamplerHashLock);
		TMap<uint32, TRefCountPtr<FVulkanSamplerState>>& SamplerMap = Device->GetSamplerMap();

		if (TRefCountPtr<FVulkanSamplerState>* Found = SamplerMap.Find(CRC))
		{
			return FSamplerStateRHIRef(*Found);
		}

		TRefCountPtr<FVulkanSamplerState> New = new FVulkanSamplerState(SamplerInfo, *Device);
		SamplerMap.Add(CRC, New);
		return FSamplerStateRHIRef(MoveTemp(New));
	}
}

FRasterizerStateRHIRef FVulkanDynamicRHI::RHICreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer)
{
	return new FVulkanRasterizerState(Initializer);
}


FDepthStencilStateRHIRef FVulkanDynamicRHI::RHICreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer)
{
	return new FVulkanDepthStencilState(Initializer);
}


FBlendStateRHIRef FVulkanDynamicRHI::RHICreateBlendState(const FBlendStateInitializerRHI& Initializer)
{
	return new FVulkanBlendState(Initializer);
}
