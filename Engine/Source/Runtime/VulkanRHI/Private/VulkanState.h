// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanState.h: Vulkan state definitions.
=============================================================================*/

#pragma once

#include "RHI.h"
#include "RHIResources.h"
#include "VulkanCommon.h"
#include "VulkanThirdParty.h"

class FVulkanDevice;

class FVulkanSamplerState : public FRHISamplerState
{
public:
	FVulkanSamplerState(const VkSamplerCreateInfo& InInfo, FVulkanDevice& InDevice, const bool bInIsImmutable = false);
	
	virtual bool IsImmutable() const final override { return bIsImmutable; }

	VkSampler Sampler;
	uint32 SamplerId;
	FRHIDescriptorHandle BindlessHandle;

	static void SetupSamplerCreateInfo(const FSamplerStateInitializerRHI& Initializer, FVulkanDevice& InDevice, VkSamplerCreateInfo& OutSamplerInfo);

	virtual FRHIDescriptorHandle GetBindlessHandle() const override final { return BindlessHandle; }

private:
	bool bIsImmutable;
};

class FVulkanRasterizerState : public FRHIRasterizerState
{
public:
	FVulkanRasterizerState(const FRasterizerStateInitializerRHI& InInitializer);

	static void ResetCreateInfo(VkPipelineRasterizationStateCreateInfo& OutInfo)
	{
		ZeroVulkanStruct(OutInfo, VK_STRUCTURE_TYPE_PIPELINE_RASTERIZATION_STATE_CREATE_INFO);
		OutInfo.frontFace = VK_FRONT_FACE_CLOCKWISE;
		OutInfo.lineWidth = 1.0f;
	}

	virtual bool GetInitializer(FRasterizerStateInitializerRHI& Out) override final
	{
		Out = Initializer;
		return true;
	}

	VkPipelineRasterizationStateCreateInfo	RasterizerState;
	FRasterizerStateInitializerRHI			Initializer;
};

class FVulkanDepthStencilState : public FRHIDepthStencilState
{
public:
	FVulkanDepthStencilState(const FDepthStencilStateInitializerRHI& InInitializer)
	{
		Initializer = InInitializer;
	}

	virtual bool GetInitializer(FDepthStencilStateInitializerRHI& Out) override final
	{
		Out = Initializer;
		return true;
	}

	void SetupCreateInfo(const FGraphicsPipelineStateInitializer& GfxPSOInit, VkPipelineDepthStencilStateCreateInfo& OutDepthStencilState);

	FDepthStencilStateInitializerRHI Initializer;
};

class FVulkanBlendState : public FRHIBlendState
{
public:
	FVulkanBlendState(const FBlendStateInitializerRHI& InInitializer);

	virtual bool GetInitializer(FBlendStateInitializerRHI& Out) override final
	{
		Out = Initializer;
		return true;
	}

	// array the pipeline state can point right to
	VkPipelineColorBlendAttachmentState BlendStates[MaxSimultaneousRenderTargets];

	FBlendStateInitializerRHI Initializer;
};
