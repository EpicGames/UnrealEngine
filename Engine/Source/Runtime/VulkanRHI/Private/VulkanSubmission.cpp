// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanSubmission.cpp: Vulkan RHI submission implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanDynamicRHI.h"
#include "VulkanContext.h"
#include "VulkanDevice.h"
#include "VulkanSubmission.h"
#include "VulkanQueue.h"
#include "VulkanDescriptorSets.h"

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"

static std::atomic<int> GGPUCrashDetected = false;

TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> FVulkanSyncPoint::MemoryPool;


void FVulkanSyncPoint::Wait() const
{
	checkf(GraphEvent, TEXT("This sync point was not created with a CPU event. Cannot wait for completion on the CPU."));

	if (!GraphEvent->IsComplete())
	{
		// Block the calling thread until the graph event is signaled by the interrupt thread.
		SCOPED_NAMED_EVENT_TEXT("SyncPoint_Wait", FColor::Turquoise);
		FVulkanDynamicRHI::Get().ProcessInterruptQueueUntil(GraphEvent);
	}

	check(GraphEvent->IsComplete());
}



FVulkanPayload::~FVulkanPayload()
{
	if (Fence)
	{
		Fence->GetOwner().ReleaseFence(Fence);
	}
}

void FVulkanPayload::Merge(FVulkanPayload& OtherPayload)
{
	check(OtherPayload.SignalSemaphores.Num() == 0);
	check(OtherPayload.WaitSemaphores.Num() == 0);

	for (int32 QueryPoolTypeIndex = 0; QueryPoolTypeIndex < OtherPayload.QueryPools.Num(); ++QueryPoolTypeIndex)
	{
		QueryPools[QueryPoolTypeIndex].Append(MoveTemp(OtherPayload.QueryPools[QueryPoolTypeIndex]));
	}

	DescriptorPoolSetContainers.Append(MoveTemp(OtherPayload.DescriptorPoolSetContainers));

	checkf(OtherPayload.CommandBuffers.IsEmpty() || (OtherPayload.CommandBuffers[0]->GetCommandBufferType() != EVulkanCommandBufferType::Primary),
		TEXT("Merging payloads should only be used on Secondary or Parallel command buffers."));
	CommandBuffers.Append(MoveTemp(OtherPayload.CommandBuffers));

	SyncPoints.Append(MoveTemp(OtherPayload.SyncPoints));
}


class FVulkanThread final : private FRunnable
{
	typedef bool(FVulkanDynamicRHI::* FQueueFunc)();

public:
	FVulkanThread(TCHAR const* InName, EThreadPriority InPriority, FVulkanDynamicRHI* InRHI, FQueueFunc InFunc)
		: RHI(InRHI)
		, Event(FPlatformProcess::GetSynchEventFromPool(false))
		, Func(InFunc)
		, Thread(FRunnableThread::Create(this, InName, 0, InPriority))
	{}

	virtual ~FVulkanThread()
	{
		bExit = true;
		Event->Trigger();

		Thread->WaitForCompletion();
		delete Thread;
		Thread = nullptr;

		FPlatformProcess::ReturnSynchEventToPool(Event);
		Event = nullptr;
	}

	void Kick() const
	{
		Event->Trigger();
	}

	void Join() const
	{
		Thread->WaitForCompletion();
	}

	uint32 GetThreadID() const
	{
		return Thread->GetThreadID();
	}

protected:

	virtual uint32 Run() override
	{
		while (!bExit)
		{
			// Process the queue until no more progress is made
			while ((RHI->*Func)());

			Event->Wait();
		}

		// :todo-jn: Drain any remaining work in the queue

		return 0;
	}

	FVulkanDynamicRHI* RHI;
	std::atomic<bool> bExit = false;
	FEvent* Event;
	FQueueFunc Func;
	FRunnableThread* Thread = nullptr;
};




static TAutoConsoleVariable<int32> CVarVulkanUseInterruptThread(
	TEXT("r.Vulkan.Submission.UseInterruptThread"),
	1,
	TEXT("  0: Process completed GPU work directly on the RHI thread.\n")
	TEXT("  1: Create a dedicated thread to process completed GPU work.\n"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarVulkanUseSubmissionThread(
	TEXT("r.Vulkan.Submission.UseSubmissionThread"),
	1,
	TEXT("  0: Submit GPU work directly from the RHI thread.\n")
	TEXT("  1: Create a dedicated thread to submit GPU work.\n"),
	ECVF_ReadOnly);

extern int32 GAllowTimelineSemaphores;

void FVulkanDynamicRHI::InitializeSubmissionPipe()
{
	const bool UseMultiThreading = FPlatformProcess::SupportsMultithreading() && FApp::ShouldUseThreadingForPerformance() && (FTaskGraphInterface::Get().GetNumWorkerThreads() > 6);
	const bool UseTimelineSemaphores = GAllowTimelineSemaphores && Device->GetOptionalExtensions().HasKHRTimelineSemaphore;
	if (UseTimelineSemaphores && UseMultiThreading)
	{
		if (CVarVulkanUseInterruptThread.GetValueOnAnyThread())
		{
			CPUTimelineSemaphore = new FVulkanSemaphore(*Device, EVulkanSemaphoreFlags::Timeline | EVulkanSemaphoreFlags::ImmediateDeletion, CPUTimelineSemaphoreValue++);
			InterruptThread = new FVulkanThread(TEXT("RHIInterruptThread"), TPri_Highest, this, &FVulkanDynamicRHI::WaitAndProcessInterruptQueue);
		}

		if (CVarVulkanUseSubmissionThread.GetValueOnAnyThread())
		{
			SubmissionThread = new FVulkanThread(TEXT("RHISubmissionThread"), TPri_Highest, this, &FVulkanDynamicRHI::ProcessSubmissionQueue);
		}
	}

	// Initialize the timing structs in each queue, and the engine GPU profilers
	{
		TArray<UE::RHI::GPUProfiler::FQueue> ProfilerQueues;
		FVulkanPlatformCommandList* Payloads = new FVulkanPlatformCommandList;
		Payloads->Reserve((int32)EVulkanQueueType::Count);
		Device->ForEachQueue([&](FVulkanQueue& Queue)
		{
			ProfilerQueues.Add(Queue.GetProfilerQueue());
			FVulkanPayload* Payload = Payloads->Emplace_GetRef(new FVulkanPayload(Queue));
			Payload->Timing = CurrentTimingPerQueue.CreateNew(Queue);
		});

		UE::RHI::GPUProfiler::InitializeQueues(ProfilerQueues);
		PendingPayloadsForSubmission.Enqueue(Payloads);
	}
}

void FVulkanDynamicRHI::ShutdownSubmissionPipe()
{
	if (SubmissionThread)
	{
		delete SubmissionThread;
		SubmissionThread = nullptr;
	}

	if (InterruptThread)
	{
		delete InterruptThread;
		InterruptThread = nullptr;
	}

	if (EopTask)
	{
		ProcessInterruptQueueUntil(EopTask);
		EopTask = nullptr;
	}

	if (CPUTimelineSemaphore)
	{
		delete CPUTimelineSemaphore;
		CPUTimelineSemaphore = nullptr;
	}
}

void FVulkanDynamicRHI::KickInterruptThread()
{
	if (InterruptThread)
	{
		checkSlow(CPUTimelineSemaphore);
		VkSemaphoreSignalInfo SignalInfo;
		ZeroVulkanStruct(SignalInfo, VK_STRUCTURE_TYPE_SEMAPHORE_SIGNAL_INFO);
		SignalInfo.semaphore = CPUTimelineSemaphore->GetHandle();
		SignalInfo.value = (uint64_t)CPUTimelineSemaphoreValue++;
		VERIFYVULKANRESULT(VulkanRHI::vkSignalSemaphoreKHR(Device->GetHandle(), &SignalInfo));

		InterruptThread->Kick();
	}
}

void FVulkanDynamicRHI::KickSubmissionThread(bool bForce)
{
	if (SubmissionThread)
	{
		SubmissionThread->Kick();
	}
	else if (bForce || !Device->UseMinimalSubmits())
	{
		FScopeLock Lock(&SubmissionCS);
		while (ProcessSubmissionQueue()) {};
	}
}

static FCriticalSection GSecondaryPayloadsCS;
FDynamicRHI::FRHIFinalizeContextsResult FVulkanDynamicRHI::RHIFinalizeContexts(FRHIFinalizeContextsArgs&& Args)
{
	FRHIFinalizeContextsResult Result;

	if (Args.UploadContext)
	{
		FVulkanPlatformCommandList* PlatformCmdList = new FVulkanPlatformCommandList;
		Result.UploadCommandList = PlatformCmdList;

		FVulkanUploadContext* UploadContext = ResourceCast(Args.UploadContext);
		UploadContext->Finalize(*PlatformCmdList);

		FVulkanUploadContext::Pool.Push(UploadContext);
	}

	for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
	{
		if (FVulkanCommandListContext* Context = static_cast<FVulkanCommandListContext*>(Args.Contexts[Pipeline]))
		{
			if (FVulkanParallelRenderPassInfo* ParallelRPInfo = Context->GetParallelRenderPassInfo())
			{
				// Handle finalizing of parallel render pass contexts.
				if (Device->GetOptionalExtensions().HasKHRDynamicRendering)
				{
					FVulkanQueryPool* OptionalTimestampQueryPool = GRHIGlobals.SupportsTimestampRenderQueries 
						? Context->GetCurrentTimestampQueryPool()
						: nullptr;

					Context->GetCommandBuffer().EndDynamicRendering(OptionalTimestampQueryPool);
				}

				FScopeLock Lock(&GSecondaryPayloadsCS);
				Context->Finalize(ParallelRPInfo->SecondaryPayloads);
			}
			else
			{
				FVulkanPlatformCommandList* PlatformCmdList = new FVulkanPlatformCommandList;
				Result.CommandLists[Pipeline] = PlatformCmdList;

				Context->Finalize(*PlatformCmdList);
			}

			if (Context->IsImmediate())
			{
				// :todo-jn: clean up immediate context for next use?
			}
			else
			{
				delete Context;
			}
		}
	}

	return Result;
}

IRHICommandContext* FVulkanDynamicRHI::RHIGetParallelCommandContext(FRHIParallelRenderPassInfo const& ParallelRenderPass)
{
	FVulkanParallelRenderPassInfo* VulkanParallelRenderPassInfo = static_cast<FVulkanParallelRenderPassInfo*>(ParallelRenderPass.RHIPlatformData);
	checkf(VulkanParallelRenderPassInfo, TEXT("Must begin parallel render pass on parent context before creating parallel contexts!"));

	FVulkanCommandListContext* ParallelCommandContext = new FVulkanCommandListContext(*Device, VulkanParallelRenderPassInfo);

	ParallelCommandContext->RenderPassInfo = ParallelRenderPass;

	if (Device->GetOptionalExtensions().HasKHRDynamicRendering)
	{
		check(VulkanParallelRenderPassInfo->DynamicRenderingInfo);

		FVulkanQueryPool* OptionalTimestampQueryPool = GRHIGlobals.SupportsTimestampRenderQueries
			? ParallelCommandContext->GetCurrentTimestampQueryPool()
			: nullptr;

		const int32 ContextIndex = VulkanParallelRenderPassInfo->NumParallelContexts++;
		if (ContextIndex == 0)
		{
			// The first context has nothing to resume, it only needs the suspend flag
			// NOTE: a dummy pass will be created to resume without suspending at the end
			VkRenderingInfo RenderingInfo = VulkanParallelRenderPassInfo->DynamicRenderingInfo->RenderingInfo;
			RenderingInfo.flags = VK_RENDERING_SUSPENDING_BIT;
			ParallelCommandContext->GetCommandBuffer().BeginDynamicRendering(RenderingInfo, OptionalTimestampQueryPool);
		}
		else
		{
			ParallelCommandContext->GetCommandBuffer().BeginDynamicRendering(VulkanParallelRenderPassInfo->DynamicRenderingInfo->RenderingInfo, OptionalTimestampQueryPool);
		}
	}

	return ParallelCommandContext;
}

bool FVulkanDynamicRHI::ProcessSubmissionQueue()
{
	SCOPED_NAMED_EVENT_TEXT("ProcessSubmissionQueue", FColor::Orange);

	// Sort all the payloads into the queues
	FVulkanPlatformCommandList* PlatformCmdList = nullptr;
	while (PendingPayloadsForSubmission.Dequeue(PlatformCmdList))
	{
	#if WITH_RHI_BREADCRUMBS
		TSharedPtr<FRHIBreadcrumbAllocatorArray> BreadcrumbAllocators{};
		if (PlatformCmdList->BreadcrumbAllocators.Num())
		{
			BreadcrumbAllocators = MakeShared<FRHIBreadcrumbAllocatorArray>(MoveTemp(PlatformCmdList->BreadcrumbAllocators));
		}
	#endif

		for (FVulkanPayload* Payload : *PlatformCmdList)
		{
		#if WITH_RHI_BREADCRUMBS
			Payload->BreadcrumbRange = PlatformCmdList->BreadcrumbRange;
			if (BreadcrumbAllocators.IsValid())
			{
				check(!Payload->BreadcrumbAllocators.IsValid());
				Payload->BreadcrumbAllocators = BreadcrumbAllocators;
			}
		#endif

			Payload->Queue.PendingSubmission.Enqueue(Payload);
		}

		delete PlatformCmdList;
	}

	bool bProgress = false;
	Device->ForEachQueue([&bProgress, &SignaledSemaphores=SignaledSemaphores](FVulkanQueue& Queue)
	{
		bProgress |= Queue.SubmitQueuedPayloads(SignaledSemaphores) > 0;
	});

	// Wake up the interrupt thread to go wait on these new payloads
	if (InterruptThread)
	{
		KickInterruptThread();
	}

	return bProgress;
}

bool FVulkanDynamicRHI::WaitAndProcessInterruptQueue()
{
	SCOPED_NAMED_EVENT_TEXT("WaitAndProcessInterruptQueue", FColor::Orange);

	bool bProgress = false;

	// Wait for progress
	{
		const int32 NumQueues = (int32)EVulkanQueueType::Count;

		// Pick up the next payload for each queue
		// NOTE: we can hold on to these because we're holding InterruptCS or we're the interrupt thread.
		FVulkanPayload* NextPayloads[NumQueues];
		uint32 NumSyncs = 0;
		Device->ForEachQueue([&NumSyncs, &NextPayloads, &bProgress](FVulkanQueue& Queue)
		{
			// Clear any existing completed payloads
			bProgress |= (Queue.ProcessInterruptQueue(0) > 0);

			// Get the next payload to wait on
			if (FVulkanPayload* Payload = Queue.PendingInterrupt.Peek())
			{
				NextPayloads[NumSyncs++] = Payload;
			}
		});

		if (NumSyncs)
		{
			const uint64 Timeout = 10ULL * 1000 * 1000;

			// Figure out if we wait on Fences or TimelineSemaphores
			if (NextPayloads[0]->Fence)
			{
				FVulkanFence* Fences[NumQueues];
				int32 NumFences = 0;
				for (uint32 Index = 0; Index < NumSyncs; ++Index)
				{
					FVulkanFence* Fence = NextPayloads[Index]->Fence;
					checkf(Fence, TEXT("Payloads should all use the same types of syncs!"));
					if (!Fence->IsSignaled())
					{
						Fences[NumFences++] = Fence;
					}
				}

				// if one of these fences is already signaled, then that guarantees progress (no wait necessary on this pass)
				if (NumFences == NumSyncs)
				{
					Device->GetFenceManager().WaitForAnyFence(MakeArrayView<FVulkanFence*>(Fences, NumFences), Timeout);
				}
			}
			else
			{
				const int32 NumSemas = NumQueues + 1;  // add a timeline sema for CPU wake
				VkSemaphore Semaphores[NumSemas];
				uint64_t Values[NumSemas];
				uint32 NumWaits = 0;
				for (uint32 Index = 0; Index < NumSyncs; ++Index)
				{
					if (NextPayloads[Index]->TimelineSemaphoreValue > 0)
					{
						Semaphores[NumWaits] = NextPayloads[Index]->Queue.GetTimelineSemaphore()->GetHandle();
						Values[NumWaits] = (uint64_t)NextPayloads[Index]->TimelineSemaphoreValue;
						NumWaits++;
					}
					else
					{
						checkf(NextPayloads[Index]->CommandBuffers.Num() == 0, TEXT("TimelineSemaphoreValue should only be 0 on unused queues."));
					}
				}

				if (CPUTimelineSemaphore)
				{
					Semaphores[NumWaits] = CPUTimelineSemaphore->GetHandle();
					Values[NumWaits] = (uint64_t)(CPUTimelineSemaphoreValue);
					NumWaits++;
				}

				VkSemaphoreWaitInfo WaitInfo;
				ZeroVulkanStruct(WaitInfo, VK_STRUCTURE_TYPE_SEMAPHORE_WAIT_INFO);
				WaitInfo.flags = VK_SEMAPHORE_WAIT_ANY_BIT;
				WaitInfo.semaphoreCount = NumWaits;
				WaitInfo.pSemaphores = Semaphores;
				WaitInfo.pValues = Values;
				VERIFYVULKANRESULT_EXPANDED(VulkanRHI::vkWaitSemaphoresKHR(Device->GetHandle(), &WaitInfo, (uint64_t)Timeout));
			}

			// Some work completed and we made progress, or we stopped waiting because of CPUTimelineSemaphore
			// Either way, we know there is work to wait on so make sure we loop back in here
			bProgress = true;
		}
	}

	return bProgress || ProcessInterruptQueue();
}

bool FVulkanDynamicRHI::ProcessInterruptQueue()
{
	SCOPED_NAMED_EVENT_TEXT("ProcessInterruptQueue", FColor::Magenta);

	bool bProgress = false;
	Device->ForEachQueue([&bProgress](FVulkanQueue& Queue)
	{
		bProgress |= Queue.ProcessInterruptQueue(0) > 0;
	});

	if (GGPUCrashDetected.load())
	{
		TerminateOnGPUCrash(TEXT("ProcessInterruptQueue"));
	}

	return bProgress;
}

void FVulkanDynamicRHI::ProcessInterruptQueueUntil(FVulkanSyncPointRef SyncPoint)
{
	checkf(SyncPoint.IsValid(), TEXT("Waiting on invalid sync point."));
	ProcessInterruptQueueUntil(SyncPoint->GetGraphEvent());
}

void FVulkanDynamicRHI::ProcessInterruptQueueUntil(FGraphEvent* GraphEvent)
{
	if (InterruptThread)
	{
		if (GraphEvent && !GraphEvent->IsComplete())
		{
			GraphEvent->Wait();
		}
	}
	else if (GraphEvent)
	{
		if (Device->UseMinimalSubmits())
		{
			KickSubmissionThread(true);
		}

		// If we're waiting for an event, accumulate the idle time
		UE::Stats::FThreadIdleStats::FScopeIdle IdleScope;

		// If we have a sync point, keep processing until the sync point is signaled.
		while (!GraphEvent->IsComplete())
		{
			if (InterruptCS.TryLock())
			{
				TGuardValue<uint32> Guard(InterruptThreadID, FPlatformTLS::GetCurrentThreadId());
				WaitAndProcessInterruptQueue();
				InterruptCS.Unlock();
			}
			else
			{
				// Failed to get the lock. Another thread is processing the interrupt queue. Try again...
				FPlatformProcess::SleepNoStats(0);
			}
		}
	}
	else
	{
		// Process until no more progress is being made, or skip if another thread is processing already
		bool bProgress;
		do
		{
			bProgress = false;
			if (InterruptCS.TryLock())
			{
				TGuardValue<uint32> Guard(InterruptThreadID, FPlatformTLS::GetCurrentThreadId());
				bProgress = ProcessInterruptQueue();
				InterruptCS.Unlock();
			}

		} while (bProgress);
	}
}

void FVulkanDynamicRHI::RHISubmitCommandLists(FRHISubmitCommandListsArgs&& Args)
{
	for (IRHIPlatformCommandList* CommandList : Args.CommandLists)
	{
		FVulkanPlatformCommandList* PlatformCmdList = ResourceCast(CommandList);
		PendingPayloadsForSubmission.Enqueue(PlatformCmdList);
	}

	KickSubmissionThread();

	ProcessInterruptQueueUntil(nullptr);
}

void FVulkanDynamicRHI::EnqueueEndOfPipeTask(TUniqueFunction<void()> TaskFunc, TUniqueFunction<void(FVulkanPayload&)> ModifyPayloadCallback)
{
	FGraphEventArray Prereqs;
	Prereqs.Reserve((int32)EVulkanQueueType::Count + 1);
	if (EopTask.IsValid())
	{
		Prereqs.Add(EopTask);
	}

	FVulkanPlatformCommandList* Payloads = new FVulkanPlatformCommandList;
	Payloads->Reserve((int32)EVulkanQueueType::Count);

	Device->ForEachQueue([&](FVulkanQueue& Queue)
	{
		FVulkanPayload* Payload = new FVulkanPayload(Queue);

		FVulkanSyncPointRef SyncPoint = FVulkanSyncPoint::Create(EVulkanSyncPointType::GPUAndCPU, TEXT("VulkanEndOfPipeTask"));
		Payload->SyncPoints.Add(SyncPoint);
		Prereqs.Add(SyncPoint->GetGraphEvent());

		if (ModifyPayloadCallback)
		{
			ModifyPayloadCallback(*Payload);
		}

		Payloads->Add(Payload);
	});

	PendingPayloadsForSubmission.Enqueue(Payloads);

	KickSubmissionThread();

	EopTask = FFunctionGraphTask::CreateAndDispatchWhenReady(
		MoveTemp(TaskFunc),
		QUICK_USE_CYCLE_STAT(FExecuteRHIThreadTask, STATGROUP_TaskGraphTasks),
		&Prereqs
	);
}


void FVulkanDynamicRHI::RHIBlockUntilGPUIdle()
{
	TArray<FVulkanSyncPointRef, TInlineAllocator<(int32)EVulkanQueueType::Count>> EndOfQueueEvent;

	// Create payloads with a signal for each queue
	{
		FVulkanPlatformCommandList* Payloads = new FVulkanPlatformCommandList;
		Payloads->Reserve((int32)EVulkanQueueType::Count);

		Device->ForEachQueue([&EndOfQueueEvent, &Payloads](FVulkanQueue& Queue)
		{
			FVulkanPayload* Payload = new FVulkanPayload(Queue);
			FVulkanSyncPointRef SyncPoint = FVulkanSyncPoint::Create(EVulkanSyncPointType::GPUAndCPU, TEXT("BlockUntilGPUIdle"));
			Payload->SyncPoints.Add(SyncPoint);
			Payloads->Add(Payload);

			EndOfQueueEvent.Add(SyncPoint);
		});

		PendingPayloadsForSubmission.Enqueue(Payloads);
	}

	KickSubmissionThread();

	// Wait on each event
	for (FVulkanSyncPointRef& Event : EndOfQueueEvent)
	{
		ProcessInterruptQueueUntil(Event);
	}
}


// Resolve any pending actions and delete the payload
void FVulkanDynamicRHI::CompletePayload(FVulkanPayload* Payload)
{
	for (FVulkanCommandBuffer* CommandBuffer : Payload->CommandBuffers)
	{
		CommandBuffer->Reset();
	}

	for (FVulkanDescriptorPoolSetContainer* DescriptorPoolSetContainer : Payload->DescriptorPoolSetContainers)
	{
		Device->GetDescriptorPoolsManager().ReleasePoolSet(*DescriptorPoolSetContainer);
	}

	struct FQueryResult
	{
		uint64 Result;
		uint64 Availability;
	};

	static TArray<FQueryResult> TempResults;

	// Fallback for devices without calibrated_timestamps (or with unexpected values from calibration)
	static bool bBypassTiming = false;
	static uint64 ReferenceCPUTimestamp = 0;
	static uint64 ReferenceGPUTimestamp = 0;

	for (TArray<FVulkanQueryPool*>& QueryPoolArray : Payload->QueryPools)
	{
		for (FVulkanQueryPool* QueryPool : QueryPoolArray)
		{
			check(QueryPool->CurrentQueryCount);

			// We need one for the result and one for availablility
			if (TempResults.Num() < (int32)QueryPool->GetMaxQueries())
			{
				TempResults.SetNumZeroed(QueryPool->GetMaxQueries());
			}

			// Using VK_QUERY_RESULT_WAIT_BIT can cause infinite waits on some versions of Adreno if the device is lost, 
			// regardless of spec saying it'll eventually return.  So create a loop which shouldn't be too costly since
			// the fence was reached guarateeing that all these queries are done anyhow.
			VkResult Result = VK_SUCCESS;
			while (Result >= VK_SUCCESS)
			{
				const VkQueryResultFlags QueryFlags = VK_QUERY_RESULT_64_BIT | VK_QUERY_RESULT_WITH_AVAILABILITY_BIT;
				Result = VulkanRHI::vkGetQueryPoolResults(
					Device->GetHandle(),
					QueryPool->GetHandle(),
					0,
					QueryPool->CurrentQueryCount * QueryPool->QueryStride,
					TempResults.Num() * TempResults.GetTypeSize(),
					(uint64_t*)TempResults.GetData(),
					sizeof(FQueryResult),
					QueryFlags);

				if (Result == VK_SUCCESS)
				{
					break;
				}
				else if (GGPUCrashDetected.load())
				{
					// The GPU is crashed and we are the current interrupt thread, 
					// terminate right away if we crashed since no more progress is possible
					TerminateOnGPUCrash(TEXT("CompletePayload"));
				}
				else
				{
					FPlatformProcess::SleepNoStats(0.005f);
					VERIFYVULKANRESULT_EXPANDED(Result);
				}
			}

			// In multiview, one "query" occupies multiple slots, one for each view.
			// For queries issued with vkCmdBeginQuery/vkCmdEndQuery, the slots contain
			// implementation-defined values, but their sum is guaranteed to represent
			// the total of the query over all views. For queries issued with
			// vkCmdWriteTimestamp, either the total over all views is written to the
			// first slot and all other slots are 0, or each slot contains the timestamp
			// for its associated view. In those cases, we want the maximum timestamp
			// across all views.
			const bool bUseMaxValue = QueryPool->GetPoolType() == EVulkanQueryPoolType::Timestamp;

			for (uint32 QueryIndex = 0; QueryIndex < QueryPool->CurrentQueryCount; ++QueryIndex)
			{
					uint64* ResultPtr = QueryPool->QueryResults[QueryIndex];
				if (!ResultPtr)
					{
					continue;
				}

					uint64 QueryResult = 0;
					for (uint32 ViewIndex = 0; ViewIndex < QueryPool->QueryStride; ++ViewIndex)
					{
						uint32 ResultIndex = QueryIndex * QueryPool->QueryStride + ViewIndex;
						if (TempResults[ResultIndex].Availability)
						{
							if (bUseMaxValue)
							{
								QueryResult = FMath::Max(QueryResult, TempResults[ResultIndex].Result);
							}
							else
							{
								QueryResult += TempResults[ResultIndex].Result;
							}
						}
						else
						{
							UE_LOGF(LogVulkanRHI, Warning, "Could not read back query: [PoolType=%d, Index=%d, View=%d]", (int32)QueryPool->GetPoolType(), QueryIndex, ViewIndex);
						}
					}

						if (QueryPool->GetPoolType() == EVulkanQueryPoolType::Timestamp)
						{
							// Convert from GPU timestamp to CPU timestamp (relative to FPlatformTime::Cycles64())
							checkSlow(Payload->Queue.Timing);
							const FVulkanTiming* CurrentTiming = Payload->Queue.Timing;

							uint64 CPUCalibrationTimestamp = CurrentTiming->CPUTimestamp;
							uint64 GPUCalibrationTimestamp = CurrentTiming->GPUTimestamp;

							// NOTE: Adreno GPU calibration values aren't valid, create a fallback
						if (bBypassTiming || (QueryResult < GPUCalibrationTimestamp) || (GPUCalibrationTimestamp == 0))
							{
								if (!bBypassTiming)
								{
									static bool bWarnOnce = false;
									if (!bWarnOnce)
									{
										bWarnOnce = true;
										UE_LOGF(LogVulkanRHI, Warning,
											"GPU timestamp can't be moved to CPU timeline.  " "Calibration data is invalid (Curr=%lld  Cal=%lld)!  " "CPU and GPU profiling data will not line up!",
										QueryResult, GPUCalibrationTimestamp);
									}

									const uint64 CurrentTimestamp = FPlatformTime::Cycles64();

							ReferenceCPUTimestamp = ((CPUCalibrationTimestamp != 0) && (CPUCalibrationTimestamp < CurrentTimestamp))
								? CPUCalibrationTimestamp
								: CurrentTimestamp;
									
								ReferenceGPUTimestamp = QueryResult;
									bBypassTiming = true;
								}

								CPUCalibrationTimestamp = ReferenceCPUTimestamp;
								GPUCalibrationTimestamp = ReferenceGPUTimestamp;

								// :todo: Clamp results within the imperfect calibration timestamps (to prevent negative values below)
							QueryResult = FMath::Max(QueryResult, GPUCalibrationTimestamp);
							}

						const uint64 GPUDelta = QueryResult - GPUCalibrationTimestamp;
							const uint64 CPUDelta = (GPUDelta * CurrentTiming->CPUFrequency) / CurrentTiming->GPUFrequency;

							(*ResultPtr) = CPUCalibrationTimestamp + CPUDelta;
						}
						else
						{
						(*ResultPtr) = QueryResult;
						}
					}

			QueryPool->QueryRefs.Empty(QueryPool->GetMaxQueries());
			Device->ReleaseQueryPool(QueryPool);
		}
		QueryPoolArray.Reset();
	}

#if WITH_RHI_BREADCRUMBS
	// For NV checkpoints, extend the lifetime of breadcrumbs by keeping an array of all the allocators
	// populated until the next frame
	if (Device->GetOptionalExtensions().HasNVDiagnosticCheckpoints &&
		Payload->Queue.GetDiagnosticBuffer() && 
		Payload->Queue.GetDiagnosticBuffer()->ExtendedBreadcrumbAllocators.Num())
	{
		FVulkanDiagnosticBuffer& DiagnosticBuffer = *Payload->Queue.GetDiagnosticBuffer();

		if (Payload->BreadcrumbAllocators.IsValid())
		{
			for (TSharedRef<FRHIBreadcrumbAllocator>& Allocator : *Payload->BreadcrumbAllocators)
			{
				DiagnosticBuffer.ExtendedBreadcrumbAllocators[DiagnosticBuffer.CurrentExtendedIndex].AddUnique(&Allocator.Get());
			}
		}

		if (Payload->EndFrameEvent.IsSet())
		{
			DiagnosticBuffer.CurrentExtendedIndex = (DiagnosticBuffer.CurrentExtendedIndex + 1) % DiagnosticBuffer.ExtendedBreadcrumbAllocators.Num();
			DiagnosticBuffer.ExtendedBreadcrumbAllocators[DiagnosticBuffer.CurrentExtendedIndex].Reset();
		}
	}
#endif // WITH_RHI_BREADCRUMBS

	if (!Payload->EventStream.IsEmpty())
	{
		check(Payload->Queue.Timing);
		Payload->Queue.Timing->EventStream.Append(MoveTemp(Payload->EventStream));
	}

	if (Payload->Timing.IsSet())
	{
		// Switch the new timing struct into the queue. This redirects timestamp results to separate each frame's work.
		Payload->Queue.Timing = Payload->Timing.GetValue();

		// Reset the fallback to recalibrated it every frame (it remains innacurate, but at least we limit drift)
		bBypassTiming = false;
		ReferenceCPUTimestamp = 0;
		ReferenceGPUTimestamp = 0;
	}

	for (FVulkanSyncPointRef& SyncPoint : Payload->SyncPoints)
	{
		SyncPoint->GetGraphEvent()->DispatchSubsequents();
	}
	Payload->SyncPoints.Empty();
}

void FVulkanPayload::PreExecute()
{
	if (PreExecuteCallback)
	{
		PreExecuteCallback(Queue.GetHandle());
	}
}

void FVulkanDynamicRHI::RHIRunOnQueue(EVulkanRHIRunOnQueueType QueueType, TFunction<void(VkQueue)>&& CodeToRun, bool bWaitForSubmission)
{
	FGraphEventRef SubmissionEvent;

	FVulkanQueue* Queue = nullptr;
	switch (QueueType)
	{
	case EVulkanRHIRunOnQueueType::Graphics:
		Queue = Device->GetQueue(EVulkanQueueType::Graphics);
		break;
	case EVulkanRHIRunOnQueueType::Transfer:
		Queue = Device->GetQueue(EVulkanQueueType::Transfer);
		break;
	default:
		checkf(false, TEXT("Unknown EVulkanRHIRunOnQueueType, skipping call."));
		return;
	}

	FVulkanPlatformCommandList* Payloads = new FVulkanPlatformCommandList;
	FVulkanPayload* Payload = new FVulkanPayload(*Queue);
	Payloads->Add(Payload);

	Payload->PreExecuteCallback = MoveTemp(CodeToRun);

	if (bWaitForSubmission)
	{
		SubmissionEvent = FGraphEvent::CreateGraphEvent();
		Payload->SubmissionEvents.Add(SubmissionEvent);
	}

	PendingPayloadsForSubmission.Enqueue(Payloads);
	KickSubmissionThread(bWaitForSubmission);

	// Use this opportunity to pump the interrupt queue
	ProcessInterruptQueueUntil(nullptr);

	if (SubmissionEvent && !SubmissionEvent->IsComplete())
	{
		SubmissionEvent->Wait();
	}
}

void FVulkanDynamicRHI::ProcessInterruptQueueOnGPUCrash()
{
	// This function will not return.
	GGPUCrashDetected.store(true);

	if (InterruptThread)
	{
		// Since we have an interrupt thread, allow it to process the GPU crash.
		// This is necessary so it can retrieve all the active payloads for resolving breadcrumbs.
		InterruptThread->Kick();

		// Wait for the interrupt thread to exit (which will never happen).
		InterruptThread->Join();
	}
	else
	{
		// If we have no interrupt thread, assume ownership on the current thread
		// (or block forever on the scope lock if multiple threads enter this function).
		FScopeLock Lock(&InterruptCS);
		TGuardValue<uint32> Guard(InterruptThreadID, FPlatformTLS::GetCurrentThreadId());

		while (true)
		{
			ProcessInterruptQueue();
		}
	}
}

bool FVulkanDynamicRHI::IsInInterruptThread() const
{
	uint32 ThisThreadID = FPlatformTLS::GetCurrentThreadId();

	// If we don't have a dedicated interrupt thread, the thread currently acting
	// as the interrupt thread is tracked via the InterruptThreadID field.

	if (InterruptThread)
	{
		return ThisThreadID == InterruptThread->GetThreadID();
	}
	else
	{
		return ThisThreadID == InterruptThreadID;
	}
}