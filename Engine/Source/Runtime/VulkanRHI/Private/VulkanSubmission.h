// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanContext.h: Class to generate Vulkan command buffers from RHI CommandLists
=============================================================================*/

#pragma once 

#include "VulkanRHIPrivate.h"
#include "VulkanQuery.h"

class FVulkanDevice;
class FVulkanDynamicRHI;
class FVulkanFence;
class FVulkanPendingGfxState;
class FVulkanPendingComputeState;
class FVulkanQueue;
class FVulkanSwapChain;
class FVulkanSemaphore;
class FVulkanTiming;
class FVulkanDescriptorPoolSetContainer;


enum class EVulkanSyncPointType
{
	// Sync points of this type do not include an FGraphEvent, so cannot
	// report completion to the CPU (via either IsComplete() or Wait())
	// The will resolve into binary or timeline semaphores at submit time.
	//GPUOnly,  :todo-jn: Move GPU syncs to new FVulkanSyncPoint class

	// Sync points of this type include an FGraphEvent. The IsComplete() and Wait() functions
	// can be used to poll for completion from the CPU, or block the CPU, respectively.
	GPUAndCPU,

	// Same as GPUAndCPU but not time critical and can be pushed back up until the end of the frame.
	// Generally meant to be used for work that can happen at any point after the command buffers
	// are done executing on the GPU.
	Context,
};

class FVulkanSyncPoint;
using FVulkanSyncPointRef = TRefCountPtr<FVulkanSyncPoint>;

//
// A sync point is a logical point on a GPU queue's timeline that can be awaited by other queues, or the CPU.
// These are used throughout the RHI as a way to abstract the underlying Vulkan semaphores and fences. 
// The submission thread manages the underlying semaphores, fences and signaled values, and reports completion 
// to the relevant sync points via an FGraphEvent.
//
// Sync points are one-shot, meaning they represent a single timeline point, and are released after use, via ref-counting.
// Use FVulkanSyncPoint::Create() to make a new sync point and hold a reference to it via a FVulkanSyncPointRef object.
//
class FVulkanSyncPoint final : public FRefCountedObject
{
	friend FVulkanDynamicRHI;
	friend FVulkanQueue;

	static TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> MemoryPool;

	// No copying or moving
	FVulkanSyncPoint(FVulkanSyncPoint const&) = delete;
	FVulkanSyncPoint(FVulkanSyncPoint&&) = delete;

	FVulkanSyncPoint(EVulkanSyncPointType InType, const TCHAR* InDebugName)
		: Type(InType)
#if RHI_USE_SYNC_POINT_DEBUG_NAME
		, DebugName(InDebugName)
#endif
	{
		if ((Type == EVulkanSyncPointType::GPUAndCPU) || (Type == EVulkanSyncPointType::Context))
		{
			GraphEvent = FGraphEvent::CreateGraphEvent();
		}
	}

public:
	static FVulkanSyncPointRef Create(EVulkanSyncPointType Type, const TCHAR* InDebugName)
	{
		LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateSyncPoint"));
		return new FVulkanSyncPoint(Type, InDebugName);
	}

	bool IsComplete() const
	{
		checkf(GraphEvent, TEXT("This sync point was not created with a CPU event. Cannot check completion on the CPU."));
		return GraphEvent->IsComplete();
	}

	void Wait() const;

	FGraphEvent* GetGraphEvent() const
	{
		checkf(GraphEvent, TEXT("This sync point was not created with a CPU event."));
		return GraphEvent;
	}

	EVulkanSyncPointType GetType() const
	{
		return Type;
	}

	void* operator new(size_t Size)
	{
		check(Size == sizeof(FVulkanSyncPoint));

		void* Memory = MemoryPool.Pop();
		if (!Memory)
		{
			Memory = FMemory::Malloc(sizeof(FVulkanSyncPoint), alignof(FVulkanSyncPoint));
		}
		return Memory;
	}

	void operator delete(void* Pointer)
	{
		MemoryPool.Push(Pointer);
	}

	inline const TCHAR* GetDebugName() const
	{
#if RHI_USE_SYNC_POINT_DEBUG_NAME
		return DebugName;
#else
		return TEXT("SyncPointBusyWait");
#endif
	}

private:
	FGraphEventRef GraphEvent;
	EVulkanSyncPointType Type;

#if RHI_USE_SYNC_POINT_DEBUG_NAME
	const TCHAR* DebugName = nullptr;
#endif
};



struct FVulkanCommitReservedResourceDesc
{
	TRefCountPtr<FRHIResource> Resource = nullptr;
	uint64 CommitSizeInBytes = 0;
};



class FVulkanPayload
{
	friend class FVulkanCommandListContext;
	friend class FVulkanContextCommon;
	friend class FVulkanDynamicRHI;
	friend class FVulkanQueue;

public:
	FVulkanPayload(FVulkanQueue& InQueue)
		: Queue(InQueue)
		, EventStream(InQueue.GetProfilerQueue())
	{}

	~FVulkanPayload();

protected:

	// Must only be used to merge Secondary or Parallel command buffers
	void Merge(FVulkanPayload& Other);

	void PreExecute();

	FVulkanQueue& Queue;
	TArray<VkPipelineStageFlags> WaitFlags;  // flags that match 1:1 with WaitSemaphores
	TArray<TRefCountPtr<FVulkanSemaphore>> WaitSemaphores; // wait before command buffers
	TArray<FVulkanCommandBuffer*> CommandBuffers;
	TArray<TRefCountPtr<FVulkanSemaphore>> SignalSemaphores; // signaled after command buffers

	// Signaled when the payload has been submitted to the GPU queue
	TArray<FGraphEventRef> SubmissionEvents;

	// For internal completion tracking of the payload
	uint64 TimelineSemaphoreValue = 0;
	FVulkanFence* Fence = nullptr;

	// Used to sync other CPU work to payload completion
	TArray<FVulkanSyncPointRef> SyncPoints;

	// Queries used in the command lists of this payload
	TStaticArray<TArray<FVulkanQueryPool*>, (int32)EVulkanQueryPoolType::Count> QueryPools;

	// Used DescriptorPoolSetContainers that can be cleaned up on completion
	TArray<FVulkanDescriptorPoolSetContainer*> DescriptorPoolSetContainers;

	// Used by RHIRunOnQueue
	TFunction<void(VkQueue)> PreExecuteCallback;

#if WITH_RHI_BREADCRUMBS
	FRHIBreadcrumbRange BreadcrumbRange{};
	TSharedPtr<FRHIBreadcrumbAllocatorArray> BreadcrumbAllocators{};
#endif

	TOptional<FVulkanTiming*> Timing;
	TOptional<uint64> ExternalGPUTime;
	TOptional<UE::RHI::GPUProfiler::FEvent::FFrameBoundary> EndFrameEvent;
	UE::RHI::GPUProfiler::FEventStream EventStream;

	// UpdateReservedResources
	TArray<FVulkanCommitReservedResourceDesc> ReservedResourcesToCommit;
};

struct FVulkanPlatformCommandList : public IRHIPlatformCommandList, public TArray<FVulkanPayload*>
{
};

template<>
struct TVulkanResourceTraits<IRHIPlatformCommandList>
{
	typedef FVulkanPlatformCommandList TConcreteType;
};

