// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanSwapChain.h: Vulkan viewport RHI definitions.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "VulkanConfiguration.h"
#include "VulkanThirdParty.h"

class FVulkanDevice;
class FVulkanFence;
class FVulkanQueue;
class FVulkanSemaphore;
class FVulkanTexture;
class FVulkanView;
enum EPixelFormat : uint8;


struct FVulkanSwapChainRecreateInfo
{
	VkSwapchainKHR SwapChain;
	VkSurfaceKHR Surface;
};


class FVulkanSwapChain
{
	FVulkanSwapChain(VkInstance InInstance, FVulkanDevice& InDevice, VkSurfaceKHR InSurface, VkSwapchainKHR InSwapChain, 
		VkSurfaceTransformFlagBitsKHR PreTransform, uint32 InWidth, uint32 InHeight, bool InIsFullscreen, uint32 NumSwapChainImages,
		int8 InLockToVsync, FVulkanPlatformWindowContext& WindowContext);

public:
	static FVulkanSwapChain* Create(
		VkInstance InInstance, FVulkanDevice& Device, EPixelFormat& InOutPixelFormat,
		uint32 Width, uint32 Height, bool bIsFullScreen,
		uint32* InOutDesiredNumBackBuffers, TArray<VkImage>& OutImages, int8 LockToVsync,
		FVulkanPlatformWindowContext& WindowContext, FVulkanSwapChainRecreateInfo* RecreateInfo);

	void Destroy(FVulkanSwapChainRecreateInfo* RecreateInfo);

	// Has to be negative as we use this also on other callbacks as the acquired image index
	enum class EStatus
	{
		Healthy = 0,
		OutOfDate = -1,
		SurfaceLost = -2,
		NotReady = -3,
	};
	EStatus Present(FVulkanQueue* PresentQueue, FVulkanSemaphore* BackBufferRenderingDoneSemaphore);

	inline int8 DoesLockToVsync() { return LockToVsync; }

	inline VkSurfaceTransformFlagBitsKHR GetCachedSurfaceTransform() const { return PreTransform; }

	/** Reset the current image index when a present is skipped (e.g. window destroyed). */
	void ResetCurrentImageIndex() { CurrentImageIndex = -1; }

protected:
	const VkInstance Instance;
	FVulkanDevice& Device;

	VkSurfaceKHR Surface = VK_NULL_HANDLE;
	VkSwapchainKHR SwapChain = VK_NULL_HANDLE;

	const VkSurfaceTransformFlagBitsKHR PreTransform;

	uint32 InternalWidth = 0;
	uint32 InternalHeight = 0;
	bool bInternalFullScreen = false;

	void* WindowHandle = nullptr;
	// Platform window id captured at creation (0 if untracked). Used to release per-window RHI
	// resources by a stable id at destroy time instead of by the WindowHandle pointer.
	uint32 WindowId = 0;

	int32 CurrentImageIndex = -1;
	int32 SemaphoreIndex = 0;
	uint32 NumPresentCalls = 0;
	uint32 NumAcquireCalls = 0;

	uint32 RTPacingSampleCount = 0;
	double RTPacingPreviousFrameCPUTime = 0;
	double RTPacingSampledDeltaTimeMS = 0;
	
	double NextPresentTargetTime = 0;

	TArray<TRefCountPtr<FVulkanSemaphore>> ImageAcquiredSemaphore;
#if VULKAN_USE_IMAGE_ACQUIRE_FENCES
	TArray<FVulkanFence*> ImageAcquiredFences;
#endif
	int8 LockToVsync = 0;

	uint32 PresentID = 0;

	int32 AcquireImageIndex(TRefCountPtr<FVulkanSemaphore>* OutSemaphore);


	friend class FVulkanViewport;
	friend class FVulkanQueue;
};

