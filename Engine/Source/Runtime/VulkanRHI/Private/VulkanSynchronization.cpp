// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanRHIPrivate.h"
#include "VulkanSynchronization.h"
#include "VulkanDevice.h"


FVulkanFence::FVulkanFence(FVulkanDevice& InDevice, FVulkanFenceManager& InOwner, bool bCreateSignaled)
	: State(bCreateSignaled ? FVulkanFence::EState::Signaled : FVulkanFence::EState::NotReady)
	, Owner(InOwner)
{
	VkFenceCreateInfo Info;
	ZeroVulkanStruct(Info, VK_STRUCTURE_TYPE_FENCE_CREATE_INFO);
	Info.flags = bCreateSignaled ? VK_FENCE_CREATE_SIGNALED_BIT : 0;
	VERIFYVULKANRESULT(VulkanRHI::vkCreateFence(InDevice.GetHandle(), &Info, VULKAN_CPU_ALLOCATOR, &Handle));
}

FVulkanFence::~FVulkanFence()
{
	checkf(Handle == VK_NULL_HANDLE, TEXT("Didn't get properly destroyed by FVulkanFenceManager!"));
}



FVulkanFenceManager::~FVulkanFenceManager()
{
	ensure(UsedFences.Num() == 0);
}

void FVulkanFenceManager::DestroyFence(FVulkanFence* Fence)
{
	// Does not need to go in the deferred deletion queue
	VulkanRHI::vkDestroyFence(Device.GetHandle(), Fence->GetHandle(), VULKAN_CPU_ALLOCATOR);
	Fence->Handle = VK_NULL_HANDLE;
	delete Fence;
}

void FVulkanFenceManager::Deinit()
{
	FScopeLock Lock(&FenceLock);
	ensureMsgf(UsedFences.Num() == 0, TEXT("No all fences are done!"));
	for (FVulkanFence* Fence : FreeFences)
	{
		DestroyFence(Fence);
	}
}

FVulkanFence* FVulkanFenceManager::AllocateFence(bool bCreateSignaled)
{
	FScopeLock Lock(&FenceLock);
	if (FreeFences.Num() != 0)
	{
		FVulkanFence* Fence = FreeFences[0];
		FreeFences.RemoveAtSwap(0, EAllowShrinking::No);
		UsedFences.Add(Fence);

		if (bCreateSignaled)
		{
			Fence->State = FVulkanFence::EState::Signaled;
		}
		return Fence;
	}

	FVulkanFence* NewFence = new FVulkanFence(Device, *this, bCreateSignaled);
	UsedFences.Add(NewFence);
	return NewFence;
}

// Sets it to nullptr
void FVulkanFenceManager::ReleaseFence(FVulkanFence*& Fence)
{
	FScopeLock Lock(&FenceLock);
	ResetFence(Fence);
	UsedFences.RemoveSingleSwap(Fence, EAllowShrinking::No);
#if VULKAN_REUSE_FENCES
	FreeFences.Add(Fence);
#else
	DestroyFence(Fence);
#endif
	Fence = nullptr;
}

void FVulkanFenceManager::WaitAndReleaseFence(FVulkanFence*& Fence, uint64 TimeInNanoseconds)
{
	FScopeLock Lock(&FenceLock);
	if (!Fence->IsSignaled())
	{
		WaitForFence(Fence, TimeInNanoseconds);
	}

	ResetFence(Fence);
	UsedFences.RemoveSingleSwap(Fence, EAllowShrinking::No);
	FreeFences.Add(Fence);
	Fence = nullptr;
}

bool FVulkanFenceManager::CheckFenceState(FVulkanFence* Fence)
{
	check(UsedFences.Contains(Fence));
	check(Fence->State == FVulkanFence::EState::NotReady);
	const VkResult Result = VulkanRHI::vkGetFenceStatus(Device.GetHandle(), Fence->Handle);
	switch (Result)
	{
	case VK_SUCCESS:
		Fence->State = FVulkanFence::EState::Signaled;
		return true;

	case VK_NOT_READY:
		break;

	default:
		VERIFYVULKANRESULT(Result);
		break;
	}

	return false;
}

bool FVulkanFenceManager::WaitForFence(FVulkanFence* Fence, uint64 TimeInNanoseconds)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanWaitFence);
#endif

	check(UsedFences.Contains(Fence));
	check(Fence->State == FVulkanFence::EState::NotReady);
	VkResult Result = VulkanRHI::vkWaitForFences(Device.GetHandle(), 1, &Fence->Handle, true, TimeInNanoseconds);
	switch (Result)
	{
	case VK_SUCCESS:
		Fence->State = FVulkanFence::EState::Signaled;
		return true;
	case VK_TIMEOUT:
		break;
	default:
		VERIFYVULKANRESULT(Result);
		break;
	}

	return false;
}

bool FVulkanFenceManager::WaitForAnyFence(TArrayView<FVulkanFence*> Fences, uint64 TimeInNanoseconds)
{
	// Mostly used for waits on queues, inlinealloc to that size
	TArray<VkFence, TInlineAllocator<(int32)EVulkanQueueType::Count>> FenceHandles;
	for (FVulkanFence* Fence : Fences)
	{
		if (Fence && !Fence->IsSignaled())
		{
			check(UsedFences.Contains(Fence));
			FenceHandles.Add(Fence->GetHandle());
		}
	}

	const VkResult Result = VulkanRHI::vkWaitForFences(Device.GetHandle(), FenceHandles.Num(), FenceHandles.GetData(), false, TimeInNanoseconds);
	switch (Result)
	{
	case VK_SUCCESS:
		if (Fences.Num() == 1)
		{
			Fences[0]->State = FVulkanFence::EState::Signaled;
		}
		return true;
	case VK_TIMEOUT:
		break;
	default:
		VERIFYVULKANRESULT(Result);
		break;
	}

	return false;
}

void FVulkanFenceManager::ResetFence(FVulkanFence* Fence)
{
	if (Fence->State != FVulkanFence::EState::NotReady)
	{
		VERIFYVULKANRESULT(VulkanRHI::vkResetFences(Device.GetHandle(), 1, &Fence->Handle));
		Fence->State = FVulkanFence::EState::NotReady;
	}
}




FVulkanSemaphore::FVulkanSemaphore(FVulkanDevice& InDevice, EVulkanSemaphoreFlags InFlags, uint64 InInitialTimelineValue)
	: Device(InDevice)
	, SemaphoreHandle(VK_NULL_HANDLE)
	, Flags(InFlags)
{
	// Create semaphore
	VkSemaphoreCreateInfo CreateInfo;
	ZeroVulkanStruct(CreateInfo, VK_STRUCTURE_TYPE_SEMAPHORE_CREATE_INFO);

	VkSemaphoreTypeCreateInfo TypeCreateInfo;
	if (IsTimeline())
	{
		checkf(InDevice.GetOptionalExtensions().HasKHRTimelineSemaphore, TEXT("Trying to create timeline semaphore without device support!"));
		ZeroVulkanStruct(TypeCreateInfo, VK_STRUCTURE_TYPE_SEMAPHORE_TYPE_CREATE_INFO);
		TypeCreateInfo.semaphoreType = VK_SEMAPHORE_TYPE_TIMELINE;
		TypeCreateInfo.initialValue = (uint64_t)InInitialTimelineValue;
		CreateInfo.pNext = &TypeCreateInfo;
	}

	VERIFYVULKANRESULT(VulkanRHI::vkCreateSemaphore(Device.GetHandle(), &CreateInfo, VULKAN_CPU_ALLOCATOR, &SemaphoreHandle));
}

FVulkanSemaphore::FVulkanSemaphore(FVulkanDevice& InDevice, const VkSemaphore& InExternalSemaphore)
	: Device(InDevice)
	, SemaphoreHandle(InExternalSemaphore)
	, Flags(EVulkanSemaphoreFlags::ExternallySignaled)
{}

FVulkanSemaphore::~FVulkanSemaphore()
{
	check(SemaphoreHandle != VK_NULL_HANDLE);
	if (EnumHasAnyFlags(Flags, EVulkanSemaphoreFlags::ImmediateDeletion))
	{
		VulkanRHI::vkDestroySemaphore(Device.GetHandle(), SemaphoreHandle, VULKAN_CPU_ALLOCATOR);
	}
	else if (!IsExternallyOwned())
	{
		Device.GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::Semaphore, SemaphoreHandle);
	}
	SemaphoreHandle = VK_NULL_HANDLE;
}

bool FVulkanSemaphore::WaitForTimelineSemaphoreValue(uint64 Value, uint64 Timeout)
{
	checkf(IsTimeline(), TEXT("WaitForTimelineSemaphoreValue not available!  Semaphore was not created with EVulkanSemaphoreFlags::Timeline."));
	checkSlow(SemaphoreHandle != VK_NULL_HANDLE);
	VkSemaphoreWaitInfo WaitInfo;
	ZeroVulkanStruct(WaitInfo, VK_STRUCTURE_TYPE_SEMAPHORE_WAIT_INFO);
	WaitInfo.semaphoreCount = 1;
	WaitInfo.pSemaphores = &SemaphoreHandle;
	WaitInfo.pValues = (uint64_t*)&Value;
	const VkResult Result = VulkanRHI::vkWaitSemaphoresKHR(Device.GetHandle(), &WaitInfo, Timeout);
	switch (Result)
	{
	case VK_SUCCESS:
		return true;
	case VK_TIMEOUT:
		break;
	default:
		VERIFYVULKANRESULT(Result);
		break;
	}
	return false;
}

void FVulkanSemaphore::GetTimelineSemaphoreValue(uint64& OutValue)
{
	checkf(IsTimeline(), TEXT("GetTimelineSemaphoreValue not available!  Semaphore was not created with EVulkanSemaphoreFlags::Timeline."));
	uint64_t TempValue = 0;
	VERIFYVULKANRESULT(VulkanRHI::vkGetSemaphoreCounterValueKHR(Device.GetHandle(), SemaphoreHandle, &TempValue));
	if (TempValue != MAX_uint64)
	{
		OutValue = TempValue;
	}
	else
	{
		// Limit how many times the message is printed (but leave a few so it is visible)
		static int32 MessageCounter = 0;
		if (MessageCounter++ < 10)
		{
			UE_LOGF(LogVulkanRHI, Warning, "Timeline semaphore value is MAX_uint64 (this can happen when the device is lost).");
		}
	}
}
