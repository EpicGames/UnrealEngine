// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Containers/Array.h"
#include "HAL/CriticalSection.h"
#include "Templates/RefCounting.h"
#include "VulkanConfiguration.h"
#include "VulkanThirdParty.h"

class FVulkanDevice;
class FVulkanFenceManager;

// Internal Vulkan Fence class that wrap VkFence management
// (not to be confused with FVulkanGPUFence class that maps to RHI's FRHIFence) 
class FVulkanFence
{
public:
	FVulkanFence(FVulkanDevice& InDevice, FVulkanFenceManager& InOwner, bool bCreateSignaled);

	inline VkFence GetHandle() const
	{
		return Handle;
	}

	inline bool IsSignaled() const
	{
		return State == EState::Signaled;
	}

	FVulkanFenceManager& GetOwner()
	{
		return Owner;
	}

protected:
	VkFence Handle;

	enum class EState
	{
		// Initial state
		NotReady,

		// After GPU processed it
		Signaled,
	};

	EState State;

	FVulkanFenceManager& Owner;

	// Only owner can delete!
	~FVulkanFence();
	friend FVulkanFenceManager;
};

class FVulkanFenceManager
{
public:
	FVulkanFenceManager(FVulkanDevice& InDevice)
		: Device(InDevice)
	{
	}
	~FVulkanFenceManager();

	void Deinit();

	FVulkanFence* AllocateFence(bool bCreateSignaled = false);

	inline bool IsFenceSignaled(FVulkanFence* Fence)
	{
		if (Fence->IsSignaled())
		{
			return true;
		}

		return CheckFenceState(Fence);
	}

	// Returns false if it timed out
	bool WaitForAnyFence(TArrayView<FVulkanFence*> Fences, uint64 TimeInNanoseconds);
	bool WaitForFence(FVulkanFence* Fence, uint64 TimeInNanoseconds);

	void ResetFence(FVulkanFence* Fence);

	// Sets it to nullptr
	void ReleaseFence(FVulkanFence*& Fence);

	// Sets it to nullptr
	void WaitAndReleaseFence(FVulkanFence*& Fence, uint64 TimeInNanoseconds);

protected:
	FVulkanDevice& Device;
	FCriticalSection FenceLock;
	TArray<FVulkanFence*> FreeFences;
	TArray<FVulkanFence*> UsedFences;

	// Returns true if signaled
	bool CheckFenceState(FVulkanFence* Fence);

	void DestroyFence(FVulkanFence* Fence);
};



enum class EVulkanSemaphoreFlags : uint8
{
	None = 0,

	// Will not delete handle on destruction
	ExternallyOwned    = 1 << 1,    

	// Inform submission pipeline that the signal will not come from a payload (acquired image semaphore for example)
	ExternallySignaled = 1 << 2,

	// Create a timeline semaphore (must be supported)
	Timeline           = 1 << 3,

	// Will not be queued for deferred deletion
	ImmediateDeletion  = 1 << 4,

	// Inform submission pipeline that the wait will not come from a payload (rendering done semaphore is consumed by vkQueuePresentKHR)
	ExternallyWaited   = 1 << 5
};
ENUM_CLASS_FLAGS(EVulkanSemaphoreFlags);


// Internal Vulkan Semaphore class that wrap VkSemaphore management (binary or timeline)
class FVulkanSemaphore : public FRefCountedObject
{
public:
	FVulkanSemaphore(FVulkanDevice& InDevice, EVulkanSemaphoreFlags InFlags=EVulkanSemaphoreFlags::None, uint64 InInitialTimelineValue=0);
	FVulkanSemaphore(FVulkanDevice& InDevice, const VkSemaphore& InExternalSemaphore);
	virtual ~FVulkanSemaphore();

	VkSemaphore GetHandle() const
	{
		return SemaphoreHandle;
	}

	bool IsExternallyOwned() const
	{
		return EnumHasAnyFlags(Flags, EVulkanSemaphoreFlags::ExternallyOwned);
	}

	bool IsExternallySignaled() const
	{
		return EnumHasAnyFlags(Flags, EVulkanSemaphoreFlags::ExternallySignaled);
	}

	bool IsExternallyWaited() const
	{
		return EnumHasAnyFlags(Flags, EVulkanSemaphoreFlags::ExternallyWaited);
	}

	bool IsTimeline() const
	{
		return EnumHasAnyFlags(Flags, EVulkanSemaphoreFlags::Timeline);
	}

	void GetTimelineSemaphoreValue(uint64& OutValue);
	bool WaitForTimelineSemaphoreValue(uint64 Value, uint64 Timeout);

private:
	FVulkanDevice& Device;
	VkSemaphore SemaphoreHandle;
	const EVulkanSemaphoreFlags Flags;
};

