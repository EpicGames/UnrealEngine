// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanTexture.cpp: Vulkan texture RHI implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanMemory.h"
#include "VulkanContext.h"
#include "VulkanPendingState.h"
#include "Containers/ResourceArray.h"
#include "VulkanLLM.h"
#include "VulkanBarriers.h"
#include "VulkanTransientResourceAllocator.h"
#include "RHICoreStats.h"
#include "RHILockTracker.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "RHICoreTexture.h"
#include "RHICoreTextureInitializer.h"

// This is a workaround for issues with AFBC on Mali GPUs before the G710
int32 GVulkanDepthStencilForceStorageBit = 0;
static FAutoConsoleVariableRef CVarVulkanDepthStencilForceStorageBit(
	TEXT("r.Vulkan.DepthStencilForceStorageBit"),
	GVulkanDepthStencilForceStorageBit,
	TEXT("Whether to force Image Usage Storage on Depth (can disable framebuffer compression).\n")
	TEXT(" 0: Not enabled\n")
	TEXT(" 1: Enables override for IMAGE_USAGE_STORAGE"),
	ECVF_Default
);

int32 GVulkanAllowConcurrentImage = 0;
static TAutoConsoleVariable<int32> GCVarAllowConcurrentImage(
	TEXT("r.Vulkan.AllowConcurrentImage"),
	GVulkanAllowConcurrentImage,
	TEXT("When async compute is supported: \n")
	TEXT(" 0 to use queue family ownership transfers with images (default)\n")
	TEXT(" 1 to use sharing mode concurrent with images"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> CVarVulkanSparseImageAllocSizeMB(
	TEXT("r.Vulkan.SparseImageAllocSizeMB"),
	16,
	TEXT("Size of the backing memory blocks for sparse images in megabytes (default 16MB)."),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> CVarVulkanSparseImageMergeBindings(
	TEXT("r.Vulkan.SparseImageMergeBindings"),
	1,
	TEXT("Merge consecutive bindings of sparse images."),
	ECVF_ReadOnly
);

extern int32 GVulkanLogDefrag;

#if ENABLE_LOW_LEVEL_MEM_TRACKER
inline ELLMTagVulkan GetMemoryTagForTextureFlags(ETextureCreateFlags UEFlags)
{
	bool bRenderTarget = EnumHasAnyFlags(UEFlags, TexCreate_RenderTargetable | TexCreate_ResolveTargetable | TexCreate_DepthStencilTargetable);
	return bRenderTarget ? ELLMTagVulkan::VulkanRenderTargets : ELLMTagVulkan::VulkanTextures;
}
#endif // ENABLE_LOW_LEVEL_MEM_TRACKER

static const VkImageTiling GVulkanViewTypeTilingMode[VK_IMAGE_VIEW_TYPE_RANGE_SIZE] =
{
	VK_IMAGE_TILING_LINEAR,		// VK_IMAGE_VIEW_TYPE_1D
	VK_IMAGE_TILING_OPTIMAL,	// VK_IMAGE_VIEW_TYPE_2D
	VK_IMAGE_TILING_OPTIMAL,	// VK_IMAGE_VIEW_TYPE_3D
	VK_IMAGE_TILING_OPTIMAL,	// VK_IMAGE_VIEW_TYPE_CUBE
	VK_IMAGE_TILING_LINEAR,		// VK_IMAGE_VIEW_TYPE_1D_ARRAY
	VK_IMAGE_TILING_OPTIMAL,	// VK_IMAGE_VIEW_TYPE_2D_ARRAY
	VK_IMAGE_TILING_OPTIMAL,	// VK_IMAGE_VIEW_TYPE_CUBE_ARRAY
};

static TStatId GetVulkanStatEnum(bool bIsCube, bool bIs3D, bool bIsRT)
{
#if STATS
	if (bIsRT == false)
	{
		// normal texture
		if (bIsCube)
		{
			return GET_STATID(STAT_TextureMemoryCube);
		}
		else if (bIs3D)
		{
			return GET_STATID(STAT_TextureMemory3D);
		}
		else
		{
			return GET_STATID(STAT_TextureMemory2D);
		}
	}
	else
	{
		// render target
		if (bIsCube)
		{
			return GET_STATID(STAT_RenderTargetMemoryCube);
		}
		else if (bIs3D)
		{
			return GET_STATID(STAT_RenderTargetMemory3D);
		}
		else
		{
			return GET_STATID(STAT_RenderTargetMemory2D);
		}
	}
#else
	return TStatId();
#endif
}

static VkImageLayout ChooseVRSLayout()
{
	if (GRHIVariableRateShadingImageDataType == VRSImage_Palette)
	{
		return VK_IMAGE_LAYOUT_FRAGMENT_SHADING_RATE_ATTACHMENT_OPTIMAL_KHR;
	}
	else if (GRHIVariableRateShadingImageDataType == VRSImage_Fractional)
	{
		return VK_IMAGE_LAYOUT_FRAGMENT_DENSITY_MAP_OPTIMAL_EXT;
	}

	checkNoEntry();
	return VK_IMAGE_LAYOUT_UNDEFINED;
}

static VkImageLayout GetInitialLayoutFromRHIAccess(ERHIAccess RHIAccess, bool bIsDepthStencilTarget, bool bSupportReadOnlyOptimal)
{
	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::RTV))
	{
		return VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL;
	}

	if (RHIAccess == ERHIAccess::Present)
	{
		return VK_IMAGE_LAYOUT_PRESENT_SRC_KHR;
	}

	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::DSVWrite))
	{
		return VK_IMAGE_LAYOUT_ATTACHMENT_OPTIMAL;
	}

	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::DSVRead))
	{
		return VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL;
	}

	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::SRVMask))
	{
		if (bIsDepthStencilTarget)
		{
			return VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL;
		}

		return bSupportReadOnlyOptimal ? VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL : VK_IMAGE_LAYOUT_GENERAL;
	}

	if (EnumHasAnyFlags(RHIAccess, ERHIAccess::UAVMask))
	{
		return VK_IMAGE_LAYOUT_GENERAL;
	}

	switch (RHIAccess)
	{
	case ERHIAccess::Unknown:	return VK_IMAGE_LAYOUT_UNDEFINED;
	case ERHIAccess::Discard:	return VK_IMAGE_LAYOUT_UNDEFINED;
	case ERHIAccess::CPURead:	return VK_IMAGE_LAYOUT_UNDEFINED;
	case ERHIAccess::CopySrc:	return VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL;
	case ERHIAccess::CopyDest:	return VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL;
	case ERHIAccess::ShadingRateSource:	return ChooseVRSLayout();
	}

	checkf(false, TEXT("Invalid initial access %d"), RHIAccess);
	return VK_IMAGE_LAYOUT_UNDEFINED;
}

static void UpdateVulkanTextureStats(const FRHITextureDesc& TextureDesc, uint64 TextureSize, bool bAllocating)
{
	const bool bOnlyStreamableTexturesInTexturePool = false;
	UE::RHICore::UpdateGlobalTextureStats(TextureDesc, TextureSize, bOnlyStreamableTexturesInTexturePool, bAllocating);
}

static void VulkanTextureAllocated(const FRHITextureDesc& TextureDesc, uint64 Size)
{
	UpdateVulkanTextureStats(TextureDesc, Size, true);
}

static void VulkanTextureDestroyed(const FRHITextureDesc& TextureDesc, uint64 Size)
{
	UpdateVulkanTextureStats(TextureDesc, Size, false);
}

void FVulkanTexture::InternalLockWrite(FVulkanContextCommon& Context, FVulkanTexture* Surface, const VkBufferImageCopy& Region, VulkanRHI::FStagingBuffer* StagingBuffer, bool bNeedsTransitions)
{
	FVulkanCommandBuffer* CmdBuffer = Context.GetActiveCmdBuffer();
	ensure(CmdBuffer->IsOutsideRenderPass());
	VkCommandBuffer StagingCommandBuffer = CmdBuffer->GetHandle();

	const VkImageSubresourceLayers& ImageSubresource = Region.imageSubresource;
	const VkImageSubresourceRange SubresourceRange = FVulkanPipelineBarrier::MakeSubresourceRange(ImageSubresource.aspectMask, ImageSubresource.mipLevel, 1, ImageSubresource.baseArrayLayer, ImageSubresource.layerCount);

	// We might skip transitions in passes where copies are batched
	if (bNeedsTransitions)
	{
		FVulkanPipelineBarrier Barrier;
		Barrier.AddImageLayoutTransition(Surface->Image, VK_IMAGE_LAYOUT_UNDEFINED, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL, SubresourceRange);
		Barrier.Execute(CmdBuffer);
	}

	VulkanRHI::vkCmdCopyBufferToImage(StagingCommandBuffer, StagingBuffer->GetHandle(), Surface->Image, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL, 1, &Region);

	if (bNeedsTransitions)
	{
		const bool bIsTargetable = EnumHasAnyFlags(Surface->GetDesc().Flags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable);
		const ERHIAccess DefaultAccess = RHIGetDefaultResourceState(Surface->GetDesc().Flags, false);
		const VkImageLayout DestLayout = GetInitialLayoutFromRHIAccess(DefaultAccess, bIsTargetable && Surface->IsDepthOrStencilAspect(), Surface->SupportsSampling());

		FVulkanPipelineBarrier Barrier;
		Barrier.AddImageLayoutTransition(Surface->Image, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL, DestLayout, SubresourceRange);
		Barrier.Execute(CmdBuffer);
	}

	Surface->Device->GetStagingManager().ReleaseBuffer(&Context, StagingBuffer);
}

void FVulkanTexture::ErrorInvalidViewType() const
{
	UE_LOGF(LogVulkanRHI, Error, "Invalid ViewType %ls", VK_TYPE_TO_STRING(VkImageViewType, GetViewType()));
}


static VkImageUsageFlags GetUsageFlagsFromCreateFlags(FVulkanDevice& InDevice, const ETextureCreateFlags& UEFlags)
{
	VkImageUsageFlags UsageFlags = VK_IMAGE_USAGE_TRANSFER_SRC_BIT | VK_IMAGE_USAGE_TRANSFER_DST_BIT | VK_IMAGE_USAGE_SAMPLED_BIT;

	if (EnumHasAnyFlags(UEFlags, TexCreate_Presentable))
	{
		UsageFlags |= VK_IMAGE_USAGE_STORAGE_BIT;
	}
	else if (EnumHasAnyFlags(UEFlags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable))
	{
		if (EnumHasAllFlags(UEFlags, TexCreate_InputAttachmentRead))
		{
			UsageFlags |= VK_IMAGE_USAGE_INPUT_ATTACHMENT_BIT;
		}
		UsageFlags |= (EnumHasAnyFlags(UEFlags, TexCreate_RenderTargetable) ? VK_IMAGE_USAGE_COLOR_ATTACHMENT_BIT : VK_IMAGE_USAGE_DEPTH_STENCIL_ATTACHMENT_BIT);
		if (EnumHasAllFlags(UEFlags, TexCreate_Memoryless) && InDevice.GetDeviceMemoryManager().SupportsMemoryless())
		{
			UsageFlags |= VK_IMAGE_USAGE_TRANSIENT_ATTACHMENT_BIT;
			// Remove the transfer and sampled bits, as they are incompatible with the transient bit.
			UsageFlags &= ~(VK_IMAGE_USAGE_TRANSFER_DST_BIT | VK_IMAGE_USAGE_TRANSFER_SRC_BIT | VK_IMAGE_USAGE_SAMPLED_BIT);
		}
	}
	else if (EnumHasAnyFlags(UEFlags, TexCreate_DepthStencilResolveTarget))
	{
		UsageFlags |= VK_IMAGE_USAGE_DEPTH_STENCIL_ATTACHMENT_BIT;
	}
	else if (EnumHasAnyFlags(UEFlags, TexCreate_ResolveTargetable))
	{
		UsageFlags |= VK_IMAGE_USAGE_COLOR_ATTACHMENT_BIT;
	}
	else if (InDevice.GetOptionalExtensions().HasEXTHostImageCopy)
	{
		// non-targets can be initialized using host transfers
		UsageFlags |= VK_IMAGE_USAGE_HOST_TRANSFER_BIT;
	}

	if (EnumHasAnyFlags(UEFlags, TexCreate_Foveation) && ValidateShadingRateDataType())
	{
		if (GRHIVariableRateShadingImageDataType == VRSImage_Palette)
		{
			UsageFlags |= VK_IMAGE_USAGE_FRAGMENT_SHADING_RATE_ATTACHMENT_BIT_KHR;
		}

		if (GRHIVariableRateShadingImageDataType == VRSImage_Fractional)
		{
			UsageFlags |= VK_IMAGE_USAGE_FRAGMENT_DENSITY_MAP_BIT_EXT;
		}
	}

	if (EnumHasAnyFlags(UEFlags, TexCreate_UAV))
	{
		//cannot have the storage bit on a memoryless texture
		ensure(!EnumHasAnyFlags(UEFlags, TexCreate_Memoryless));
		UsageFlags |= VK_IMAGE_USAGE_STORAGE_BIT;
	}

	return UsageFlags;
}



void FVulkanTexture::GenerateImageCreateInfo(
	FImageCreateInfo& OutImageCreateInfo,
	FVulkanDevice& InDevice,
	const FRHITextureDesc& InDesc,
	VkFormat* OutStorageFormat,
	VkFormat* OutViewFormat,
	bool bForceLinearTexture)
{
	const VkPhysicalDeviceProperties& DeviceProperties = InDevice.GetDeviceProperties();
	const FPixelFormatInfo& FormatInfo = GPixelFormats[InDesc.Format];
	VkFormat TextureFormat = (VkFormat)FormatInfo.PlatformFormat;

	const ETextureCreateFlags UEFlags = InDesc.Flags;
	if(EnumHasAnyFlags(UEFlags, TexCreate_CPUReadback))
	{
		bForceLinearTexture = true;
	}

	// Works arround an AMD driver bug where InterlockedMax() on a R32 Texture2D ends up with incorrect memory order swizzling
	if (IsRHIDeviceAMD() && (InDesc.Format == PF_R32_UINT && UEFlags == (TexCreate_ShaderResource | TexCreate_UAV | TexCreate_AtomicCompatible)))
	{
		bForceLinearTexture = true;
	}

	checkf(TextureFormat != VK_FORMAT_UNDEFINED, TEXT("PixelFormat %d, is not supported for images"), (int32)InDesc.Format);
	VkImageCreateInfo& ImageCreateInfo = OutImageCreateInfo.ImageCreateInfo;
	ZeroVulkanStruct(ImageCreateInfo, VK_STRUCTURE_TYPE_IMAGE_CREATE_INFO);

	const VkImageViewType ResourceType = UETextureDimensionToVkImageViewType(InDesc.Dimension);
	switch(ResourceType)
	{
	case VK_IMAGE_VIEW_TYPE_1D:
		ImageCreateInfo.imageType = VK_IMAGE_TYPE_1D;
		check((uint32)InDesc.Extent.X <= DeviceProperties.limits.maxImageDimension1D);
		break;
	case VK_IMAGE_VIEW_TYPE_CUBE:
	case VK_IMAGE_VIEW_TYPE_CUBE_ARRAY:
		check(InDesc.Extent.X == InDesc.Extent.Y);
		check((uint32)InDesc.Extent.X <= DeviceProperties.limits.maxImageDimensionCube);
		check((uint32)InDesc.Extent.Y <= DeviceProperties.limits.maxImageDimensionCube);
		ImageCreateInfo.imageType = VK_IMAGE_TYPE_2D;
		break;
	case VK_IMAGE_VIEW_TYPE_2D:
	case VK_IMAGE_VIEW_TYPE_2D_ARRAY:
		check((uint32)InDesc.Extent.X <= DeviceProperties.limits.maxImageDimension2D);
		check((uint32)InDesc.Extent.Y <= DeviceProperties.limits.maxImageDimension2D);
		ImageCreateInfo.imageType = VK_IMAGE_TYPE_2D;
		break;
	case VK_IMAGE_VIEW_TYPE_3D:
		check((uint32)InDesc.Extent.Y <= DeviceProperties.limits.maxImageDimension3D);
		ImageCreateInfo.imageType = VK_IMAGE_TYPE_3D;
		break;
	default:
		checkf(false, TEXT("Unhandled image type %d"), (int32)ResourceType);
		break;
	}

	VkFormat srgbFormat = UEToVkTextureFormat(InDesc.Format, EnumHasAllFlags(UEFlags, TexCreate_SRGB));
	VkFormat nonSrgbFormat = UEToVkTextureFormat(InDesc.Format, false);

	ImageCreateInfo.format = EnumHasAnyFlags(UEFlags, TexCreate_UAV) ? nonSrgbFormat : srgbFormat;

	checkf(ImageCreateInfo.format != VK_FORMAT_UNDEFINED, TEXT("Pixel Format %d not defined!"), (int32)InDesc.Format);
	if (OutViewFormat)
	{
		*OutViewFormat = srgbFormat;
	}
	if (OutStorageFormat)
	{
		*OutStorageFormat = nonSrgbFormat;
	}

	ImageCreateInfo.extent.width = InDesc.Extent.X;
	ImageCreateInfo.extent.height = InDesc.Extent.Y;
	ImageCreateInfo.extent.depth = ResourceType == VK_IMAGE_VIEW_TYPE_3D ? InDesc.Depth : 1;
	ImageCreateInfo.mipLevels = InDesc.NumMips;
	const uint32 LayerCount = (ResourceType == VK_IMAGE_VIEW_TYPE_CUBE || ResourceType == VK_IMAGE_VIEW_TYPE_CUBE_ARRAY) ? 6 : 1;
	ImageCreateInfo.arrayLayers = InDesc.ArraySize * LayerCount;
	check(ImageCreateInfo.arrayLayers <= DeviceProperties.limits.maxImageArrayLayers);

	ImageCreateInfo.flags = (ResourceType == VK_IMAGE_VIEW_TYPE_CUBE || ResourceType == VK_IMAGE_VIEW_TYPE_CUBE_ARRAY) ? VK_IMAGE_CREATE_CUBE_COMPATIBLE_BIT : 0;

	if (EnumHasAnyFlags(UEFlags, TexCreate_ReservedResource))
	{
		ImageCreateInfo.flags |= VK_IMAGE_CREATE_SPARSE_BINDING_BIT; // no VK_IMAGE_CREATE_SPARSE_RESIDENCY_BIT yet since only TexCreate_ImmediateCommit is supported
	}

	const bool bHasUAVFormat = (InDesc.UAVFormat != PF_Unknown && InDesc.UAVFormat != InDesc.Format);
	const bool bNeedsMutableFormat = (EnumHasAllFlags(UEFlags, TexCreate_SRGB) || (InDesc.Format == PF_R64_UINT) || bHasUAVFormat || !InDesc.AliasableFormats.IsEmpty());
	if (bNeedsMutableFormat)
	{
		if (InDevice.GetOptionalExtensions().HasKHRImageFormatList)
		{
			VkImageFormatListCreateInfoKHR& ImageFormatListCreateInfo = OutImageCreateInfo.ImageFormatListCreateInfo;
			ZeroVulkanStruct(ImageFormatListCreateInfo, VK_STRUCTURE_TYPE_IMAGE_FORMAT_LIST_CREATE_INFO_KHR);
			ImageFormatListCreateInfo.pNext = ImageCreateInfo.pNext;
			ImageCreateInfo.pNext = &ImageFormatListCreateInfo;

			// Allow non-SRGB views to be created for SRGB textures
			if (EnumHasAllFlags(UEFlags, TexCreate_SRGB) && (nonSrgbFormat != srgbFormat))
			{
				OutImageCreateInfo.FormatsUsed.Add(nonSrgbFormat);
				OutImageCreateInfo.FormatsUsed.Add(srgbFormat);
			}

			// Make it possible to create R32G32 views of R64 images for utilities like clears
			if (InDesc.Format == PF_R64_UINT)
			{
				OutImageCreateInfo.FormatsUsed.AddUnique(nonSrgbFormat);
				OutImageCreateInfo.FormatsUsed.AddUnique(UEToVkTextureFormat(PF_R32G32_UINT, false));
			}

			if (bHasUAVFormat)
			{
				OutImageCreateInfo.FormatsUsed.AddUnique(nonSrgbFormat);
				OutImageCreateInfo.FormatsUsed.AddUnique(UEToVkTextureFormat(InDesc.UAVFormat, false));
			}

			for (EPixelFormat AliasPixelFormat : InDesc.AliasableFormats)
			{
				const VkFormat AliasFormat = UEToVkTextureFormat(AliasPixelFormat, false);
				if (ImageCreateInfo.format != AliasFormat)
				{
					OutImageCreateInfo.FormatsUsed.AddUnique(AliasFormat);
				}
			}

			ImageFormatListCreateInfo.pViewFormats = OutImageCreateInfo.FormatsUsed.GetData();
			ImageFormatListCreateInfo.viewFormatCount = OutImageCreateInfo.FormatsUsed.Num();
		}

		ImageCreateInfo.flags |= VK_IMAGE_CREATE_MUTABLE_FORMAT_BIT;
		if (bHasUAVFormat && IsAnyBlockCompressedPixelFormat(InDesc.Format) && !IsAnyBlockCompressedPixelFormat(InDesc.UAVFormat))
		{
			ImageCreateInfo.flags |= VK_IMAGE_CREATE_BLOCK_TEXEL_VIEW_COMPATIBLE_BIT;
		}
	}

	if ((ImageCreateInfo.imageType == VK_IMAGE_TYPE_3D) && !EnumHasAnyFlags(UEFlags, TexCreate_ReservedResource))
	{
		ImageCreateInfo.flags |= VK_IMAGE_CREATE_2D_ARRAY_COMPATIBLE_BIT;
	}

	if (InDevice.GetOptionalExtensions().HasEXTFragmentDensityMapOffset && GRHISupportsVariableRateShadingImageOffsets)
	{
		ImageCreateInfo.flags |= VK_IMAGE_CREATE_FRAGMENT_DENSITY_MAP_OFFSET_BIT_EXT;
	}

	ImageCreateInfo.tiling = bForceLinearTexture ? VK_IMAGE_TILING_LINEAR : GVulkanViewTypeTilingMode[ResourceType];
	if (EnumHasAnyFlags(UEFlags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable | TexCreate_ResolveTargetable | TexCreate_DepthStencilResolveTarget))
	{
		ImageCreateInfo.tiling = VK_IMAGE_TILING_OPTIMAL;
	}

	ImageCreateInfo.usage = GetUsageFlagsFromCreateFlags(InDevice, UEFlags);

	if (EnumHasAnyFlags(UEFlags, TexCreate_External))
	{
		VkExternalMemoryImageCreateInfoKHR& ExternalMemImageCreateInfo = OutImageCreateInfo.ExternalMemImageCreateInfo;
		ZeroVulkanStruct(ExternalMemImageCreateInfo, VK_STRUCTURE_TYPE_EXTERNAL_MEMORY_IMAGE_CREATE_INFO_KHR);
#if PLATFORM_WINDOWS
		ExternalMemImageCreateInfo.handleTypes = VK_EXTERNAL_MEMORY_HANDLE_TYPE_OPAQUE_WIN32_BIT_KHR;
#else
	    ExternalMemImageCreateInfo.handleTypes = VK_EXTERNAL_MEMORY_HANDLE_TYPE_OPAQUE_FD_BIT_KHR;
#endif
		ExternalMemImageCreateInfo.pNext = ImageCreateInfo.pNext;
    	ImageCreateInfo.pNext = &ExternalMemImageCreateInfo;
	}

	//#todo-rco: If using CONCURRENT, make sure to NOT do so on render targets as that kills DCC compression
	ImageCreateInfo.sharingMode = VK_SHARING_MODE_EXCLUSIVE;
	ImageCreateInfo.queueFamilyIndexCount = 0;
	ImageCreateInfo.pQueueFamilyIndices = nullptr;

	uint8 NumSamples = InDesc.NumSamples;
	if (ImageCreateInfo.tiling == VK_IMAGE_TILING_LINEAR && NumSamples > 1)
	{
		UE_LOGF(LogVulkanRHI, Warning, "Not allowed to create Linear textures with %d samples, reverting to 1 sample", NumSamples);
		NumSamples = 1;
	}

	switch (NumSamples)
	{
	case 1:
		ImageCreateInfo.samples = VK_SAMPLE_COUNT_1_BIT;
		break;
	case 2:
		ImageCreateInfo.samples = VK_SAMPLE_COUNT_2_BIT;
		break;
	case 4:
		ImageCreateInfo.samples = VK_SAMPLE_COUNT_4_BIT;
		break;
	case 8:
		ImageCreateInfo.samples = VK_SAMPLE_COUNT_8_BIT;
		break;
	case 16:
		ImageCreateInfo.samples = VK_SAMPLE_COUNT_16_BIT;
		break;
	case 32:
		ImageCreateInfo.samples = VK_SAMPLE_COUNT_32_BIT;
		break;
	case 64:
		ImageCreateInfo.samples = VK_SAMPLE_COUNT_64_BIT;
		break;
	default:
		checkf(0, TEXT("Unsupported number of samples %d"), NumSamples);
		break;
	}

	FVulkanPlatform::SetImageMemoryRequirementWorkaround(ImageCreateInfo);

	const VkFormatProperties& FormatProperties = InDevice.GetFormatProperties(ImageCreateInfo.format);
	const VkFormatFeatureFlags FormatFlags = ImageCreateInfo.tiling == VK_IMAGE_TILING_LINEAR ? 
		FormatProperties.linearTilingFeatures : 
		FormatProperties.optimalTilingFeatures;

	if (!VKHasAnyFlags(FormatFlags, VK_FORMAT_FEATURE_SAMPLED_IMAGE_BIT))
	{
		// Some formats don't support sampling and that's ok, we'll use a STORAGE_IMAGE
		check(EnumHasAnyFlags(UEFlags, TexCreate_UAV | TexCreate_CPUReadback));
		ImageCreateInfo.usage &= ~VK_IMAGE_USAGE_SAMPLED_BIT;
	}

	if (bHasUAVFormat)
	{
		const VkFormat UAVFormat = UEToVkTextureFormat(InDesc.UAVFormat, false);
		const VkFormatProperties& UAVFormatProperties = InDevice.GetFormatProperties(UAVFormat);
		const VkFormatFeatureFlags UAVFormatFlags = ImageCreateInfo.tiling == VK_IMAGE_TILING_LINEAR ?
			UAVFormatProperties.linearTilingFeatures :
			UAVFormatProperties.optimalTilingFeatures;

		ensure((UAVFormatFlags & VK_FORMAT_FEATURE_STORAGE_IMAGE_BIT) != 0);
	}

	if (!VKHasAnyFlags(FormatFlags, VK_FORMAT_FEATURE_STORAGE_IMAGE_BIT))
	{
		ensure((ImageCreateInfo.usage & VK_IMAGE_USAGE_STORAGE_BIT) == 0 || bHasUAVFormat);
		if (bHasUAVFormat)
		{
			ImageCreateInfo.flags |= VK_IMAGE_CREATE_EXTENDED_USAGE_BIT;
		}
		else
		{
			ImageCreateInfo.usage &= ~VK_IMAGE_USAGE_STORAGE_BIT;
		}
	}

	if (!VKHasAnyFlags(FormatFlags, VK_FORMAT_FEATURE_COLOR_ATTACHMENT_BIT))
	{
		ensure((ImageCreateInfo.usage & VK_IMAGE_USAGE_COLOR_ATTACHMENT_BIT) == 0);
		ImageCreateInfo.usage &= ~VK_IMAGE_USAGE_COLOR_ATTACHMENT_BIT;
	}

	if (!VKHasAnyFlags(FormatFlags, VK_FORMAT_FEATURE_DEPTH_STENCIL_ATTACHMENT_BIT))
	{
		ensure((ImageCreateInfo.usage & VK_IMAGE_USAGE_DEPTH_STENCIL_ATTACHMENT_BIT) == 0);
		ImageCreateInfo.usage &= ~VK_IMAGE_USAGE_DEPTH_STENCIL_ATTACHMENT_BIT;
	}

	if (!VKHasAnyFlags(FormatFlags, VK_FORMAT_FEATURE_TRANSFER_SRC_BIT))
	{
		// this flag is used unconditionally, strip it without warnings 
		ImageCreateInfo.usage &= ~VK_IMAGE_USAGE_TRANSFER_SRC_BIT;
	}
		
	if (!VKHasAnyFlags(FormatFlags, VK_FORMAT_FEATURE_TRANSFER_DST_BIT))
	{
		// this flag is used unconditionally, strip it without warnings 
		ImageCreateInfo.usage &= ~VK_IMAGE_USAGE_TRANSFER_DST_BIT;
	}

	if (GVulkanDepthStencilForceStorageBit && EnumHasAnyFlags(UEFlags, TexCreate_DepthStencilTargetable) && (TextureFormat != VK_FORMAT_D16_UNORM && TextureFormat != VK_FORMAT_D32_SFLOAT))
	{
		ImageCreateInfo.usage |= VK_IMAGE_USAGE_STORAGE_BIT;
	}

	ZeroVulkanStruct(OutImageCreateInfo.CompressionControl, VK_STRUCTURE_TYPE_IMAGE_COMPRESSION_CONTROL_EXT);
	OutImageCreateInfo.CompressionFixedRateFlags = 0;

	if (EnumHasAnyFlags(InDesc.Flags, TexCreate_LossyCompression | TexCreate_LossyCompressionLowBitrate) && InDevice.GetOptionalExtensions().HasEXTImageCompressionControl)
	{
		VkImageCompressionControlEXT& CompressionControl = OutImageCreateInfo.CompressionControl;
		CompressionControl = VkImageCompressionControlEXT{ VK_STRUCTURE_TYPE_IMAGE_COMPRESSION_CONTROL_EXT };
		CompressionControl.flags = VK_IMAGE_COMPRESSION_FIXED_RATE_DEFAULT_EXT;

		VkImageCompressionPropertiesEXT ImageCompressionProperties{ VK_STRUCTURE_TYPE_IMAGE_COMPRESSION_PROPERTIES_EXT };
		VkImageFormatProperties2 ImageFormatProperties{ VK_STRUCTURE_TYPE_IMAGE_FORMAT_PROPERTIES_2 };
		ImageFormatProperties.pNext = &ImageCompressionProperties;

		VkPhysicalDeviceImageFormatInfo2 ImageFormatInfo{ VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_IMAGE_FORMAT_INFO_2 };
		ImageFormatInfo.pNext = &CompressionControl;
		ImageFormatInfo.format = ImageCreateInfo.format;
		ImageFormatInfo.type = ImageCreateInfo.imageType;
		ImageFormatInfo.tiling = ImageCreateInfo.tiling;
		ImageFormatInfo.usage = ImageCreateInfo.usage;
		ImageFormatInfo.flags = ImageCreateInfo.flags;

		if (VulkanRHI::vkGetPhysicalDeviceImageFormatProperties2(InDevice.GetPhysicalHandle(), &ImageFormatInfo, &ImageFormatProperties) == VK_SUCCESS)
		{
			if (ImageCompressionProperties.imageCompressionFlags == VK_IMAGE_COMPRESSION_FIXED_RATE_EXPLICIT_EXT)
			{
				CompressionControl.pNext = ImageCreateInfo.pNext;
				ImageCreateInfo.pNext = &CompressionControl;

				if (EnumHasAllFlags(InDesc.Flags, TexCreate_LossyCompressionLowBitrate) && ImageCompressionProperties.imageCompressionFixedRateFlags)
				{
					CompressionControl.flags = VK_IMAGE_COMPRESSION_FIXED_RATE_EXPLICIT_EXT;
					OutImageCreateInfo.CompressionFixedRateFlags = 1 << FMath::CountTrailingZeros(ImageCompressionProperties.imageCompressionFixedRateFlags);
					CompressionControl.compressionControlPlaneCount = 1;
					CompressionControl.pFixedRateFlags = &OutImageCreateInfo.CompressionFixedRateFlags;
				}
			}
		}
	}

	if (InDevice.HasMultipleQueues() && (GVulkanAllowConcurrentImage != 0))
	{
		ImageCreateInfo.sharingMode = VK_SHARING_MODE_CONCURRENT;
		ImageCreateInfo.queueFamilyIndexCount = InDevice.GetActiveQueueFamilies().Num();
		ImageCreateInfo.pQueueFamilyIndices = (uint32_t*)InDevice.GetActiveQueueFamilies().GetData();
	}
	else
	{
		ImageCreateInfo.sharingMode = VK_SHARING_MODE_EXCLUSIVE;
	}
}

void FVulkanTexture::InternalMoveSurface(FVulkanDevice& InDevice, FVulkanCommandListContext& Context, VulkanRHI::FVulkanAllocation& DestAllocation)
{
	FImageCreateInfo ImageCreateInfo;
	const FRHITextureDesc& Desc = GetDesc();
	FVulkanTexture::GenerateImageCreateInfo(ImageCreateInfo, InDevice, Desc, &StorageFormat, &ViewFormat);

	VkImage MovedImage;
	VERIFYVULKANRESULT(VulkanRHI::vkCreateImage(InDevice.GetHandle(), &ImageCreateInfo.ImageCreateInfo, VULKAN_CPU_ALLOCATOR, &MovedImage));
	checkf(Tiling == ImageCreateInfo.ImageCreateInfo.tiling, TEXT("Move has changed image tiling:  before [%s] != after [%s]"), VK_TYPE_TO_STRING(VkImageTiling, Tiling), VK_TYPE_TO_STRING(VkImageTiling, ImageCreateInfo.ImageCreateInfo.tiling));

	const ETextureCreateFlags UEFlags = Desc.Flags;
	const bool bRenderTarget = EnumHasAnyFlags(UEFlags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable | TexCreate_ResolveTargetable);
	const bool bCPUReadback = EnumHasAnyFlags(UEFlags, TexCreate_CPUReadback);
	const bool bMemoryless = EnumHasAnyFlags(UEFlags, TexCreate_Memoryless);
	const bool bExternal = EnumHasAnyFlags(UEFlags, TexCreate_External);
	checkf(!bCPUReadback, TEXT("Move of CPUReadback surfaces not currently supported.   UEFlags=0x%x"), (int32)UEFlags);
	checkf(!bMemoryless || !InDevice.GetDeviceMemoryManager().SupportsMemoryless(), TEXT("Move of Memoryless surfaces not currently supported.   UEFlags=0x%x"), (int32)UEFlags);
	checkf(!bExternal, TEXT("Move of external memory not supported. UEFlags=0x%x"), (int32)UEFlags);

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
	// This shouldn't change
	VkMemoryRequirements MovedMemReqs;
	VulkanRHI::vkGetImageMemoryRequirements(InDevice.GetHandle(), MovedImage, &MovedMemReqs);
	checkf((MemoryRequirements.alignment == MovedMemReqs.alignment), TEXT("Memory requirements changed: alignment %d -> %d"), (int32)MemoryRequirements.alignment, (int32)MovedMemReqs.alignment);
	checkf((MemoryRequirements.size == MovedMemReqs.size), TEXT("Memory requirements changed: size %d -> %d"), (int32)MemoryRequirements.size, (int32)MovedMemReqs.size);
	checkf((MemoryRequirements.memoryTypeBits == MovedMemReqs.memoryTypeBits), TEXT("Memory requirements changed: memoryTypeBits %d -> %d"), (int32)MemoryRequirements.memoryTypeBits, (int32)MovedMemReqs.memoryTypeBits);
#endif // UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT

	DestAllocation.BindImage(&InDevice, MovedImage);

	// Copy Original -> Moved
	FVulkanCommandBuffer& CommandBuffer = Context.GetCommandBuffer();
	VkCommandBuffer CommandBufferHandle = CommandBuffer.GetHandle();
	ensure(CommandBuffer.IsOutsideRenderPass());

	{
		const uint32 NumberOfArrayLevels = GetNumberOfArrayLevels();
		const VkImageSubresourceRange FullSubresourceRange = FVulkanPipelineBarrier::MakeSubresourceRange(FullAspectMask);

		const ERHIAccess OriginalAccess = GetTrackedAccess_Unsafe();
		const VkImageLayout OriginalLayout = GetInitialLayoutFromRHIAccess(OriginalAccess, IsDepthOrStencilAspect(), SupportsSampling());

		// Transition to copying layouts
		{
			FVulkanPipelineBarrier Barrier;
			Barrier.AddImageLayoutTransition(Image, OriginalLayout, VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL, FullSubresourceRange);
			Barrier.AddImageLayoutTransition(MovedImage, VK_IMAGE_LAYOUT_UNDEFINED, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL, FullSubresourceRange);
			Barrier.Execute(&CommandBuffer);
		}
		{
			VkImageCopy Regions[MAX_TEXTURE_MIP_COUNT];
			check(Desc.NumMips <= MAX_TEXTURE_MIP_COUNT);
			FMemory::Memzero(Regions);
			for (uint32 i = 0; i < Desc.NumMips; ++i)
			{
				VkImageCopy& Region = Regions[i];
				Region.extent.width = FMath::Max(1, Desc.Extent.X >> i);
				Region.extent.height = FMath::Max(1, Desc.Extent.Y >> i);
				Region.extent.depth = FMath::Max(1, Desc.Depth >> i);
				Region.srcSubresource.aspectMask = FullAspectMask;
				Region.dstSubresource.aspectMask = FullAspectMask;
				Region.srcSubresource.baseArrayLayer = 0;
				Region.dstSubresource.baseArrayLayer = 0;
				Region.srcSubresource.layerCount = NumberOfArrayLevels;
				Region.dstSubresource.layerCount = NumberOfArrayLevels;
				Region.srcSubresource.mipLevel = i;
				Region.dstSubresource.mipLevel = i;
			}

			VulkanRHI::vkCmdCopyImage(CommandBufferHandle,
				Image, VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL,
				MovedImage, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL,
				Desc.NumMips, &Regions[0]);
		}

		// Put the destination image in exactly the same layout the original image was
		{
			FVulkanPipelineBarrier Barrier;
			Barrier.AddImageLayoutTransition(Image, VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL, OriginalLayout, FullSubresourceRange);
			Barrier.AddImageLayoutTransition(MovedImage, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL, OriginalLayout, FullSubresourceRange);
			Barrier.Execute(&CommandBuffer);
		}
	}

	{
		check(Image != VK_NULL_HANDLE);
		InDevice.NotifyDeletedImage(Image, bRenderTarget);
		InDevice.GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::Image, Image);

		if (GVulkanLogDefrag)
		{
			FGenericPlatformMisc::LowLevelOutputDebugStringf(TEXT("** MOVE IMAGE %p -> %p\n"), Image, MovedImage);
		}
	}

	Image = MovedImage;
}

void FVulkanTexture::DestroySurface()
{
	const bool bIsLocalOwner = (ImageOwnerType == EImageOwnerType::LocalOwner);
	const bool bHasExternalOwner = (ImageOwnerType == EImageOwnerType::ExternalOwner);

	if (CpuReadbackBuffer)
	{
		Device->GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::Buffer, CpuReadbackBuffer->Buffer);
		Device->GetMemoryManager().FreeVulkanAllocation(Allocation);
		delete CpuReadbackBuffer;

	}
	else if (bIsLocalOwner || bHasExternalOwner)
	{
		const bool bRenderTarget = EnumHasAnyFlags(GetDesc().Flags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable | TexCreate_ResolveTargetable);
		Device->NotifyDeletedImage(Image, bRenderTarget);

		if (bIsLocalOwner)
		{
			// If we don't own the allocation, it's transient memory not included in stats
			if (Allocation.HasAllocation())
			{
				VulkanTextureDestroyed(GetDesc(), Allocation.Size);
			}

			if (Image != VK_NULL_HANDLE)
			{
				Device->GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::Image, Image);

				if (ReservedResourceData)
				{
					check(!Allocation.HasAllocation());
					CommitReservedResource(0);
					VulkanTextureDestroyed(GetDesc(), MemoryRequirements.size);
				}
				else
				{
					Device->GetMemoryManager().FreeVulkanAllocation(Allocation);
				}

				Image = VK_NULL_HANDLE;
			}
		}
		else
		{
			Image = VK_NULL_HANDLE;
			if (ExternalImageDeleteCallbackInfo.Function)
			{
				ExternalImageDeleteCallbackInfo.Function(ExternalImageDeleteCallbackInfo.UserData);
			}
		}

		ImageOwnerType = EImageOwnerType::None;
	}
}


void FVulkanTexture::InvalidateMappedMemory()
{
	Allocation.InvalidateMappedMemory(Device);

}
void* FVulkanTexture::GetMappedPointer()
{
	return Allocation.GetMappedPointer(Device);
}

#if RHI_ENABLE_RESOURCE_INFO
bool FVulkanTexture::GetResourceInfo(FRHIResourceInfo& OutResourceInfo) const
{
	OutResourceInfo = FRHIResourceInfo{};
	OutResourceInfo.Name = GetName();
	OutResourceInfo.Type = GetType();
	OutResourceInfo.VRamAllocation.AllocationSize = GetMemorySize();
	OutResourceInfo.IsTransient = Allocation.bTransient;

	return true;
}
#endif

VkDeviceMemory FVulkanTexture::GetAllocationHandle() const
{
	if (Allocation.IsValid())
	{
		return Allocation.GetDeviceMemoryHandle(Device);
	}
	else
	{
		return VK_NULL_HANDLE;
	}
}

uint64 FVulkanTexture::GetAllocationOffset() const
{
	if (Allocation.IsValid())
	{
		return Allocation.Offset;
	}
	else
	{
		return 0;
	}
}

void FVulkanTexture::GetMipStride(uint32 MipIndex, uint32& Stride)
{
	// Calculate the width of the MipMap.
	const FRHITextureDesc& Desc = GetDesc();
	const EPixelFormat PixelFormat = Desc.Format;
	const uint32 BlockSizeX = GPixelFormats[PixelFormat].BlockSizeX;
	const uint32 MipSizeX = FMath::Max<uint32>(Desc.Extent.X >> MipIndex, BlockSizeX);
	uint32 NumBlocksX = (MipSizeX + BlockSizeX - 1) / BlockSizeX;

	if (PixelFormat == PF_PVRTC2 || PixelFormat == PF_PVRTC4)
	{
		// PVRTC has minimum 2 blocks width
		NumBlocksX = FMath::Max<uint32>(NumBlocksX, 2);
	}

	const uint32 BlockBytes = GPixelFormats[PixelFormat].BlockBytes;

	Stride = NumBlocksX * BlockBytes;
}


void FVulkanTexture::GetMipSize(uint32 MipIndex, uint64& MipBytes)
{
	// Calculate the dimensions of mip-map level.
	const FRHITextureDesc& Desc = GetDesc();
	const EPixelFormat PixelFormat = Desc.Format;
	const uint32 BlockSizeX = GPixelFormats[PixelFormat].BlockSizeX;
	const uint32 BlockSizeY = GPixelFormats[PixelFormat].BlockSizeY;
	const uint32 BlockBytes = GPixelFormats[PixelFormat].BlockBytes;
	const uint32 MipSizeX = FMath::Max<uint32>(Desc.Extent.X >> MipIndex, BlockSizeX);
	const uint32 MipSizeY = FMath::Max<uint32>(Desc.Extent.Y >> MipIndex, BlockSizeY);
	uint32 NumBlocksX = (MipSizeX + BlockSizeX - 1) / BlockSizeX;
	uint32 NumBlocksY = (MipSizeY + BlockSizeY - 1) / BlockSizeY;

	if (PixelFormat == PF_PVRTC2 || PixelFormat == PF_PVRTC4)
	{
		// PVRTC has minimum 2 blocks width and height
		NumBlocksX = FMath::Max<uint32>(NumBlocksX, 2);
		NumBlocksY = FMath::Max<uint32>(NumBlocksY, 2);
	}

	// Size in bytes
	MipBytes = NumBlocksX * NumBlocksY * BlockBytes * Desc.Depth;
}

void FVulkanTexture::SetInitialImageState(FRHICommandListBase& RHICmdList, VkImageLayout InitialLayout, bool bClear, const FClearValueBinding& ClearValueBinding, bool bIsTransientResource)
{
	RHICmdList.EnqueueLambda(TEXT("FVulkanTexture::SetInitialImageState"),
		[VulkanTexture = this, InitialLayout, bClear, bIsTransientResource, ClearValueBinding](FRHICommandListBase& ExecutingCmdList)
		{
			// Can't use TransferQueue as Vulkan requires that queue to also have Gfx or Compute capabilities...
			// NOTE: Transient resources' memory might have belonged to another resource earlier in the ActiveCmdBuffer, so we can't use UploadCmdBuffer
			FVulkanCommandBuffer& CommandBuffer = bIsTransientResource ?
				FVulkanCommandListContext::Get(ExecutingCmdList).GetCommandBuffer() :
				FVulkanUploadContext::Get(ExecutingCmdList).GetCommandBuffer();
			ensure(CommandBuffer.IsOutsideRenderPass());

			VkImageSubresourceRange SubresourceRange = FVulkanPipelineBarrier::MakeSubresourceRange(VulkanTexture->GetFullAspectMask());

			VkImageLayout CurrentLayout = VK_IMAGE_LAYOUT_UNDEFINED;
			if (bClear && !bIsTransientResource)
			{
				{
					FVulkanPipelineBarrier Barrier;
					Barrier.AddImageLayoutTransition(VulkanTexture->Image, VK_IMAGE_LAYOUT_UNDEFINED, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL, SubresourceRange);
					Barrier.Execute(&CommandBuffer);
				}

				if (VulkanTexture->GetFullAspectMask() == VK_IMAGE_ASPECT_COLOR_BIT)
				{
					VkClearColorValue Color;
					FMemory::Memzero(Color);
					Color.float32[0] = ClearValueBinding.Value.Color[0];
					Color.float32[1] = ClearValueBinding.Value.Color[1];
					Color.float32[2] = ClearValueBinding.Value.Color[2];
					Color.float32[3] = ClearValueBinding.Value.Color[3];

					VulkanRHI::vkCmdClearColorImage(CommandBuffer.GetHandle(), VulkanTexture->Image, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL, &Color, 1, &SubresourceRange);
				}
				else
				{
					check(VulkanTexture->IsDepthOrStencilAspect());
					VkClearDepthStencilValue Value;
					FMemory::Memzero(Value);
					Value.depth = ClearValueBinding.Value.DSValue.Depth;
					Value.stencil = ClearValueBinding.Value.DSValue.Stencil;

					VulkanRHI::vkCmdClearDepthStencilImage(CommandBuffer.GetHandle(), VulkanTexture->Image, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL, &Value, 1, &SubresourceRange);
				}

				CurrentLayout = VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL;
			}

			if ((InitialLayout != CurrentLayout) && (InitialLayout != VK_IMAGE_LAYOUT_UNDEFINED))
			{
				FVulkanPipelineBarrier Barrier;
				Barrier.AddFullImageLayoutTransition(*VulkanTexture, CurrentLayout, InitialLayout);
				Barrier.Execute(&CommandBuffer);
			}
		});
}

TArray<VkSparseMemoryBind> FVulkanTexture::CommitReservedResource(uint64 RequiredCommitSizeInBytes)
{
	checkf((RequiredCommitSizeInBytes == UINT64_MAX) || (RequiredCommitSizeInBytes == 0), TEXT("Only full resource commits are supported for images."));
	checkf(ReservedResourceData, TEXT("CommitReservedResource called on a resource without any ReservedResourceData.  Resource needs to be created with the ReservedResource flag."));

	const bool bMergeConsecutiveBindings = CVarVulkanSparseImageMergeBindings.GetValueOnAnyThread() != 0;

	const uint64 SparseBlockSize = (uint64)CVarVulkanSparseImageAllocSizeMB.GetValueOnAnyThread() * 1024 * 1024;
	const uint64 TileSize = GRHIGlobals.ReservedResources.TileSizeInBytes;
	RequiredCommitSizeInBytes = Align(FMath::Min<uint64>(RequiredCommitSizeInBytes, ReservedResourceData->MemoryRequirements.size), TileSize);

	// Figure out the allocation needs
	const uint64 RequiredBlockCount = Align(RequiredCommitSizeInBytes, SparseBlockSize) / SparseBlockSize;
	const uint64 CurrentBlockCount = ReservedResourceData->Allocations.Num();

	// Figure out the tile needs
	const uint64 RequiredTileCount = Align(RequiredCommitSizeInBytes, TileSize) / TileSize;
	const uint64 CurrentTileCount = ReservedResourceData->NumCommittedTiles;

	TArray<VkSparseMemoryBind> Binds;
	if (RequiredTileCount > CurrentTileCount)
	{
		VkMemoryRequirements BlockMemoryRequirements = ReservedResourceData->MemoryRequirements;
		BlockMemoryRequirements.size = SparseBlockSize;

		// Make sure we will have all the memory we need
		ReservedResourceData->Allocations.Reserve(RequiredBlockCount);
		for (uint64 BlockCount = CurrentBlockCount; BlockCount < RequiredBlockCount; BlockCount++)
		{
			const uint32 BlockSize = (BlockCount == ReservedResourceData->BlockCount - 1) ? ReservedResourceData->LastBlockSize : SparseBlockSize;
			FVulkanReservedResourceData::FSparseAllocation& NewBlockAlloc = ReservedResourceData->Allocations.AddDefaulted_GetRef();
			if (!Device->GetMemoryManager().AllocateImageMemory(
				NewBlockAlloc.Allocation, nullptr, BlockMemoryRequirements,
				VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT, VulkanRHI::EVulkanAllocationMetaImageOther,
				false, __FILE__, __LINE__))
			{
				Device->GetMemoryManager().HandleOOM();
			}
			NewBlockAlloc.ResourceOffset = BlockCount * SparseBlockSize;
		}

		// Bind all the individual tiles
		Binds.Reserve(RequiredTileCount - CurrentTileCount);
		uint32 LastAllocIndex = MAX_uint32;
		for (uint64 TileCount = CurrentTileCount; TileCount < RequiredTileCount; TileCount++)
		{
			const uint32 TileOffset = (TileCount * TileSize);

			const uint32 AllocIndex = TileOffset / SparseBlockSize;
			const FVulkanReservedResourceData::FSparseAllocation& SparseAlloc = ReservedResourceData->Allocations[AllocIndex];

			if (bMergeConsecutiveBindings && !Binds.IsEmpty() && (LastAllocIndex == AllocIndex))
			{
				Binds.Last().size += TileSize;
			}
			else
			{
				VkSparseMemoryBind& NewBind = Binds.AddZeroed_GetRef();
				NewBind.size = TileSize;
				NewBind.memory = SparseAlloc.Allocation.GetDeviceMemoryHandle(Device);
				NewBind.memoryOffset = SparseAlloc.Allocation.Offset + TileOffset - SparseAlloc.ResourceOffset;
				NewBind.resourceOffset = TileOffset;
			}

			LastAllocIndex = AllocIndex;
		}

		const int64 CommitDeltaInBytes = TileSize * (RequiredTileCount - CurrentTileCount);
		UE::RHICore::UpdateReservedResourceStatsOnCommit(CommitDeltaInBytes, false /* bBuffer */, true /* bCommitting */);
	}
	else if (RequiredTileCount < CurrentTileCount)
	{
		if (bMergeConsecutiveBindings)
		{
			VkSparseMemoryBind& NewBind = Binds.AddZeroed_GetRef();
			NewBind.size = (CurrentTileCount - RequiredTileCount) * TileSize;
			NewBind.memory = VK_NULL_HANDLE;
			NewBind.memoryOffset = 0;
			NewBind.resourceOffset = (RequiredTileCount * TileSize);
		}
		else
		{
			for (int64 TileIndex = CurrentTileCount - 1; TileIndex >= (int64)RequiredTileCount; TileIndex--)
			{
				VkSparseMemoryBind& NewBind = Binds.AddZeroed_GetRef();
				NewBind.size = TileSize;
				NewBind.memory = VK_NULL_HANDLE;
				NewBind.memoryOffset = 0;
				NewBind.resourceOffset = (TileIndex * TileSize);
			}
		}

		// Free up unused blocks (deletion is deferred, don't have to wait for binds to be submitted)
		while (ReservedResourceData->Allocations.Num() > RequiredBlockCount)
		{
			ReservedResourceData->Allocations.Last().Allocation.Free(*Device);
			ReservedResourceData->Allocations.Pop();
		}

		const int64 CommitDeltaInBytes = TileSize * (CurrentTileCount - RequiredTileCount);
		UE::RHICore::UpdateReservedResourceStatsOnCommit(CommitDeltaInBytes, false /* bBuffer */, false /* bCommitting */);
	}

	ReservedResourceData->NumCommittedTiles = RequiredTileCount;
	return Binds;
}


/*-----------------------------------------------------------------------------
	Texture allocator support.
-----------------------------------------------------------------------------*/

void FVulkanDynamicRHI::RHIGetTextureMemoryStats(FTextureMemoryStats& OutStats)
{
	UE::RHICore::FillBaselineTextureMemoryStats(OutStats);

	check(Device);
	const uint64 TotalGPUMemory = Device->GetDeviceMemoryManager().GetTotalMemory(true);
	const uint64 TotalCPUMemory = Device->GetDeviceMemoryManager().GetTotalMemory(false);

	OutStats.DedicatedVideoMemory = TotalGPUMemory;
	OutStats.DedicatedSystemMemory = TotalCPUMemory;
	OutStats.SharedSystemMemory = -1;
	OutStats.TotalGraphicsMemory = TotalGPUMemory ? TotalGPUMemory : -1;

	OutStats.LargestContiguousAllocation = OutStats.StreamingMemorySize;
}

bool FVulkanDynamicRHI::RHIGetTextureMemoryVisualizeData( FColor* /*TextureData*/, int32 /*SizeX*/, int32 /*SizeY*/, int32 /*Pitch*/, int32 /*PixelSize*/ )
{
	VULKAN_SIGNAL_UNIMPLEMENTED();

	return false;
}

uint32 FVulkanDynamicRHI::RHIComputeMemorySize(FRHITexture* TextureRHI)
{
	if(!TextureRHI)
	{
		return 0;
	}

	return ResourceCast(TextureRHI)->GetMemorySize();
}

class FVulkanTextureReference : public FRHITextureReference
{
public:
	FVulkanTextureReference(FRHITexture* InReferencedTexture)
		: FRHITextureReference(InReferencedTexture)
	{
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FVulkanTextureReference(FRHITexture* InReferencedTexture, FVulkanShaderResourceView* InBindlessView)
		: FRHITextureReference(InReferencedTexture, InBindlessView->GetBindlessHandle())
		, BindlessView(InBindlessView)
	{
	}

	TRefCountPtr<FVulkanShaderResourceView> BindlessView;
#endif
};

template<>
struct TVulkanResourceTraits<FRHITextureReference>
{
	using TConcreteType = FVulkanTextureReference;
};

FTextureReferenceRHIRef FVulkanDynamicRHI::RHICreateTextureReference(FRHICommandListBase& RHICmdList, FRHITexture* InReferencedTexture)
{
	FRHITexture* ReferencedTexture = InReferencedTexture ? InReferencedTexture : FRHITextureReference::GetDefaultTexture();

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	// If the referenced texture is configured for bindless, make sure we also create an SRV to use for bindless.
	if (ReferencedTexture && ReferencedTexture->GetDefaultBindlessHandle().IsValid())
	{
		FShaderResourceViewRHIRef BindlessView = RHICmdList.CreateShaderResourceView(
			ReferencedTexture,
			FRHIViewDesc::CreateTextureSRV()
				.SetDimensionFromTexture(ReferencedTexture)
				.SetMipRange(0, 1));
		return new FVulkanTextureReference(ReferencedTexture, ResourceCast(BindlessView.GetReference()));
	}
#endif

	return new FVulkanTextureReference(ReferencedTexture);
}

void FVulkanDynamicRHI::RHIUpdateTextureReference(FRHICommandListBase& RHICmdList, FRHITextureReference* TextureRef, FRHITexture* InNewTexture)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (Device->SupportsBindless() && TextureRef && TextureRef->IsBindless())
	{
		// Workaround for a crash bug where FRHITextureReferences are deleted before this command is executed on the RHI thread.
		// Take a reference on the FRHITextureReference object to keep it alive.
		// @todo dev-pr - This should be refactored out when we eventually remove FRHITextureReference.
		TRefCountPtr<FVulkanTextureReference> Ref = ResourceCast(TextureRef);

		RHICmdList.EnqueueLambdaMultiPipe(GetEnabledRHIPipelines(), FRHICommandListBase::EThreadFence::Enabled, TEXT("FVulkanDynamicRHI::RHIUpdateTextureReference"),
			[TextureRef = MoveTemp(Ref), NewTexture = ResourceCast(InNewTexture ? InNewTexture : FRHITextureReference::GetDefaultTexture())](const FVulkanContextArray& Contexts)
			{
				FVulkanShaderResourceView* VulkanTextureRefSRV = TextureRef->BindlessView;
				FRHIDescriptorHandle DestHandle = VulkanTextureRefSRV->GetBindlessHandle();

				if (DestHandle.IsValid())
				{
					checkf(VulkanTextureRefSRV->IsInitialized(), TEXT("TextureReference should always be created with a view of the default texture at least"));

					const FRHITextureDesc& Desc = NewTexture->GetDesc();

					VulkanTextureRefSRV->Invalidate();

					FVulkanTextureViewDesc VulkanViewDesc;
					VulkanViewDesc.ViewType = NewTexture->GetViewType();
					VulkanViewDesc.AspectFlags = NewTexture->GetPartialAspectMask();
					VulkanViewDesc.UEFormat = Desc.Format;
					VulkanViewDesc.Format = NewTexture->ViewFormat;
					VulkanViewDesc.FirstMip = 0u;
					VulkanViewDesc.NumMips = FMath::Max(Desc.NumMips, (uint8)1u);
					VulkanViewDesc.ArraySliceIndex = 0u;
					VulkanViewDesc.NumArraySlices = NewTexture->GetNumberOfArrayLevels();
					VulkanViewDesc.bUseIdentitySwizzle = !NewTexture->SupportsSampling();
					VulkanTextureRefSRV->UpdateTextureView(Contexts, NewTexture->Image, VulkanViewDesc);
				}
			});
	}
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING

	FDynamicRHI::RHIUpdateTextureReference(RHICmdList, TextureRef, InNewTexture);
}

/*-----------------------------------------------------------------------------
	2D texture support.
-----------------------------------------------------------------------------*/

FVulkanDynamicRHI::FCreateTextureResult FVulkanDynamicRHI::BeginCreateTextureInternal(const FRHITextureCreateDesc& CreateDesc, const FRHITransientHeapAllocation* InTransientHeapAllocation)
{
	LLM_SCOPE_VULKAN(GetMemoryTagForTextureFlags(CreateDesc.Flags));
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.OwnerName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CreateDesc.DebugName, CreateDesc.GetTraceClassName(), CreateDesc.OwnerName);

	FVulkanTexture* Texture = new FVulkanTexture(*Device, CreateDesc, InTransientHeapAllocation);

	const bool bNeedsAllPlanes = Device->NeedsAllPlanes();

	if (bNeedsAllPlanes)
	{
		Texture->AllPlanesTrackedAccess[0] = CreateDesc.InitialState;
		Texture->AllPlanesTrackedAccess[1] = CreateDesc.InitialState;
	}

	const bool bIsTransientResource = (InTransientHeapAllocation != nullptr);
	const bool bIsTargetable = EnumHasAnyFlags(CreateDesc.Flags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable);
	const bool bDoInitialClear = VKHasAnyFlags(Texture->ImageUsageFlags, VK_IMAGE_USAGE_SAMPLED_BIT) && bIsTargetable && !bIsTransientResource;

	FCreateTextureResult Result{};
	Result.Texture = Texture;
	Result.DefaultLayout = GetInitialLayoutFromRHIAccess(CreateDesc.InitialState, bIsTargetable && Texture->IsDepthOrStencilAspect(), Texture->SupportsSampling());
	Result.bTransientResource = bIsTransientResource;
	Result.bClear = bDoInitialClear;

	return Result;
}

FVulkanTexture* FVulkanDynamicRHI::FinalizeCreateTextureInternal(FRHICommandListBase& RHICmdList, FCreateTextureResult CreateResult)
{
	FVulkanTexture* Texture = CreateResult.Texture;
	const FRHITextureDesc& Desc = Texture->GetDesc();

	if (EnumHasAnyFlags(Desc.Flags, TexCreate_ImmediateCommit))
	{
		checkf(EnumHasAnyFlags(Desc.Flags, TexCreate_ReservedResource), TEXT("The ImmediateCommit flag can only be used on textures created with the ReservedResource flag."));

		RHICmdList.EnqueueLambda(TEXT("FVulkanTexture::ImmediateCommit"),
			[Texture](FRHICommandListBase& ExecutingCmdList)
			{
				FVulkanCommandListContext& Context = FVulkanCommandListContext::Get(ExecutingCmdList);

				FVulkanCommitReservedResourceDesc CommitDesc;
				CommitDesc.Resource = Texture;
				CommitDesc.CommitSizeInBytes = UINT64_MAX;
				Context.GetPayload(FVulkanCommandListContext::EPhase::UpdateReservedResources).ReservedResourcesToCommit.Add(CommitDesc);
			});
	}

	if (!EnumHasAnyFlags(Desc.Flags, TexCreate_CPUReadback))
	{
		if (CreateResult.DefaultLayout != VK_IMAGE_LAYOUT_UNDEFINED || CreateResult.bClear)
		{
			Texture->SetInitialImageState(RHICmdList, CreateResult.DefaultLayout, CreateResult.bClear, Desc.ClearValue, CreateResult.bTransientResource);
		}
	}

	return Texture;
}

size_t VulkanCalculateTextureSize(FVulkanTexture* Texture)
{
	const FRHITextureDesc& Desc = Texture->GetDesc();
	const FPixelFormatInfo& FormatInfo = GPixelFormats[Desc.Format];

	const uint32 WidthInBlocks = FMath::DivideAndRoundUp<uint32>(Desc.Extent.X, FormatInfo.BlockSizeX);
	const uint32 HeightInBlocks = FMath::DivideAndRoundUp<uint32>(Desc.Extent.Y, FormatInfo.BlockSizeY);

	const VkPhysicalDeviceLimits& Limits = Texture->Device->GetLimits();

	const size_t StagingPitch = static_cast<size_t>(WidthInBlocks) * FormatInfo.BlockBytes;
	const size_t StagingBufferSize = Align(StagingPitch * HeightInBlocks, Limits.minMemoryMapAlignment);
	return StagingBufferSize;
}

namespace UE::VulkanTexture
{
	// Vulkan stores subresources by layer first. 
	// ie: Layer0.Mip0->Layer1.Mip0->Layer0.Mip1->Layer1->Mip1
	static uint64 CalculateSubresourceOffset(const FRHITextureDesc& Desc, FRHITextureInitializer::FSubresourceIndex SubresourceIndex, uint64& OutStride, uint64& OutSize)
	{
		const uint64 FaceCount = Desc.IsTextureCube() ? 6 : 1;
		const uint64 LayerCount = FaceCount * Desc.ArraySize;
		const uint64 LayerIndex = SubresourceIndex.FaceIndex + SubresourceIndex.ArrayIndex * FaceCount;

		uint64 LayerOffset = 0;

		uint64 MipOffset = 0;
		uint64 MipStride = 0;

		for (int32 Index = 0; Index <= SubresourceIndex.MipIndex; Index++)
		{
			const uint64 MipSize = UE::RHITextureUtils::CalculateTextureMipSize(Desc, Index, MipStride);

			if (Index == SubresourceIndex.MipIndex)
			{
				LayerOffset += MipSize * LayerIndex;

				OutSize = MipSize;
				OutStride = MipStride;
			}
			else
			{
				LayerOffset += MipSize * LayerCount;
			}
		}

		return LayerOffset;
	}
}

void FVulkanTexture::UploadInitialData(FRHICommandListBase& RHICmdList, VulkanRHI::FStagingBuffer* UploadBuffer)
{
	RHICmdList.EnqueueLambda(TEXT("FVulkanTexture::UploadInitialData"),
		[this, UploadBuffer](FRHICommandListBase& ExecutingCmdList) mutable
		{
			const FRHITextureDesc& Desc = GetDesc();

			const uint32 LayerCount = GetNumberOfArrayLevels();

			TArray<VkBufferImageCopy, TInlineAllocator<12>> Regions;
			Regions.AddZeroed(Desc.NumMips);

			uint64 MipOffset = 0;

			for (uint32 MipIndex = 0; MipIndex < Desc.NumMips; MipIndex++)
			{
				VkBufferImageCopy& Region = Regions[MipIndex];

				check(MipOffset < UploadBuffer->GetSize());

				const FUintVector3 MipExtents = UE::RHITextureUtils::CalculateMipExtents(Desc, MipIndex);

				Region.bufferOffset = MipOffset;

				Region.imageExtent.width = MipExtents.X;
				Region.imageExtent.height = MipExtents.Y;
				Region.imageExtent.depth = MipExtents.Z;

				Region.imageSubresource.aspectMask = FullAspectMask;
				Region.imageSubresource.layerCount = LayerCount;
				Region.imageSubresource.mipLevel = MipIndex;

				uint64 Stride = 0;
				const uint64 MipSize = UE::RHITextureUtils::CalculateTextureMipSize(Desc, MipIndex, Stride);
				const uint64 SubresourceSize = MipSize * LayerCount;

				MipOffset += SubresourceSize;
			}

			FVulkanUploadContext& Context = FVulkanUploadContext::Get(ExecutingCmdList);

			FVulkanCommandBuffer* CmdBuffer = Context.GetActiveCmdBuffer();
			ensure(CmdBuffer->IsOutsideRenderPass());
			VkCommandBuffer StagingCommandBuffer = CmdBuffer->GetHandle();

			// The layers refer to slices in 3D textures created with VK_IMAGE_CREATE_2D_ARRAY_COMPATIBLE_BIT if Maintenance9 is enabled 
			const bool bUseSlices =
				(Desc.Dimension == ETextureDimension::Texture3D) &&
				Device->GetOptionalExtensions().HasKHRMaintenance9 &&
				!EnumHasAnyFlags(Desc.Flags, TexCreate_ReservedResource);

			const VkImageSubresourceRange SubresourceRange = FVulkanPipelineBarrier::MakeSubresourceRange(FullAspectMask, 0, Desc.NumMips, 0, bUseSlices ? VK_REMAINING_ARRAY_LAYERS : LayerCount);

			{
				FVulkanPipelineBarrier Barrier;
				Barrier.AddImageLayoutTransition(Image, VK_IMAGE_LAYOUT_UNDEFINED, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL, SubresourceRange);
				Barrier.Execute(CmdBuffer);
			}

			VulkanRHI::vkCmdCopyBufferToImage(StagingCommandBuffer, UploadBuffer->GetHandle(), Image, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL, Regions.Num(), Regions.GetData());

			{
				FVulkanPipelineBarrier Barrier;
				Barrier.AddImageLayoutTransition(Image, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL, VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL, SubresourceRange);
				Barrier.Execute(CmdBuffer);
			}

			Device->GetStagingManager().ReleaseBuffer(&Context, UploadBuffer);
		});
}

void FVulkanTexture::HostUploadInitialData(FRHICommandListBase& RHICmdList, const uint8* BulkData, uint32 BulkDataSize)
{
	check(Device->GetOptionalExtensions().HasEXTHostImageCopy);
	check(BulkData);

	const FRHITextureDesc& Desc = GetDesc();
	const uint32 LayerCount = GetNumberOfArrayLevels();

	const VkImageSubresourceRange FullSubresourceRange = {
			.aspectMask = GetFullAspectMask(),
			.baseMipLevel = 0,
			.levelCount = Desc.NumMips,
			.baseArrayLayer = 0,
			.layerCount = LayerCount
	};

	// Transition the entire texture to GENERAL
	VkHostImageLayoutTransitionInfo TransitionInfo = {
		.sType = VK_STRUCTURE_TYPE_HOST_IMAGE_LAYOUT_TRANSITION_INFO,
		.pNext = nullptr,
		.image = Image,
		.oldLayout = VK_IMAGE_LAYOUT_UNDEFINED,
		.newLayout = VK_IMAGE_LAYOUT_GENERAL,
		.subresourceRange = FullSubresourceRange
	};
	VERIFYVULKANRESULT(VulkanRHI::vkTransitionImageLayoutEXT(Device->GetHandle(), 1, &TransitionInfo));

	TArray<VkMemoryToImageCopy, TInlineAllocator<12>> Regions;
	Regions.AddZeroed(Desc.NumMips);
	{
		uint64 MipOffset = 0;
		for (uint32 MipIndex = 0; MipIndex < Desc.NumMips; MipIndex++)
		{
			VkMemoryToImageCopy& Region = Regions[MipIndex];

			check(MipOffset < BulkDataSize);

			const FUintVector3 MipExtents = UE::RHITextureUtils::CalculateMipExtents(Desc, MipIndex);

			Region.pHostPointer = BulkData + MipOffset;

			Region.imageExtent.width = MipExtents.X;
			Region.imageExtent.height = MipExtents.Y;
			Region.imageExtent.depth = MipExtents.Z;

			Region.imageSubresource.aspectMask = FullAspectMask;
			Region.imageSubresource.layerCount = LayerCount;
			Region.imageSubresource.mipLevel = MipIndex;

			uint64 Stride = 0;
			const uint64 MipSize = UE::RHITextureUtils::CalculateTextureMipSize(Desc, MipIndex, Stride);
			const uint64 SubresourceSize = MipSize * LayerCount;

			MipOffset += SubresourceSize;
		}
	}

	VkCopyMemoryToImageInfo CopyMemoryToImageInfo = {
		.sType = VK_STRUCTURE_TYPE_COPY_MEMORY_TO_IMAGE_INFO,
		.pNext = nullptr,
		.flags = 0,
		.dstImage = Image,
		.dstImageLayout = VK_IMAGE_LAYOUT_GENERAL,
		.regionCount = (uint32)Regions.Num(),
		.pRegions = Regions.GetData()
	};
	VERIFYVULKANRESULT(VulkanRHI::vkCopyMemoryToImageEXT(Device->GetHandle(), &CopyMemoryToImageInfo));

	// Now queue up a barrier to execute on the GPU to bring it back to a read-only layout
	{
		FVulkanUploadContext& Context = FVulkanUploadContext::Get(RHICmdList);
		FVulkanCommandBuffer& CommandBuffer = Context.GetCommandBuffer();
		ensure(CommandBuffer.IsOutsideRenderPass());

		FVulkanPipelineBarrier Barrier;
		Barrier.AddImageLayoutTransition(Image, VK_IMAGE_LAYOUT_GENERAL, VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL, FullSubresourceRange);
		Barrier.Execute(CommandBuffer.GetHandle());
	}
}


FVulkanTexture* FVulkanDynamicRHI::CreateTextureInternal(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc)
{
	FCreateTextureResult CreateResult = BeginCreateTextureInternal(CreateDesc, nullptr);
	return FinalizeCreateTextureInternal(RHICmdList, CreateResult);
}

FVulkanTexture* FVulkanDynamicRHI::CreateTextureInternal(const FRHITextureCreateDesc& CreateDesc, const FRHITransientHeapAllocation& InTransientHeapAllocation)
{
	checkf(!EnumHasAnyFlags(CreateDesc.Flags, TexCreate_ReservedResource), TEXT("Can't use reserved resources with transient allocation."));
	FCreateTextureResult CreateResult = BeginCreateTextureInternal(CreateDesc, &InTransientHeapAllocation);
	return CreateResult.Texture;
}

FRHITextureInitializer FVulkanDynamicRHI::RHICreateTextureInitializer(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.OwnerName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CreateDesc.DebugName, CreateDesc.GetTraceClassName(), CreateDesc.OwnerName);

	if (CreateDesc.InitAction == ERHITextureInitAction::Default)
	{
		return UE::RHICore::FDefaultTextureInitializer(RHICmdList, CreateTextureInternal(RHICmdList, CreateDesc));
	}

	check(EnumHasAnyFlags(CreateDesc.InitialState, ERHIAccess::SRVMask));

	// TODO: add something to force VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL as the default layout

	FCreateTextureResult CreateResult = BeginCreateTextureInternal(CreateDesc, nullptr);
	const bool bUseHostImageCopies = Device->GetOptionalExtensions().HasEXTHostImageCopy && 
		!CreateResult.bClear && CreateResult.Texture && 
		VKHasAllFlags(CreateResult.Texture->ImageUsageFlags, VK_IMAGE_USAGE_HOST_TRANSFER_BIT);
	if (bUseHostImageCopies)
	{
		// We don't want finalize to transition the image when done from host
		CreateResult.DefaultLayout = VK_IMAGE_LAYOUT_UNDEFINED;
	}
	FVulkanTexture* Texture = FinalizeCreateTextureInternal(RHICmdList, CreateResult);

	if (bUseHostImageCopies)
	{
		if (CreateDesc.InitAction == ERHITextureInitAction::BulkData)
		{
			checkf(CreateDesc.BulkData, TEXT("InitAction was BulkData, but none was specified!"));
			Texture->HostUploadInitialData(RHICmdList, reinterpret_cast<const uint8*>(CreateDesc.BulkData->GetResourceBulkData()), CreateDesc.BulkData->GetResourceBulkDataSize());

			// Discard the bulk data's contents.
			CreateDesc.BulkData->Discard();

			return UE::RHICore::FDefaultTextureInitializer(RHICmdList, Texture);
		}
		else if (CreateDesc.InitAction == ERHITextureInitAction::Initializer)
		{
			const uint64 UploadMemorySize = UE::RHITextureUtils::CalculateTextureSize(CreateDesc);
			uint8* UploadMemory = reinterpret_cast<uint8*>(FMemory::Malloc(UploadMemorySize));

			return UE::RHICore::FBaseTextureInitializerImplementation(RHICmdList, Texture, UploadMemory, UploadMemorySize,
				[Texture = TRefCountPtr<FVulkanTexture>(Texture), UploadMemory, UploadMemorySize](FRHICommandListBase& RHICmdList) mutable
				{
					Texture->HostUploadInitialData(RHICmdList, UploadMemory, UploadMemorySize);
					FMemory::Free(UploadMemory);
					return TRefCountPtr<FRHITexture>(MoveTemp(Texture));
				},
				[Texture = TRefCountPtr<FVulkanTexture>(Texture), UploadMemory, UploadMemorySize](FRHITextureInitializer::FSubresourceIndex SubresourceIndex) mutable
				{
					const FRHITextureDesc& TextureDesc = Texture->GetDesc();

					uint64 SubresourceStride = 0;
					uint64 SubresourceSize = 0;
					const uint64 Offset = UE::VulkanTexture::CalculateSubresourceOffset(TextureDesc, SubresourceIndex, SubresourceStride, SubresourceSize);
					checkf(Offset + SubresourceSize <= UploadMemorySize,
						TEXT("Subresource(Face=%d,Array=%d,Mip=%d) access out of bounds: Offset=%lld + SubresourceSize=%lld > UploadMemorySize=%lld"), 
						SubresourceIndex.FaceIndex, SubresourceIndex.ArrayIndex, SubresourceIndex.MipIndex,
						Offset, SubresourceSize, UploadMemorySize);

					FRHITextureSubresourceInitializer Result{};
					Result.Data = UploadMemory + Offset;
					Result.Stride = SubresourceStride;
					Result.Size = SubresourceSize;

					return Result;
				}
			);
		}
	}
	else
	{
		const uint64 UploadMemorySize = UE::RHITextureUtils::CalculateTextureSize(CreateDesc);

		// Transfer bulk data
		VulkanRHI::FStagingBuffer* UploadBuffer = Device->GetStagingManager().AcquireBuffer(UploadMemorySize);
		void* const UploadData = UploadBuffer->GetMappedPointer();

		UE::RHICore::FBaseTextureInitializerImplementation Initializer(RHICmdList, Texture, UploadData, UploadMemorySize,
			[Texture = TRefCountPtr<FVulkanTexture>(Texture), UploadBuffer](FRHICommandListBase& RHICmdList) mutable
			{
				Texture->UploadInitialData(RHICmdList, UploadBuffer);
				return TRefCountPtr<FRHITexture>(MoveTemp(Texture));
			},
			[Texture = TRefCountPtr<FVulkanTexture>(Texture), UploadBuffer](FRHITextureInitializer::FSubresourceIndex SubresourceIndex) mutable
			{
				const FRHITextureDesc& TextureDesc = Texture->GetDesc();

				uint64 SubresourceStride = 0;
				uint64 SubresourceSize = 0;
				const uint64 Offset = UE::VulkanTexture::CalculateSubresourceOffset(TextureDesc, SubresourceIndex, SubresourceStride, SubresourceSize);
				check(Offset + SubresourceSize <= UploadBuffer->GetSize());

				FRHITextureSubresourceInitializer Result{};
				Result.Data = reinterpret_cast<uint8*>(UploadBuffer->GetMappedPointer()) + Offset;
				Result.Stride = SubresourceStride;
				Result.Size = SubresourceSize;

				return Result;
			}
		);

		if (CreateDesc.InitAction == ERHITextureInitAction::BulkData)
		{
			check(CreateDesc.BulkData);
			check(UploadMemorySize >= CreateDesc.BulkData->GetResourceBulkDataSize());

			FMemory::Memcpy(UploadData, CreateDesc.BulkData->GetResourceBulkData(), CreateDesc.BulkData->GetResourceBulkDataSize());

			// Discard the bulk data's contents.
			CreateDesc.BulkData->Discard();

			return MoveTemp(Initializer);
		}
		else if (CreateDesc.InitAction == ERHITextureInitAction::Initializer)
		{
			return MoveTemp(Initializer);
		}
	}

	return UE::RHICore::HandleUnknownTextureInitializerInitAction(RHICmdList, CreateDesc);
}

FTextureRHIRef FVulkanDynamicRHI::RHIAsyncCreateTexture2D(uint32 SizeX,uint32 SizeY,uint8 Format,uint32 NumMips,ETextureCreateFlags Flags, ERHIAccess InResourceState,void** InitialMipData,uint32 NumInitialMips, const TCHAR* DebugName, FGraphEventRef& OutCompletionEvent)
{
	UE_LOGF(LogVulkan, Fatal, "RHIAsyncCreateTexture2D is not supported");
	VULKAN_SIGNAL_UNIMPLEMENTED();
	return FTextureRHIRef();
}

static FCriticalSection GTextureMapLock;

FRHILockTextureResult FVulkanDynamicRHI::RHILockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanTextures);

	FVulkanTexture* Texture = ResourceCast(Arguments.Texture);

	FRHILockTextureResult Result{};
	Texture->GetMipSize(Arguments.MipIndex, Result.ByteCount);
	Texture->GetMipStride(Arguments.MipIndex, Result.Stride);

	VulkanRHI::FStagingBuffer* StagingBuffer = Device->GetStagingManager().AcquireBuffer(Result.ByteCount);
	{
		FScopeLock Lock(&GTextureMapLock);
		GRHILockTracker.Lock(Arguments, reinterpret_cast<void*>(StagingBuffer), false);
	}

	Result.Data = StagingBuffer->GetMappedPointer();
	
	return Result;
}

void FVulkanDynamicRHI::RHIUnlockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanTextures);

	VulkanRHI::FStagingBuffer* StagingBuffer = nullptr;
	{
		FScopeLock Lock(&GTextureMapLock);
		const FRHILockTracker::FTextureLockParams Params = GRHILockTracker.Unlock(Arguments);
		StagingBuffer = reinterpret_cast<VulkanRHI::FStagingBuffer*>(Params.Data);
		checkf(StagingBuffer, TEXT("Texture was not locked!"));
	}

	FVulkanTexture* Texture = ResourceCast(Arguments.Texture);
	const FRHITextureDesc& Desc = Texture->GetDesc();

	const uint32 ArrayIndex = UE::RHICore::GetLockArrayIndex(Desc, Arguments);

	uint32 MipWidth  = FMath::Max<uint32>(Desc.Extent.X >> Arguments.MipIndex, 0);
	uint32 MipHeight = FMath::Max<uint32>(Desc.Extent.Y >> Arguments.MipIndex, 0);

	ensure(!(MipHeight == 0 && MipWidth == 0));

	MipWidth = FMath::Max<uint32>(MipWidth, 1);
	MipHeight = FMath::Max<uint32>(MipHeight, 1);

	VkBufferImageCopy Region{};
	Region.imageSubresource.aspectMask = Texture->GetPartialAspectMask();
	Region.imageSubresource.mipLevel = Arguments.MipIndex;
	Region.imageSubresource.baseArrayLayer = ArrayIndex;
	Region.imageSubresource.layerCount = 1;
	Region.imageExtent.width = MipWidth;
	Region.imageExtent.height = MipHeight;
	Region.imageExtent.depth = 1;

	const bool bNeedsTransition = RHICmdList.NeedsExtraTransitions();
	RHICmdList.EnqueueLambda(TEXT("FVulkanTexture::InternalLockWrite"),
		[Texture, Region, StagingBuffer, bNeedsTransition](FRHICommandListBase& ExecutingCmdList)
		{
			FVulkanTexture::InternalLockWrite(FVulkanCommandListContext::Get(ExecutingCmdList), Texture, Region, StagingBuffer, bNeedsTransition);
		});
}


void FVulkanDynamicRHI::InternalUpdateTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, uint32 MipIndex, const FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData)
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanTextures);

	const FPixelFormatInfo& FormatInfo = GPixelFormats[TextureRHI->GetFormat()];

	check(UpdateRegion.Width  % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.Height % FormatInfo.BlockSizeY == 0);
	check(UpdateRegion.DestX  % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.DestY  % FormatInfo.BlockSizeY == 0);
	check(UpdateRegion.SrcX   % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.SrcY   % FormatInfo.BlockSizeY == 0);

	const uint32 SrcXInBlocks   = FMath::DivideAndRoundUp<uint32>(UpdateRegion.SrcX,   FormatInfo.BlockSizeX);
	const uint32 SrcYInBlocks   = FMath::DivideAndRoundUp<uint32>(UpdateRegion.SrcY,   FormatInfo.BlockSizeY);
	const uint32 WidthInBlocks  = FMath::DivideAndRoundUp<uint32>(UpdateRegion.Width,  FormatInfo.BlockSizeX);
	const uint32 HeightInBlocks = FMath::DivideAndRoundUp<uint32>(UpdateRegion.Height, FormatInfo.BlockSizeY);

	const VkPhysicalDeviceLimits& Limits = Device->GetLimits();

	const size_t StagingPitch = static_cast<size_t>(WidthInBlocks) * FormatInfo.BlockBytes;
	const size_t StagingBufferSize = Align(StagingPitch * HeightInBlocks, Limits.minMemoryMapAlignment);

	VulkanRHI::FStagingBuffer* StagingBuffer = Device->GetStagingManager().AcquireBuffer(StagingBufferSize);
	void* RESTRICT StagingMemory = StagingBuffer->GetMappedPointer();

	const uint8* CopySrc = SourceData + FormatInfo.BlockBytes * SrcXInBlocks + SourcePitch * SrcYInBlocks;
	uint8* CopyDst = (uint8*)StagingMemory;
	for (uint32 BlockRow = 0; BlockRow < HeightInBlocks; BlockRow++)
	{
		FMemory::Memcpy(CopyDst, CopySrc, WidthInBlocks * FormatInfo.BlockBytes);
		CopySrc += SourcePitch;
		CopyDst += StagingPitch;
	}

	const FIntVector MipDimensions = TextureRHI->GetMipDimensions(MipIndex);
	VkBufferImageCopy Region{};
	Region.imageSubresource.aspectMask = VK_IMAGE_ASPECT_COLOR_BIT;
	Region.imageSubresource.mipLevel = MipIndex;
	Region.imageSubresource.layerCount = 1;
	Region.imageOffset.x = UpdateRegion.DestX;
	Region.imageOffset.y = UpdateRegion.DestY;
	Region.imageExtent.width = FMath::Min(UpdateRegion.Width, static_cast<uint32>(MipDimensions.X) - UpdateRegion.DestX);
	Region.imageExtent.height = FMath::Min(UpdateRegion.Height, static_cast<uint32>(MipDimensions.Y) - UpdateRegion.DestY);
	Region.imageExtent.depth = 1;

	FVulkanTexture* Texture = ResourceCast(TextureRHI);
	const bool bNeedsTransition = RHICmdList.NeedsExtraTransitions();
	RHICmdList.EnqueueLambda(TEXT("FVulkanTexture::InternalLockWrite"),
		[Texture, Region, StagingBuffer, bNeedsTransition](FRHICommandListBase& ExecutingCmdList)
		{
			FVulkanTexture::InternalLockWrite(FVulkanCommandListContext::Get(ExecutingCmdList), Texture, Region, StagingBuffer, bNeedsTransition);
		});
}

FUpdateTexture3DData FVulkanDynamicRHI::RHIBeginUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion)
{
	const int32 FormatSize = PixelFormatBlockBytes[Texture->GetFormat()];
	const int32 RowPitch = UpdateRegion.Width * FormatSize;
	const int32 DepthPitch = UpdateRegion.Width * UpdateRegion.Height * FormatSize;

	SIZE_T MemorySize = static_cast<SIZE_T>(DepthPitch)* UpdateRegion.Depth;
	uint8* Data = (uint8*)FMemory::Malloc(MemorySize);

	return FUpdateTexture3DData(Texture, MipIndex, UpdateRegion, RowPitch, DepthPitch, Data, MemorySize, GFrameNumberRenderThread);
}

void FVulkanDynamicRHI::RHIEndUpdateTexture3D(FRHICommandListBase& RHICmdList, FUpdateTexture3DData& UpdateData)
{
	check(IsInParallelRenderingThread());
	check(GFrameNumberRenderThread == UpdateData.FrameNumber);

	InternalUpdateTexture3D(RHICmdList, UpdateData.Texture, UpdateData.MipIndex, UpdateData.UpdateRegion, UpdateData.RowPitch, UpdateData.DepthPitch, UpdateData.Data);
	
	FMemory::Free(UpdateData.Data);
	UpdateData.Data = nullptr;
}

void FVulkanDynamicRHI::InternalUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, uint32 MipIndex, const FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData)
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanTextures);
	FVulkanTexture* Texture = ResourceCast(TextureRHI);

	const EPixelFormat PixelFormat = Texture->GetDesc().Format;
	const int32 BlockSizeX = GPixelFormats[PixelFormat].BlockSizeX;
	const int32 BlockSizeY = GPixelFormats[PixelFormat].BlockSizeY;
	const int32 BlockSizeZ = GPixelFormats[PixelFormat].BlockSizeZ;
	const int32 BlockBytes = GPixelFormats[PixelFormat].BlockBytes;
	const VkFormat Format = UEToVkTextureFormat(PixelFormat, false);

	ensure(BlockSizeZ == 1);

	const VkPhysicalDeviceLimits& Limits = Device->GetLimits();

	VulkanRHI::FStagingBuffer* StagingBuffer = nullptr;
	const uint32 NumBlocksX = (uint32)FMath::DivideAndRoundUp<int32>(UpdateRegion.Width, (uint32)BlockSizeX);
	const uint32 NumBlocksY = (uint32)FMath::DivideAndRoundUp<int32>(UpdateRegion.Height, (uint32)BlockSizeY);
	check(NumBlocksX * BlockBytes <= SourceRowPitch);
	check(NumBlocksX * BlockBytes * NumBlocksY <= SourceDepthPitch);

	const uint32 DestRowPitch = NumBlocksX * BlockBytes;
	const uint32 DestSlicePitch = DestRowPitch * NumBlocksY;

	const uint32 BufferSize = Align(DestSlicePitch * UpdateRegion.Depth, Limits.minMemoryMapAlignment);
	StagingBuffer = Device->GetStagingManager().AcquireBuffer(BufferSize);
	void* RESTRICT Memory = StagingBuffer->GetMappedPointer();

	ensure(UpdateRegion.SrcX == 0);
	ensure(UpdateRegion.SrcY == 0);

	uint8* RESTRICT DestData = (uint8*)Memory;
	for (uint32 Depth = 0; Depth < UpdateRegion.Depth; Depth++)
	{
		uint8* RESTRICT SourceRowData = (uint8*)SourceData + SourceDepthPitch * Depth;
		for (uint32 Height = 0; Height < NumBlocksY; ++Height)
		{
			FMemory::Memcpy(DestData, SourceRowData, NumBlocksX * BlockBytes);
			DestData += DestRowPitch;
			SourceRowData += SourceRowPitch;
		}
	}
	uint32 TextureSizeX = FMath::Max(1u, TextureRHI->GetSizeX() >> MipIndex);
	uint32 TextureSizeY = FMath::Max(1u, TextureRHI->GetSizeY() >> MipIndex);
	uint32 TextureSizeZ = FMath::Max(1u, TextureRHI->GetSizeZ() >> MipIndex);

	// The layers refer to slices in 3D textures created with VK_IMAGE_CREATE_2D_ARRAY_COMPATIBLE_BIT if Maintenance9 is enabled 
	// (Maintenance5 allows the use of VK_REMAINING_ARRAY_LAYERS for copies)
	const bool bUseSlices = Device->GetOptionalExtensions().HasKHRMaintenance5 && Device->GetOptionalExtensions().HasKHRMaintenance9 && !EnumHasAnyFlags(Texture->GetDesc().Flags, TexCreate_ReservedResource);

	const VkBufferImageCopy Region = {
		.bufferOffset = 0,
		.bufferRowLength = 0,   // zero assumes tightly packed buffer
		.bufferImageHeight = 0, // zero assumes tightly packed buffer
		.imageSubresource = {
			.aspectMask = VK_IMAGE_ASPECT_COLOR_BIT,
			.mipLevel = MipIndex,
			.baseArrayLayer = 0,
			.layerCount = bUseSlices ? VK_REMAINING_ARRAY_LAYERS : 1
		},
		.imageOffset = { 
			.x = (int32)UpdateRegion.DestX, 
			.y = (int32)UpdateRegion.DestY,
			.z = (int32)UpdateRegion.DestZ
		},
		.imageExtent = {
			.width = (uint32)FMath::Min((int32)(TextureSizeX - UpdateRegion.DestX), (int32)UpdateRegion.Width),
			.height = (uint32)FMath::Min((int32)(TextureSizeY - UpdateRegion.DestY), (int32)UpdateRegion.Height),
			.depth = (uint32)FMath::Min((int32)(TextureSizeZ - UpdateRegion.DestZ), (int32)UpdateRegion.Depth)
		}
	};

	const bool bNeedsTransition = RHICmdList.NeedsExtraTransitions();
	RHICmdList.EnqueueLambda(TEXT("FVulkanTexture::InternalLockWrite"),
		[Texture, Region, StagingBuffer, bNeedsTransition](FRHICommandListBase& ExecutingCmdList)
		{
			FVulkanTexture::InternalLockWrite(FVulkanCommandListContext::Get(ExecutingCmdList), Texture, Region, StagingBuffer, bNeedsTransition);
		});
}




FVulkanTexture::FVulkanTexture(FVulkanDevice& InDevice, const FRHITextureCreateDesc& InCreateDesc, const FRHITransientHeapAllocation* InTransientHeapAllocation)
	: FRHITexture(InCreateDesc)
	, Device(&InDevice)
	, ImageOwnerType(EImageOwnerType::LocalOwner)
{
	VULKAN_TRACK_OBJECT_CREATE(FVulkanTexture, this);

	if (EnumHasAnyFlags(InCreateDesc.Flags, TexCreate_CPUReadback))
	{
		check(InCreateDesc.NumSamples == 1); //not implemented
		check(InCreateDesc.ArraySize == 1);  //not implemented

		CpuReadbackBuffer = new FVulkanCpuReadbackBuffer;
		uint64 Size = 0;
		for (uint32 Mip = 0; Mip < InCreateDesc.NumMips; ++Mip)
		{
			uint64 LocalSize = 0;
			GetMipSize(Mip, LocalSize);
			CpuReadbackBuffer->MipOffsets[Mip] = Size;
			Size += LocalSize;
		}

		CpuReadbackBuffer->Buffer = InDevice.CreateBuffer(Size, VK_BUFFER_USAGE_TRANSFER_DST_BIT);

		// Set minimum alignment to 16 bytes, as some buffers are used with CPU SIMD instructions
		const uint32 ForcedMinAlignment = 16u;
		const VulkanRHI::EVulkanAllocationFlags AllocFlags = VulkanRHI::EVulkanAllocationFlags::HostCached | VulkanRHI::EVulkanAllocationFlags::AutoBind;
		InDevice.GetMemoryManager().AllocateBufferMemory(Allocation, CpuReadbackBuffer->Buffer, AllocFlags, InCreateDesc.DebugName, ForcedMinAlignment);

		void* Memory = Allocation.GetMappedPointer(Device);
		FMemory::Memzero(Memory, Size);
		Allocation.FlushMappedMemory(Device);

		ImageOwnerType = EImageOwnerType::None;
		ViewFormat = StorageFormat = UEToVkTextureFormat(InCreateDesc.Format, false);

		// :todo-jn: Kept around temporarily for legacy defrag/eviction/stats
		VulkanRHI::vkGetBufferMemoryRequirements(InDevice.GetHandle(), CpuReadbackBuffer->Buffer, &MemoryRequirements);

		return;
	}

	FImageCreateInfo ImageCreateInfo;
	FVulkanTexture::GenerateImageCreateInfo(ImageCreateInfo, InDevice, InCreateDesc, &StorageFormat, &ViewFormat);

	VERIFYVULKANRESULT(VulkanRHI::vkCreateImage(InDevice.GetHandle(), &ImageCreateInfo.ImageCreateInfo, VULKAN_CPU_ALLOCATOR, &Image));

	// Fetch image size
	VulkanRHI::vkGetImageMemoryRequirements(InDevice.GetHandle(), Image, &MemoryRequirements);

	VULKAN_SET_DEBUG_NAME(InDevice, VK_OBJECT_TYPE_IMAGE, Image, TEXT("%s:(FVulkanTexture*)0x%p"), InCreateDesc.DebugName ? InCreateDesc.DebugName : TEXT("?"), this);

	FullAspectMask = VulkanRHI::GetAspectMaskFromUEFormat(InCreateDesc.Format, true, true);
	PartialAspectMask = VulkanRHI::GetAspectMaskFromUEFormat(InCreateDesc.Format, false, true);

	// If VK_IMAGE_TILING_OPTIMAL is specified,
	// memoryTypeBits in vkGetImageMemoryRequirements will become 1
	// which does not support VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT.
	if (ImageCreateInfo.ImageCreateInfo.tiling != VK_IMAGE_TILING_OPTIMAL)
	{
		MemProps |= VK_MEMORY_PROPERTY_HOST_VISIBLE_BIT;
	}

	const bool bRenderTarget = EnumHasAnyFlags(InCreateDesc.Flags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable | TexCreate_ResolveTargetable);
	const bool bUAV = EnumHasAnyFlags(InCreateDesc.Flags, TexCreate_UAV);
	const bool bExternal = EnumHasAnyFlags(InCreateDesc.Flags, TexCreate_External);
	const bool bIsTransientResource = (InTransientHeapAllocation != nullptr) && InTransientHeapAllocation->IsValid();

	VkMemoryPropertyFlags MemoryFlags = VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT;

	bool bMemoryless = EnumHasAnyFlags(InCreateDesc.Flags, TexCreate_Memoryless) && InDevice.GetDeviceMemoryManager().SupportsMemoryless();
	if (bMemoryless)
	{
		if (ensureMsgf(bRenderTarget, TEXT("Memoryless surfaces can only be used for render targets")))
		{
			MemoryFlags |= VK_MEMORY_PROPERTY_LAZILY_ALLOCATED_BIT;
		}
		else
		{
			bMemoryless = false;
		}
	}

	if (EnumHasAnyFlags(InCreateDesc.Flags, TexCreate_ReservedResource))
	{
		check(!bIsTransientResource);
		ReservedResourceData = MakeUnique<FVulkanReservedResourceData>();
		ReservedResourceData->MemoryRequirements = MemoryRequirements;

		const uint64 SparseBlockSize = (uint64)CVarVulkanSparseImageAllocSizeMB.GetValueOnAnyThread() * 1024 * 1024;
		ReservedResourceData->LastBlockSize = Align(MemoryRequirements.size % SparseBlockSize, GRHIGlobals.ReservedResources.TileSizeInBytes);
		ReservedResourceData->BlockCount = MemoryRequirements.size / SparseBlockSize;
		if (ReservedResourceData->LastBlockSize == 0)
		{
			ReservedResourceData->LastBlockSize = SparseBlockSize;
		}
		else
		{
			ReservedResourceData->BlockCount++;
		}

		checkf(SparseBlockSize % ReservedResourceData->MemoryRequirements.alignment == 0,
			TEXT("The value of r.Vulkan.SparseImageAllocSizeMB (%dMB) is not compatible with this resource's tile size requirements (%lld)."),
			CVarVulkanSparseImageAllocSizeMB.GetValueOnAnyThread(), ReservedResourceData->MemoryRequirements.alignment);
		checkf(ReservedResourceData->LastBlockSize % ReservedResourceData->MemoryRequirements.alignment == 0,
			TEXT("The value of GRHIGlobals.ReservedResources.TileSizeInBytes (%d) is not compatible with this resource's tile size requirements (%lld)."),
			GRHIGlobals.ReservedResources.TileSizeInBytes, ReservedResourceData->MemoryRequirements.alignment);

		VulkanTextureAllocated(GetDesc(), MemoryRequirements.size);
	}
	else
	{
		if (bIsTransientResource)
		{
			check(!bMemoryless);
			check(InTransientHeapAllocation->Offset % MemoryRequirements.alignment == 0);
			check(InTransientHeapAllocation->Size >= MemoryRequirements.size);
			Allocation = FVulkanTransientHeap::GetVulkanAllocation(*InTransientHeapAllocation);
		}
		else
		{
			VulkanRHI::EVulkanAllocationMetaType MetaType = (bRenderTarget || bUAV) ? VulkanRHI::EVulkanAllocationMetaImageRenderTarget : VulkanRHI::EVulkanAllocationMetaImageOther;
	#if VULKAN_SUPPORTS_DEDICATED_ALLOCATION
			extern int32 GVulkanEnableDedicatedImageMemory;
			// Per https://developer.nvidia.com/what%E2%80%99s-your-vulkan-memory-type
			VkDeviceSize SizeToBeConsideredForDedicated = 12 * 1024 * 1024;
			if ((bRenderTarget || MemoryRequirements.size >= SizeToBeConsideredForDedicated) && !bMemoryless && GVulkanEnableDedicatedImageMemory)
			{
				constexpr bool bForceDedicated = false;  // it should be considered for dedicated, but don't force it and let driver decide
				if (!InDevice.GetMemoryManager().AllocateDedicatedImageMemory(Allocation, this, Image, MemoryFlags, MetaType, bExternal, bForceDedicated, __FILE__, __LINE__))
				{
					checkNoEntry();
				}
			}
			else
	#endif
			{
				if (!InDevice.GetMemoryManager().AllocateImageMemory(Allocation, this, MemoryRequirements, MemoryFlags, MetaType, bExternal, __FILE__, __LINE__))
				{
					checkNoEntry();
				}
			}

			// update rhi stats
			VulkanTextureAllocated(GetDesc(), Allocation.Size);
		}
		Allocation.BindImage(Device, Image);
	}

	Tiling = ImageCreateInfo.ImageCreateInfo.tiling;
	check(Tiling == VK_IMAGE_TILING_LINEAR || Tiling == VK_IMAGE_TILING_OPTIMAL);
	ImageUsageFlags = ImageCreateInfo.ImageCreateInfo.usage;

	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanTextures);
	const VkImageViewType ViewType = GetViewType();
	const bool bIsSRGB = EnumHasAllFlags(InCreateDesc.Flags, TexCreate_SRGB);
	if (ViewFormat == VK_FORMAT_UNDEFINED)
	{
		StorageFormat = UEToVkTextureFormat(InCreateDesc.Format, false);
		ViewFormat = UEToVkTextureFormat(InCreateDesc.Format, bIsSRGB);
		checkf(StorageFormat != VK_FORMAT_UNDEFINED, TEXT("Pixel Format %d not defined!"), (int32)InCreateDesc.Format);
	}

	InitViews(ImageUsageFlags);
}

FVulkanTexture::FVulkanTexture(FVulkanDevice& InDevice, const FRHITextureCreateDesc& InCreateDesc, VkImage InImage, const FVulkanRHIExternalImageDeleteCallbackInfo& InExternalImageDeleteCallbackInfo)
	: FRHITexture(InCreateDesc)
	, Device(&InDevice)
	, Image(InImage)
	, ExternalImageDeleteCallbackInfo(InExternalImageDeleteCallbackInfo)
	, ImageOwnerType(EImageOwnerType::ExternalOwner)
{
	VULKAN_TRACK_OBJECT_CREATE(FVulkanTexture, this);

	{
		StorageFormat = UEToVkTextureFormat(InCreateDesc.Format, false);

		checkf(InCreateDesc.Format == PF_Unknown || StorageFormat != VK_FORMAT_UNDEFINED, TEXT("PixelFormat %d, is not supported for images"), (int32)InCreateDesc.Format);

		ViewFormat = UEToVkTextureFormat(InCreateDesc.Format, EnumHasAllFlags(InCreateDesc.Flags, TexCreate_SRGB));
		FullAspectMask = VulkanRHI::GetAspectMaskFromUEFormat(InCreateDesc.Format, true, true);
		PartialAspectMask = VulkanRHI::GetAspectMaskFromUEFormat(InCreateDesc.Format, false, true);

		// Purely informative patching, we know that "TexCreate_Presentable" uses optimal tiling
		if (EnumHasAllFlags(InCreateDesc.Flags, TexCreate_Presentable) && GetTiling() == VK_IMAGE_TILING_MAX_ENUM)
		{
			Tiling = VK_IMAGE_TILING_OPTIMAL;
		}

		if (Image != VK_NULL_HANDLE)
		{
			ImageUsageFlags = GetUsageFlagsFromCreateFlags(InDevice, InCreateDesc.Flags);
#if VULKAN_ENABLE_WRAP_LAYER
			FWrapLayer::CreateImage(VK_SUCCESS, InDevice.GetHandle(), nullptr, &Image);
#endif
			VULKAN_SET_DEBUG_NAME(InDevice, VK_OBJECT_TYPE_IMAGE, Image, TEXT("%s:(FVulkanTexture*)0x%p"), InCreateDesc.DebugName ? InCreateDesc.DebugName : TEXT("?"), this);

			const bool bRenderTarget = EnumHasAnyFlags(InCreateDesc.Flags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable);
			const bool bDoInitialClear = bRenderTarget;
			const VkImageLayout InitialLayout = VK_IMAGE_LAYOUT_UNDEFINED;  // use undefinied to avoid transitioning the texture when aliasing

			if (!EnumHasAnyFlags(InCreateDesc.Flags, TexCreate_Presentable))
			{
				FRHICommandList& RHICmdList = FRHICommandListExecutor::GetImmediateCommandList();
				SetInitialImageState(RHICmdList, InitialLayout, bDoInitialClear, InCreateDesc.ClearValue, false);
			}
		}
	}

	InitViews(DefaultImageUsageFlags);
}

#if PLATFORM_ANDROID

struct FVulkanAndroidTextureResources
{
	FVulkanDevice* Device;
	VkImage Image;
	VkDeviceMemory DeviceMemory;
	VkSamplerYcbcrConversion SamplerYcbcrConversion;
	AHardwareBuffer* HardwareBuffer;
};

static void CleanupVulkanAndroidTextureResources(void* UserData)
{
	check(UserData);
	FVulkanAndroidTextureResources* VulkanResources = static_cast<FVulkanAndroidTextureResources*>(UserData);

	if (VulkanResources->Device)
	{
		VulkanResources->Device->GetDeferredDeletionQueue().EnqueueFunc([VulkanResources](FVulkanDevice& Device)
		{
			if (VulkanResources->SamplerYcbcrConversion != VK_NULL_HANDLE)
			{
				VulkanRHI::vkDestroySamplerYcbcrConversion(Device.GetHandle(), VulkanResources->SamplerYcbcrConversion, VULKAN_CPU_ALLOCATOR);
			}

			if (VulkanResources->DeviceMemory != VK_NULL_HANDLE)
			{
				VulkanRHI::vkFreeMemory(Device.GetHandle(), VulkanResources->DeviceMemory, VULKAN_CPU_ALLOCATOR);
			}

			if (VulkanResources->Image != VK_NULL_HANDLE)
			{
				VulkanRHI::vkDestroyImage(Device.GetHandle(), VulkanResources->Image, VULKAN_CPU_ALLOCATOR);
			}

			if (VulkanResources->HardwareBuffer)
			{
				AHardwareBuffer_release(VulkanResources->HardwareBuffer);
			}

			delete VulkanResources;
		});
	}

}

FVulkanTexture::FVulkanTexture(FVulkanDevice& InDevice, const FRHITextureCreateDesc& InCreateDesc, const AHardwareBuffer_Desc& HardwareBufferDesc, AHardwareBuffer* HardwareBuffer)
	: FRHITexture(InCreateDesc)
	, Device(&InDevice)
	, ImageOwnerType(EImageOwnerType::ExternalOwner)
{
	VULKAN_TRACK_OBJECT_CREATE(FVulkanTexture, this);

	check(HardwareBuffer);
	AHardwareBuffer_acquire(HardwareBuffer);

	VkDevice VulkanDevice = InDevice.GetHandle();

	VkAndroidHardwareBufferFormatPropertiesANDROID HardwareBufferFormatProperties;
	ZeroVulkanStruct(HardwareBufferFormatProperties, VK_STRUCTURE_TYPE_ANDROID_HARDWARE_BUFFER_FORMAT_PROPERTIES_ANDROID);

	VkAndroidHardwareBufferPropertiesANDROID HardwareBufferProperties;
	ZeroVulkanStruct(HardwareBufferProperties, VK_STRUCTURE_TYPE_ANDROID_HARDWARE_BUFFER_PROPERTIES_ANDROID);
	HardwareBufferProperties.pNext = &HardwareBufferFormatProperties;

	VERIFYVULKANRESULT(VulkanRHI::vkGetAndroidHardwareBufferPropertiesANDROID(VulkanDevice, HardwareBuffer, &HardwareBufferProperties));

	VkExternalFormatANDROID ExternalFormat;
	ZeroVulkanStruct(ExternalFormat, VK_STRUCTURE_TYPE_EXTERNAL_FORMAT_ANDROID);
	ExternalFormat.externalFormat = HardwareBufferFormatProperties.externalFormat;

	VkExternalMemoryImageCreateInfo ExternalMemoryImageCreateInfo;
	ZeroVulkanStruct(ExternalMemoryImageCreateInfo, VK_STRUCTURE_TYPE_EXTERNAL_MEMORY_IMAGE_CREATE_INFO);
	ExternalMemoryImageCreateInfo.handleTypes = VK_EXTERNAL_MEMORY_HANDLE_TYPE_ANDROID_HARDWARE_BUFFER_BIT_ANDROID;
	ExternalMemoryImageCreateInfo.pNext = &ExternalFormat;

	VkImageCreateInfo ImageCreateInfo;
	ZeroVulkanStruct(ImageCreateInfo, VK_STRUCTURE_TYPE_IMAGE_CREATE_INFO);
	ImageCreateInfo.imageType = VK_IMAGE_TYPE_2D;
	ImageCreateInfo.format = VK_FORMAT_UNDEFINED;
	ImageCreateInfo.extent.width = HardwareBufferDesc.width;
	ImageCreateInfo.extent.height = HardwareBufferDesc.height;
	ImageCreateInfo.extent.depth = 1;
	ImageCreateInfo.mipLevels = 1;
	ImageCreateInfo.arrayLayers = HardwareBufferDesc.layers;

	ImageCreateInfo.flags = 0;
	ImageCreateInfo.samples = VK_SAMPLE_COUNT_1_BIT;

	ImageCreateInfo.tiling = VK_IMAGE_TILING_OPTIMAL;
	ImageCreateInfo.usage = VK_IMAGE_USAGE_SAMPLED_BIT;
	ImageCreateInfo.sharingMode = VK_SHARING_MODE_EXCLUSIVE;
	ImageCreateInfo.queueFamilyIndexCount = 0;
	ImageCreateInfo.pQueueFamilyIndices = nullptr;
	ImageCreateInfo.initialLayout = VK_IMAGE_LAYOUT_UNDEFINED;

	ImageCreateInfo.pNext = &ExternalMemoryImageCreateInfo;

	VkImage VulkanImage;
	VERIFYVULKANRESULT(VulkanRHI::vkCreateImage(VulkanDevice, &ImageCreateInfo, VULKAN_CPU_ALLOCATOR, &VulkanImage));

	VkMemoryDedicatedAllocateInfo MemoryDedicatedAllocateInfo;
	ZeroVulkanStruct(MemoryDedicatedAllocateInfo, VK_STRUCTURE_TYPE_MEMORY_DEDICATED_ALLOCATE_INFO);
	MemoryDedicatedAllocateInfo.image = VulkanImage;
	MemoryDedicatedAllocateInfo.buffer = VK_NULL_HANDLE;

	VkImportAndroidHardwareBufferInfoANDROID ImportAndroidHardwareBufferInfo;
	ZeroVulkanStruct(ImportAndroidHardwareBufferInfo, VK_STRUCTURE_TYPE_IMPORT_ANDROID_HARDWARE_BUFFER_INFO_ANDROID);
	ImportAndroidHardwareBufferInfo.buffer = HardwareBuffer;
	ImportAndroidHardwareBufferInfo.pNext = &MemoryDedicatedAllocateInfo;

	uint32 MemoryTypeBits = HardwareBufferProperties.memoryTypeBits;
	check(MemoryTypeBits > 0); // No index available, this should never happen
	uint32 MemoryTypeIndex = 0;
	for (;(MemoryTypeBits & 1) != 1; ++MemoryTypeIndex)
	{
		MemoryTypeBits >>= 1;
	}

	VkMemoryAllocateInfo MemoryAllocateInfo;
	ZeroVulkanStruct(MemoryAllocateInfo, VK_STRUCTURE_TYPE_MEMORY_ALLOCATE_INFO);
	MemoryAllocateInfo.allocationSize = HardwareBufferProperties.allocationSize;
	MemoryAllocateInfo.memoryTypeIndex = MemoryTypeIndex;
	MemoryAllocateInfo.pNext = &ImportAndroidHardwareBufferInfo;

	VkDeviceMemory VulkanDeviceMemory;
	VERIFYVULKANRESULT(VulkanRHI::vkAllocateMemory(VulkanDevice, &MemoryAllocateInfo, VULKAN_CPU_ALLOCATOR, &VulkanDeviceMemory));
	VERIFYVULKANRESULT(VulkanRHI::vkBindImageMemory(VulkanDevice, VulkanImage, VulkanDeviceMemory, 0));

	VkSamplerYcbcrConversionCreateInfo SamplerYcbcrConversionCreateInfo;
	ZeroVulkanStruct(SamplerYcbcrConversionCreateInfo, VK_STRUCTURE_TYPE_SAMPLER_YCBCR_CONVERSION_CREATE_INFO);
	SamplerYcbcrConversionCreateInfo.format = VK_FORMAT_UNDEFINED;
	SamplerYcbcrConversionCreateInfo.ycbcrModel = HardwareBufferFormatProperties.suggestedYcbcrModel;
	SamplerYcbcrConversionCreateInfo.ycbcrRange = HardwareBufferFormatProperties.suggestedYcbcrRange;
	SamplerYcbcrConversionCreateInfo.components = HardwareBufferFormatProperties.samplerYcbcrConversionComponents;
	SamplerYcbcrConversionCreateInfo.xChromaOffset = HardwareBufferFormatProperties.suggestedXChromaOffset;
	SamplerYcbcrConversionCreateInfo.yChromaOffset = HardwareBufferFormatProperties.suggestedYChromaOffset;
	SamplerYcbcrConversionCreateInfo.chromaFilter = VK_FILTER_LINEAR;
	SamplerYcbcrConversionCreateInfo.forceExplicitReconstruction = VK_FALSE;
	SamplerYcbcrConversionCreateInfo.pNext = &ExternalFormat;

	VkSamplerYcbcrConversion SamplerYcbcrConversion;
	VERIFYVULKANRESULT(VulkanRHI::vkCreateSamplerYcbcrConversion(VulkanDevice, &SamplerYcbcrConversionCreateInfo, VULKAN_CPU_ALLOCATOR, &SamplerYcbcrConversion));

	FVulkanAndroidTextureResources* VulkanAndroidTextureResources = new FVulkanAndroidTextureResources
	{
		&InDevice,
		VulkanImage,
		VulkanDeviceMemory,
		SamplerYcbcrConversion,
		HardwareBuffer
	};

	ExternalImageDeleteCallbackInfo =
	{
		VulkanAndroidTextureResources,
		CleanupVulkanAndroidTextureResources
	};

	Image = VulkanImage;

	// From here this is the same as the ctor that takes an VkImage, excepct for passing the SamplerYcbcrConversion to the view, 
	// possibly some code could be shared.
	{
		StorageFormat = UEToVkTextureFormat(InCreateDesc.Format, false);

		checkf(InCreateDesc.Format == PF_Unknown || StorageFormat != VK_FORMAT_UNDEFINED, TEXT("PixelFormat %d, is not supported for images"), (int32)InCreateDesc.Format);

		ViewFormat = UEToVkTextureFormat(InCreateDesc.Format, EnumHasAllFlags(InCreateDesc.Flags, TexCreate_SRGB));
		FullAspectMask = VulkanRHI::GetAspectMaskFromUEFormat(InCreateDesc.Format, true, true);
		PartialAspectMask = VulkanRHI::GetAspectMaskFromUEFormat(InCreateDesc.Format, false, true);

		// Purely informative patching, we know that "TexCreate_Presentable" uses optimal tiling
		if (EnumHasAllFlags(InCreateDesc.Flags, TexCreate_Presentable) && GetTiling() == VK_IMAGE_TILING_MAX_ENUM)
		{
			Tiling = VK_IMAGE_TILING_OPTIMAL;
		}

		if (Image != VK_NULL_HANDLE)
		{
			ImageUsageFlags = GetUsageFlagsFromCreateFlags(InDevice, InCreateDesc.Flags);
#if VULKAN_ENABLE_WRAP_LAYER
			FWrapLayer::CreateImage(VK_SUCCESS, InDevice.GetHandle(), nullptr, &Image);
#endif
			VULKAN_SET_DEBUG_NAME(InDevice, VK_OBJECT_TYPE_IMAGE, Image, TEXT("%s:(FVulkanTexture*)0x%p"), InCreateDesc.DebugName ? InCreateDesc.DebugName : TEXT("?"), this);

			const bool bRenderTarget = EnumHasAnyFlags(InCreateDesc.Flags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable);
			const bool bDoInitialClear = bRenderTarget;
			const VkImageLayout InitialLayout = VK_IMAGE_LAYOUT_UNDEFINED;  // use undefinied to avoid transitioning the texture when aliasing

			FRHICommandList& RHICmdList = FRHICommandListImmediate::Get();
			SetInitialImageState(RHICmdList, InitialLayout, bDoInitialClear, InCreateDesc.ClearValue, false);
		}
	}

	InitViews(DefaultImageUsageFlags, SamplerYcbcrConversion);
}

#endif // PLATFORM_ANDROID


static FVulkanTextureViewDesc CreateTextureViewDesc(FVulkanTexture* Texture, VkImageUsageFlags ViewUsageFlags, VkSamplerYcbcrConversion SamplerYcbcrConversion)
{
	const FRHITextureDesc& Desc = Texture->GetDesc();
	const bool bHasUAVFormat = (Desc.UAVFormat != PF_Unknown) && (Desc.UAVFormat != Desc.Format);

	if (EnumHasAllFlags(Desc.Flags, ETextureCreateFlags::SRGB) || (bHasUAVFormat && !UE::PixelFormat::HasCapabilities(Desc.Format, EPixelFormatCapabilities::UAV)))
	{
		ViewUsageFlags = (ViewUsageFlags & ~VK_IMAGE_USAGE_STORAGE_BIT);
	}

	const bool bUseIdentitySwizzle =
		!Texture->SupportsSampling() ||
		(Texture->ViewFormat == VK_FORMAT_UNDEFINED); // External buffer textures also require identity swizzle

	FVulkanTextureViewDesc ViewDesc;
	ViewDesc.ViewType = Texture->GetViewType();
	ViewDesc.AspectFlags = Texture->GetFullAspectMask();
	ViewDesc.UEFormat = Desc.Format;
	ViewDesc.Format = Texture->ViewFormat;
	ViewDesc.FirstMip = 0;
	ViewDesc.NumMips = FMath::Max(Desc.NumMips, (uint8)1u);
	ViewDesc.ArraySliceIndex = 0;
	ViewDesc.NumArraySlices = Texture->GetNumberOfArrayLevels();
	ViewDesc.bUseIdentitySwizzle = bUseIdentitySwizzle;
	ViewDesc.ImageUsageFlags = ViewUsageFlags;
	ViewDesc.SamplerYcbcrConversion = SamplerYcbcrConversion;

	return ViewDesc;
}

void FVulkanTexture::InitViews(VkImageUsageFlags ViewUsageFlags, VkSamplerYcbcrConversion SamplerYcbcrConversion)
{
	if (Image != VK_NULL_HANDLE && GetViewType() != VK_IMAGE_VIEW_TYPE_MAX_ENUM)
	{
		const FVulkanTextureViewDesc ViewDesc = CreateTextureViewDesc(this, ViewUsageFlags, SamplerYcbcrConversion);

		if (DefaultView)
		{
			DefaultView->Invalidate();
			DefaultView->CreateTextureView(Image, ViewDesc);
		}
		else
		{
			const ERHIDescriptorType DescriptorType =
				(FullAspectMask != PartialAspectMask) ? ERHIDescriptorType::Invalid :
				SupportsSampling() ? ERHIDescriptorType::TextureSRV : ERHIDescriptorType::TextureUAV;

			DefaultView = new FVulkanView(*Device, DescriptorType);
			DefaultView->CreateTextureView(Image, ViewDesc);
		}

		if (FullAspectMask == PartialAspectMask)
		{
			PartialView = DefaultView;
		}
		else
		{
			FVulkanTextureViewDesc PartialViewDesc = ViewDesc;

			PartialViewDesc.AspectFlags = PartialAspectMask;
			PartialViewDesc.bUseIdentitySwizzle = false;
			PartialViewDesc.SamplerYcbcrConversion = VK_NULL_HANDLE;

			if (PartialView)
			{
				PartialView->Invalidate();
				PartialView->CreateTextureView(Image, PartialViewDesc);
			}
			else
			{
				const ERHIDescriptorType DescriptorType =
					SupportsSampling() ? ERHIDescriptorType::TextureSRV : ERHIDescriptorType::TextureUAV;

				PartialView = new FVulkanView(*Device, DescriptorType);
				PartialView->CreateTextureView(Image, PartialViewDesc);
			}
		}
	}
}

void FVulkanTexture::UpdateViews(const FVulkanContextArray& Contexts, VkImageUsageFlags ViewUsageFlags, VkSamplerYcbcrConversion SamplerYcbcrConversion)
{
	if (Image != VK_NULL_HANDLE && GetViewType() != VK_IMAGE_VIEW_TYPE_MAX_ENUM)
	{
		const FVulkanTextureViewDesc ViewDesc = CreateTextureViewDesc(this, ViewUsageFlags, SamplerYcbcrConversion);
		const ERHIDescriptorType DescriptorType = SupportsSampling() ? ERHIDescriptorType::TextureSRV : ERHIDescriptorType::TextureUAV;

		if (DefaultView)
		{
			DefaultView->Invalidate();
			DefaultView->UpdateTextureView(Contexts, Image, ViewDesc);
		}
		else
		{
			DefaultView = new FVulkanView(*Device, DescriptorType);
			DefaultView->CreateTextureView(Image, ViewDesc);
		}

		if (FullAspectMask == PartialAspectMask)
		{
			PartialView = DefaultView;
		}
		else
		{
			FVulkanTextureViewDesc PartialViewDesc = ViewDesc;

			PartialViewDesc.AspectFlags = PartialAspectMask;
			PartialViewDesc.bUseIdentitySwizzle = false;
			PartialViewDesc.SamplerYcbcrConversion = VK_NULL_HANDLE;

			if (PartialView)
			{
				PartialView->Invalidate();
				PartialView->UpdateTextureView(Contexts, Image, PartialViewDesc);
			}
			else
			{
				PartialView = new FVulkanView(*Device, DescriptorType);
				PartialView->CreateTextureView(Image, PartialViewDesc);
			}
		}
	}
}

static FRWLock GInternalViewRWLock; // Use single global lock for now
FVulkanView* FVulkanTexture::FindOrAddInternalView(const FVulkanTextureViewDesc& ViewDesc)
{
	if ((ImageOwnerType == EImageOwnerType::Aliased) && AliasedTexture.IsValid())
	{
		return ResourceCast(AliasedTexture)->FindOrAddInternalView(ViewDesc);
	}

	{
		FRWScopeLock ScopedLock(GInternalViewRWLock, SLT_ReadOnly);
		if (FVulkanView** VulkanView = InternalViews.Find(ViewDesc.GetHash()))
		{
			return *VulkanView;
		}
	}

	FRWScopeLock ScopedLock(GInternalViewRWLock, SLT_Write);
	FVulkanView*& VulkanView = InternalViews.FindOrAdd(ViewDesc.GetHash(), nullptr);
	if (!VulkanView)
	{
		// When creating a view for a framebuffer, only VK_IMAGE_USAGE_COLOR_ATTACHMENT_BIT will be set
		const ERHIDescriptorType DescriptorType = VKHasAnyFlags(ViewDesc.ImageUsageFlags, VK_IMAGE_USAGE_SAMPLED_BIT) ? ERHIDescriptorType::TextureSRV : ERHIDescriptorType::Invalid;
		VulkanView = new FVulkanView(*Device, DescriptorType);
		VulkanView->CreateTextureView(Image, ViewDesc);
	}
	return VulkanView;
}

FVulkanTexture::FVulkanTexture(FVulkanDevice& InDevice, const FRHITextureCreateDesc& InCreateDesc, FTextureRHIRef& SrcTextureRHI)
	: FRHITexture(InCreateDesc)
	, Device(&InDevice)
	, ImageOwnerType(EImageOwnerType::Aliased)
{
	VULKAN_TRACK_OBJECT_CREATE(FVulkanTexture, this);

	{
		StorageFormat = UEToVkTextureFormat(InCreateDesc.Format, false);

		checkf(InCreateDesc.Format == PF_Unknown || StorageFormat != VK_FORMAT_UNDEFINED, TEXT("PixelFormat %d, is not supported for images"), (int32)InCreateDesc.Format);

		ViewFormat = UEToVkTextureFormat(InCreateDesc.Format, EnumHasAllFlags(InCreateDesc.Flags, TexCreate_SRGB));
		FullAspectMask = VulkanRHI::GetAspectMaskFromUEFormat(InCreateDesc.Format, true, true);
		PartialAspectMask = VulkanRHI::GetAspectMaskFromUEFormat(InCreateDesc.Format, false, true);

		// Purely informative patching, we know that "TexCreate_Presentable" uses optimal tiling
		if (EnumHasAllFlags(InCreateDesc.Flags, TexCreate_Presentable) && GetTiling() == VK_IMAGE_TILING_MAX_ENUM)
		{
			Tiling = VK_IMAGE_TILING_OPTIMAL;
		}

		FImageCreateInfo ImageCreateInfo;
		FVulkanTexture::GenerateImageCreateInfo(ImageCreateInfo, InDevice, InCreateDesc, &StorageFormat, &ViewFormat);

		ImageUsageFlags = ImageCreateInfo.ImageCreateInfo.usage;
	}

	AliasTextureResources(SrcTextureRHI);
}

FVulkanTexture::~FVulkanTexture()
{
	VULKAN_TRACK_OBJECT_DELETE(FVulkanTexture, this);
	if (ImageOwnerType != EImageOwnerType::Aliased)
	{
		if (PartialView != DefaultView)
		{
			delete PartialView;
		}

		delete DefaultView;

		for (auto& Pair : InternalViews)
		{
			delete Pair.Value;
		}
		InternalViews.Reset();
		
		DestroySurface();
	}
}

void FVulkanTexture::AliasTextureResources(FTextureRHIRef& SrcTextureRHI)
{
	FVulkanTexture* SrcTexture = ResourceCast(SrcTextureRHI);

	Image = SrcTexture->Image;
	DefaultView = SrcTexture->DefaultView;
	PartialView = SrcTexture->PartialView;
	AliasedTexture = SrcTexture;
}

void FVulkanTexture::UpdateLinkedViews(const FVulkanContextArray& Contexts)
{
	UpdateViews(Contexts, ImageUsageFlags);

	const bool bRenderTarget = EnumHasAnyFlags(GetDesc().Flags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable | TexCreate_ResolveTargetable);
	if (bRenderTarget)
	{
		Device->NotifyDeletedImage(Image, true);
	}

	for (TPair<uint32, FVulkanView*>& Pair : InternalViews)
	{
		delete Pair.Value;
	}
	InternalViews.Reset();

	FVulkanViewableResource::UpdateLinkedViews(Contexts);
}

void FVulkanTexture::Move(FVulkanDevice& InDevice, const FVulkanContextArray& Contexts, VulkanRHI::FVulkanAllocation& NewAllocation)
{
	const uint64 Size = GetMemorySize();
	static uint64 TotalSize = 0;
	TotalSize += Size;
	if (GVulkanLogDefrag)
	{
		UE_LOGF(LogVulkanRHI, Display, "Moving Surface, %d <<-- %d    :::: %ls\n", NewAllocation.Offset, 42, *GetName().ToString());
		UE_LOGF(LogVulkanRHI, Display, "Moved %8.4fkb %8.4fkb   TB %p  :: IMG %p   %-40ls\n", Size / (1024.f), TotalSize / (1024.f), this, reinterpret_cast<const void*>(Image), *GetName().ToString());
	}

	const ETextureCreateFlags UEFlags = GetDesc().Flags;
	const bool bRenderTarget = EnumHasAnyFlags(UEFlags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable | TexCreate_ResolveTargetable);
	const bool bUAV = EnumHasAnyFlags(UEFlags, TexCreate_UAV);
	checkf(bRenderTarget || bUAV, TEXT("Surface must be a RenderTarget or a UAV in order to be moved.  UEFlags=0x%x"), (int32)UEFlags);
	checkf(Tiling == VK_IMAGE_TILING_OPTIMAL, TEXT("Tiling [%s] is not supported for move, only VK_IMAGE_TILING_OPTIMAL"), VK_TYPE_TO_STRING(VkImageTiling, Tiling));

	InternalMoveSurface(InDevice, *Contexts[ERHIPipeline::Graphics], NewAllocation);
	
	// Swap in the new allocation for this surface
	Allocation.Swap(NewAllocation);

	UpdateLinkedViews(Contexts);
}

void FVulkanTexture::Evict(FVulkanDevice& InDevice, const FVulkanContextArray& Contexts)
{
	check(AliasedTexture == nullptr); //can't evict textures we don't own
	const uint64 Size = GetMemorySize();
	static uint64 TotalSize = 0;
	TotalSize += Size;
	if (GVulkanLogDefrag)
	{
		FGenericPlatformMisc::LowLevelOutputDebugStringf(TEXT("Evicted %8.4fkb %8.4fkb   TB %p  :: IMG %p   %-40s\n"), Size / (1024.f), TotalSize / (1024.f), this, Image, *GetName().ToString());
	}

	{
		check(0 == CpuReadbackBuffer);
		checkf(MemProps == VK_MEMORY_PROPERTY_DEVICE_LOCAL_BIT, TEXT("Can't evict surface that isn't device local.  MemoryProperties=%s"), VK_FLAGS_TO_STRING(VkMemoryPropertyFlags, MemProps));
		checkf(VulkanRHI::GetAspectMaskFromUEFormat(GetDesc().Format, true, true) == FullAspectMask, TEXT("FullAspectMask (%s) does not match with PixelFormat (%d)"), VK_FLAGS_TO_STRING(VkImageAspectFlags, FullAspectMask), (int32)GetDesc().Format);
		checkf(VulkanRHI::GetAspectMaskFromUEFormat(GetDesc().Format, false, true) == PartialAspectMask, TEXT("PartialAspectMask (%s) does not match with PixelFormat (%d)"), VK_FLAGS_TO_STRING(VkImageAspectFlags, PartialAspectMask), (int32)GetDesc().Format);

		const ETextureCreateFlags UEFlags = GetDesc().Flags;
		const bool bRenderTarget = EnumHasAnyFlags(UEFlags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable | TexCreate_ResolveTargetable);
		const bool bUAV = EnumHasAnyFlags(UEFlags, TexCreate_UAV);
		//none of this is supported for eviction
		checkf(!bRenderTarget, TEXT("RenderTargets do not support evict."));
		checkf(!bUAV, TEXT("UAV do not support evict."));

		MemProps = InDevice.GetDeviceMemoryManager().GetEvictedMemoryProperties();

		// Create a new host allocation to move the surface to
		VulkanRHI::FVulkanAllocation HostAllocation;
		const VulkanRHI::EVulkanAllocationMetaType MetaType = VulkanRHI::EVulkanAllocationMetaImageOther;
		if (!InDevice.GetMemoryManager().AllocateImageMemory(HostAllocation, this, MemoryRequirements, MemProps, MetaType, false, __FILE__, __LINE__))
		{
			InDevice.GetMemoryManager().HandleOOM();
			checkNoEntry();
		}

		InternalMoveSurface(InDevice, *Contexts[ERHIPipeline::Graphics], HostAllocation);

		// Delete the original allocation and swap in the new host allocation
		Device->GetMemoryManager().FreeVulkanAllocation(Allocation);
		Allocation.Swap(HostAllocation);

		VULKAN_SET_DEBUG_NAME(InDevice, VK_OBJECT_TYPE_IMAGE, Image, TEXT("(FVulkanTexture*)0x%p [hostimage]"), this);

		UpdateLinkedViews(Contexts);
	}
}

bool FVulkanTexture::GetTextureResourceInfo(FRHIResourceInfo& OutResourceInfo) const
{
	OutResourceInfo = FRHIResourceInfo();
	OutResourceInfo.VRamAllocation.AllocationSize = GetMemorySize();
	return true;
}

void FVulkanDynamicRHI::RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, const TCHAR* Name)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	TextureRHI->SetName(Name);

	SetVulkanResourceName(Device, ResourceCast(TextureRHI), Name);
#endif
}

FDynamicRHI::FRHICalcTextureSizeResult FVulkanDynamicRHI::RHICalcTexturePlatformSize(FRHITextureDesc const& Desc, uint32 FirstMipIndex)
{
	// FIXME: this function ignores FirstMipIndex!

	// Zero out the members which don't affect the size since we'll use this as a key in the map of already computed sizes.
	FRHITextureDesc CleanDesc = Desc;
	CleanDesc.UAVFormat = PF_Unknown;
	CleanDesc.ClearValue = FClearValueBinding::None;
	CleanDesc.ExtData = 0;

	// Adjust number of mips as UTexture can request non-valid # of mips
	CleanDesc.NumMips = (uint8)FMath::Min(FMath::FloorLog2(FMath::Max(CleanDesc.Extent.X, FMath::Max(CleanDesc.Extent.Y, (int32)CleanDesc.Depth))) + 1, (uint32)CleanDesc.NumMips);

	static TMap<FRHITextureDesc, VkMemoryRequirements> TextureSizes;
	static FCriticalSection TextureSizesLock;

	VkMemoryRequirements* Found = nullptr;
	{
		FScopeLock Lock(&TextureSizesLock);
		Found = TextureSizes.Find(CleanDesc);
		if (Found)
		{
			return { (uint64)Found->size, (uint32)Found->alignment };
		}
	}

	// Create temporary image to measure the memory requirements.
	FVulkanTexture::FImageCreateInfo TmpCreateInfo;
	FVulkanTexture::GenerateImageCreateInfo(TmpCreateInfo, *Device, CleanDesc, nullptr, nullptr, false);

	VkMemoryRequirements OutMemReq;

	if (Device->GetOptionalExtensions().HasKHRMaintenance4)
	{
		VkDeviceImageMemoryRequirements ImageMemReq;
		ZeroVulkanStruct(ImageMemReq, VK_STRUCTURE_TYPE_DEVICE_IMAGE_MEMORY_REQUIREMENTS);
		ImageMemReq.pCreateInfo = &TmpCreateInfo.ImageCreateInfo;
		ImageMemReq.planeAspect = (VulkanRHI::GetAspectMaskFromUEFormat(CleanDesc.Format, true, true) == VK_IMAGE_ASPECT_COLOR_BIT) ? VK_IMAGE_ASPECT_COLOR_BIT : VK_IMAGE_ASPECT_DEPTH_BIT;  // should be ignored

		VkMemoryRequirements2 MemReq2;
		ZeroVulkanStruct(MemReq2, VK_STRUCTURE_TYPE_MEMORY_REQUIREMENTS_2);

		VulkanRHI::vkGetDeviceImageMemoryRequirementsKHR(Device->GetHandle(), &ImageMemReq, &MemReq2);
		OutMemReq = MemReq2.memoryRequirements;
	}
	else
	{
		VkImage TmpImage;
		VERIFYVULKANRESULT(VulkanRHI::vkCreateImage(Device->GetHandle(), &TmpCreateInfo.ImageCreateInfo, VULKAN_CPU_ALLOCATOR, &TmpImage));
		VulkanRHI::vkGetImageMemoryRequirements(Device->GetHandle(), TmpImage, &OutMemReq);
		VulkanRHI::vkDestroyImage(Device->GetHandle(), TmpImage, VULKAN_CPU_ALLOCATOR);
	}

	{
		FScopeLock Lock(&TextureSizesLock);
		TextureSizes.Add(CleanDesc, OutMemReq);
	}

	return { (uint64)OutMemReq.size, (uint32)OutMemReq.alignment };
}

void FVulkanCommandListContext::RHICopyTexture(FRHITexture* SourceTexture, FRHITexture* DestTexture, const FRHICopyTextureInfo& CopyInfo)
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanTextures);
	check(SourceTexture && DestTexture);

	FVulkanTexture* Source = ResourceCast(SourceTexture);
	FVulkanTexture* Dest = ResourceCast(DestTexture);

	FVulkanCommandBuffer& CommandBuffer = GetCommandBuffer();
	check(CommandBuffer.IsOutsideRenderPass());

	const FPixelFormatInfo& PixelFormatInfo = GPixelFormats[DestTexture->GetDesc().Format];
	const FRHITextureDesc& SourceDesc = SourceTexture->GetDesc();
	const FRHITextureDesc& DestDesc = DestTexture->GetDesc();
	const FIntVector SourceXYZ = SourceDesc.GetSize();
	const FIntVector DestXYZ = DestDesc.GetSize();

	check(!EnumHasAnyFlags(Source->GetDesc().Flags, TexCreate_CPUReadback));
	if (EnumHasAllFlags(Dest->GetDesc().Flags, TexCreate_CPUReadback))
	{
		checkf(CopyInfo.DestSliceIndex == 0, TEXT("Slices not supported in TexCreate_CPUReadback textures"));
		checkf(CopyInfo.DestPosition.IsZero(), TEXT("Destination position not supported in TexCreate_CPUReadback textures"));
		FIntVector Size = CopyInfo.Size;
		if (Size == FIntVector::ZeroValue)
		{
			ensure(SourceXYZ.X <= DestXYZ.X && SourceXYZ.Y <= DestXYZ.Y);
			Size.X = FMath::Max<uint32>(1u, SourceXYZ.X >> CopyInfo.SourceMipIndex);
			Size.Y = FMath::Max<uint32>(1u, SourceXYZ.Y >> CopyInfo.SourceMipIndex);
			Size.Z = FMath::Max<uint32>(1u, SourceXYZ.Z >> CopyInfo.SourceMipIndex);
		}
		VkBufferImageCopy CopyRegion[MAX_TEXTURE_MIP_COUNT];
		FMemory::Memzero(CopyRegion);

		const FVulkanCpuReadbackBuffer* CpuReadbackBuffer = Dest->GetCpuReadbackBuffer();
		const uint32 SourceSliceIndex = CopyInfo.SourceSliceIndex;
		const uint32 SourceMipIndex = CopyInfo.SourceMipIndex;
		const uint32 DestMipIndex = CopyInfo.DestMipIndex;
		for (uint32 Index = 0; Index < CopyInfo.NumMips; ++Index)
		{
			CopyRegion[Index].bufferOffset = CpuReadbackBuffer->MipOffsets[DestMipIndex + Index];
			CopyRegion[Index].bufferRowLength = Size.X;
			CopyRegion[Index].bufferImageHeight = Size.Y;
			CopyRegion[Index].imageSubresource.aspectMask = Source->GetFullAspectMask();
			CopyRegion[Index].imageSubresource.mipLevel = SourceMipIndex + Index;
			CopyRegion[Index].imageSubresource.baseArrayLayer = SourceSliceIndex;
			CopyRegion[Index].imageSubresource.layerCount = 1;
			CopyRegion[Index].imageOffset.x = CopyInfo.SourcePosition.X;
			CopyRegion[Index].imageOffset.y = CopyInfo.SourcePosition.Y;
			CopyRegion[Index].imageOffset.z = CopyInfo.SourcePosition.Z;
			CopyRegion[Index].imageExtent.width = Size.X;
			CopyRegion[Index].imageExtent.height = Size.Y;
			CopyRegion[Index].imageExtent.depth = Size.Z;

			Size.X = FMath::Max(1, Size.X / 2);
			Size.Y = FMath::Max(1, Size.Y / 2);
			Size.Z = FMath::Max(1, Size.Z / 2);
		}

		VulkanRHI::vkCmdCopyImageToBuffer(CommandBuffer.GetHandle(), Source->Image, VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL, CpuReadbackBuffer->Buffer, CopyInfo.NumMips, &CopyRegion[0]);

		FVulkanPipelineBarrier BarrierMemory;
		BarrierMemory.AddMemoryBarrier(VK_ACCESS_TRANSFER_WRITE_BIT, VK_ACCESS_HOST_READ_BIT, VK_PIPELINE_STAGE_ALL_COMMANDS_BIT, VK_PIPELINE_STAGE_HOST_BIT);
		BarrierMemory.Execute(&CommandBuffer);
	}
	else
	{
		VkImageCopy Region;
		FMemory::Memzero(Region);
		if (CopyInfo.Size == FIntVector::ZeroValue)
		{
			// Copy whole texture when zero vector is specified for region size
			Region.extent.width  = FMath::Max<uint32>(1u, SourceXYZ.X >> CopyInfo.SourceMipIndex);
			Region.extent.height = FMath::Max<uint32>(1u, SourceXYZ.Y >> CopyInfo.SourceMipIndex);
			Region.extent.depth  = FMath::Max<uint32>(1u, SourceXYZ.Z >> CopyInfo.SourceMipIndex);
			ensure(Region.extent.width <= (uint32)DestXYZ.X && Region.extent.height <= (uint32)DestXYZ.Y);
		}
		else
		{
			ensure(CopyInfo.Size.X > 0 && CopyInfo.Size.X <= DestXYZ.X && CopyInfo.Size.Y > 0 && CopyInfo.Size.Y <= DestXYZ.Y);
			Region.extent.width  = FMath::Max(1, CopyInfo.Size.X);
			Region.extent.height = FMath::Max(1, CopyInfo.Size.Y);
			Region.extent.depth  = FMath::Max(1, CopyInfo.Size.Z);
		}
		Region.srcSubresource.aspectMask = Source->GetFullAspectMask();
		Region.srcSubresource.baseArrayLayer = CopyInfo.SourceSliceIndex;
		Region.srcSubresource.layerCount = CopyInfo.NumSlices;
		Region.srcSubresource.mipLevel = CopyInfo.SourceMipIndex;
		Region.srcOffset.x = CopyInfo.SourcePosition.X;
		Region.srcOffset.y = CopyInfo.SourcePosition.Y;
		Region.srcOffset.z = CopyInfo.SourcePosition.Z;
		Region.dstSubresource.aspectMask = Dest->GetFullAspectMask();
		Region.dstSubresource.baseArrayLayer = CopyInfo.DestSliceIndex;
		Region.dstSubresource.layerCount = CopyInfo.NumSlices;
		Region.dstSubresource.mipLevel = CopyInfo.DestMipIndex;
		Region.dstOffset.x = CopyInfo.DestPosition.X;
		Region.dstOffset.y = CopyInfo.DestPosition.Y;
		Region.dstOffset.z = CopyInfo.DestPosition.Z;

		for (uint32 Index = 0; Index < CopyInfo.NumMips; ++Index)
		{
			VulkanRHI::vkCmdCopyImage(CommandBuffer.GetHandle(),
				Source->Image, VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL,
				Dest->Image, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL,
				1, &Region);

			++Region.srcSubresource.mipLevel;
			++Region.dstSubresource.mipLevel;

			// Scale down the copy region if there is another mip to proceed.
			if (Index != CopyInfo.NumMips - 1)
			{
				Region.srcOffset.x /= 2;
				Region.srcOffset.y /= 2;
				Region.srcOffset.z /= 2;

				Region.dstOffset.x /= 2;
				Region.dstOffset.y /= 2;
				Region.dstOffset.z /= 2;

				Region.extent.width  = FMath::Max<uint32>(Region.extent.width  / 2, 1u);
				Region.extent.height = FMath::Max<uint32>(Region.extent.height / 2, 1u);
				Region.extent.depth  = FMath::Max<uint32>(Region.extent.depth  / 2, 1u);

				// RHICopyTexture is allowed to copy mip regions only if are aligned on the block size to prevent unexpected / inconsistent results.
				ensure(Region.srcOffset.x % PixelFormatInfo.BlockSizeX == 0 && Region.srcOffset.y % PixelFormatInfo.BlockSizeY == 0 && Region.srcOffset.z % PixelFormatInfo.BlockSizeZ == 0);
				ensure(Region.dstOffset.x % PixelFormatInfo.BlockSizeX == 0 && Region.dstOffset.y % PixelFormatInfo.BlockSizeY == 0 && Region.dstOffset.z % PixelFormatInfo.BlockSizeZ == 0);
				// For extent, the condition is harder to verify since on Vulkan, the extent must not be aligned on block size if it would exceed the surface limit.
			}
		}
	}
}

void FVulkanCommandListContext::RHICopyBufferRegion(FRHIBuffer* DstBuffer, uint64 DstOffset, FRHIBuffer* SrcBuffer, uint64 SrcOffset, uint64 NumBytes)
{
	if (!DstBuffer || !SrcBuffer || DstBuffer == SrcBuffer || !NumBytes)
	{
		return;
	}

	FVulkanBuffer* DstBufferVk = ResourceCast(DstBuffer);
	FVulkanBuffer* SrcBufferVk = ResourceCast(SrcBuffer);

	check(DstBufferVk && SrcBufferVk);
	check(DstOffset + NumBytes <= DstBuffer->GetSize() && SrcOffset + NumBytes <= SrcBuffer->GetSize());

	uint64 DstOffsetVk = DstBufferVk->GetOffset() + DstOffset;
	uint64 SrcOffsetVk = SrcBufferVk->GetOffset() + SrcOffset;

	FVulkanCommandBuffer& CommandBuffer = GetCommandBuffer();
	check(CommandBuffer.IsOutsideRenderPass());
	VkCommandBuffer CommandBufferHandle = CommandBuffer.GetHandle();

	VkMemoryBarrier BarrierBefore = { VK_STRUCTURE_TYPE_MEMORY_BARRIER, nullptr, VK_ACCESS_MEMORY_READ_BIT|VK_ACCESS_MEMORY_WRITE_BIT, VK_ACCESS_TRANSFER_READ_BIT| VK_ACCESS_TRANSFER_WRITE_BIT };
	VulkanRHI::vkCmdPipelineBarrier(CommandBufferHandle, VK_PIPELINE_STAGE_ALL_COMMANDS_BIT, VK_PIPELINE_STAGE_TRANSFER_BIT, 0, 1, &BarrierBefore, 0, nullptr, 0, nullptr);

	VkBufferCopy Region = {};
	Region.srcOffset = SrcOffsetVk;
	Region.dstOffset = DstOffsetVk;
	Region.size = NumBytes;
	VulkanRHI::vkCmdCopyBuffer(CommandBufferHandle, SrcBufferVk->GetHandle(), DstBufferVk->GetHandle(), 1, &Region);

	VkMemoryBarrier BarrierAfter = { VK_STRUCTURE_TYPE_MEMORY_BARRIER, nullptr, VK_ACCESS_TRANSFER_READ_BIT | VK_ACCESS_TRANSFER_WRITE_BIT, VK_ACCESS_MEMORY_READ_BIT | VK_ACCESS_MEMORY_WRITE_BIT };
	VulkanRHI::vkCmdPipelineBarrier(CommandBufferHandle, VK_PIPELINE_STAGE_TRANSFER_BIT, VK_PIPELINE_STAGE_ALL_COMMANDS_BIT, 0, 1, &BarrierAfter, 0, nullptr, 0, nullptr);
}
