// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanTransientResourceAllocator.h"

#include "VulkanRHIPrivate.h"
#include "VulkanCommandWrappers.h"
#include "VulkanDevice.h"
#include "VulkanMemory.h"

FVulkanTransientHeap::FVulkanTransientHeap(const FInitializer& Initializer, FVulkanDevice& InDevice)
	: FRHITransientHeap(Initializer)
	, Device(InDevice)
{
	EBufferUsageFlags UEBufferUsageFlags = BUF_UniformBuffer | BUF_VertexBuffer | BUF_IndexBuffer | BUF_DrawIndirect 
		| BUF_UnorderedAccess | BUF_StructuredBuffer | BUF_ShaderResource | BUF_KeepCPUAccessible;

	if (InDevice.GetOptionalExtensions().HasRaytracingExtensions())
	{
		UEBufferUsageFlags |= BUF_RayTracingScratch;
		// AccelerationStructure not yet supported as TransientResource see FVulkanTransientResourceAllocator::CreateBuffer
		//UEBufferUsageFlags |= BUF_AccelerationStructure;
	}

	const bool bZeroSize = false;
	const VkBufferUsageFlags BufferUsageFlags = FVulkanBuffer::UEToVKBufferUsageFlags(InDevice, UEBufferUsageFlags, bZeroSize);
	VulkanBuffer = InDevice.CreateBuffer(Initializer.Size, BufferUsageFlags);
	if (UseVulkanDescriptorCache())
	{
		HandleID = ++GVulkanBufferHandleIdCounter;
	}

	// Find the alignment that works for everyone
	const uint32 MinBufferAlignment = FMath::Max<uint32>(Initializer.Alignment, VulkanRHI::FMemoryManager::CalculateBufferAlignment(InDevice, UEBufferUsageFlags, bZeroSize));

	const VulkanRHI::EVulkanAllocationFlags AllocFlags = VulkanRHI::EVulkanAllocationFlags::Dedicated | VulkanRHI::EVulkanAllocationFlags::AutoBind;
	InDevice.GetMemoryManager().AllocateBufferMemory(InternalAllocation, VulkanBuffer, AllocFlags, TEXT("VulkanTransientHeap"), MinBufferAlignment);
}

FVulkanTransientHeap::~FVulkanTransientHeap()
{
	Device.GetMemoryManager().FreeVulkanAllocation(InternalAllocation);
	Device.GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::Buffer, VulkanBuffer);
	VulkanBuffer = VK_NULL_HANDLE;
	HandleID = 0;
}

VkDeviceMemory FVulkanTransientHeap::GetMemoryHandle()
{
	return InternalAllocation.GetDeviceMemoryHandle(&Device);
}

VulkanRHI::FVulkanAllocation FVulkanTransientHeap::GetVulkanAllocation(const FRHITransientHeapAllocation& HeapAllocation)
{
	FVulkanTransientHeap* Heap = static_cast<FVulkanTransientHeap*>(HeapAllocation.Heap);
	check(Heap);

	VulkanRHI::FVulkanAllocation TransientAlloc;
	TransientAlloc.Reference(Heap->InternalAllocation);
	TransientAlloc.VulkanHandle = (uint64)Heap->VulkanBuffer;
	TransientAlloc.HandleId = Heap->HandleID;
	TransientAlloc.Offset += HeapAllocation.Offset;
	TransientAlloc.Size = HeapAllocation.Size;
	TransientAlloc.bTransient = true;
	check((TransientAlloc.Offset + TransientAlloc.Size) <= Heap->InternalAllocation.Size);
	return TransientAlloc;
}

FVulkanTransientHeapCache* FVulkanTransientHeapCache::Create(FVulkanDevice& InDevice)
{
	FRHITransientHeapCache::FInitializer Initializer = FRHITransientHeapCache::FInitializer::CreateDefault();

	// Respect a minimum alignment
	Initializer.HeapAlignment = FMath::Max((uint32)InDevice.GetLimits().bufferImageGranularity, 256u);

	// Mix resource types onto the same heap.
	Initializer.bSupportsAllHeapFlags = true;

	return new FVulkanTransientHeapCache(Initializer, InDevice);
}

FVulkanTransientHeapCache::FVulkanTransientHeapCache(const FRHITransientHeapCache::FInitializer& Initializer, FVulkanDevice& InDevice)
	: FRHITransientHeapCache(Initializer)
	, Device(InDevice)
{
}

FRHITransientHeap* FVulkanTransientHeapCache::CreateHeap(const FRHITransientHeap::FInitializer& HeapInitializer)
{
	return new FVulkanTransientHeap(HeapInitializer, Device);
}


FVulkanTransientResourceAllocator::FVulkanTransientResourceAllocator(FVulkanTransientHeapCache& InHeapCache)
	: FRHITransientResourceHeapAllocator(InHeapCache)
	, Device(InHeapCache.Device)
{
}

FRHITransientTexture* FVulkanTransientResourceAllocator::CreateTexture(const FRHITextureCreateInfo& InCreateInfo, const TCHAR* InDebugName, const FRHITransientAllocationFences& Fences)
{
	FDynamicRHI::FRHICalcTextureSizeResult MemReq = GDynamicRHI->RHICalcTexturePlatformSize(InCreateInfo, 0);

	return CreateTextureInternal(InCreateInfo, InDebugName, Fences, MemReq.Size, MemReq.Align,
		[InCreateInfo, InDebugName, MemReq](const FRHITransientHeap::FResourceInitializer& Initializer)
	{
		FRHITextureCreateDesc CreateDesc(InCreateInfo, ERHIAccess::Discard, InDebugName);

		FRHITexture* Texture = FVulkanDynamicRHI::Get().CreateTextureInternal(CreateDesc, Initializer.Allocation);
		return new FRHITransientTexture(Texture, 0/*GpuVirtualAddress*/, Initializer.Hash, MemReq.Size, ERHITransientAllocationType::Heap, InCreateInfo);
	});
}

FRHITransientBuffer* FVulkanTransientResourceAllocator::CreateBuffer(const FRHIBufferCreateInfo& InCreateInfo, const TCHAR* InDebugName, const FRHITransientAllocationFences& Fences)
{
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::Create(InDebugName, InCreateInfo)
		.SetInitialState(ERHIAccess::Discard);

	checkf(!EnumHasAnyFlags(CreateDesc.Usage, BUF_AccelerationStructure), TEXT("AccelerationStructure not yet supported as TransientResource."));
	checkf(!EnumHasAnyFlags(CreateDesc.Usage, BUF_Volatile), TEXT("The volatile flag is not supported for transient resources."));

	const bool bZeroSize = (CreateDesc.Size == 0);
	const uint32 Alignment = VulkanRHI::FMemoryManager::CalculateBufferAlignment(Device, CreateDesc.Usage, bZeroSize);
	uint64 Size = Align(CreateDesc.Size, Alignment);

	return CreateBufferInternal(CreateDesc, InDebugName, Fences, Size, Alignment,
	[&Device=Device, CreateDesc, InDebugName, Size](const FRHITransientHeap::FResourceInitializer& Initializer)
	{
		FRHIBuffer* Buffer = new FVulkanBuffer(Device, CreateDesc, &Initializer.Allocation);
		return new FRHITransientBuffer(Buffer, 0/*GpuVirtualAddress*/, Initializer.Hash, Size, ERHITransientAllocationType::Heap, CreateDesc);
	});
}
