// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VulkanMemory.h"
#include "VulkanResources.h"
#include "RHICoreTransientResourceAllocator.h"

class FVulkanTransientHeap final
	: public FRHITransientHeap
	, public FRefCountedObject
{
public:
	FVulkanTransientHeap(const FInitializer& Initializer, FVulkanDevice& InDevice);
	~FVulkanTransientHeap();

	VkDeviceMemory GetMemoryHandle();
	static VulkanRHI::FVulkanAllocation GetVulkanAllocation(const FRHITransientHeapAllocation& TransientInitializer);

private:
	FVulkanDevice& Device;
	VkBuffer VulkanBuffer = VK_NULL_HANDLE;
	uint32 HandleID = 0;
	VkMemoryRequirements MemoryRequirements;
	VulkanRHI::FVulkanAllocation InternalAllocation;
};


class FVulkanTransientHeapCache final
	: public FRHITransientHeapCache
{
public:
	static FVulkanTransientHeapCache* Create(FVulkanDevice& InDevice);

	FRHITransientHeap* CreateHeap(const FRHITransientHeap::FInitializer& Initializer) override;

	FVulkanDevice& Device;

private:
	FVulkanTransientHeapCache(const FInitializer& Initializer, FVulkanDevice& InDevice);
};


class FVulkanTransientResourceAllocator final
	: public FRHITransientResourceHeapAllocator
{
public:
	FVulkanTransientResourceAllocator(FVulkanTransientHeapCache& InHeapCache);

	//! IRHITransientResourceAllocator Overrides
	bool SupportsResourceType(ERHITransientResourceType InType) const override
	{
		switch (InType)
		{
		case ERHITransientResourceType::Buffer: return true;
		case ERHITransientResourceType::Texture: return true;
		default: checkNoEntry(); return false;
		}
	}
	FRHITransientTexture* CreateTexture(const FRHITextureCreateInfo& InCreateInfo, const TCHAR* InDebugName, const FRHITransientAllocationFences& Fences) override;
	FRHITransientBuffer* CreateBuffer(const FRHIBufferCreateInfo& InCreateInfo, const TCHAR* InDebugName, const FRHITransientAllocationFences& Fences) override;

	FVulkanDevice& Device;
};