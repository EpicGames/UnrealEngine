// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanRHIPrivate.h"
#include "VulkanContext.h"
#include "VulkanDescriptorSets.h"
#include "VulkanLLM.h"
#include "ClearReplacementShaders.h"
#include "VulkanRayTracing.h"

static VkImageViewType GetVkImageViewTypeForDimensionUAV(FRHIViewDesc::EDimension DescDimension, VkImageViewType TextureViewType)
{
	switch (DescDimension)
	{
	case FRHIViewDesc::EDimension::Texture2D:        return VK_IMAGE_VIEW_TYPE_2D;
	case FRHIViewDesc::EDimension::Texture2DArray:   return VK_IMAGE_VIEW_TYPE_2D_ARRAY;
	case FRHIViewDesc::EDimension::Texture3D:        return VK_IMAGE_VIEW_TYPE_3D;
	case FRHIViewDesc::EDimension::TextureCube:      return VK_IMAGE_VIEW_TYPE_2D_ARRAY;
	case FRHIViewDesc::EDimension::TextureCubeArray: return VK_IMAGE_VIEW_TYPE_2D_ARRAY;
	case FRHIViewDesc::EDimension::Unknown:          return TextureViewType;
	default: break;
	}

	checkf(false, TEXT("Unknown texture dimension value!"));
	return VK_IMAGE_VIEW_TYPE_MAX_ENUM;

}

inline ERHIDescriptorType GetUAVDescriptorType(const FRHIViewDesc& ViewDesc)
{
	if (ViewDesc.IsTexture())
	{
		return ERHIDescriptorType::TextureUAV;
	}

	switch (ViewDesc.Buffer.UAV.BufferType)
	{
	case FRHIViewDesc::EBufferType::Raw:
	case FRHIViewDesc::EBufferType::Structured:
		return ERHIDescriptorType::BufferUAV;

	case FRHIViewDesc::EBufferType::Typed:
		return ERHIDescriptorType::TypedBufferUAV;

	case FRHIViewDesc::EBufferType::AccelerationStructure:
		return ERHIDescriptorType::AccelerationStructure;

	default:
		checkNoEntry();
		return ERHIDescriptorType::Invalid;
	}
}

FVulkanUnorderedAccessView::FVulkanUnorderedAccessView(FRHICommandListBase& RHICmdList, FVulkanDevice& InDevice, FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc)
	: FRHIUnorderedAccessView(InResource, InViewDesc)
	, FVulkanLinkedView(InDevice, GetUAVDescriptorType(InViewDesc))
{
	RHICmdList.EnqueueLambda([this](FRHICommandListBase&)
	{
		LinkHead(GetBaseResource()->LinkedViews);
		UpdateView({});
	});

	RHICmdList.RHIThreadFence(true);
}

FVulkanViewableResource* FVulkanUnorderedAccessView::GetBaseResource() const
{
	if (IsBuffer())
	{
		return ResourceCast(GetBuffer());
	}

	return ResourceCast(GetTexture());
}

void FVulkanUnorderedAccessView::UpdateView(const FVulkanContextArray& Contexts)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanUAVUpdateTime);
#endif

	Invalidate();

	if (IsBuffer())
	{
		FVulkanBuffer* Buffer = ResourceCast(GetBuffer());
		auto const Info = ViewDesc.Buffer.UAV.GetViewInfo(Buffer);

		checkf(!Info.bAppendBuffer && !Info.bAtomicCounter, TEXT("UAV counters not implemented in Vulkan RHI."));

		if (!Info.bNullView)
		{
			switch (Info.BufferType)
			{
			case FRHIViewDesc::EBufferType::Raw:
			case FRHIViewDesc::EBufferType::Structured:
				InitAsStructuredBufferView(Contexts, Buffer, Info.OffsetInBytes, Info.SizeInBytes);
				break;

			case FRHIViewDesc::EBufferType::Typed:
				check(VKHasAllFlags(Buffer->GetBufferUsageFlags(), VK_BUFFER_USAGE_STORAGE_TEXEL_BUFFER_BIT));
				InitAsTypedBufferView(Contexts, Buffer, Info.Format, Info.OffsetInBytes, Info.SizeInBytes);
				break;

			case FRHIViewDesc::EBufferType::AccelerationStructure:
				checkNoEntry(); // @todo implement
				break;

			default:
				checkNoEntry();
				break;
			}
		}
	}
	else
	{
		FVulkanTexture* Texture = ResourceCast(GetTexture());
		auto const Info = ViewDesc.Texture.UAV.GetViewInfo(Texture);

		uint32 ArrayFirst = Info.ArrayRange.First;
		uint32 ArrayNum = Info.ArrayRange.Num;
		if (Info.Dimension == FRHIViewDesc::EDimension::TextureCube || Info.Dimension == FRHIViewDesc::EDimension::TextureCubeArray)
		{
			ArrayFirst *= 6;
			ArrayNum *= 6;
			checkf((ArrayFirst + ArrayNum) <= Texture->GetNumberOfArrayLevels(), TEXT("View extends beyond original cube texture level count!"));
		}

		FVulkanTextureViewDesc VulkanViewDesc;
		VulkanViewDesc.ViewType = GetVkImageViewTypeForDimensionUAV(Info.Dimension, Texture->GetViewType());
		VulkanViewDesc.AspectFlags = Texture->GetPartialAspectMask();
		VulkanViewDesc.UEFormat = Info.Format;
		VulkanViewDesc.Format = UEToVkTextureFormat(Info.Format, false);
		VulkanViewDesc.FirstMip = Info.MipLevel;
		VulkanViewDesc.NumMips = 1;
		VulkanViewDesc.ArraySliceIndex = ArrayFirst;
		VulkanViewDesc.NumArraySlices = ArrayNum;
		VulkanViewDesc.bUseIdentitySwizzle = true;
		InitAsTextureView(Contexts, Texture->Image, VulkanViewDesc);
	}
}

void FVulkanUnorderedAccessView::Clear(TRHICommandList_RecursiveHazardous<FVulkanCommandListContext>& RHICmdList, const void* ClearValue, bool bFloat)
{
	auto GetValueType = [bFloat](EPixelFormat Format)
	{
		if (bFloat)
		{
			return EClearReplacementValueType::Float;
		}

		switch (Format)
		{
		case PF_R32_SINT:
		case PF_R16_SINT:
		case PF_R16G16B16A16_SINT:
			return EClearReplacementValueType::Int32;
		}

		return EClearReplacementValueType::Uint32;
	};

	if (IsBuffer())
	{
		FVulkanBuffer* Buffer = ResourceCast(GetBuffer());
		auto const Info = ViewDesc.Buffer.UAV.GetViewInfo(Buffer);

		switch (Info.BufferType)
		{
		case FRHIViewDesc::EBufferType::Raw:
		case FRHIViewDesc::EBufferType::Structured:
			RHICmdList.RunOnContext([this, Buffer, Info, ClearValue = *static_cast<const uint32*>(ClearValue)](FVulkanCommandListContext& Context)
			{
				FVulkanCommandBuffer& CmdBuffer = Context.GetCommandBuffer();

				// vkCmdFillBuffer is treated as a transfer operation for the purposes of synchronization barriers.
				{
					FVulkanPipelineBarrier BeforeBarrier;
					BeforeBarrier.AddMemoryBarrier(VK_ACCESS_MEMORY_READ_BIT | VK_ACCESS_MEMORY_WRITE_BIT, VK_ACCESS_TRANSFER_WRITE_BIT, VK_PIPELINE_STAGE_ALL_COMMANDS_BIT, VK_PIPELINE_STAGE_TRANSFER_BIT);
					BeforeBarrier.Execute(&CmdBuffer);
				}

				VulkanRHI::vkCmdFillBuffer(
					  CmdBuffer.GetHandle()
					, Buffer->GetHandle()
					, Buffer->GetOffset() + Info.OffsetInBytes
					, Info.SizeInBytes
					, ClearValue
				);

				{
					FVulkanPipelineBarrier AfterBarrier;
					AfterBarrier.AddMemoryBarrier(VK_ACCESS_TRANSFER_WRITE_BIT, VK_ACCESS_MEMORY_READ_BIT | VK_ACCESS_MEMORY_WRITE_BIT, VK_PIPELINE_STAGE_TRANSFER_BIT, VK_PIPELINE_STAGE_ALL_COMMANDS_BIT);
					AfterBarrier.Execute(&CmdBuffer);
				}
			});
			break;

		case FRHIViewDesc::EBufferType::Typed:
			{
				const uint32 ComputeWorkGroupCount = FMath::DivideAndRoundUp(Info.NumElements, (uint32)ClearReplacementCS::TThreadGroupSize<EClearReplacementResourceType::Buffer>::X);

				FVulkanDevice& TargetDevice = FVulkanCommandListContext::Get(RHICmdList).Device;
				const bool bOversizedBuffer = (ComputeWorkGroupCount > TargetDevice.GetLimits().maxComputeWorkGroupCount[0]);

				if (bOversizedBuffer)
				{
					ClearUAVShader_T<EClearReplacementResourceType::LargeBuffer, 4, false>(RHICmdList, this, Info.NumElements, 1, 1, ClearValue, GetValueType(Info.Format));
				}
				else
				{
					ClearUAVShader_T<EClearReplacementResourceType::Buffer, 4, false>(RHICmdList, this, Info.NumElements, 1, 1, ClearValue, GetValueType(Info.Format));
				}
			}
			break;

		default:
			checkNoEntry();
			break;
		}
	}
	else
	{
		FVulkanTexture* Texture = ResourceCast(GetTexture());
		auto const Info = ViewDesc.Texture.UAV.GetViewInfo(Texture);

		FIntVector SizeXYZ = Texture->GetMipDimensions(Info.MipLevel);

		switch (Texture->GetDesc().Dimension)
		{
		case ETextureDimension::Texture2D:
			ClearUAVShader_T<EClearReplacementResourceType::Texture2D, 4, false>(RHICmdList, this, SizeXYZ.X, SizeXYZ.Y, SizeXYZ.Z, ClearValue, GetValueType(Info.Format));
			break;

		case ETextureDimension::Texture2DArray:
			ClearUAVShader_T<EClearReplacementResourceType::Texture2DArray, 4, false>(RHICmdList, this, SizeXYZ.X, SizeXYZ.Y, Info.ArrayRange.Num, ClearValue, GetValueType(Info.Format));
			break;

		case ETextureDimension::TextureCube:
		case ETextureDimension::TextureCubeArray:
			ClearUAVShader_T<EClearReplacementResourceType::Texture2DArray, 4, false>(RHICmdList, this, SizeXYZ.X, SizeXYZ.Y, Info.ArrayRange.Num * 6, ClearValue, GetValueType(Info.Format));
			break;

		case ETextureDimension::Texture3D:
			ClearUAVShader_T<EClearReplacementResourceType::Texture3D, 4, false>(RHICmdList, this, SizeXYZ.X, SizeXYZ.Y, SizeXYZ.Z, ClearValue, GetValueType(Info.Format));
			break;

		default:
			checkNoEntry();
			break;
		}
	}
}

void FVulkanCommandListContext::RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values)
{
	TRHICommandList_RecursiveHazardous<FVulkanCommandListContext> RHICmdList(this);
	ResourceCast(UnorderedAccessViewRHI)->Clear(RHICmdList, &Values, true);
}

void FVulkanCommandListContext::RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values)
{
	TRHICommandList_RecursiveHazardous<FVulkanCommandListContext> RHICmdList(this);
	ResourceCast(UnorderedAccessViewRHI)->Clear(RHICmdList, &Values, false);
}

FUnorderedAccessViewRHIRef FVulkanDynamicRHI::RHICreateUnorderedAccessView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
{
	return new FVulkanUnorderedAccessView(RHICmdList, *Device, Resource, ViewDesc);
}
