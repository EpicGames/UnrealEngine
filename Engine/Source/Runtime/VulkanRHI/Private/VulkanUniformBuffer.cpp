// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanUniformBuffer.cpp: Vulkan Constant buffer implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanContext.h"
#include "VulkanLLM.h"
#include "ShaderParameterStruct.h"
#include "RHIUniformBufferDataShared.h"
#include "VulkanDescriptorSets.h"
#include "VulkanBindlessDescriptorManager.h"

static int32 GVulkanAllowUniformUpload = 1;
static FAutoConsoleVariableRef CVarVulkanAllowUniformUpload(
	TEXT("r.Vulkan.AllowUniformUpload"),
	GVulkanAllowUniformUpload,
	TEXT("Allow Uniform Buffer uploads outside of renderpasses\n")
	TEXT(" 0: Disabled, buffers are always reallocated\n")
	TEXT(" 1: Enabled, buffers are uploaded outside renderpasses"),
	ECVF_Default
);


/*-----------------------------------------------------------------------------
	Uniform buffer RHI object
-----------------------------------------------------------------------------*/


void FVulkanUniformBuffer::AllocateMemory(const void* InitialData)
{
	const int32 DataSize = GetLayout().ConstantBufferSize;

	// Free any previous allocations
	Device.GetMemoryManager().FreeUniformBuffer(Allocation);

	void* DestinationData = nullptr;
	if (Usage == UniformBuffer_SingleDraw || Usage == UniformBuffer_SingleFrame)
	{
		const int32 UBAlignment = FMath::Max<uint32>(Device.GetLimits().minUniformBufferOffsetAlignment, 16u);
		DestinationData = Device.GetTempBlockAllocator().Alloc(DataSize, UBAlignment, nullptr, Allocation);
	}
	else
	{
		Device.GetMemoryManager().AllocUniformBuffer(Allocation, DataSize);
		DestinationData = Allocation.GetMappedPointer(&Device);
	}

	if (DestinationData && InitialData)
	{
		UE::RHICore::UpdateUniformBufferConstants(DestinationData, InitialData, GetLayout(), Device.SupportsBindless());
		Allocation.FlushMappedMemory(&Device);
	}
}

void FVulkanUniformBuffer::SetupUniformBufferView()
{
	if (UniformViewSRV && GetBufferHandle() == VK_NULL_HANDLE)
	{
		const FRHIViewDesc::FBufferSRV& SRVInfo = UniformViewSRV->GetDesc().Buffer.SRV;
		FVulkanBuffer* Buffer = ResourceCast(UniformViewSRV->GetBuffer());
		Allocation.Reference(Buffer->GetCurrentAllocation());
		check(Allocation.Size >= PLATFORM_MAX_UNIFORM_BUFFER_RANGE);
		//Adjust Allocation.Size ???
		Allocation.Offset += SRVInfo.OffsetInBytes;
	}
}

FVulkanUniformBuffer::FVulkanUniformBuffer(FVulkanDevice& InDevice, const FRHIUniformBufferLayout* InLayout, const void* Contents, EUniformBufferUsage InUsage, EUniformBufferValidation Validation)
	: FRHIUniformBuffer(InLayout)
	, Usage(InUsage)
	, Device(InDevice)
{
#if VULKAN_ENABLE_AGGRESSIVE_STATS
	SCOPE_CYCLE_COUNTER(STAT_VulkanUniformBufferCreateTime);
#endif

	// Verify the correctness of our thought pattern how the resources are delivered
	//	- If we have at least one resource, we also expect ResourceOffset to have an offset
	//	- Meaning, there is always a uniform buffer with a size specified larged than 0 bytes
	check(InLayout->Resources.Num() > 0 || InLayout->ConstantBufferSize > 0);
	const uint32 NumResources = InLayout->Resources.Num();

	// Setup resource table
	if (NumResources > 0)
	{
		// Transfer the resource table to an internal resource-array
		ResourceTable.Empty(NumResources);
		ResourceTable.AddZeroed(NumResources);

		if (Contents)
		{
			for (uint32 Index = 0; Index < NumResources; ++Index)
			{
				ResourceTable[Index] = GetShaderParameterResourceRHI(Contents, InLayout->Resources[Index].MemberOffset, InLayout->Resources[Index].MemberType);
			}
		}
	}

	if (EnumHasAnyFlags(InLayout->Flags, ERHIUniformBufferFlags::UniformView))
	{
		// For uniform view we expect an buffer SRV as a first resource
		check(InLayout->Resources.Num() > 0);
		EUniformBufferBaseType ResourceBaseType = InLayout->Resources[0].MemberType;
		if (ResourceBaseType == UBMT_SRV || ResourceBaseType == UBMT_RDG_BUFFER_SRV)
		{
			UniformViewSRV = (FRHIShaderResourceView*)GetShaderParameterResourceRHI(Contents, InLayout->Resources[0].MemberOffset, ResourceBaseType);
		}
		check(UniformViewSRV)
		return;
	}

	if (InLayout->ConstantBufferSize > 0)
	{
		AllocateMemory(Contents);
		BindlessHandle = Device.GetBindlessDescriptorManager()->AllocateDescriptor(ERHIDescriptorType::CBV);
		UpdateBindlessHandle({}, true);
	}
}

FVulkanUniformBuffer::~FVulkanUniformBuffer()
{
	if (BindlessHandle.IsValid())
	{
		Device.GetDeferredDeletionQueue().EnqueueBindlessHandle(BindlessHandle);
	}

	Device.GetMemoryManager().FreeUniformBuffer(Allocation);
}

void FVulkanUniformBuffer::UpdateResourceTable(const FRHIUniformBufferLayout& InLayout, const void* Contents, int32 NumResources)
{
	check(ResourceTable.Num() == NumResources);

	for (int32 Index = 0; Index < NumResources; ++Index)
	{
		const auto Parameter = InLayout.Resources[Index];
		ResourceTable[Index] = GetShaderParameterResourceRHI(Contents, Parameter.MemberOffset, Parameter.MemberType);
	}
}

void FVulkanUniformBuffer::UpdateResourceTable(TArrayView<FRHIResource*> Resources)
{
	check(Resources.Num() <= ResourceTable.Num());
	for (int32 ResourceIndex = 0; ResourceIndex < Resources.Num(); ++ResourceIndex)
	{
		ResourceTable[ResourceIndex] = Resources[ResourceIndex];
	}
}

void FVulkanUniformBuffer::UpdateBindlessHandle(const FVulkanContextArray& Contexts, bool bImmediate)
{
	if (BindlessHandle.IsValid() && Device.GetBindlessDescriptorManager()->IsSupported())
	{
		const VkDeviceAddress BufferAddress = GetDeviceAddress();
		Device.GetBindlessDescriptorManager()->UpdateDescriptor(Contexts, BindlessHandle, BufferAddress, GetSize(), bImmediate);
	}
}

VkDeviceAddress FVulkanUniformBuffer::GetDeviceAddress() const
{
	if (Device.GetOptionalExtensions().HasBufferDeviceAddress)
	{
		const VkBufferDeviceAddressInfo BufferInfo = {
			.sType = VK_STRUCTURE_TYPE_BUFFER_DEVICE_ADDRESS_INFO,
			.pNext = nullptr,
			.buffer = GetBufferHandle()
		};
		VkDeviceAddress BufferAddress = VulkanRHI::vkGetBufferDeviceAddressKHR(Device.GetHandle(), &BufferInfo);
		BufferAddress += GetOffset();
		return BufferAddress;
	}
	return 0;
}

FUniformBufferRHIRef FVulkanDynamicRHI::RHICreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation)
{
	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanUniformBuffers);

	return new FVulkanUniformBuffer(*Device, Layout, Contents, Usage, Validation);
}

void FVulkanDynamicRHI::RHIUpdateUniformBuffer(FRHICommandListBase& RHICmdList, FRHIUniformBuffer* UniformBufferRHI, const void* Contents)
{
	FVulkanUniformBuffer* UniformBuffer = ResourceCast(UniformBufferRHI);

	SCOPE_CYCLE_COUNTER(STAT_VulkanUpdateUniformBuffers);

	const FRHIUniformBufferLayout& Layout = UniformBuffer->GetLayout();
	const int32 ConstantBufferSize = Layout.ConstantBufferSize;
	const bool bBypass = RHICmdList.Bypass();

	TArrayView<FRHIResource*> CmdListResources;
	if (const int32 NumResources = Layout.Resources.Num())
	{
		CmdListResources = bBypass
			? TArrayView<FRHIResource*>((FRHIResource**)FMemory_Alloca(sizeof(FRHIResource*) * NumResources), NumResources)
			: RHICmdList.AllocArrayUninitialized<FRHIResource*>(NumResources);

		for (int32 Index = 0; Index < NumResources; Index++)
		{
			const FRHIUniformBufferResource& Parameter = Layout.Resources[Index];
			CmdListResources[Index] = GetShaderParameterResourceRHI(Contents, Parameter.MemberOffset, Parameter.MemberType);
		}
	}

	if (!bBypass)
	{
		for (FRHIResource* Resource : CmdListResources)
		{
			Resource->AddRef();
		}
	}

	const void* const TempContents = (bBypass || ConstantBufferSize == 0)
		? Contents
		: RHICmdList.AllocCopy(Contents, ConstantBufferSize, 16);

	RHICmdList.EnqueueLambdaMultiPipe(GetEnabledRHIPipelines(), FRHICommandListBase::EThreadFence::Enabled, TEXT("RHIUpdateUniformBuffer"),
		[UniformBuffer, TempContents, ConstantBufferSize, bBypass, CmdListResources = MoveTemp(CmdListResources)](const FVulkanContextArray& Contexts)
	{
		if (ConstantBufferSize > 0)
		{
			UniformBuffer->AllocateMemory(TempContents);
			UniformBuffer->UpdateBindlessHandle(Contexts, false);
		}

		UniformBuffer->UpdateResourceTable(CmdListResources);
		if (!bBypass)
		{
			for (FRHIResource* Resource : CmdListResources)
			{
				Resource->Release();
			}
		}
	});
}

