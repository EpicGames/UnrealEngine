// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanUtil.cpp: Vulkan Utility implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanUtil.h"
#include "VulkanPendingState.h"
#include "VulkanContext.h"
#include "VulkanMemory.h"
#include "Engine/GameEngine.h"
#include "Misc/App.h"
#include "Misc/OutputDeviceRedirector.h"
#include "RHIValidationContext.h"
#include "PipelineStateCache.h"
#include "Async/ParallelFor.h"
#include "HAL/FileManager.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "RenderCore.h"
#include "RHICoreNvidiaAftermath.h"
#include "VulkanRayTracing.h"
#include "VulkanQuery.h"

#if PLATFORM_ANDROID
#include "Android/AndroidStats.h"
#endif

FVulkanDynamicRHI*	GVulkanRHI = nullptr;

extern CORE_API bool GIsGPUCrashed;

void SetVulkanResourceName(FVulkanDevice* Device, FVulkanTexture* Texture, const TCHAR* Name)
{
#if VULKAN_ENABLE_IMAGE_TRACKING_LAYER
	VulkanRHI::BindDebugLabelName(Texture->Image, Name);
#endif

#if VULKAN_ENABLE_DUMP_LAYER
// TODO: this dies in the printf on android. Needs investigation.
#if !PLATFORM_ANDROID
	VulkanRHI::PrintfBegin(*FString::Printf(TEXT("vkDebugMarkerSetObjectNameEXT(0x%p=%s)\n"), Texture->Image, Name));
#endif
#endif

#if VULKAN_ENABLE_DRAW_MARKERS
	if (PFN_vkSetDebugUtilsObjectNameEXT SetDebugName = Device->GetSetDebugName())
	{
		FAnsiString AnsiName(Name);
		VulkanRHI::SetDebugName(SetDebugName, Device->GetHandle(), Texture->Image, GetData(AnsiName));
	}
#endif
}

#if NV_AFTERMATH
struct FShaderPair
{
	const FRHIShader*    RHIShader = nullptr;
	const FVulkanShader* VulkanShader = nullptr;
};

template<typename T>
void ConditionallyAddShader(const T* Value, TArray<FShaderPair>& Shaders, TSet<const FRHIShader*>& ShaderSet)
{
	if (Value && !ShaderSet.Contains(Value))
	{
		Shaders.Add(FShaderPair {
			static_cast<const FRHIShader*>(Value),
			static_cast<const FVulkanShader*>(Value),
		});
		
		ShaderSet.Add(Value);
	}
}

bool IsPipelineUnused(const VulkanResourceFrameCounter& Counter, uint32_t Threshold)
{
	// Check in case that it's ahead of the adapter, just in case it entered a strange state
	const uint32 PipelineFrameIndex = Counter.Get();
	return (PipelineFrameIndex <= GFrameNumberRenderThread) && (GFrameNumberRenderThread - PipelineFrameIndex) > Threshold;
}

void AftermathLateAssociate(float TimeLimitSeconds, uint32 FrameLimit)
{
	uint64 CycleStart = FPlatformTime::Cycles64();

	UE_LOGF(LogVulkanRHI, Log, "Starting late shader associations...");

	TArray<FShaderPair>     Shaders;
	TSet<const FRHIShader*> ShaderSet;

	uint32_t IgnoredPipelines = 0;

	// Get active pipelines, allow one second for consolidation to finish
	TArray<TRefCountPtr<FRHIResource>> PipelineResources;
	PipelineStateCache::GetPipelineStates(PipelineResources, true, UE::FTimeout(FTimespan::FromSeconds(1)));

	// Deduplicate shaders, Aftermath hashes are not local to the parent pipeline
	for (FRHIResource* Resource : PipelineResources)
	{
		if (!Resource)
		{
			continue;
		}
		
		switch (Resource->GetType())
		{
			default:
				checkNoEntry();
			case RRT_GraphicsPipelineState:
			{
				auto* Pipeline = static_cast<FVulkanGraphicsPipelineState*>(Resource);
					
				if (IsPipelineUnused(Pipeline->FrameCounter, FrameLimit))
				{
					IgnoredPipelines++;
					continue;
				}
					
				ConditionallyAddShader(static_cast<FVulkanVertexShader*>(Pipeline->GetShader(SF_Vertex)), Shaders, ShaderSet);
				ConditionallyAddShader(static_cast<FVulkanGeometryShader*>(Pipeline->GetShader(SF_Geometry)), Shaders, ShaderSet);
				ConditionallyAddShader(static_cast<FVulkanGeometryShader*>(Pipeline->GetShader(SF_Amplification)), Shaders, ShaderSet);
				ConditionallyAddShader(static_cast<FVulkanGeometryShader*>(Pipeline->GetShader(SF_Mesh)), Shaders, ShaderSet);
				ConditionallyAddShader(static_cast<FVulkanPixelShader*>(Pipeline->GetShader(SF_Pixel)), Shaders, ShaderSet);
				break;
			}
			case RRT_ComputePipelineState:
			{
				auto* Pipeline = static_cast<FVulkanComputePipeline*>(Resource);
				
				if (IsPipelineUnused(Pipeline->FrameCounter, FrameLimit))
				{
					IgnoredPipelines++;
					continue;
				}
					
				ConditionallyAddShader(static_cast<FVulkanComputeShader*>(Pipeline->GetComputeShader()), Shaders, ShaderSet);
				break;
			}
			case RRT_RayTracingPipelineState:
			{
				auto* Pipeline = static_cast<FVulkanRayTracingPipelineState*>(Resource);
					
				if (IsPipelineUnused(Pipeline->FrameCounter, FrameLimit))
				{
					IgnoredPipelines++;
					continue;
				}

				for (int32 i = 0; i < Pipeline->GetVulkanShaderNum(SF_RayGen); i++)
				{
					ConditionallyAddShader(Pipeline->GetVulkanShader(SF_RayGen, i), Shaders, ShaderSet);
				}

				for (int32 i = 0; i < Pipeline->GetVulkanShaderNum(SF_RayCallable); i++)
				{
					ConditionallyAddShader(Pipeline->GetVulkanShader(SF_RayCallable, i), Shaders, ShaderSet);
				}

				for (int32 i = 0; i < Pipeline->GetVulkanShaderNum(SF_RayHitGroup); i++)
				{
					ConditionallyAddShader(Pipeline->GetVulkanShader(SF_RayHitGroup, i), Shaders, ShaderSet);
				}

				for (int32 i = 0; i < Pipeline->GetVulkanShaderNum(SF_RayMiss); i++)
				{
					ConditionallyAddShader(Pipeline->GetVulkanShader(SF_RayMiss, i), Shaders, ShaderSet);
				}
				break;
			}
		}
	}

	UE_LOGF(LogVulkanRHI, Log, "Late shader associations ignored %u pipelines based on frame fences", IgnoredPipelines);

	// Parallelize as much as possible to avoid timeouts
	ParallelFor(Shaders.Num(), [CycleStart, TimeLimitSeconds, &Shaders](int32 Index)
	{
		// Aftermath handling is time constrained, if we hit the limit just stop
		float Elapsed = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - CycleStart);
		if (Elapsed >= TimeLimitSeconds)
		{
			UE_CALL_ONCE([Elapsed]
			{
				UE_LOGF(LogVulkanRHI, Warning, "Late shader associations timed out at %.0fs", Elapsed);
			});
			return;
		}

		FShaderPair               Shader = Shaders[Index];
		FVulkanShader::FSpirvCode Code   = Shader.VulkanShader->GetSpirvCode();

		FString DebugName;
		if (Shader.RHIShader->HasShaderName())
		{
			DebugName = Shader.RHIShader->GetShaderName();	
		}
		else
		{
			ANSICHAR AnsiName[1024];
			Shader.VulkanShader->GetEntryPoint(AnsiName, 1024);
			DebugName = AnsiName;
		}
		
		UE::RHICore::Nvidia::Aftermath::RegisterShaderBinary(
			Code.GetCodeView().GetData(),
			Code.GetCodeView().Num() * sizeof(uint32_t),
			DebugName
		);
	});
	
	double TimeMS = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - CycleStart);
	UE_LOGF(LogVulkanRHI, Log, "Created late shader associations, took %.0fms", TimeMS);
}
#endif // NV_AFTERMATH


FVulkanStagingBuffer::~FVulkanStagingBuffer()
{
	if (StagingBuffer)
	{
		check(Device);
		Device->GetStagingManager().ReleaseBuffer(nullptr, StagingBuffer);
	}
}

void* FVulkanStagingBuffer::Lock(uint32 Offset, uint32 NumBytes)
{
	check(!bIsLocked);
	bIsLocked = true;
	const uint32 EndOffset = Offset + NumBytes;
	checkf(EndOffset <= QueuedNumBytes, TEXT("Lock at Offset (%u) and NumBytes (%u) reads beyond the allocated size of the staging buffer (%u)"), Offset, NumBytes, QueuedNumBytes);
	// make sure cached memory is invalidated
	StagingBuffer->InvalidateMappedMemory();
	return (void*)((uint8*)StagingBuffer->GetMappedPointer() + Offset);
}

void FVulkanStagingBuffer::Unlock()
{
	check(bIsLocked);
	bIsLocked = false;
}

FStagingBufferRHIRef FVulkanDynamicRHI::RHICreateStagingBuffer()
{
	return new FVulkanStagingBuffer();
}

void* FVulkanDynamicRHI::RHILockStagingBuffer(FRHIStagingBuffer* StagingBufferRHI, FRHIGPUFence* FenceRHI, uint32 Offset, uint32 NumBytes)
{
	FVulkanStagingBuffer* StagingBuffer = ResourceCast(StagingBufferRHI);
	checkSlow(!FenceRHI || FenceRHI->Poll());
	return StagingBuffer->Lock(Offset, NumBytes);
}

void FVulkanDynamicRHI::RHIUnlockStagingBuffer(FRHIStagingBuffer* StagingBufferRHI)
{
	FVulkanStagingBuffer* StagingBuffer = ResourceCast(StagingBufferRHI);
	StagingBuffer->Unlock();
}

extern int32 GAllowTimelineSemaphores;

FVulkanGPUFence::FVulkanGPUFence(FVulkanDevice& InDevice, FName InName)
	: FRHIGPUFence(InName)
	, Device(InDevice)
{
	if (!GAllowTimelineSemaphores)
	{
		VkEventCreateInfo EventCreateInfo;
		ZeroVulkanStruct(EventCreateInfo, VK_STRUCTURE_TYPE_EVENT_CREATE_INFO);
		VERIFYVULKANRESULT(VulkanRHI::vkCreateEvent(InDevice.GetHandle(), &EventCreateInfo, VULKAN_CPU_ALLOCATOR, &Event));
		VERIFYVULKANRESULT(VulkanRHI::vkResetEvent(InDevice.GetHandle(), Event));
	}
}

FVulkanGPUFence::~FVulkanGPUFence()
{
	if (Event)
	{
		Device.GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::Event, Event);
		Event = VK_NULL_HANDLE;
	}
}

void FVulkanGPUFence::Clear()
{
	CompletedSyncPoint = nullptr;

	if (Event)
	{
		VERIFYVULKANRESULT(VulkanRHI::vkResetEvent(Device.GetHandle(), Event));
		SubmittedSyncPoint = nullptr;
	}
}

bool FVulkanGPUFence::Poll() const
{
	if (CompletedSyncPoint.IsValid() && CompletedSyncPoint->IsComplete())
	{
		return true;
	}

	if (Event && SubmittedSyncPoint.IsValid() && SubmittedSyncPoint->IsComplete())
	{
		return (VulkanRHI::vkGetEventStatus(Device.GetHandle(), Event) == VK_EVENT_SET);
	}

	return false;
}

void FVulkanGPUFence::Wait(FRHICommandListImmediate& RHICmdList, FRHIGPUMask GPUMask) const
{
	if (!Poll())
	{
		SCOPED_NAMED_EVENT_TEXT("FVulkanGPUFence_Wait", FColor::Turquoise);
		FVulkanDynamicRHI::Get().ProcessInterruptQueueUntil(CompletedSyncPoint);
	}
}

FGPUFenceRHIRef FVulkanDynamicRHI::RHICreateGPUFence(const FName& Name)
{
	return new FVulkanGPUFence(*Device, Name);
}

void FVulkanDynamicRHI::RHIWriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIGPUFence* FenceRHI)
{
	FVulkanGPUFence* Fence = ResourceCast(FenceRHI);
	check(Fence);

	checkf(!Fence->SubmittedSyncPoint.IsValid() && !Fence->CompletedSyncPoint.IsValid(), TEXT("The fence for the current GPU node has already been issued."));

	// When using minimal submits, the simple threading scheme means we can push back when fence results are communicated
	const EVulkanSyncPointType SyncPointType = Device->UseMinimalSubmits() ? EVulkanSyncPointType::Context : EVulkanSyncPointType::GPUAndCPU;
	Fence->CompletedSyncPoint = FVulkanSyncPoint::Create(SyncPointType, TEXT("GPUFence"));

	if (Fence->Event)
	{
		Fence->SubmittedSyncPoint = FGraphEvent::CreateGraphEvent();
	}

	Fence->NumPendingWriteCommands.Increment();
	RHICmdList.EnqueueLambda([FenceRef=TRefCountPtr<FVulkanGPUFence>(Fence), CompletedSyncPoint=Fence->CompletedSyncPoint, SubmittedSyncPoint=Fence->SubmittedSyncPoint](FRHICommandListBase& CmdList)
	{
		FVulkanCommandListContext& Context = FVulkanCommandListContext::Get(CmdList);

		if (FenceRef->Event)
		{
			VulkanRHI::vkCmdSetEvent(Context.GetCommandBuffer().GetHandle(), FenceRef->Event, VK_PIPELINE_STAGE_BOTTOM_OF_PIPE_BIT);
			Context.AddSubmissionEvent(SubmittedSyncPoint);
			Context.AddPendingSyncPoint(CompletedSyncPoint);
		}
		else
		{
			Context.SignalSyncPoint(CompletedSyncPoint);
		}

		FenceRef->NumPendingWriteCommands.Decrement();
	});
}

namespace VulkanRHI
{
	VkBuffer CreateBuffer(FVulkanDevice* InDevice, VkDeviceSize Size, VkBufferUsageFlags BufferUsageFlags, VkMemoryRequirements& OutMemoryRequirements)
	{
		VkDevice Device = InDevice->GetHandle();
		VkBuffer Buffer = VK_NULL_HANDLE;

		VkBufferCreateInfo BufferCreateInfo;
		ZeroVulkanStruct(BufferCreateInfo, VK_STRUCTURE_TYPE_BUFFER_CREATE_INFO);
		BufferCreateInfo.size = Size;
		BufferCreateInfo.usage = BufferUsageFlags;
		VERIFYVULKANRESULT_EXPANDED(VulkanRHI::vkCreateBuffer(Device, &BufferCreateInfo, VULKAN_CPU_ALLOCATOR, &Buffer));

		VulkanRHI::vkGetBufferMemoryRequirements(Device, Buffer, &OutMemoryRequirements);

		return Buffer;
	}

	/**
	 * Checks that the given result isn't a failure.  If it is, the application exits with an appropriate error message.
	 * @param	Result - The result code to check
	 * @param	Code - The code which yielded the result.
	 * @param	VkFunction - Tested function name.
	 * @param	Filename - The filename of the source file containing Code.
	 * @param	Line - The line number of Code within Filename.
	 */
	void VerifyVulkanResult(VkResult Result, const ANSICHAR* VkFunction, const ANSICHAR* Filename, uint32 Line)
	{
		// It's possible for multiple threads to catch GPU crashes or other Vulkan errors at the same time. Make sure we only log the error once
		// by acquiring this critical section inside VerifyVulkanResult (and never releasing it, because those functions don't return).
		static FCriticalSection CrashCritSec;

		bool bDumpMemory = false;
		bool bIsGPUCrashed = false;
		FString ErrorString;
		switch (Result)
		{
#define VKERRORCASE(x)	case x: ErrorString = TEXT(#x)
		VKERRORCASE(VK_NOT_READY); break;
		VKERRORCASE(VK_TIMEOUT); break;
		VKERRORCASE(VK_EVENT_SET); break;
		VKERRORCASE(VK_EVENT_RESET); break;
		VKERRORCASE(VK_INCOMPLETE); break;
		VKERRORCASE(VK_ERROR_OUT_OF_HOST_MEMORY); bDumpMemory = true; break;
		VKERRORCASE(VK_ERROR_OUT_OF_DEVICE_MEMORY); bDumpMemory = true; break;
		VKERRORCASE(VK_ERROR_INITIALIZATION_FAILED); break;
		VKERRORCASE(VK_ERROR_DEVICE_LOST); 
			bIsGPUCrashed = true;
#if PLATFORM_ANDROID
			FAndroidStats::LogGPUStats();
#endif
			break;
		VKERRORCASE(VK_ERROR_MEMORY_MAP_FAILED); break;
		VKERRORCASE(VK_ERROR_LAYER_NOT_PRESENT); break;
		VKERRORCASE(VK_ERROR_EXTENSION_NOT_PRESENT); break;
		VKERRORCASE(VK_ERROR_FEATURE_NOT_PRESENT); break;
		VKERRORCASE(VK_ERROR_INCOMPATIBLE_DRIVER); break;
		VKERRORCASE(VK_ERROR_TOO_MANY_OBJECTS); break;
		VKERRORCASE(VK_ERROR_FORMAT_NOT_SUPPORTED); break;
		VKERRORCASE(VK_ERROR_SURFACE_LOST_KHR); break;
		VKERRORCASE(VK_ERROR_NATIVE_WINDOW_IN_USE_KHR); break;
		VKERRORCASE(VK_SUBOPTIMAL_KHR); break;
		VKERRORCASE(VK_ERROR_OUT_OF_DATE_KHR); break;
		VKERRORCASE(VK_ERROR_INCOMPATIBLE_DISPLAY_KHR); break;
		VKERRORCASE(VK_ERROR_VALIDATION_FAILED_EXT); break;
		VKERRORCASE(VK_ERROR_INVALID_SHADER_NV); break;
		VKERRORCASE(VK_ERROR_FRAGMENTED_POOL); break;
		VKERRORCASE(VK_ERROR_OUT_OF_POOL_MEMORY_KHR); break;
		VKERRORCASE(VK_ERROR_INVALID_EXTERNAL_HANDLE_KHR); break;
		VKERRORCASE(VK_ERROR_NOT_PERMITTED_EXT); break;
#undef VKERRORCASE
		default:
			ErrorString = VK_TYPE_TO_STRING(VkResult, Result);
			break;
		}

		FString Message = FString::Printf(TEXT("%s failed, VkResult=%d\n at %s:%u \n with error %s"), ANSI_TO_TCHAR(VkFunction), (int32)Result, ANSI_TO_TCHAR(Filename), Line, *ErrorString);
#define VULKAN_PRINT_ERROR_MSG(Verbosity) UE_LOGF(LogVulkanRHI, Verbosity, "%ls", *Message)

		if (bIsGPUCrashed)
		{
			if (GVulkanRHI->IsInInterruptThread())
			{
				//
				// We're already on the interrupt thread. We must not block on the lock, as the interrupt thread is responsible
				// for reporting the GPU crash (we can only safely access the RHI breadcrumbs / active payloads from the interrupt thread).
				//
				// Attempt to take the crash lock to prevent other threads from log spamming, but don't give up if we fail to do so.
				//
				if (CrashCritSec.TryLock())
				{
					// We're the first thread to take the lock... log the error
					VULKAN_PRINT_ERROR_MSG(Error);
				}

				// This function does not return.
				GVulkanRHI->TerminateOnGPUCrash(*Message);
			}
			else
			{
				// Take the lock to ensure we report the error only once
				CrashCritSec.Lock();
				VULKAN_PRINT_ERROR_MSG(Error);

				// The interrupt thread must be the one to handle VK_ERROR_DEVICE_LOST
				// This function does not return.
				GVulkanRHI->ProcessInterruptQueueOnGPUCrash();
			}
		}
		else
		{
			CrashCritSec.Lock();

			VULKAN_PRINT_ERROR_MSG(Error);

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
			if (bDumpMemory)
			{
				GVulkanRHI->DumpMemory();
			}
#endif
		}

		GLog->Panic();

		VULKAN_PRINT_ERROR_MSG(Fatal);

		// Force shutdown, we can't do anything useful anymore.
		FPlatformMisc::RequestExit(true, TEXT("VerifyVulkanResult"));

#undef VULKAN_PRINT_ERROR_MSG
	}

}

void FVulkanDynamicRHI::DumpCrashMarkers()
{
	// Only one device in Vulkan
	const uint32 DeviceIndex = 0;

	// Log shader asserts / prints
	{
		FString ShaderDiagnostics;

		Device->ForEachQueue([&ShaderDiagnostics](FVulkanQueue& Queue)
			{
				FVulkanDiagnosticBuffer* DiagnosticBuffer = Queue.GetDiagnosticBuffer();
				if (DiagnosticBuffer)
				{
					const EVulkanQueueType QueueType = Queue.GetQueueType();
					const TCHAR* QueueName = GetVulkanQueueTypeName(QueueType);
					ShaderDiagnostics += DiagnosticBuffer->GetShaderDiagnosticMessages(DeviceIndex, (uint32)QueueType, QueueName);
				}

			});

		if (!ShaderDiagnostics.IsEmpty())
		{
			UE_LOGF(LogVulkanRHI, Error, "Shader diagnostic messages and asserts:%ls\r\n", *ShaderDiagnostics);
		}
	}

	if (Device->GetOptionalExtensions().HasEXTDeviceFault)
	{
		const VkDevice DeviceHandle = Device->GetHandle();
			VkResult Result;

			VkDeviceFaultCountsEXT FaultCounts;
			ZeroVulkanStruct(FaultCounts, VK_STRUCTURE_TYPE_DEVICE_FAULT_COUNTS_EXT);
			Result = VulkanRHI::vkGetDeviceFaultInfoEXT(DeviceHandle, &FaultCounts, nullptr);
			if (Result == VK_SUCCESS)
			{
				VkDeviceFaultInfoEXT FaultInfo;
				ZeroVulkanStruct(FaultInfo, VK_STRUCTURE_TYPE_DEVICE_FAULT_INFO_EXT);

				TArray<VkDeviceFaultAddressInfoEXT> AddressInfos;
				AddressInfos.SetNumZeroed(FaultCounts.addressInfoCount);
				FaultInfo.pAddressInfos = AddressInfos.GetData();

				TArray<VkDeviceFaultVendorInfoEXT> VendorInfos;
				VendorInfos.SetNumZeroed(FaultCounts.vendorInfoCount);
				FaultInfo.pVendorInfos = VendorInfos.GetData();

				TArray<uint8> VendorBinaryData;
				VendorBinaryData.SetNumZeroed(FaultCounts.vendorBinarySize);
				FaultInfo.pVendorBinaryData = VendorBinaryData.GetData();

				Result = VulkanRHI::vkGetDeviceFaultInfoEXT(DeviceHandle, &FaultCounts, &FaultInfo);
				if (Result == VK_SUCCESS)
				{
					// :todo-jn: match these up to resources

					auto ReportAddrToStr = [&AddressInfos]() {
						FString AddrStr;
						for (const VkDeviceFaultAddressInfoEXT& AddrInfo : AddressInfos)
						{
							const uint64_t LowerAddress = (AddrInfo.reportedAddress & ~(AddrInfo.addressPrecision - 1));
							const uint64_t UpperAddress = (AddrInfo.reportedAddress | (AddrInfo.addressPrecision - 1));

							AddrStr += FString::Printf(TEXT("\n    - %s : 0x%016llX (range:0x%016llX-0x%016llX)"), 
								VK_TYPE_TO_STRING(VkDeviceFaultAddressTypeEXT, AddrInfo.addressType),
								AddrInfo.reportedAddress, LowerAddress, UpperAddress);
						}
						return AddrStr;
					};

					auto ReportVendorToStr = [&VendorInfos]() {
						FString VendorStr;
						for (const VkDeviceFaultVendorInfoEXT& VendorInfo : VendorInfos)
						{
							VendorStr += FString::Printf(TEXT("\n    - %s (code:0x%016llX data:0x%016llX)"),
								StringCast<TCHAR>((UTF8CHAR*)VendorInfo.description).Get(), VendorInfo.vendorFaultCode, VendorInfo.vendorFaultData);
						}
						return VendorStr;
					};

					UE_LOGF(LogVulkanRHI, Error, 
						"\nDEVICE FAULT REPORT:\n" "* Description: %ls\n" "* Address Info: %ls\n" "* Vendor Info: %ls\n" "* Vendor Binary Size: %llu\n",

						StringCast<TCHAR>((UTF8CHAR*)FaultInfo.description).Get(), *ReportAddrToStr(), *ReportVendorToStr(), FaultCounts.vendorBinarySize
					);
				}
			}
		}


#if WITH_RHI_BREADCRUMBS

	// Log Nvidia checkpoints
#if NV_AFTERMATH
	if (Device->GetOptionalExtensions().HasNVDiagnosticCheckpoints)
	{
		struct FCheckpointDataNV : public VkCheckpointDataNV
		{
			FCheckpointDataNV()
			{
				ZeroVulkanStruct(*this, VK_STRUCTURE_TYPE_CHECKPOINT_DATA_NV);
			}
		};

		Device->ForEachQueue([](FVulkanQueue& Queue)
		{
			TArray<FCheckpointDataNV> Checkpoints;
			uint32 Num = 0;
			const VkQueue QueueHandle = Queue.GetHandle();
			VulkanDynamicAPI::vkGetQueueCheckpointDataNV(QueueHandle, &Num, nullptr);
			if (Num > 0)
			{
				Checkpoints.AddDefaulted(Num);
				VulkanDynamicAPI::vkGetQueueCheckpointDataNV(QueueHandle, &Num, &Checkpoints[0]);
				check(Num == Checkpoints.Num());
				for (uint32 Index = 0; Index < Num; ++Index)
				{
					check(Checkpoints[Index].sType == VK_STRUCTURE_TYPE_CHECKPOINT_DATA_NV);
					FRHIBreadcrumbNode* Breadcrumb = (FRHIBreadcrumbNode*)Checkpoints[Index].pCheckpointMarker;
					UE_LOGF(LogVulkanRHI, Error, "[VK_NV_device_diagnostic_checkpoints] %i: Stage %ls (0x%08x), %ls",
						Index, VK_TYPE_TO_STRING(VkPipelineStageFlagBits, Checkpoints[Index].stage), Checkpoints[Index].stage,
						*Breadcrumb->GetFullPath());
				}
				GLog->Panic();
			}
		});
	}
#endif // NV_AFTERMATH

	// Log RHI breadcrumb data
	if (UE::RHI::UseGPUCrashBreadcrumbs())
	{
		TMap<FRHIBreadcrumbState::FQueueID, TArray<FRHIBreadcrumbRange>> QueueRanges;
		FRHIBreadcrumbState BreadcrumbState;

		Device->ForEachQueue([&QueueRanges, &BreadcrumbState, DeviceIndex](FVulkanQueue& Queue)
		{
			FVulkanDiagnosticBuffer* DiagnosticBuffer = Queue.GetDiagnosticBuffer();
			if (DiagnosticBuffer)
			{
				const EVulkanQueueType QueueType = Queue.GetQueueType();
				const TCHAR* QueueName = GetVulkanQueueTypeName(QueueType);

				ERHIPipeline Pipeline;
				switch (QueueType)
				{
				default: return; // Skip pipelines that the RHI doesn't handle
				case EVulkanQueueType::Graphics:     Pipeline = ERHIPipeline::Graphics; break;
				case EVulkanQueueType::AsyncCompute: Pipeline = ERHIPipeline::AsyncCompute; break;
				}

				TArray<FRHIBreadcrumbRange>& Ranges = QueueRanges.Add({ DeviceIndex, Pipeline });

				// Extract the breadcrumb ranges for the payloads still in the interrupt queue.
				for (FVulkanPayload* Payload : Queue.PendingInterrupt)
				{
					if (Payload->BreadcrumbRange)
					{
						Ranges.AddUnique(Payload->BreadcrumbRange);
					}
				}

				BreadcrumbState.Devices[DeviceIndex].Pipelines[Pipeline].MarkerOut = DiagnosticBuffer->ReadMarkerOut();
				BreadcrumbState.Devices[DeviceIndex].Pipelines[Pipeline].MarkerIn = DiagnosticBuffer->ReadMarkerIn();
			}
		});

		// Traverse the breadcrumb tree and log active GPU work
		if (!QueueRanges.IsEmpty())
		{
			BreadcrumbState.DumpActiveBreadcrumbs(QueueRanges);
		}
	}
#endif // WITH_RHI_BREADCRUMBS
}




void FVulkanDynamicRHI::TerminateOnGPUCrash(const TCHAR* Message)
{
	GIsGPUCrashed = true;

	DumpCrashMarkers();

	// Make sure we wait on the Aftermath crash dump before we crash.
#if NV_AFTERMATH
	TArray<UE::RHICore::Nvidia::Aftermath::FCrashResult> AftermathResults;
	UE::RHICore::Nvidia::Aftermath::OnGPUCrash(AftermathResults);
#endif

	UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
	if (GameEngine)
	{
		GameEngine->OnGPUCrash();
	}

	// hard break here when the debugger is attached
	if (FPlatformMisc::IsDebuggerPresent())
	{
		UE_DEBUG_BREAK();
	}

	FVulkanPlatform::OnGPUCrash(Message);

	// Force shutdown, we can't do anything useful anymore.
	FPlatformMisc::RequestExit(true, TEXT("FVulkanDynamicRHI.TerminateOnGPUCrash"));
}



// PSO Stats
DEFINE_STAT(STAT_VulkanNumPSOs);
DEFINE_STAT(STAT_VulkanNumGraphicsPSOs);
DEFINE_STAT(STAT_VulkanNumPSOLRU);
DEFINE_STAT(STAT_VulkanNumPSOLRUSize);
DEFINE_STAT(STAT_VulkanPSOLookupTime);
DEFINE_STAT(STAT_VulkanPSOCreationTime);
DEFINE_STAT(STAT_VulkanPSOHeaderInitTime);
DEFINE_STAT(STAT_VulkanPSOVulkanCreationTime);
DEFINE_STAT(STAT_VulkanNumComputePSOs);
DEFINE_STAT(STAT_VulkanPSOKeyMemory);


// Graphic pipeline library stats
DEFINE_STAT(STAT_VulkanRegPSOCompileCount);
DEFINE_STAT(STAT_VulkanLibPSOCompileCount);
DEFINE_STAT(STAT_VulkanNumVertexInputLibs);
DEFINE_STAT(STAT_VulkanNumVSLibs);
DEFINE_STAT(STAT_VulkanNumMSLibs);
DEFINE_STAT(STAT_VulkanNumPSLibs);
DEFINE_STAT(STAT_VulkanNumFragOutputStateLibs);
DEFINE_STAT(STAT_VulkanNumShaderModule);


// General Stats
DEFINE_STAT(STAT_VulkanDrawCallTime);
DEFINE_STAT(STAT_VulkanDispatchCallTime);
DEFINE_STAT(STAT_VulkanDrawCallPrepareTime);
DEFINE_STAT(STAT_VulkanCustomPresentTime);
DEFINE_STAT(STAT_VulkanDispatchCallPrepareTime);
DEFINE_STAT(STAT_VulkanGetOrCreatePipeline);
DEFINE_STAT(STAT_VulkanGetDescriptorSet);
DEFINE_STAT(STAT_VulkanPipelineBind);
DEFINE_STAT(STAT_VulkanNumCmdBuffers);
DEFINE_STAT(STAT_VulkanNumRenderPasses);
DEFINE_STAT(STAT_VulkanNumFrameBuffers);
DEFINE_STAT(STAT_VulkanNumBufferViews);
DEFINE_STAT(STAT_VulkanNumImageViews);
DEFINE_STAT(STAT_VulkanNumPhysicalMemAllocations);
DEFINE_STAT(STAT_VulkanUniformBufferCreateTime);
DEFINE_STAT(STAT_VulkanApplyPackedUniformBuffers);
DEFINE_STAT(STAT_VulkanSRVUpdateTime);
DEFINE_STAT(STAT_VulkanUAVUpdateTime);
DEFINE_STAT(STAT_VulkanDeletionQueue);
DEFINE_STAT(STAT_VulkanQueueSubmit);
DEFINE_STAT(STAT_VulkanQueuePresent);
DEFINE_STAT(STAT_VulkanNumQueries);
DEFINE_STAT(STAT_VulkanNumQueryPools);
DEFINE_STAT(STAT_VulkanWaitFence);
DEFINE_STAT(STAT_VulkanWaitSwapchain);
DEFINE_STAT(STAT_VulkanAcquireBackBuffer);
DEFINE_STAT(STAT_VulkanStagingBuffer);
DEFINE_STAT(STAT_VulkanVkCreateDescriptorPool);
DEFINE_STAT(STAT_VulkanNumDescPools);
DEFINE_STAT(STAT_VulkanUpdateUniformBuffers);
DEFINE_STAT(STAT_VulkanUpdateUniformBuffersRename);
DEFINE_STAT(STAT_VulkanNumPooledBarrierEvents);
#if VULKAN_ENABLE_AGGRESSIVE_STATS
DEFINE_STAT(STAT_VulkanBarrierTime);
DEFINE_STAT(STAT_VulkanUpdateDescriptorSets);
DEFINE_STAT(STAT_VulkanNumUpdateDescriptors);
DEFINE_STAT(STAT_VulkanNumDescSets);
DEFINE_STAT(STAT_VulkanSetUniformBufferTime);
DEFINE_STAT(STAT_VulkanVkUpdateDS);
DEFINE_STAT(STAT_VulkanBindVertexStreamsTime);
#endif
DEFINE_STAT(STAT_VulkanNumDescSetsTotal);
