// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanVertexDeclaration.cpp: Vulkan vertex declaration RHI implementation.
=============================================================================*/

#include "PipelineFileCache.h"
#include "VulkanRHIPrivate.h"

struct FVulkanVertexDeclarationKey
{
	FVertexDeclarationElementList VertexElements;

	uint32 Hash;
	uint32 HashNoStride;

	explicit FVulkanVertexDeclarationKey(const FVertexDeclarationElementList& InElements)
		: VertexElements(InElements)
	{
		Hash = FCrc::MemCrc_DEPRECATED(VertexElements.GetData(), VertexElements.Num()*sizeof(FVertexElement));

		FVertexDeclarationElementList SortedElements = VertexElements;
		FPipelineCacheFileFormatPSO::SortVertexElements(SortedElements);

		HashNoStride = 0;
		for (int32 ElementIndex = 0; ElementIndex < SortedElements.Num(); ElementIndex++)
		{
			FVertexElement Element = SortedElements[ElementIndex];
			Element.Stride = 0;
			HashNoStride = FCrc::MemCrc32(&Element, sizeof(Element), HashNoStride);
		}
	}
};

inline uint32 GetTypeHash(const FVulkanVertexDeclarationKey& Key)
{
	return Key.Hash;
}

bool operator==(const FVulkanVertexDeclarationKey& A, const FVulkanVertexDeclarationKey& B)
{
	return (A.VertexElements.Num() == B.VertexElements.Num()
			&& !memcmp(A.VertexElements.GetData(), B.VertexElements.GetData(), A.VertexElements.Num() * sizeof(FVertexElement)));
}

FVulkanVertexDeclaration::FVulkanVertexDeclaration(const FVertexDeclarationElementList& InElements, uint32 InHash, uint32 InHashNoStrides) 
	: Elements(InElements)
	, Hash(InHash)
	, HashNoStrides(InHashNoStrides)
{
}

TMap<FVulkanVertexDeclarationKey, FVertexDeclarationRHIRef> GVertexDeclarationCache;

void FVulkanVertexDeclaration::EmptyCache()
{
	GVertexDeclarationCache.Empty(0);
}

FVertexDeclarationRHIRef FVulkanDynamicRHI::RHICreateVertexDeclaration(const FVertexDeclarationElementList& Elements)
{
	FVulkanVertexDeclarationKey Key(Elements);

	static FCriticalSection CS;
	FScopeLock ScopeLock(&CS);
	FVertexDeclarationRHIRef* VertexDeclarationRefPtr = GVertexDeclarationCache.Find(Key);
	if (VertexDeclarationRefPtr == nullptr)
	{
		VertexDeclarationRefPtr = &GVertexDeclarationCache.Add(Key, new FVulkanVertexDeclaration(Elements, Key.Hash, Key.HashNoStride));
	}

	check(VertexDeclarationRefPtr);
	check(IsValidRef(*VertexDeclarationRefPtr));

	return *VertexDeclarationRefPtr;
}

FVulkanVertexInputStateInfo::~FVulkanVertexInputStateInfo()
{
}
FVulkanVertexInputStateInfo::FVulkanVertexInputStateInfo()
	: Hash(0)
	, BindingsNum(0)
	, BindingsMask(0)
	, AttributesNum(0)
{
	FMemory::Memzero(Info);
	FMemory::Memzero(Attributes);
	FMemory::Memzero(Bindings);
}

bool FVulkanVertexInputStateInfo::operator ==(const FVulkanVertexInputStateInfo& Other)
{
	if(AttributesNum != Other.AttributesNum)
	{
		return false;
	}
	for(uint32 i = 0; i < AttributesNum; ++i)
	{
		if(0 != FMemory::Memcmp(&Attributes[i], &Other.Attributes[i], sizeof(Attributes[i])))
		{
			return false;
		}
	}
	return true;
}

void FVulkanVertexInputStateInfo::Generate(FVulkanVertexDeclaration* VertexDeclaration, uint32 VertexHeaderInOutAttributeMask)
{
	// GenerateVertexInputState is expected to be called only once!
	check(Info.sType == 0);

	// Generate vertex attribute Layout
	const FVertexDeclarationElementList& VertexInput = VertexDeclaration->Elements;

	// Generate Bindings
	for (const FVertexElement& Element : VertexInput)
	{
		if ((1<<Element.AttributeIndex) & VertexHeaderInOutAttributeMask)
		{
			check(Element.StreamIndex < MaxVertexElementCount);

			VkVertexInputBindingDescription& CurrBinding = Bindings[Element.StreamIndex];
			if ((BindingsMask & (1 << Element.StreamIndex)) != 0)
			{
				// If exists, validate.
				// Info must be the same
				check(CurrBinding.binding == Element.StreamIndex);
				check(CurrBinding.inputRate == Element.bUseInstanceIndex ? VK_VERTEX_INPUT_RATE_INSTANCE : VK_VERTEX_INPUT_RATE_VERTEX);
				check(CurrBinding.stride == Element.Stride);
			}
			else
			{
				// Zeroed outside
				check(CurrBinding.binding == 0 && CurrBinding.inputRate == 0 && CurrBinding.stride == 0);
				CurrBinding.binding = Element.StreamIndex;
				CurrBinding.inputRate = Element.bUseInstanceIndex ? VK_VERTEX_INPUT_RATE_INSTANCE : VK_VERTEX_INPUT_RATE_VERTEX;
				CurrBinding.stride = Element.Stride;

				// Add mask flag and increment number of bindings
				BindingsMask |= 1 << Element.StreamIndex;
			}
		}
	}

	// Remove gaps between bindings
	BindingsNum = 0;
	BindingToStream.Reset();
	StreamToBinding.Reset();
	for (int32 i=0; i<UE_ARRAY_COUNT(Bindings); i++)
	{
		if (!((1<<i) & BindingsMask))
		{
			continue;
		}

		BindingToStream.Add(BindingsNum, i);
		StreamToBinding.Add(i, BindingsNum);
		VkVertexInputBindingDescription& CurrBinding = Bindings[BindingsNum];
		CurrBinding = Bindings[i];
		CurrBinding.binding = BindingsNum;
		BindingsNum++;
	}

	// Clean originally placed bindings
	FMemory::Memset(Bindings + BindingsNum, 0, sizeof(Bindings[0]) * (UE_ARRAY_COUNT(Bindings)-BindingsNum));

	// Attributes are expected to be uninitialized/empty
	check(AttributesNum == 0);
	for (const FVertexElement& CurrElement : VertexInput)
	{
		// Mask-out unused vertex input
		if ((!((1<<CurrElement.AttributeIndex) & VertexHeaderInOutAttributeMask))
			||	!StreamToBinding.Contains(CurrElement.StreamIndex))
		{
			continue;
		}

		VkVertexInputAttributeDescription& CurrAttribute = Attributes[AttributesNum++]; // Zeroed at the begin of the function
		check(CurrAttribute.location == 0 && CurrAttribute.binding == 0 && CurrAttribute.format == 0 && CurrAttribute.offset == 0);

		CurrAttribute.binding = StreamToBinding.FindChecked(CurrElement.StreamIndex);
		CurrAttribute.location = CurrElement.AttributeIndex;
		CurrAttribute.format = UEToVkBufferFormat(CurrElement.Type);
		CurrAttribute.offset = CurrElement.Offset;
	}

	// Vertex Input
	Info.sType = VK_STRUCTURE_TYPE_PIPELINE_VERTEX_INPUT_STATE_CREATE_INFO;

	// Its possible to have no vertex buffers
	if (BindingsNum == 0)
	{
		check(Hash == 0);
		return;
	}

	Info.vertexBindingDescriptionCount = BindingsNum;
	Info.pVertexBindingDescriptions = Bindings;

	check(AttributesNum > 0);
	Info.vertexAttributeDescriptionCount = AttributesNum;
	Info.pVertexAttributeDescriptions = Attributes;

	Hash = FCrc::MemCrc32(Bindings, BindingsNum * sizeof(Bindings[0]));
	check(AttributesNum > 0);
	Hash = FCrc::MemCrc32(Attributes, AttributesNum * sizeof(Attributes[0]), Hash);
}
