// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanRHIPrivate.h"
#include "VulkanBindlessDescriptorManager.h"
#include "VulkanDevice.h"
#include "VulkanLLM.h"
#include "VulkanContext.h"

uint32 FVulkanTextureViewDesc::GetHash() const
{
	uint32 Hash = 0;
	Hash = FCrc::TypeCrc32(ViewType, Hash);
	Hash = FCrc::TypeCrc32(AspectFlags, Hash);
	Hash = FCrc::TypeCrc32(UEFormat, Hash);
	Hash = FCrc::TypeCrc32(Format, Hash);
	Hash = FCrc::TypeCrc32(FirstMip, Hash);
	Hash = FCrc::TypeCrc32(NumMips, Hash);
	Hash = FCrc::TypeCrc32(ArraySliceIndex, Hash);
	Hash = FCrc::TypeCrc32(NumArraySlices, Hash);
	Hash = FCrc::TypeCrc32(bUseIdentitySwizzle, Hash);
	Hash = FCrc::TypeCrc32(ImageUsageFlags, Hash);
	Hash = FCrc::TypeCrc32(SamplerYcbcrConversion, Hash);
	return Hash;
}

FVulkanView::FVulkanView(FVulkanDevice& InDevice, ERHIDescriptorType InDescriptorType)
	: Device(InDevice)
	, DescriptorType(InDescriptorType)
{
	// Some views might not want bindless handles (like views used strictly for framebuffers)
	if (InDescriptorType != ERHIDescriptorType::Invalid)
	{
		BindlessHandle = Device.GetBindlessDescriptorManager()->AllocateDescriptor(InDescriptorType);
	}
}

FVulkanView::~FVulkanView()
{
	Invalidate();

	if (BindlessHandle.IsValid())
	{
		Device.GetDeferredDeletionQueue().EnqueueBindlessHandle(BindlessHandle);
		BindlessHandle = FRHIDescriptorHandle();
	}
}

void FVulkanView::Invalidate()
{
	// Carry forward its initialized state
	const bool bIsInitialized = IsInitialized();

	switch (GetViewType())
	{
	default: checkNoEntry(); [[fallthrough]];
	case EType::Null:
		break;

	case EType::TypedBuffer:
		DEC_DWORD_STAT(STAT_VulkanNumBufferViews);
		Device.GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::BufferView, Storage.Get<FTypedBufferView>().View);
		break;

	case EType::Texture:
		DEC_DWORD_STAT(STAT_VulkanNumImageViews);
		Device.GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::ImageView, Storage.Get<FTextureView>().View);
		break;

	case EType::StructuredBuffer:
		// Nothing to do
		break;

	case EType::AccelerationStructure:
		Device.GetDeferredDeletionQueue().EnqueueResource(VulkanRHI::FDeferredDeletionQueue2::EType::AccelerationStructure, Storage.Get<FAccelerationStructureView>().Handle);
		break;
	}

	Storage.Emplace<FInvalidatedState>();
	Storage.Get<FInvalidatedState>().bInitialized = bIsInitialized;
}

void FVulkanView::InitAsTypedBufferView(const FVulkanContextArray& Contexts, FVulkanBuffer* Buffer, EPixelFormat UEFormat, uint32 InOffset, uint32 InSize)
{
	// We will need a deferred update if the descriptor was already in use
	const bool bImmediateUpdate = !IsInitialized();

	check(GetViewType() == EType::Null);
	Storage.Emplace<FTypedBufferView>();
	FTypedBufferView& TBV = Storage.Get<FTypedBufferView>();

	const uint32 TotalOffset = Buffer->GetOffset() + InOffset;

	check(UEFormat != PF_Unknown);
	const VkFormat Format = GVulkanBufferFormat[UEFormat];
	check(Format != VK_FORMAT_UNDEFINED);

	VkBufferViewCreateInfo ViewInfo;
	ZeroVulkanStruct(ViewInfo, VK_STRUCTURE_TYPE_BUFFER_VIEW_CREATE_INFO);
	ViewInfo.buffer = Buffer->GetHandle();
	ViewInfo.offset = TotalOffset;
	ViewInfo.format = Format;

	const uint32 TypeSize = VulkanRHI::GetNumBitsPerPixel(Format) / 8u;
	// View size has to be a multiple of element size
	// Commented out because there are multiple places in the high level rendering code which re-purpose buffers for a new format while there are still
	// views with the old format lying around, and then lock them with a size computed based on the new stride, triggering this assert when the old views
	// are re-created. These places need to be fixed before re-enabling this check (UE-211785).
	//check(IsAligned(InSize, TypeSize));

	//#todo-rco: Revisit this if buffer views become VK_BUFFER_USAGE_STORAGE_BUFFER_BIT instead of VK_BUFFER_USAGE_UNIFORM_TEXEL_BUFFER_BIT
	const VkPhysicalDeviceLimits& Limits = Device.GetLimits();
	const uint64 MaxSize = (uint64)Limits.maxTexelBufferElements * TypeSize;
	ViewInfo.range = FMath::Min<uint64>(InSize, MaxSize);
	// TODO: add a check() for exceeding MaxSize, to catch code which blindly makes views without checking the platform limits.

	check(Buffer->GetBufferUsageFlags() & (VK_BUFFER_USAGE_UNIFORM_TEXEL_BUFFER_BIT | VK_BUFFER_USAGE_STORAGE_TEXEL_BUFFER_BIT));
	check(IsAligned(InOffset, Limits.minTexelBufferOffsetAlignment));

	VERIFYVULKANRESULT(VulkanRHI::vkCreateBufferView(Device.GetHandle(), &ViewInfo, VULKAN_CPU_ALLOCATOR, &TBV.View));

	TBV.bVolatile = Buffer->IsVolatile();
	if (!TBV.bVolatile && UseVulkanDescriptorCache())
	{
		TBV.ViewId = ++GVulkanBufferViewHandleIdCounter;
	}

	INC_DWORD_STAT(STAT_VulkanNumBufferViews);
	// :todo-jn: the buffer view is actually not needed in bindless anymore

	Device.GetBindlessDescriptorManager()->UpdateDescriptor(Contexts, BindlessHandle, ViewInfo, bImmediateUpdate);
}

void FVulkanView::InitAsTextureView(const FVulkanContextArray& Contexts, VkImage InImage, const FVulkanTextureViewDesc& ViewDesc)
{
	// We will need a deferred update if the descriptor was already in use
	const bool bImmediateUpdate = !IsInitialized();

	check(GetViewType() == EType::Null);
	Storage.Emplace<FTextureView>();
	FTextureView& TV = Storage.Get<FTextureView>();

	LLM_SCOPE_VULKAN(ELLMTagVulkan::VulkanTextures);

	VkImageViewCreateInfo ViewInfo;
	ZeroVulkanStruct(ViewInfo, VK_STRUCTURE_TYPE_IMAGE_VIEW_CREATE_INFO);
	ViewInfo.image = InImage;
	ViewInfo.viewType = ViewDesc.ViewType;
	ViewInfo.format = ViewDesc.Format;

#if VULKAN_SUPPORTS_ASTC_DECODE_MODE
	VkImageViewASTCDecodeModeEXT DecodeMode;
	if (Device.GetOptionalExtensions().HasEXTASTCDecodeMode && IsAstcLdrFormat(ViewDesc.Format) && !IsAstcSrgbFormat(ViewDesc.Format))
	{
		ZeroVulkanStruct(DecodeMode, VK_STRUCTURE_TYPE_IMAGE_VIEW_ASTC_DECODE_MODE_EXT);
		DecodeMode.decodeMode = VK_FORMAT_R8G8B8A8_UNORM;
		DecodeMode.pNext = ViewInfo.pNext;
		ViewInfo.pNext = &DecodeMode;
	}
#endif

	VkSamplerYcbcrConversionInfo SamplerYcbcrConversionInfo;
	if (ViewDesc.SamplerYcbcrConversion)
	{
		ZeroVulkanStruct(SamplerYcbcrConversionInfo, VK_STRUCTURE_TYPE_SAMPLER_YCBCR_CONVERSION_INFO);
		SamplerYcbcrConversionInfo.conversion = ViewDesc.SamplerYcbcrConversion;
		SamplerYcbcrConversionInfo.pNext = ViewInfo.pNext;
		ViewInfo.pNext = &SamplerYcbcrConversionInfo;
	}

	if (ViewDesc.bUseIdentitySwizzle)
	{
		ViewInfo.components.a = VK_COMPONENT_SWIZZLE_IDENTITY;
		ViewInfo.components.r = VK_COMPONENT_SWIZZLE_IDENTITY;
		ViewInfo.components.g = VK_COMPONENT_SWIZZLE_IDENTITY;
		ViewInfo.components.b = VK_COMPONENT_SWIZZLE_IDENTITY;
	}
	else
	{
		ViewInfo.components = Device.GetFormatComponentMapping(ViewDesc.UEFormat);
	}

	ViewInfo.subresourceRange.aspectMask = ViewDesc.AspectFlags;
	ViewInfo.subresourceRange.baseMipLevel = ViewDesc.FirstMip;
	ensure(ViewDesc.NumMips != 0xFFFFFFFF);
	ViewInfo.subresourceRange.levelCount = ViewDesc.NumMips;

	ensure(ViewDesc.ArraySliceIndex != 0xFFFFFFFF);
	ensure(ViewDesc.NumArraySlices != 0xFFFFFFFF);
	ViewInfo.subresourceRange.baseArrayLayer = ViewDesc.ArraySliceIndex;
	ViewInfo.subresourceRange.layerCount = ViewDesc.NumArraySlices;

	//HACK.  DX11 on PC currently uses a D24S8 depthbuffer and so needs an X24_G8 SRV to visualize stencil.
	//So take that as our cue to visualize stencil.  In the future, the platform independent code will have a real format
	//instead of PF_DepthStencil, so the cross-platform code could figure out the proper format to pass in for this.
	if (ViewDesc.UEFormat == PF_X24_G8)
	{
		ensure((ViewInfo.format == (VkFormat)GPixelFormats[PF_DepthStencil].PlatformFormat) && (ViewInfo.format != VK_FORMAT_UNDEFINED));
		ViewInfo.subresourceRange.aspectMask = VK_IMAGE_ASPECT_STENCIL_BIT;
	}

	// Inform the driver the view will only be used with a subset of usage flags (to help performance and/or compatibility)
	VkImageViewUsageCreateInfo ImageViewUsageCreateInfo;
	if (ViewDesc.ImageUsageFlags != 0)
	{
		ZeroVulkanStruct(ImageViewUsageCreateInfo, VK_STRUCTURE_TYPE_IMAGE_VIEW_USAGE_CREATE_INFO);
		ImageViewUsageCreateInfo.usage = ViewDesc.ImageUsageFlags;

		ImageViewUsageCreateInfo.pNext = (void*)ViewInfo.pNext;
		ViewInfo.pNext = &ImageViewUsageCreateInfo;
	}

	INC_DWORD_STAT(STAT_VulkanNumImageViews);
	VERIFYVULKANRESULT(VulkanRHI::vkCreateImageView(Device.GetHandle(), &ViewInfo, VULKAN_CPU_ALLOCATOR, &TV.View));

	TV.Image = InImage;

	if (UseVulkanDescriptorCache())
	{
		TV.ViewId = ++GVulkanImageViewHandleIdCounter;
	}

	if (BindlessHandle.IsValid())
	{
		if (Device.GetBindlessDescriptorManager()->UseDescriptorHeaps())
		{
			const bool bIsDepthStencil = VKHasAnyFlags(ViewDesc.AspectFlags, (VK_IMAGE_ASPECT_DEPTH_BIT | VK_IMAGE_ASPECT_STENCIL_BIT));
			const VkImageDescriptorInfoEXT ImageDescriptorInfo = {
				.sType = VK_STRUCTURE_TYPE_IMAGE_DESCRIPTOR_INFO_EXT,
				.pNext = nullptr,
				.pView = &ViewInfo,
				.layout = (DescriptorType == ERHIDescriptorType::TextureUAV) ? VK_IMAGE_LAYOUT_GENERAL : bIsDepthStencil ? VK_IMAGE_LAYOUT_READ_ONLY_OPTIMAL : VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL
			};

			Device.GetBindlessDescriptorManager()->UpdateDescriptor(Contexts, BindlessHandle, ImageDescriptorInfo, bImmediateUpdate);
		}
		else
		{
			const bool bDepthOrStencilAspect = (ViewDesc.AspectFlags & (VK_IMAGE_ASPECT_DEPTH_BIT | VK_IMAGE_ASPECT_STENCIL_BIT)) != 0;
			Device.GetBindlessDescriptorManager()->UpdateDescriptor(Contexts, BindlessHandle, TV.View, bDepthOrStencilAspect, bImmediateUpdate);
		}
	}
}

void FVulkanView::CreateTextureView(VkImage InImage, const FVulkanTextureViewDesc& ViewDesc)
{
	InitAsTextureView({}, InImage, ViewDesc);
}

void FVulkanView::UpdateTextureView(const FVulkanContextArray& Contexts, VkImage InImage, const FVulkanTextureViewDesc& ViewDesc)
{
	InitAsTextureView(Contexts, InImage, ViewDesc);
}

void FVulkanView::InitAsStructuredBufferView(const FVulkanContextArray& Contexts, FVulkanBuffer* Buffer, uint32 InOffset, uint32 InSize)
{
	// We will need a deferred update if the descriptor was already in use
	const bool bImmediateUpdate = !IsInitialized();

	check(GetViewType() == EType::Null);
	Storage.Emplace<FStructuredBufferView>();
	FStructuredBufferView& SBV = Storage.Get<FStructuredBufferView>();

	const uint32 TotalOffset = Buffer->GetOffset() + InOffset;

	SBV.Buffer = Buffer->GetHandle();
	SBV.HandleId = Buffer->GetCurrentAllocation().HandleId;
	SBV.Offset = TotalOffset;

	// :todo-jn: Volatile buffers use temporary allocations that can be smaller than the buffer creation size.  Check if the savings are still worth it.
	if (Buffer->IsVolatile())
	{
		InSize = FMath::Min<uint64>(InSize, Buffer->GetCurrentSize());
	}

	SBV.Size = InSize;

	Device.GetBindlessDescriptorManager()->UpdateDescriptor(Contexts, BindlessHandle, Buffer->GetHandle(), TotalOffset, InSize, bImmediateUpdate);
}

void FVulkanView::InitAsAccelerationStructureView(const FVulkanContextArray& Contexts, FVulkanBuffer* Buffer, uint32 Offset, uint32 Size)
{
	check(GetViewType() == EType::Null);
	Storage.Emplace<FAccelerationStructureView>();
	FAccelerationStructureView& ASV = Storage.Get<FAccelerationStructureView>();

	const VkAccelerationStructureCreateInfoKHR CreateInfo = {
		.sType = VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_CREATE_INFO_KHR,
		.pNext = nullptr,
		.createFlags = 0,
		.buffer = Buffer->GetHandle(),
		.offset = Buffer->GetOffset() + Offset,
		.size = Size,
		.type = VK_ACCELERATION_STRUCTURE_TYPE_TOP_LEVEL_KHR,
		.deviceAddress = 0
	};
	VERIFYVULKANRESULT(VulkanDynamicAPI::vkCreateAccelerationStructureKHR(Device.GetHandle(), &CreateInfo, VULKAN_CPU_ALLOCATOR, &ASV.Handle));

	if (Device.GetBindlessDescriptorManager()->UseDescriptorHeaps())
	{
		const VkAccelerationStructureDeviceAddressInfoKHR DeviceAddressInfo = {
			.sType = VK_STRUCTURE_TYPE_ACCELERATION_STRUCTURE_DEVICE_ADDRESS_INFO_KHR,
			.pNext = nullptr,
			.accelerationStructure = ASV.Handle
		};

		const VkDeviceAddressRangeEXT DeviceAddressRange = {
			.address = VulkanDynamicAPI::vkGetAccelerationStructureDeviceAddressKHR(Device.GetHandle(), &DeviceAddressInfo),
			.size = 0
		};

		Device.GetBindlessDescriptorManager()->UpdateDescriptor(Contexts, BindlessHandle, DeviceAddressRange, true);
	}
	else
	{
		Device.GetBindlessDescriptorManager()->UpdateDescriptor(Contexts, BindlessHandle, ASV.Handle, true);
	}
}

//
// FVulkanViewableResource

void FVulkanViewableResource::UpdateLinkedViews(const FVulkanContextArray& Contexts)
{
	for (FVulkanLinkedView* View = LinkedViews; View; View = View->Next())
	{
		View->UpdateView(Contexts);
	}
}
