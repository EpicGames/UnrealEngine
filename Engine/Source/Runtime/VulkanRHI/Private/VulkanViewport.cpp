// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanViewport.cpp: Vulkan viewport RHI implementation.
=============================================================================*/

#include "VulkanRHIPrivate.h"
#include "VulkanSwapChain.h"
#include "VulkanPendingState.h"
#include "VulkanContext.h"
#include "VulkanBarriers.h"
#include "GlobalShader.h"
#include "HAL/PlatformAtomics.h"
#include "Engine/RendererSettings.h"
#include "StereoRenderUtils.h"
#include "CommonRenderResources.h"
#include "ScreenRendering.h"
#include "RHIStaticStates.h"

FVulkanBackBuffer::FVulkanBackBuffer(FVulkanDevice& Device, FVulkanViewport* InViewport, EPixelFormat Format, uint32 SizeX, uint32 SizeY, ETextureCreateFlags UEFlags)
	: FVulkanTexture(Device, FRHITextureCreateDesc::Create2D(TEXT("FVulkanBackBuffer"), SizeX, SizeY, Format).SetFlags(UEFlags).SetInitialState(ERHIAccess::Present), VK_NULL_HANDLE, {})
	, Viewport(InViewport)
{
}

void FVulkanBackBuffer::ReleaseAcquiredImage()
{
	if (DefaultView)
	{
		// Do not invalidate view here, just remove a reference to it
		DefaultView = nullptr;
		PartialView = nullptr;
	}

	Image = VK_NULL_HANDLE;
}

void FVulkanBackBuffer::ReleaseViewport()
{
	Viewport = nullptr;
	ReleaseAcquiredImage();
}

void FVulkanBackBuffer::OnGetBackBufferImage(FRHICommandListImmediate& RHICmdList)
{
	check(Viewport);
	if (GVulkanDelayAcquireImage == EDelayAcquireImageType::None)
	{
		FVulkanCommandListContext& Context = (FVulkanCommandListContext&)RHICmdList.GetContext().GetLowestLevelContext();
		AcquireBackBufferImage(Context);
	}
}

void FVulkanBackBuffer::OnAdvanceBackBufferFrame(FRHICommandListImmediate& RHICmdList)
{
	check(Viewport);
	ReleaseAcquiredImage();
}

void FVulkanBackBuffer::AcquireBackBufferImage(FVulkanCommandListContext& Context)
{
	check(Viewport);
	
	if (Image == VK_NULL_HANDLE)
	{
		if (Viewport->TryAcquireImageIndex(Context))
		{
			const int32 AcquiredImageIndex = Viewport->AcquiredImageIndex;
			check(AcquiredImageIndex >= 0 && AcquiredImageIndex < Viewport->BackBufferData.Num());

			FVulkanView* ImageView = Viewport->BackBufferData[AcquiredImageIndex].TextureView.Get();

			Image = ImageView->GetTextureView().Image;
			DefaultView = ImageView;
			PartialView = ImageView;
		}
		else
		{
			// fallback to a 'dummy' backbuffer
			check(Viewport->RenderingBackBuffer);
			FVulkanView* DummyView = Viewport->RenderingBackBuffer->DefaultView;
			Image = DummyView->GetTextureView().Image;
			DefaultView = DummyView;
			PartialView = DummyView;
		}
	}
}

FVulkanBackBuffer::~FVulkanBackBuffer()
{
	check(IsImageOwner() == false);
	// Clear ImageOwnerType so ~FVulkanTexture2D() doesn't try to re-destroy it
	ImageOwnerType = EImageOwnerType::None;
	ReleaseAcquiredImage();
}

FVulkanViewport::FVulkanViewport(FVulkanDevice& InDevice, void* InWindowHandle, uint32 InSizeX, uint32 InSizeY, bool bInIsFullscreen, EPixelFormat InPreferredPixelFormat)
	: Device(InDevice)
	, SizeX(InSizeX)
	, SizeY(InSizeY)
	, bIsFullscreen(bInIsFullscreen)
	, PixelFormat(InPreferredPixelFormat)
	, AcquiredImageIndex(-1)
	, SwapChain(nullptr)
	, WindowHandle(InWindowHandle)
	, PresentCount(0)
{
	check(IsInGameThread());

	static IConsoleVariable* CVarVsync = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VSync"));
	LockToVsync = CVarVsync->GetInt() != 0;

	FVulkanDynamicRHI::Get().Viewports.Add(this);

	// Make sure Instance is created
	FVulkanDynamicRHI::Get().InitInstance();

	FVulkanPlatformWindowContext WindowContext(InWindowHandle);

	ENQUEUE_RENDER_COMMAND(CreateSwapchain)(
		[&WindowContext, VulkanViewport = this](FRHICommandListImmediate& RHICmdList)
		{
			RHICmdList.EnqueueLambda([&WindowContext, VulkanViewport](FRHICommandListBase& ExecutingCmdList)
				{
					VulkanViewport->CreateSwapchain(FVulkanCommandListContext::Get(ExecutingCmdList), nullptr, WindowContext);
				});
			RHICmdList.SubmitAndBlockUntilGPUIdle();
		});
	FlushRenderingCommands();

	if (SupportsStandardSwapchain())
	{
		FCoreDelegates::OnSystemResolutionChanged.AddRaw(this, &FVulkanViewport::OnSystemResolutionChanged);
	}
}

FVulkanViewport::~FVulkanViewport()
{
	RenderingBackBuffer = nullptr;
	
	if (RHIBackBuffer)
	{
		RHIBackBuffer->ReleaseViewport();
		RHIBackBuffer = nullptr;
	}
	
	if (SupportsStandardSwapchain())
	{
		for (int32 Index = 0; Index < BackBufferData.Num(); ++Index)
		{
			FBackbufferImageData& Data = BackBufferData[Index];

			// FIXME: race condition on TransitionAndLayoutManager, could this be called from RT while RHIT is active?
			if (Data.Texture.IsValid())
			{
				Device.NotifyDeletedImage(Data.Texture->Image, true);
			}
		}
		BackBufferData.Empty();

		if (SwapChain)
		{
			SwapChain->Destroy(nullptr);
			delete SwapChain;
			SwapChain = nullptr;
		}

		FCoreDelegates::OnSystemResolutionChanged.RemoveAll(this);
	}

	FVulkanDynamicRHI::Get().Viewports.Remove(this);
}

bool FVulkanViewport::DoCheckedSwapChainJob(FVulkanCommandListContext& Context)
{
	FVulkanQueue* PresentQueue = Device.GetPresentQueue();

	// Present the back buffer to the viewport window.
	auto SwapChainJob = [PresentQueue, Viewport=this]()
	{
		// The swapchain may have been destroyed if the window was closed.
		if (!Viewport->SwapChain)
		{
			return (int32)FVulkanSwapChain::EStatus::Healthy;
		}

		// May happen if swapchain was recreated in DoCheckedSwapChainJob()
		if (Viewport->AcquiredImageIndex == -1)
		{
			// Skip present silently if image has not been acquired
			return (int32)FVulkanSwapChain::EStatus::Healthy;
		}

		return (int32)Viewport->SwapChain->Present(PresentQueue, Viewport->BackBufferData[Viewport->AcquiredImageIndex].RenderingDoneSemaphores);
	};


	int32 AttemptsPending = FVulkanPlatform::RecreateSwapchainOnFail() ? 4 : 0;
	int32 Status = SwapChainJob();

	while (FVulkanPlatformWindowContext::CanCreateSwapchainOnDemand() && Status < 0 && AttemptsPending > 0)
	{
		// Don't attempt to recreate if the window has been destroyed.
		if (!FVulkanPlatform::IsWindowValid(WindowHandle))
		{
			return true;
		}

		if (Status == (int32)FVulkanSwapChain::EStatus::OutOfDate)
		{
			UE_LOGF(LogVulkanRHI, Verbose, "Swapchain is out of date! Trying to recreate the swapchain.");
		}
		else if (Status == (int32)FVulkanSwapChain::EStatus::SurfaceLost)
		{
			UE_LOGF(LogVulkanRHI, Warning, "Swapchain surface lost! Trying to recreate the swapchain.");
		}
		else
		{
			check(0);
		}

		FVulkanPlatformWindowContext WindowContext(WindowHandle);
		RecreateSwapchain(Context, WindowContext);

		// Swapchain creation pushes some commands - flush the command buffers now to begin with a fresh state
		Context.FlushCommands(EVulkanFlushFlags::WaitForCompletion);

		Status = SwapChainJob();

		--AttemptsPending;
	}

	return Status >= 0;
}

bool FVulkanViewport::TryAcquireImageIndex(FVulkanCommandListContext& Context)
{
	if (SwapChain && FVulkanPlatform::IsWindowValid(WindowHandle))
	{
		TRefCountPtr<FVulkanSemaphore> AcquiredSemaphore;
		const int32 Result = SwapChain->AcquireImageIndex(&AcquiredSemaphore);
		if (Result >= 0)
		{
			AcquiredImageIndex = Result;

			// Wait for semaphore signal before writing to backbuffer image
			Context.AddWaitSemaphore(VK_PIPELINE_STAGE_COLOR_ATTACHMENT_OUTPUT_BIT | VK_PIPELINE_STAGE_TRANSFER_BIT, AcquiredSemaphore);

			if (BackBufferData[AcquiredImageIndex].bFirstUse)
			{
				FBackbufferImageData& Data = BackBufferData[AcquiredImageIndex];
				Data.bFirstUse = false;

				VkClearColorValue ClearColor;
				FMemory::Memzero(ClearColor);

				const VkImage BackbufferImage = Data.Texture->Image;
				const VkImageSubresourceRange Range = FVulkanPipelineBarrier::MakeSubresourceRange(VK_IMAGE_ASPECT_COLOR_BIT);

				FVulkanCommandBuffer& CommandBuffer = Context.GetCommandBuffer();
				VulkanSetImageLayout(&CommandBuffer, BackbufferImage, VK_IMAGE_LAYOUT_UNDEFINED, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL, Range);
				VulkanRHI::vkCmdClearColorImage(CommandBuffer.GetHandle(), BackbufferImage, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL, &ClearColor, 1, &Range);
				VulkanSetImageLayout(&CommandBuffer, BackbufferImage, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL, VK_IMAGE_LAYOUT_PRESENT_SRC_KHR, Range);
			}

			return true;
		}
	}
	return false;
}

FTextureRHIRef FVulkanViewport::GetBackBuffer(FRHICommandListImmediate& RHICmdList)
{
	check(IsInRenderingThread());

	// make sure we aren't in the middle of swapchain recreation (which can happen on e.g. RHI thread)
	FScopeLock LockSwapchain(&RecreatingSwapchain);

	if (SupportsStandardSwapchain() && GVulkanDelayAcquireImage != EDelayAcquireImageType::DelayAcquire)
	{
		check(RHIBackBuffer);
		
		RHICmdList.EnqueueLambda([this](FRHICommandListImmediate& CmdList)
		{
			this->RHIBackBuffer->OnGetBackBufferImage(CmdList);
		});

		return RHIBackBuffer.GetReference();
	}
	
	return RenderingBackBuffer.GetReference();
}

void FVulkanViewport::AdvanceBackBufferFrame(FRHICommandListImmediate& RHICmdList, bool bPresent)
{
	check(IsInRenderingThread());

	if (SupportsStandardSwapchain() && GVulkanDelayAcquireImage != EDelayAcquireImageType::DelayAcquire && bPresent)
	{
		check(RHIBackBuffer);
		
		RHICmdList.EnqueueLambda([this](FRHICommandListImmediate& CmdList)
		{
			this->RHIBackBuffer->OnAdvanceBackBufferFrame(CmdList);
		});
	}
}

void FVulkanViewport::WaitForFrameEventCompletion()
{
	if (FVulkanPlatform::RequiresWaitingForFrameCompletionEvent())
	{
		static FCriticalSection CS;
		FScopeLock ScopeLock(&CS);
		if (LastFrameSyncPoint.IsValid())
		{
			// If last frame's fence hasn't been signaled already, wait for it here
			if (!LastFrameSyncPoint->IsComplete())
			{
				FVulkanDynamicRHI::Get().ProcessInterruptQueueUntil(LastFrameSyncPoint);
			}
		}
	}
}

void FVulkanViewport::IssueFrameEvent()
{
	if (FVulkanPlatform::RequiresWaitingForFrameCompletionEvent())
	{
		FVulkanCommandListContext& ImmediateContext = Device.GetImmediateContext();
		LastFrameSyncPoint = ImmediateContext.GetContextSyncPoint();
		ImmediateContext.FlushCommands();
	}
}


// Tear down and recreate swapchain and related resources.
void FVulkanViewport::RecreateSwapchain(FVulkanCommandListContext& Context, FVulkanPlatformWindowContext& WindowContext)
{
	// Make sure everything is submitted and submission queue is idle
	Context.FlushCommands(EVulkanFlushFlags::WaitForCompletion);

	FScopeLock LockSwapchain(&RecreatingSwapchain);

	FVulkanSwapChainRecreateInfo RecreateInfo = { VK_NULL_HANDLE, VK_NULL_HANDLE };
	DestroySwapchain(&RecreateInfo);
	CreateSwapchain(Context, &RecreateInfo, WindowContext);
	check(RecreateInfo.Surface == VK_NULL_HANDLE);
	check(RecreateInfo.SwapChain == VK_NULL_HANDLE);
}

void FVulkanDynamicRHI::RHITick(float DeltaTime)
{
	check(IsInGameThread());
	for (FVulkanViewport* Viewport : Viewports)
	{
		Viewport->CheckSwapChain();
	}
}

void FVulkanViewport::CheckSwapChain()
{
	check(IsInGameThread());

	// Honor platform-side recreate requests (Linux uses this after a Hide/Show cycle
	// Vulkan WSI's OUT_OF_DATE detection across xdg role destroy/recreate is unreliable).
	// Must NOT gate on SwapChain FVulkanSwapChain::Create can leave it null after a
	// graceful surface-lost bail, and we still need to recreate when the platform asks.
	const bool bPlatformRequestsRecreate =
		FVulkanPlatform::TryConsumeSwapchainRecreateRequest(WindowHandle);

	const bool bVsyncChanged =
		SwapChain && FPlatformAtomics::AtomicRead(&LockToVsync) != SwapChain->DoesLockToVsync();

	if (bPlatformRequestsRecreate || bVsyncChanged)
	{
		FVulkanPlatformWindowContext WindowContext(WindowHandle);

		ENQUEUE_RENDER_COMMAND(UpdateVsync)(
			[this, &WindowContext](FRHICommandListImmediate& RHICmdList)
		{
			RecreateSwapchainFromRT(RHICmdList, PixelFormat, WindowContext);
		});
		FlushRenderingCommands();
	}
}

void FVulkanViewport::Resize(FRHICommandListImmediate& RHICmdList, uint32 InSizeX, uint32 InSizeY, bool bInIsFullscreen, EPixelFormat PreferredPixelFormat, FVulkanPlatformWindowContext& WindowContext)
{
	check(IsInRenderingThread());

	RHICmdList.EnqueueLambda([this, InSizeX, InSizeY, bInIsFullscreen, PreferredPixelFormat](FRHICommandListBase& ExecutingCmdList)
	{
		SizeX = InSizeX;
		SizeY = InSizeY;
		bIsFullscreen = bInIsFullscreen;
		PixelFormat = PreferredPixelFormat;
	});
	RecreateSwapchainFromRT(RHICmdList, PreferredPixelFormat, WindowContext);
}

void FVulkanViewport::RecreateSwapchainFromRT(FRHICommandListImmediate& RHICmdList, EPixelFormat PreferredPixelFormat, FVulkanPlatformWindowContext& WindowContext)
{
	check(IsInRenderingThread());

	RHICmdList.EnqueueLambda([this, PreferredPixelFormat, &WindowContext](FRHICommandListBase& ExecutingCmdList)
	{
		// Skip swapchain recreation if the window has been destroyed or is no longer valid.
		if (!FVulkanPlatform::IsWindowValid(WindowHandle))
		{
			return;
		}

		FVulkanSwapChainRecreateInfo RecreateInfo = { VK_NULL_HANDLE, VK_NULL_HANDLE };
		DestroySwapchain(&RecreateInfo);
		PixelFormat = PreferredPixelFormat;
		CreateSwapchain(FVulkanCommandListContext::Get(ExecutingCmdList), &RecreateInfo, WindowContext);
		check(RecreateInfo.Surface == VK_NULL_HANDLE);
		check(RecreateInfo.SwapChain == VK_NULL_HANDLE);
	});

	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
}

void FVulkanViewport::InitImages(TConstArrayView<VkImage> Images)
{
	const FVulkanTextureViewDesc ViewDesc =
	{
		.ViewType = VK_IMAGE_VIEW_TYPE_2D,
		.AspectFlags = VK_IMAGE_ASPECT_COLOR_BIT,
		.UEFormat = PixelFormat,
		.Format = UEToVkTextureFormat(PixelFormat, false),
		.FirstMip = 0,
		.NumMips = 1,
		.ArraySliceIndex = 0,
		.NumArraySlices = 1
	};

	BackBufferData.SetNum(Images.Num());

	for (int32 Index = 0; Index < Images.Num(); ++Index)
	{
		FBackbufferImageData& Data = BackBufferData[Index];

		Data.bFirstUse = true;
		if (!Data.RenderingDoneSemaphores.IsValid())
		{
			// Consumed by vkQueuePresentKHR rather than by a payload wait.
			Data.RenderingDoneSemaphores = new FVulkanSemaphore(Device, EVulkanSemaphoreFlags::ExternallyWaited);
		}

		const VkSurfaceTransformFlagBitsKHR CachedSurfaceTransform = SwapChain->GetCachedSurfaceTransform();
		const bool bSwapImageDimensions = (CachedSurfaceTransform == VK_SURFACE_TRANSFORM_ROTATE_90_BIT_KHR || CachedSurfaceTransform == VK_SURFACE_TRANSFORM_ROTATE_270_BIT_KHR);
		const uint32 ImageSizeX = bSwapImageDimensions ? SizeY : SizeX;
		const uint32 ImageSizeY = bSwapImageDimensions ? SizeX : SizeY;
		Data.Texture = (FVulkanTexture*)FVulkanDynamicRHI::Get().RHICreateTexture2DFromResource(PixelFormat, ImageSizeX, ImageSizeY, 1, 1, Images[Index], ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::Presentable).GetReference();

		Data.TextureView = MakeUnique<FVulkanView>(Device, ERHIDescriptorType::TextureSRV);
		Data.TextureView->CreateTextureView(Images[Index], ViewDesc);

#if VULKAN_ENABLE_DRAW_MARKERS
		if (Device.GetSetDebugName())
		{
			VulkanRHI::SetDebugName(Device.GetSetDebugName(), Device.GetHandle(), Data.Texture->Image, "VulkanBackBuffer");
		}
#endif
	}
}

void FVulkanViewport::CreateSwapchain(FVulkanCommandListContext& Context, FVulkanSwapChainRecreateInfo* RecreateInfo, FVulkanPlatformWindowContext& WindowContext)
{
	// Release a previous swapchain 'dummy' and a real backbuffer if any
	RenderingBackBuffer = nullptr;
	RHIBackBuffer = nullptr;

	if (SupportsStandardSwapchain())
	{
		check(SwapChain == nullptr);

		if (WindowContext.IsValid())
		{
			uint32 DesiredNumBackBuffers = NumRequestedSwapchainImages;
			TArray<VkImage> Images;
			SwapChain = FVulkanSwapChain::Create(
				FVulkanDynamicRHI::Get().Instance, Device,
				PixelFormat, SizeX, SizeY, bIsFullscreen,
				&DesiredNumBackBuffers,
				Images,
				LockToVsync,
				WindowContext,
				RecreateInfo
			);

			if (SwapChain)
			{
				ensureMsgf(Images.Num() >= NumRequestedSwapchainImages,
					TEXT("SwapChain Images: Requested Count = %i images , actual count: %i , previous count: %i"), 
					NumRequestedSwapchainImages, Images.Num(), BackBufferData.Num());

				InitImages(Images);
			}
		}

		RHIBackBuffer = new FVulkanBackBuffer(Device, this, PixelFormat, SizeX, SizeY, TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_ResolveTargetable);
	}
	else
	{
		PixelFormat = GetPixelFormatForNonDefaultSwapchain();
		if (RecreateInfo != nullptr)
		{
			if (RecreateInfo->SwapChain)
			{
				FVulkanPlatform::DestroySwapchainKHR(Device.GetHandle(), RecreateInfo->SwapChain, VULKAN_CPU_ALLOCATOR);
				RecreateInfo->SwapChain = VK_NULL_HANDLE;
			}
			if (RecreateInfo->Surface)
			{
				FVulkanPlatform::DestroySurface(WindowContext.GetWindowId(), FVulkanDynamicRHI::Get().Instance, RecreateInfo->Surface);
				RecreateInfo->Surface = VK_NULL_HANDLE;
			}
		}
	}

	// We always create a 'dummy' backbuffer to gracefully handle SurfaceLost cases
	{
		const uint32 BackBufferSizeX = RequiresRenderingBackBuffer() ? SizeX : 1;
		const uint32 BackBufferSizeY = RequiresRenderingBackBuffer() ? SizeY : 1;

		const UE::StereoRenderUtils::FStereoShaderAspects Aspects(GMaxRHIShaderPlatform);

		const int kMultiViewCount = 2; // TODO: number of subresources may change in the future
		const FRHITextureCreateDesc CreateDesc = (Aspects.IsMobileMultiViewEnabled() ?			
			FRHITextureCreateDesc::Create2DArray(TEXT("RenderingBackBufferArr"), BackBufferSizeX, BackBufferSizeY, kMultiViewCount, PixelFormat) :
			FRHITextureCreateDesc::Create2D(TEXT("RenderingBackBuffer"), BackBufferSizeX, BackBufferSizeY, PixelFormat))
			.SetClearValue(FClearValueBinding::None)
			.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource | ETextureCreateFlags::ResolveTargetable)
			.SetInitialState(ERHIAccess::Present);


		RenderingBackBuffer = FVulkanDynamicRHI::Get().CreateTextureInternal(CreateDesc, {});

		FVulkanPipelineBarrier Barrier;
		Barrier.AddImageLayoutTransition(RenderingBackBuffer->Image, VK_IMAGE_LAYOUT_UNDEFINED, VK_IMAGE_LAYOUT_PRESENT_SRC_KHR, FVulkanPipelineBarrier::MakeSubresourceRange(VK_IMAGE_ASPECT_COLOR_BIT, 0, 1));
		Barrier.Execute(&Context.GetCommandBuffer());

#if VULKAN_ENABLE_DRAW_MARKERS
		if (Device.GetSetDebugName())
		{
			VulkanRHI::SetDebugName(Device.GetSetDebugName(), Device.GetHandle(), RenderingBackBuffer->Image, "RenderingBackBuffer");
		}
#endif
	}

	AcquiredImageIndex = -1;
}

void FVulkanViewport::DestroySwapchain(FVulkanSwapChainRecreateInfo* RecreateInfo)
{
	Device.WaitUntilIdle();

	// Intentionally leave RenderingBackBuffer alive, so it can be used a dummy backbuffer while we don't have swapchain images
	// RenderingBackBuffer = nullptr;
	
	if (RHIBackBuffer)
	{
		RHIBackBuffer->ReleaseAcquiredImage();
		// We release this RHIBackBuffer when we create a new swapchain
	}
		
	if (SupportsStandardSwapchain() && SwapChain)
	{
		for (int32 Index = 0; Index < BackBufferData.Num(); ++Index)
		{
			FBackbufferImageData& Data = BackBufferData[Index];
			if (Data.Texture.IsValid())
			{
				Device.NotifyDeletedImage(Data.Texture->Image, true);
				Data.Texture = nullptr;
			}
			Data.TextureView = nullptr;
		}
		
		Device.GetDeferredDeletionQueue().ReleaseResources(true);

		SwapChain->Destroy(RecreateInfo);
		delete SwapChain;
		SwapChain = nullptr;

		Device.GetDeferredDeletionQueue().ReleaseResources(true);

		BackBufferData.Empty();
	}

	AcquiredImageIndex = -1;
}

inline static void CopyImageToBackBuffer(FVulkanCommandListContext& Context, FVulkanTexture& SrcSurface, FVulkanTexture& DstSurface, int32 SizeX, int32 SizeY, int32 WindowSizeX, int32 WindowSizeY, VkSurfaceTransformFlagBitsKHR CachedSurfaceTransform)
{
	const bool bNeedsVulkanPreTransform = CachedSurfaceTransform != VK_SURFACE_TRANSFORM_IDENTITY_BIT_KHR;

	FVulkanCommandBuffer* CmdBuffer = &Context.GetCommandBuffer();
	check(CmdBuffer->IsOutsideRenderPass());

	const VkImageLayout SrcSurfaceLayout = bNeedsVulkanPreTransform ? VK_IMAGE_LAYOUT_SHADER_READ_ONLY_OPTIMAL : VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL;
	const VkImageLayout DstSurfaceLayout = bNeedsVulkanPreTransform ? VK_IMAGE_LAYOUT_COLOR_ATTACHMENT_OPTIMAL : VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL;

	{
		FVulkanPipelineBarrier Barrier;
		Barrier.AddImageLayoutTransition(SrcSurface.Image, VK_IMAGE_LAYOUT_PRESENT_SRC_KHR, SrcSurfaceLayout, FVulkanPipelineBarrier::MakeSubresourceRange(VK_IMAGE_ASPECT_COLOR_BIT, 0, 1));
		Barrier.AddImageLayoutTransition(DstSurface.Image, VK_IMAGE_LAYOUT_UNDEFINED, DstSurfaceLayout, FVulkanPipelineBarrier::MakeSubresourceRange(VK_IMAGE_ASPECT_COLOR_BIT, 0, 1));
		Barrier.Execute(CmdBuffer);
	}

	VulkanRHI::DebugHeavyWeightBarrier(CmdBuffer->GetHandle(), 32);

	// Copy and rotate the intermediate image to the BackBuffer with a pixel shader
	if (bNeedsVulkanPreTransform)
	{
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		// No alpha blending, no depth tests or writes, no stencil tests or writes, no backface culling.
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		TRHICommandList_RecursiveHazardous<FVulkanCommandListContext> RHICmdList(&Context);

		RHICmdList.BeginRenderPass(FRHIRenderPassInfo(&DstSurface, ERenderTargetActions::DontLoad_Store), TEXT("SurfaceTransform"));

		auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FImagePreTransformVS> VertexShader(ShaderMap);
		TShaderMapRef<FScreenPS> PixelShader(ShaderMap);

		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		FImagePreTransformVS::FParameters VSParameters;
		FMatrix44f RenderPassTransformMatrix = FRotationMatrix44f(FRotator3f(0.0f, -180.0f * (FMath::Log2(static_cast<float>(CachedSurfaceTransform))) / 2, 0.0f));
		VSParameters.PreTransform.X = RenderPassTransformMatrix.M[0][0];
		VSParameters.PreTransform.Y = RenderPassTransformMatrix.M[0][1];
		VSParameters.PreTransform.Z = RenderPassTransformMatrix.M[1][0];
		VSParameters.PreTransform.W = RenderPassTransformMatrix.M[1][1];

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSParameters);
		SetShaderParametersLegacyPS(RHICmdList, PixelShader, TStaticSamplerState<SF_Point>::GetRHI(), &SrcSurface);

		RHICmdList.DrawPrimitive(0, 2, 1);

		RHICmdList.EndRenderPass();
	}
	else
	{
		if (SizeX != WindowSizeX || SizeY != WindowSizeY)
		{
			VkImageBlit Region;
			FMemory::Memzero(Region);
			Region.srcOffsets[0].x = 0;
			Region.srcOffsets[0].y = 0;
			Region.srcOffsets[0].z = 0;
			Region.srcOffsets[1].x = SizeX;
			Region.srcOffsets[1].y = SizeY;
			Region.srcOffsets[1].z = 1;
			Region.srcSubresource.aspectMask = VK_IMAGE_ASPECT_COLOR_BIT;
			Region.srcSubresource.layerCount = 1;
			Region.dstOffsets[0].x = 0;
			Region.dstOffsets[0].y = 0;
			Region.dstOffsets[0].z = 0;
			Region.dstOffsets[1].x = WindowSizeX;
			Region.dstOffsets[1].y = WindowSizeY;
			Region.dstOffsets[1].z = 1;
			Region.dstSubresource.aspectMask = VK_IMAGE_ASPECT_COLOR_BIT;
			Region.dstSubresource.baseArrayLayer = 0;
			Region.dstSubresource.layerCount = 1;
			VulkanRHI::vkCmdBlitImage(CmdBuffer->GetHandle(),
				SrcSurface.Image, VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL,
				DstSurface.Image, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL,
				1, &Region, VK_FILTER_LINEAR);
		}
		else
		{
			VkImageCopy Region;
			FMemory::Memzero(Region);
			Region.extent.width = SizeX;
			Region.extent.height = SizeY;
			Region.extent.depth = 1;
			Region.srcSubresource.aspectMask = VK_IMAGE_ASPECT_COLOR_BIT;
			//Region.srcSubresource.baseArrayLayer = 0;
			Region.srcSubresource.layerCount = 1;
			//Region.srcSubresource.mipLevel = 0;
			Region.dstSubresource.aspectMask = VK_IMAGE_ASPECT_COLOR_BIT;
			//Region.dstSubresource.baseArrayLayer = 0;
			Region.dstSubresource.layerCount = 1;
			//Region.dstSubresource.mipLevel = 0;
			VulkanRHI::vkCmdCopyImage(CmdBuffer->GetHandle(),
				SrcSurface.Image, VK_IMAGE_LAYOUT_TRANSFER_SRC_OPTIMAL,
				DstSurface.Image, VK_IMAGE_LAYOUT_TRANSFER_DST_OPTIMAL,
				1, &Region);
		}
	}

	{
		FVulkanPipelineBarrier Barrier;
		Barrier.AddImageLayoutTransition(SrcSurface.Image, SrcSurfaceLayout, VK_IMAGE_LAYOUT_PRESENT_SRC_KHR, FVulkanPipelineBarrier::MakeSubresourceRange(VK_IMAGE_ASPECT_COLOR_BIT, 0, 1));
		Barrier.AddImageLayoutTransition(DstSurface.Image, DstSurfaceLayout, VK_IMAGE_LAYOUT_PRESENT_SRC_KHR, FVulkanPipelineBarrier::MakeSubresourceRange(VK_IMAGE_ASPECT_COLOR_BIT, 0, 1));
		Barrier.Execute(CmdBuffer);
	}
}

bool FVulkanViewport::Present(FVulkanCommandListContext& Context, const FRHIPresentArgs& InPresentArgs)
{
	check(Context.IsImmediate());

#if PLATFORM_ANDROID && USE_ANDROID_STANDALONE
	if (FVulkanPlatform::IsWaitingForWindowCreated())
	{
		UE_LOGF(LogVulkanRHI, Log, "Present: skipping frame, waiting for WINDOW_CREATED");
		return true;
	}
#endif

	if (!InPresentArgs.bPresent)
	{
		return false;
	}

	FPlatformAtomics::AtomicStore(&LockToVsync, InPresentArgs.bLockToVsync ? 1 : 0);

	if (PresentBeginLatencyCallback)
	{
		PresentBeginLatencyCallback();
	}

	//Transition back buffer to presentable and submit that command
	if (SupportsStandardSwapchain())
	{
		bool bFailedToDelayAcquireBackbuffer = false;

		if (GVulkanDelayAcquireImage == EDelayAcquireImageType::DelayAcquire && RenderingBackBuffer)
		{
			SCOPE_CYCLE_COUNTER(STAT_VulkanAcquireBackBuffer);
			// swapchain can go out of date, do not crash at this point
			if (LIKELY(TryAcquireImageIndex(Context)))
			{
				const uint32 WindowSizeX = FMath::Min(SizeX, SwapChain->InternalWidth);
				const uint32 WindowSizeY = FMath::Min(SizeY, SwapChain->InternalHeight);

				FBackbufferImageData& Data = BackBufferData[AcquiredImageIndex];
				CopyImageToBackBuffer(Context, *RenderingBackBuffer.GetReference(), *Data.Texture.GetReference(), SizeX, SizeY, WindowSizeX, WindowSizeY, SwapChain->GetCachedSurfaceTransform());
			}
			else
			{
				bFailedToDelayAcquireBackbuffer = true;
			}
		}
		else
		{
			if (AcquiredImageIndex != -1)
			{
				FBackbufferImageData& Data = BackBufferData[AcquiredImageIndex];
				FVulkanCommandBuffer& CommandBuffer = Context.GetCommandBuffer();
				check(CommandBuffer.IsOutsideRenderPass());
				check(RHIBackBuffer != nullptr && RHIBackBuffer->Image == Data.Texture->Image);
				VulkanSetImageLayout(&CommandBuffer, Data.Texture->Image, VK_IMAGE_LAYOUT_UNDEFINED, VK_IMAGE_LAYOUT_PRESENT_SRC_KHR, FVulkanPipelineBarrier::MakeSubresourceRange(VK_IMAGE_ASPECT_COLOR_BIT));
			}
			else
			{
				// When we have failed to acquire backbuffer image we fallback to using 'dummy' backbuffer
				check(RHIBackBuffer != nullptr && RHIBackBuffer->Image == RenderingBackBuffer->Image);
			}
		}

		if (LIKELY(!bFailedToDelayAcquireBackbuffer))
		{
			if (AcquiredImageIndex >= 0)
			{
				Context.AddSignalSemaphore(BackBufferData[AcquiredImageIndex].RenderingDoneSemaphores);
			}
		}
		else
		{
			if (FVulkanPlatformWindowContext::CanCreateSwapchainOnDemand()) // on android we dont want to attempt to recreate the window when we dont have the window lock...
			{
				// failing to do the delayacquire can only happen if we were in this mode to begin with
				check(GVulkanDelayAcquireImage == EDelayAcquireImageType::DelayAcquire);

				// Don't attempt to recreate the swapchain if the window has been destroyed.
				if (!FVulkanPlatform::IsWindowValid(WindowHandle))
				{
					return true;
				}

				UE_LOGF(LogVulkanRHI, Log, "AcquireNextImage() failed due to the outdated swapchain, not even attempting to present.");

				FVulkanPlatformWindowContext WindowContext(GetWindowHandle());
				RecreateSwapchain(Context, WindowContext);

				// Swapchain creation pushes some commands - flush the command buffers now to begin with a fresh state
				Context.FlushCommands(EVulkanFlushFlags::WaitForCompletion);
			}

			// early exit
			return (int32)FVulkanSwapChain::EStatus::Healthy;
		}
	}

	const bool bHasCustomPresent = IsValidRef(CustomPresent);

	if (bHasCustomPresent)
	{
		// If we don't do the actual Present call (e.g. XR device rendering to an offscreen
		// compositor buffer), we need to manually set the end of the frame marker
		// for tools like RenderDoc to be able to judge when the frame has ended before we submit
		// the active command buffer below
		Context.InsertEndFrameDebugLabel();
	}

	// Submit any accumulated commands or syncs, wait until they hit the queue so that we can present
	Context.FlushCommands(EVulkanFlushFlags::WaitForSubmission);

	//#todo-rco: Proper SyncInterval bLockToVsync ? RHIConsoleVariables::SyncInterval : 0
	int32 SyncInterval = 0;
	bool bNeedNativePresent = true;

	if (bHasCustomPresent)
	{
		SCOPE_CYCLE_COUNTER(STAT_VulkanCustomPresentTime);
		bNeedNativePresent = CustomPresent->Present(this, Context, SyncInterval);
	}

	bool bResult = false;
	if (bNeedNativePresent && (!SupportsStandardSwapchain() || GVulkanDelayAcquireImage == EDelayAcquireImageType::DelayAcquire || RHIBackBuffer != nullptr))
	{
		// Skip present if the native window has been destroyed. On Wayland, presenting
		// to a destroyed surface crashes in the driver's wl_proxy_marshal.
		if (!FVulkanPlatform::IsWindowValid(WindowHandle))
		{
			AcquiredImageIndex = -1;
			if (SwapChain)
			{
				SwapChain->ResetCurrentImageIndex();
			}
			return false;
		}

		if (SupportsStandardSwapchain() && !DoCheckedSwapChainJob(Context))
		{
			UE_LOGF(LogVulkanRHI, Error, "Swapchain present failed!");
			bResult = false;
		}
		else
		{
			bResult = true;
		}

		if (bHasCustomPresent)
		{
			CustomPresent->PostPresent();
		}
	}

	if (PresentEndLatencyCallback)
	{
		PresentEndLatencyCallback();
	}

	if (FVulkanPlatform::RequiresWaitingForFrameCompletionEvent() && !bHasCustomPresent)
	{
		// Wait for the GPU to finish rendering the previous frame before finishing this frame.
		WaitForFrameEventCompletion();
		IssueFrameEvent();
	}

	AcquiredImageIndex = -1;

	++PresentCount;
	++GVulkanRHI->TotalPresentCount;

	return bResult;
}

bool FVulkanViewport::SupportsStandardSwapchain()
{
	return !FVulkanDynamicRHI::Get().bIsStandaloneStereoDevice;
}

bool FVulkanViewport::RequiresRenderingBackBuffer()
{
	return !FVulkanDynamicRHI::Get().bIsStandaloneStereoDevice;
}

EPixelFormat FVulkanViewport::GetPixelFormatForNonDefaultSwapchain()
{
	if (FVulkanDynamicRHI::Get().bIsStandaloneStereoDevice)
	{
		return PF_R8G8B8A8;
	}
	else
	{
		checkf(0, TEXT("Platform Requires Standard Swapchain!"));
		return PF_Unknown;
	}
}

void FVulkanViewport::OnSystemResolutionChanged(uint32 ResX, uint32 ResY)
{
	EDeviceScreenOrientation CurrentOrientation = FPlatformMisc::GetDeviceOrientation();

	// The swap chain needs to be recreated after a rotation
	// Only 180-degree rotations need to be handled here because 90-degree rotations will resize the viewport and recreate the swap chain.
	if ((CachedOrientation == EDeviceScreenOrientation::Portrait && CurrentOrientation == EDeviceScreenOrientation::PortraitUpsideDown)
		|| (CachedOrientation == EDeviceScreenOrientation::PortraitUpsideDown && CurrentOrientation == EDeviceScreenOrientation::Portrait)
		|| (CachedOrientation == EDeviceScreenOrientation::LandscapeRight && CurrentOrientation == EDeviceScreenOrientation::LandscapeLeft)
		|| (CachedOrientation == EDeviceScreenOrientation::LandscapeLeft && CurrentOrientation == EDeviceScreenOrientation::LandscapeRight))
	{
		check(IsInGameThread());
		FVulkanPlatformWindowContext WindowContext(GetWindowHandle());

		ENQUEUE_RENDER_COMMAND(RecreateSwapchain)(
			[this,&WindowContext](FRHICommandListImmediate& RHICmdList)
			{
				RecreateSwapchainFromRT(RHICmdList, WindowContext);
			});
		FlushRenderingCommands();
	}
	CachedOrientation = CurrentOrientation;
}

/*=============================================================================
 *	The following RHI functions must be called from the main thread.
 *=============================================================================*/
FViewportRHIRef FVulkanDynamicRHI::RHICreateViewport(void* WindowHandle, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat)
{
	check(IsInGameThread());
	return new FVulkanViewport(*Device, WindowHandle, SizeX, SizeY, bIsFullscreen, PixelFormat);
}

void FVulkanDynamicRHI::RHIResizeViewport(FRHIViewport* ViewportRHI, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat)
{
	check(IsInGameThread());
	FVulkanViewport* Viewport = ResourceCast(ViewportRHI);

	if (Viewport->GetSizeXY() != FIntPoint(SizeX, SizeY) || Viewport->IsFullscreen() != bIsFullscreen || Viewport->PixelFormat != PixelFormat)
	{
		FVulkanPlatformWindowContext WindowContext(Viewport->WindowHandle);

		ENQUEUE_RENDER_COMMAND(ResizeViewport)(
			[Viewport, SizeX, SizeY, bIsFullscreen, PixelFormat, &WindowContext](FRHICommandListImmediate& RHICmdList)
			{
				Viewport->Resize(RHICmdList, SizeX, SizeY, bIsFullscreen, PixelFormat, WindowContext);
			});
		FlushRenderingCommands();
	}
}

FTextureRHIRef FVulkanDynamicRHI::RHIGetViewportBackBuffer(FRHIViewport* ViewportRHI)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	check(ViewportRHI);
	FVulkanViewport* Viewport = ResourceCast(ViewportRHI);

	return Viewport->GetBackBuffer(RHICmdList);
}

void FVulkanCommandListContext::RHISetViewport(float MinX, float MinY, float MinZ, float MaxX, float MaxY, float MaxZ)
{
	PendingGfxState->SetViewport(MinX, MinY, MinZ, MaxX, MaxY, MaxZ);
}

void FVulkanCommandListContext::RHISetStereoViewport(float LeftMinX, float RightMinX, float LeftMinY, float RightMinY, float MinZ, float LeftMaxX, float RightMaxX, float LeftMaxY, float RightMaxY, float MaxZ)
{
	TStaticArray<VkViewport, 2> Viewports;

	Viewports[0].x = FMath::FloorToInt(LeftMinX);
	Viewports[0].y = FMath::FloorToInt(LeftMinY);
	Viewports[0].width = FMath::CeilToInt(LeftMaxX - LeftMinX);
	Viewports[0].height = FMath::CeilToInt(LeftMaxY - LeftMinY);
	Viewports[0].minDepth = MinZ;
	Viewports[0].maxDepth = MaxZ;

	Viewports[1].x = FMath::FloorToInt(RightMinX);
	Viewports[1].y = FMath::FloorToInt(RightMinY);
	Viewports[1].width = FMath::CeilToInt(RightMaxX - RightMinX);
	Viewports[1].height = FMath::CeilToInt(RightMaxY - RightMinY);
	Viewports[1].minDepth = MinZ;
	Viewports[1].maxDepth = MaxZ;

	PendingGfxState->SetMultiViewport(Viewports);
}

void FVulkanCommandListContext::RHISetMultipleViewports(uint32 Count, const FViewportBounds* Data)
{
	VULKAN_SIGNAL_UNIMPLEMENTED();
}

void FVulkanCommandListContext::RHISetScissorRect(bool bEnable, uint32 MinX, uint32 MinY, uint32 MaxX, uint32 MaxY)
{
	PendingGfxState->SetScissor(bEnable, MinX, MinY, MaxX, MaxY);
}
