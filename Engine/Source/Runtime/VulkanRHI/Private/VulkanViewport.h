// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanViewport.h: Vulkan viewport RHI definitions.
=============================================================================*/

#pragma once

#include "VulkanResources.h"
#include "HAL/CriticalSection.h"

class FVulkanDynamicRHI;
class FVulkanSwapChain;
class FVulkanQueue;
class FVulkanViewport;
class FVulkanSemaphore;

namespace VulkanRHI
{
	class FSemaphore;
}

class FVulkanBackBuffer : public FVulkanTexture
{
public:
	FVulkanBackBuffer(FVulkanDevice& Device, FVulkanViewport* InViewport, EPixelFormat Format, uint32 SizeX, uint32 SizeY, ETextureCreateFlags UEFlags);
	virtual ~FVulkanBackBuffer();
	
	void OnGetBackBufferImage(FRHICommandListImmediate& RHICmdList);
	void OnAdvanceBackBufferFrame(FRHICommandListImmediate& RHICmdList);

	void ReleaseViewport();
	void ReleaseAcquiredImage();

private:
	void AcquireBackBufferImage(FVulkanCommandListContext& Context);

private:
	FVulkanViewport* Viewport;
};


class FVulkanViewport : public FRHIViewport
{
public:
	FVulkanViewport(FVulkanDevice& InDevice, void* InWindowHandle, uint32 InSizeX,uint32 InSizeY,bool bInIsFullscreen, EPixelFormat InPreferredPixelFormat);
	~FVulkanViewport();

	FTextureRHIRef GetBackBuffer(FRHICommandListImmediate& RHICmdList);
	void AdvanceBackBufferFrame(FRHICommandListImmediate& RHICmdList, bool bPresent);

	virtual void WaitForFrameEventCompletion() override;

	virtual void IssueFrameEvent() override;

	inline FIntPoint GetSizeXY() const
	{
		return FIntPoint(SizeX, SizeY);
	}

	virtual void SetCustomPresent(FRHICustomPresent* InCustomPresent) override final
	{
		CustomPresent = InCustomPresent;
	}

	virtual FRHICustomPresent* GetCustomPresent() const override final
	{
		return CustomPresent;
	}

	void CheckSwapChain();
	
	bool Present(FVulkanCommandListContext& Context, const FRHIPresentArgs& InPresentArgs);

	inline uint32 GetPresentCount() const
	{
		return PresentCount;
	}

	inline bool IsFullscreen() const
	{
		return bIsFullscreen;
	}

	inline uint32 GetBackBufferImageCount()
	{
		return (uint32)BackBufferData.Num();
	}

	inline VkImage GetBackBufferImage(uint32 Index)
	{
		if (BackBufferData.Num() > 0)
		{
			return BackBufferData[Index].Texture->Image;
		}
		else
		{
			return VK_NULL_HANDLE;
		}
	}

	inline FVulkanSwapChain* GetSwapChain()
	{
		return SwapChain;
	}

	inline void* GetWindowHandle() { return WindowHandle; }

	void DestroySwapchain(struct FVulkanSwapChainRecreateInfo* RecreateInfo);
	void RecreateSwapchain(FVulkanCommandListContext& Context, FVulkanPlatformWindowContext& WindowContext);

	// This is the number of images we're aiming for, driver might return more
	constexpr static uint32 NumRequestedSwapchainImages = 3;

protected:
	FVulkanDevice& Device;

	struct FBackbufferImageData
	{
		TRefCountPtr<FVulkanTexture> Texture;
		TRefCountPtr<FVulkanSemaphore> RenderingDoneSemaphores;
		TUniquePtr<FVulkanView> TextureView;
		bool bFirstUse = true;
	};
	TArray<FBackbufferImageData, TInlineAllocator<NumRequestedSwapchainImages*2>> BackBufferData;
	TRefCountPtr<FVulkanBackBuffer> RHIBackBuffer;

	// 'Dummy' back buffer
	TRefCountPtr<FVulkanTexture>	RenderingBackBuffer;
	
	/** narrow-scoped section that locks access to back buffer during its recreation*/
	FCriticalSection RecreatingSwapchain;

	uint32 SizeX;
	uint32 SizeY;
	bool bIsFullscreen;
	EPixelFormat PixelFormat;
	int32 AcquiredImageIndex;
	FVulkanSwapChain* SwapChain;
	void* WindowHandle;
	uint32 PresentCount;

	int8 LockToVsync;

	FCustomPresentRHIRef CustomPresent;

	FVulkanSyncPointRef LastFrameSyncPoint;

	EDeviceScreenOrientation CachedOrientation = EDeviceScreenOrientation::Unknown;
	void OnSystemResolutionChanged(uint32 ResX, uint32 ResY);

	void CreateSwapchain(FVulkanCommandListContext& Context, struct FVulkanSwapChainRecreateInfo* RecreateInfo, FVulkanPlatformWindowContext& WindowContext);
	bool TryAcquireImageIndex(FVulkanCommandListContext& Context);
	void InitImages(TConstArrayView<VkImage> Images);

	void RecreateSwapchainFromRT(FRHICommandListImmediate& RHICmdList, EPixelFormat PreferredPixelFormat, FVulkanPlatformWindowContext& WindowContext);
	void RecreateSwapchainFromRT(FRHICommandListImmediate& RHICmdList, FVulkanPlatformWindowContext& WindowContext)
	{
		RecreateSwapchainFromRT(RHICmdList, PixelFormat, WindowContext);
	}
	void Resize(FRHICommandListImmediate& RHICmdList, uint32 InSizeX, uint32 InSizeY, bool bIsFullscreen, EPixelFormat PreferredPixelFormat, FVulkanPlatformWindowContext& WindowContext);

	bool DoCheckedSwapChainJob(FVulkanCommandListContext& Context);
	bool SupportsStandardSwapchain();
	bool RequiresRenderingBackBuffer();
	EPixelFormat GetPixelFormatForNonDefaultSwapchain();

	friend class FVulkanDynamicRHI;
	friend class FVulkanCommandListContext;
	friend struct FRHICommandAcquireBackBuffer;
	friend class FVulkanBackBuffer;
};

template<>
struct TVulkanResourceTraits<FRHIViewport>
{
	typedef FVulkanViewport TConcreteType;
};
