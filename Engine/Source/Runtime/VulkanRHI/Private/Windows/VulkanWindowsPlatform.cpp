// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanWindowsPlatform.h"
#include "../VulkanRHIPrivate.h"
#include "../VulkanDevice.h"
#include "../VulkanRayTracing.h"
#include "../VulkanExtensions.h"
#include "HAL/Platform.h"
#include "Misc/CommandLine.h"

// Disable warning about forward declared enumeration without a type, since the D3D specific enums are not used in this translation unit
#if WITH_AMD_AGS
#pragma warning(push)
#pragma warning(disable : 4471)
#include "amd_ags.h"
#pragma warning(pop)
#endif

#include "Windows/AllowWindowsPlatformTypes.h"
static HMODULE GVulkanDLLModule = nullptr;
bool FVulkanWindowsPlatform::bAttemptedLoad = false;

static PFN_vkGetInstanceProcAddr GGetInstanceProcAddr = nullptr;

// Vulkan function pointers
#define DEFINE_VK_ENTRYPOINTS(Type,Func) VULKANRHI_API Type VulkanDynamicAPI::Func = NULL;
ENUM_VK_ENTRYPOINTS_ALL(DEFINE_VK_ENTRYPOINTS)

#define CHECK_VK_ENTRYPOINTS(Type,Func) if (VulkanDynamicAPI::Func == NULL) { bFoundAllEntryPoints = false; UE_LOGF(LogRHI, Warning, "Failed to find entry point for %ls", TEXT(#Func)); }

PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
bool FVulkanWindowsPlatform::LoadVulkanLibrary()
{
	if (bAttemptedLoad)
	{
		return (GVulkanDLLModule != nullptr);
	}
	bAttemptedLoad = true;

#if VULKAN_HAS_DEBUGGING_ENABLED
	const FString VulkanSDK = FPlatformMisc::GetEnvironmentVariable(TEXT("VULKAN_SDK"));
	UE_LOGF(LogVulkanRHI, Warning, "Found VULKAN_SDK=%ls", *VulkanSDK);
	const bool bHasVulkanSDK = !VulkanSDK.IsEmpty();
	UE_LOGF(LogVulkanRHI, Display, "Registering provided Vulkan validation layers");

	// if vulkan SDK is installed, we'll append our built-in validation layers to VK_ADD_LAYER_PATH,
	// otherwise we append to VK_LAYER_PATH (which is probably empty)

	// Change behavior of loading Vulkan layers by setting environment variable "VarToUse" to UE specific directory
	FString VarToUse = (bHasVulkanSDK) ? TEXT("VK_ADD_LAYER_PATH") : TEXT("VK_LAYER_PATH");
	FString PreviousEnvVar = FPlatformMisc::GetEnvironmentVariable(*VarToUse);
	FString UELayerPath = FPaths::EngineDir();
	UELayerPath.Append(TEXT("Binaries/ThirdParty/Vulkan/Win64"));
	
	if(!PreviousEnvVar.IsEmpty())
	{
		PreviousEnvVar.Append(TEXT(";"));
	}

	PreviousEnvVar.Append(*UELayerPath);
	FPlatformMisc::SetEnvironmentVar(*VarToUse, *PreviousEnvVar);
	UE_LOGF(LogVulkanRHI, Display, "Updated %ls=%ls", *VarToUse, *PreviousEnvVar);
#endif // VULKAN_HAS_DEBUGGING_ENABLED

	// The vulkan dll must exist, otherwise the driver doesn't support Vulkan
	GVulkanDLLModule = ::LoadLibraryW(TEXT("vulkan-1.dll"));

	if (GVulkanDLLModule)
	{
#define GET_VK_ENTRYPOINTS(Type,Func) VulkanDynamicAPI::Func = (Type)FPlatformProcess::GetDllExport(GVulkanDLLModule, L""#Func);
		ENUM_VK_ENTRYPOINTS_BASE(GET_VK_ENTRYPOINTS);

		bool bFoundAllEntryPoints = true;
		ENUM_VK_ENTRYPOINTS_BASE(CHECK_VK_ENTRYPOINTS);
		if (!bFoundAllEntryPoints)
		{
			FreeVulkanLibrary();
			return false;
		}

		ENUM_VK_ENTRYPOINTS_OPTIONAL_BASE(GET_VK_ENTRYPOINTS);
#if UE_BUILD_DEBUG
		ENUM_VK_ENTRYPOINTS_OPTIONAL_BASE(CHECK_VK_ENTRYPOINTS);
#endif

		ENUM_VK_ENTRYPOINTS_PLATFORM_BASE(GET_VK_ENTRYPOINTS);
		ENUM_VK_ENTRYPOINTS_PLATFORM_BASE(CHECK_VK_ENTRYPOINTS);

#undef GET_VK_ENTRYPOINTS

		return true;
	}

	return false;
}

bool FVulkanWindowsPlatform::LoadVulkanInstanceFunctions(VkInstance inInstance)
{
	if (!GVulkanDLLModule)
	{
		return false;
	}

	GGetInstanceProcAddr = (PFN_vkGetInstanceProcAddr)FPlatformProcess::GetDllExport(GVulkanDLLModule, TEXT("vkGetInstanceProcAddr"));

	if (!GGetInstanceProcAddr)
	{
		return false;
	}

	bool bFoundAllEntryPoints = true;
#define CHECK_VK_ENTRYPOINTS(Type,Func) if (VulkanDynamicAPI::Func == NULL) { bFoundAllEntryPoints = false; UE_LOGF(LogRHI, Warning, "Failed to find entry point for %ls", TEXT(#Func)); }

	// Initialize all of the entry points we have to query manually
#define GETINSTANCE_VK_ENTRYPOINTS(Type, Func) VulkanDynamicAPI::Func = (Type)VulkanDynamicAPI::vkGetInstanceProcAddr(inInstance, #Func);
	ENUM_VK_ENTRYPOINTS_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_INSTANCE(CHECK_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_SURFACE_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_SURFACE_INSTANCE(CHECK_VK_ENTRYPOINTS);
	if (!bFoundAllEntryPoints)
	{
		FreeVulkanLibrary();
		return false;
	}

	ENUM_VK_ENTRYPOINTS_OPTIONAL_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_OPTIONAL_PLATFORM_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
#if UE_BUILD_DEBUG
	ENUM_VK_ENTRYPOINTS_OPTIONAL_INSTANCE(CHECK_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_OPTIONAL_PLATFORM_INSTANCE(CHECK_VK_ENTRYPOINTS);
#endif

	const bool bFoundRayTracingEntries = FVulkanRayTracingPlatform::CheckVulkanInstanceFunctions(inInstance);
	if (!bFoundRayTracingEntries)
	{
		UE_LOGF(LogVulkanRHI, Warning, "Vulkan RHI ray tracing is enabled, but failed to load instance functions.");
	}
	
	ENUM_VK_ENTRYPOINTS_PLATFORM_INSTANCE(GETINSTANCE_VK_ENTRYPOINTS);
	ENUM_VK_ENTRYPOINTS_PLATFORM_INSTANCE(CHECK_VK_ENTRYPOINTS);

#undef GETINSTANCE_VK_ENTRYPOINTS
#undef CHECK_VK_ENTRYPOINTS

	return true;
}
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

void FVulkanWindowsPlatform::FreeVulkanLibrary()
{
	if (GVulkanDLLModule != nullptr)
	{
		::FreeLibrary(GVulkanDLLModule);
		GVulkanDLLModule = nullptr;
	}
	bAttemptedLoad = false;
}

#include "Windows/HideWindowsPlatformTypes.h"

void FVulkanWindowsPlatform::GetInstanceExtensions(FVulkanInstanceExtensionArray& OutExtensions)
{
	OutExtensions.Add(MakeUnique<FVulkanInstanceExtension>(VK_KHR_WIN32_SURFACE_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));
	OutExtensions.Add(MakeUnique<FVulkanInstanceExtension>(VK_KHR_GET_SURFACE_CAPABILITIES_2_EXTENSION_NAME, VULKAN_SUPPORTS_FULLSCREEN_EXCLUSIVE, VULKAN_EXTENSION_NOT_PROMOTED));
}


void FVulkanWindowsPlatform::GetDeviceExtensions(FVulkanDevice* Device, FVulkanDeviceExtensionArray& OutExtensions)
{
	OutExtensions.Add(MakeUnique<FVulkanDeviceExtension>(Device, VK_EXT_FULL_SCREEN_EXCLUSIVE_EXTENSION_NAME, VULKAN_SUPPORTS_FULLSCREEN_EXCLUSIVE, 
														VULKAN_EXTENSION_NOT_PROMOTED, DEVICE_EXT_FLAG_SETTER(HasEXTFullscreenExclusive)));

	OutExtensions.Add(MakeUnique<FVulkanDeviceExtension>(Device, VK_KHR_EXTERNAL_SEMAPHORE_WIN32_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));
	OutExtensions.Add(MakeUnique<FVulkanDeviceExtension>(Device, VK_KHR_EXTERNAL_MEMORY_WIN32_EXTENSION_NAME, VULKAN_EXTENSION_ENABLED, VULKAN_EXTENSION_NOT_PROMOTED));
}

void FVulkanWindowsPlatform::CreateSurface(FVulkanPlatformWindowContext& WindowContext, VkInstance Instance, VkSurfaceKHR* OutSurface)
{
	VkWin32SurfaceCreateInfoKHR SurfaceCreateInfo;
	ZeroVulkanStruct(SurfaceCreateInfo, VK_STRUCTURE_TYPE_WIN32_SURFACE_CREATE_INFO_KHR);
	SurfaceCreateInfo.hinstance = GetModuleHandle(nullptr);
	SurfaceCreateInfo.hwnd = (HWND)WindowContext.GetWindowHandle();
	VERIFYVULKANRESULT(VulkanDynamicAPI::vkCreateWin32SurfaceKHR(Instance, &SurfaceCreateInfo, VULKAN_CPU_ALLOCATOR, OutSurface));
}

