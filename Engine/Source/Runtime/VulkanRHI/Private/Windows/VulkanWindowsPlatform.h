// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// moved the setup from VulkanRHIPrivate.h to the platform headers
#include "Windows/WindowsHWrapper.h"
#include "RHI.h"

#define VK_USE_PLATFORM_WIN32_KHR					1
#define VK_USE_PLATFORM_WIN32_KHX					1

#define	VULKAN_SHOULD_ENABLE_DRAW_MARKERS			(UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)
#define VULKAN_USE_CREATE_WIN32_SURFACE				1
#define VULKAN_DYNAMICALLYLOADED					1
#define VULKAN_SHOULD_ENABLE_DESKTOP_HMD_SUPPORT	1
#define VULKAN_SIGNAL_UNIMPLEMENTED()				checkf(false, TEXT("Unimplemented vulkan functionality: %s"), StringCast<TCHAR>(__FUNCTION__).Get())
#define VULKAN_SUPPORTS_SCALAR_BLOCK_LAYOUT			1
#define VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH	1
#define VULKAN_SUPPORTS_FULLSCREEN_EXCLUSIVE		1

#define UE_VK_API_VERSION							VK_API_VERSION_1_1


#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
#	include "vk_enum_string_helper.h"
#	define VK_TYPE_TO_STRING(Type, Value) ANSI_TO_TCHAR(string_##Type(Value))
#	define VK_FLAGS_TO_STRING(Type, Value) ANSI_TO_TCHAR(string_##Type(Value).c_str())
#endif

#define VULKAN_SHOULD_USE_LLM					(UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)

#define ENUM_VK_ENTRYPOINTS_PLATFORM_BASE(EnumMacro)

#define ENUM_VK_ENTRYPOINTS_PLATFORM_INSTANCE(EnumMacro)	\
	EnumMacro(PFN_vkCreateWin32SurfaceKHR, vkCreateWin32SurfaceKHR)

#define ENUM_VK_ENTRYPOINTS_OPTIONAL_PLATFORM_INSTANCE(EnumMacro)

// and now, include the GenericPlatform class
#include "../VulkanGenericPlatform.h"

typedef FVulkanGenericPlatformWindowContext FVulkanPlatformWindowContext;

class FVulkanWindowsPlatform : public FVulkanGenericPlatform
{
public:
	static bool LoadVulkanLibrary();
	static bool LoadVulkanInstanceFunctions(VkInstance inInstance);
	static void FreeVulkanLibrary();

	static void GetInstanceExtensions(FVulkanInstanceExtensionArray& OutExtensions);
	static void GetInstanceLayers(TArray<const ANSICHAR*>& OutLayers) {}
	static void GetDeviceExtensions(FVulkanDevice* Device, FVulkanDeviceExtensionArray& OutExtensions);
	static void GetDeviceLayers(TArray<const ANSICHAR*>& OutLayers) {}

	static void CreateSurface(FVulkanPlatformWindowContext& WindowContext, VkInstance Instance, VkSurfaceKHR* OutSurface);

	static VkTimeDomainKHR GetTimeDomain() { return VK_TIME_DOMAIN_QUERY_PERFORMANCE_COUNTER_KHR; }

private:
	static bool bAttemptedLoad;
};

typedef FVulkanWindowsPlatform FVulkanPlatform;
