// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHI.h"
#include "VulkanThirdParty.h"

struct FVulkanRHIAllocationInfo
{
	VkDeviceMemory Handle{};
	uint64 Offset{};
	uint64 Size{};
	VkMemoryPropertyFlags Flags{};
};

struct FVulkanRHIImageViewInfo
{
	VkImageView ImageView;
	VkImage Image;
	VkImageSubresourceRange SubresourceRange;
	VkFormat Format;
	uint32 Width;
	uint32 Height;
	uint32 Depth;
	ETextureCreateFlags UEFlags;
};

struct FVulkanRHIExternalImageDeleteCallbackInfo
{
	void* UserData = nullptr;
	void (*Function)(void* UserData) = nullptr;
};


// Base class used for external code (such as plugins) to enable extensions in the Vulkan RHI. The lifetime is managed through the use of TSharedRef.
// Basic usage:  Using this as a base class, pass in the ExtensionName and overload GetFeaturesStruct() to return the extension specific features struct.
// IMPORTANT:    Make sure the feature struct has its sType correctly set and pNext to NULL.
// 
// Example to enable VK_EXT_memory_decompression:
// class FDeviceExtensionMemoryDecompression : public FVulkanRHIExternalDeviceExtensionBase
// {
//    FDeviceExtensionMemoryDecompression() : FVulkanRHIExternalDeviceExtensionBase(VK_EXT_MEMORY_DECOMPRESSION_EXTENSION_NAME)
//    {
//        ZeroVulkanStruct(Features, VK_STRUCTURE_TYPE_PHYSICAL_DEVICE_MEMORY_DECOMPRESSION_FEATURES_EXT);
//    }
//    virtual VkBaseOutStructure* GetFeaturesStruct() final override
//    {
//       return reinterpret_cast<VkBaseOutStructure*>(&Features);
//    }
//    VkPhysicalDeviceMemoryDecompressionFeaturesEXT Features;
// }
// 
// - If properties are needed, overload GetPropertiesStruct() to return the extension specific properties struct (with sType and pNext initialized).
// - FeaturesCallback() / PropertiesCallback() are called after those property structures are filled.  
// - If inspection of the features and/or properties reveals the extension should not be loaded, the callbacks should return false.
// - If the extension is already handled by the engine or loaded by another plugin, DuplicateCallback() will be called.
class FVulkanRHIExternalDeviceExtensionBase : public TSharedFromThis<FVulkanRHIExternalDeviceExtensionBase>
{
public:
	FVulkanRHIExternalDeviceExtensionBase(const ANSICHAR* InExtensionName) : ExtensionName(InExtensionName) {}
	virtual ~FVulkanRHIExternalDeviceExtensionBase() {}

	// Returns a pointer to the extension's feature structure
	virtual VkBaseOutStructure* GetFeaturesStruct() { return nullptr; }

	// Callback when the features structure has been filled.  Return true if the features meet your requirements.
	virtual bool FeaturesCallback() { return true;  }

	// Pointer to the extension's properties structure
	virtual VkBaseOutStructure* GetPropertiesStruct() { return nullptr;  }

	// Callback when the properties structure has been filled.  Return true if the properties meet your requirements.
	virtual bool PropertiesCallback() { return true; }

	// Callback if the extension is found to be a duplicate (already loaded by the engine or other plugin).
	virtual void DuplicateCallback() {}

	// Name of the extension to be loaded
	const FAnsiString ExtensionName;
};

enum class EVulkanRHIRunOnQueueType
{
	Graphics = 0,
	Transfer,
};

// Patch allocations into the engine's allocators
class IVulkanMemoryAllocator
{
public:
	typedef void* AllocHandle;

	virtual ~IVulkanMemoryAllocator() {}

	virtual VkResult AllocateImageMemory(VkImage Image, VkMemoryPropertyFlags Flags, bool bForceDedicatedAllocation, AllocHandle* OutAllocHandle) = 0;
	virtual VkResult AllocateBufferMemory(VkBuffer Buffer, VkMemoryPropertyFlags Flags, bool bForceDedicatedAllocation, AllocHandle* OutAllocHandle) = 0;
	virtual void FreeMemory(const AllocHandle&) = 0;
	virtual FVulkanRHIAllocationInfo GetAllocInfo(const AllocHandle&) = 0;

	virtual void* MapMemory(const AllocHandle&) = 0;
	virtual void UnmapMemory(const AllocHandle&) = 0;

	virtual void FlushMemory(const AllocHandle&, VkDeviceSize Offset, VkDeviceSize Size) = 0;
	virtual void InvalidateMemory(const AllocHandle&, VkDeviceSize Offset, VkDeviceSize Size) = 0;

	virtual FRHIMemoryStats GetMemoryStats() = 0;
};

struct IVulkanDynamicRHI : public FDynamicRHI
{
	virtual ERHIInterfaceType GetInterfaceType() const override { return ERHIInterfaceType::Vulkan; }

	virtual uint32           RHIGetVulkanVersion() const = 0;
	virtual VkInstance       RHIGetVkInstance() const = 0;
	virtual VkDevice         RHIGetVkDevice() const = 0;
	virtual const uint8*     RHIGetVulkanDeviceUUID() const = 0;
	virtual VkPhysicalDevice RHIGetVkPhysicalDevice() const = 0;
	virtual const VkAllocationCallbacks* RHIGetVkAllocationCallbacks() = 0;

	virtual IVulkanMemoryAllocator* RHIGetVulkanMemoryAllocator() = 0;

	virtual VkQueue          RHIGetGraphicsVkQueue() const = 0;
	virtual uint32           RHIGetGraphicsQueueIndex() const = 0;
	virtual uint32           RHIGetGraphicsQueueFamilyIndex() const = 0;

	UE_DEPRECATED(5.8, "RHIGetActiveVkCommandBuffer is deprecated, and dangerous because it is incompatible with parallel translation. Use RHIGetVkCommandBuffer(FRHICommandListBase&).")
	VkCommandBuffer RHIGetActiveVkCommandBuffer()
	{
		checkNoEntry();
		return {};
	}

	//
	// Returns the command buffer for the currently active pipeline on the given RHI command list.
	// Can only be called on executing command lists (i.e. ExecutingCmdList.IsBottomOfPipe() must be true),
	// 
	virtual VkCommandBuffer  RHIGetVkCommandBuffer(FRHICommandListBase& ExecutingCmdList) const = 0;

	virtual uint64           RHIGetGraphicsAdapterLUID(VkPhysicalDevice InPhysicalDevice) const = 0;
	virtual bool             RHIDoesAdapterMatchDevice(const void* InAdapterId) const = 0;
	virtual void*            RHIGetVkDeviceProcAddr(const char* InName) const = 0;
	virtual void*            RHIGetVkInstanceProcAddr(const char* InName) const = 0;
	/**
	 * Version of RHIGetVkInstanceProcAddr that uses nullptr as the instance argument.
	 * See vkGetInstanceProcAddr manpage for distinction between "global" and non-global commands.
	 */
	virtual void*            RHIGetVkInstanceGlobalProcAddr(const char* InName) const = 0;
	virtual VkFormat         RHIGetSwapChainVkFormat(EPixelFormat InFormat) const = 0;
	virtual bool             RHISupportsEXTFragmentDensityMap2() const = 0;

	virtual void 			 RHISetFDMOffsets(const FVector2f GazePoint[2], FIntPoint FramebufferSize) = 0;

	virtual TArray<VkExtensionProperties> RHIGetAllInstanceExtensions() const = 0;
	virtual TArray<VkExtensionProperties> RHIGetAllDeviceExtensions(VkPhysicalDevice InPhysicalDevice) const = 0;
	virtual TArray<FAnsiString> RHIGetLoadedDeviceExtensions() const = 0;

	virtual FTextureRHIRef   RHICreateTexture2DFromResource(EPixelFormat Format, uint32 SizeX, uint32 SizeY, uint32 NumMips, uint32 NumSamples, VkImage Resource, ETextureCreateFlags Flags, const FClearValueBinding& ClearValueBinding = FClearValueBinding::Transparent, const FVulkanRHIExternalImageDeleteCallbackInfo& ExternalImageDeleteCallbackInfo = {}) = 0;
#if PLATFORM_ANDROID
	virtual FTextureRHIRef   RHICreateTexture2DFromAndroidHardwareBuffer(AHardwareBuffer* HardwareBuffer) = 0;
#endif
	virtual FTextureRHIRef   RHICreateTexture2DArrayFromResource(EPixelFormat Format, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, VkImage Resource, ETextureCreateFlags Flags, const FClearValueBinding& ClearValueBinding = FClearValueBinding::Transparent) = 0;
	virtual FTextureRHIRef   RHICreateTextureCubeFromResource(EPixelFormat Format, uint32 Size, bool bArray, uint32 ArraySize, uint32 NumMips, VkImage Resource, ETextureCreateFlags Flags, const FClearValueBinding& ClearValueBinding = FClearValueBinding::Transparent) = 0;

	virtual VkImage                  RHIGetVkImage(FRHITexture* InTexture) const = 0;
	virtual VkFormat                 RHIGetViewVkFormat(FRHITexture* InTexture) const = 0;
	virtual FVulkanRHIAllocationInfo RHIGetAllocationInfo(FRHITexture* InTexture) const = 0;
	virtual FVulkanRHIImageViewInfo  RHIGetImageViewInfo(FRHITexture* InTexture) const = 0;
	virtual FVulkanRHIAllocationInfo RHIGetAllocationInfo(FRHIBuffer* InBuffer) const = 0;

	virtual void           RHIFinishExternalComputeWork(VkCommandBuffer InCommandBuffer) = 0;
	virtual void           RHIVerifyResult(VkResult Result, const ANSICHAR* VkFuntion, const ANSICHAR* Filename, uint32 Line) = 0;

	static VULKANRHI_API void AddEnabledInstanceExtensionsAndLayers(TArrayView<const ANSICHAR* const> InInstanceExtensions, TArrayView<const ANSICHAR* const> InInstanceLayers);
	static void AddEnabledDeviceExtensionsAndLayers(TArrayView<const ANSICHAR* const> InDeviceExtensions, TArrayView<const ANSICHAR* const> InDeviceLayers)
	{
		TArray<TSharedRef<FVulkanRHIExternalDeviceExtensionBase>> DeviceExtensions;
		DeviceExtensions.Reserve(InDeviceExtensions.Num());
		for (const ANSICHAR* const ExtensionName : InDeviceExtensions)
		{
			DeviceExtensions.Add(MakeShared<FVulkanRHIExternalDeviceExtensionBase>(ExtensionName));
		}
		AddEnabledDeviceExtensionsAndLayers(DeviceExtensions, InDeviceLayers);
	}
	static VULKANRHI_API void AddEnabledDeviceExtensionsAndLayers(TArrayView<TSharedRef<FVulkanRHIExternalDeviceExtensionBase>> InDeviceExtensions, TArrayView<const ANSICHAR* const> InDeviceLayers);


	// Runs code on SubmissionThread with access to VkQueue.  Useful for plugins. 
	virtual void RHIRunOnQueue(EVulkanRHIRunOnQueueType QueueType, TFunction<void(VkQueue)>&& CodeToRun, bool bWaitForSubmission) = 0;

	UE_DEPRECATED(5.8, "RHIRegisterWork is deprecated, and there is no replacement. Remove calls to this function.")
	void RHIRegisterWork(uint32 NumPrimitives)
	{
	}
};

inline IVulkanDynamicRHI* GetIVulkanDynamicRHI()
{
	checkf(GDynamicRHI, TEXT("Tried to fetch RHI too early"));
	check(GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::Vulkan);
	return GetDynamicRHI<IVulkanDynamicRHI>();
}

#define VERIFYVULKANRESULT_EXTERNAL(VkFunction) { const VkResult ScopedResult = VkFunction; if (ScopedResult != VK_SUCCESS) { GetIVulkanDynamicRHI()->RHIVerifyResult(ScopedResult, #VkFunction, __FILE__, __LINE__); }}
