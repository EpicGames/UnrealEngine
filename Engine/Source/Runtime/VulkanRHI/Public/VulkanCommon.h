// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanCommon.h: Common definitions used for both runtime and compiling shaders.
=============================================================================*/

#pragma once

#include "RHIDefinitions.h"
#include "Logging/LogMacros.h"

#ifndef VULKAN_SUPPORTS_GEOMETRY_SHADERS
	#define VULKAN_SUPPORTS_GEOMETRY_SHADERS					PLATFORM_SUPPORTS_GEOMETRY_SHADERS
#endif

// This defines controls shader generation (so will cause a format rebuild)
// be careful wrt cooker/target platform not matching define-wise!!!
// ONLY used for debugging binding table/descriptor set bugs/mismatches.
#define VULKAN_ENABLE_BINDING_DEBUG_NAMES						0

namespace ShaderStage
{
	// There should be one value for each value in EShaderFrequency.
	// These values are meant to be used as indices in contexts where values for different bind points can overlap (Graphics/Compute/RayTracing)
	// like shader arrays in pipeline states or UB binding indices for Graphics (Vertex==0, Pixel==1) that can overlap with Compute (Compute==0). 
	// IMPORTANT: Adjusting these requires a full shader rebuild (ie modify the GUID in VulkanCommon.usf)
	enum EStage
	{
		Vertex = 0,
		Pixel = 1,
		Geometry = 2,
		Mesh = 3,
		Task = 4,

		NumGraphicsStages = 5,

		RayGen = 0,
		RayMiss = 1,
		RayHitGroup = 2,
		RayCallable = 3,

		NumRayTracingStages = 4,

		Compute = 0,

		NumComputeStages = 1,

		MaxNumStages = 6, // work with even count to simplify bindless alignment requirements

		Invalid = -1,
	};

	static_assert(MaxNumStages >= FMath::Max(NumComputeStages, FMath::Max(NumGraphicsStages, NumRayTracingStages)), "MaxNumStages too small!");

	inline EStage GetStageForFrequency(EShaderFrequency Stage)
	{
		switch (Stage)
		{
		case SF_Vertex:			return Vertex;
		case SF_Mesh:			return Mesh;
		case SF_Amplification:	return Task;
		case SF_Pixel:			return Pixel;
		case SF_Geometry:		return Geometry;
		case SF_RayGen:			return RayGen;
		case SF_RayMiss:		return RayMiss;
		case SF_RayHitGroup:	return RayHitGroup;
		case SF_RayCallable:	return RayCallable;
		case SF_Compute:		return Compute;
		default:
			checkf(0, TEXT("Invalid shader Stage %d"), (int32)Stage);
			break;
		}

		return Invalid;
	}

	inline EShaderFrequency GetFrequencyForGfxStage(EStage Stage)
	{
		switch (Stage)
		{
		case EStage::Vertex:	return SF_Vertex;
		case EStage::Pixel:		return SF_Pixel;
		case EStage::Geometry:	return SF_Geometry;
		case EStage::Mesh:		return SF_Mesh;
		case EStage::Task:		return SF_Amplification;
		default:
			checkf(0, TEXT("Invalid graphic shader stage: %d"), (int32)Stage);
			break;
		}

		return SF_NumFrequencies;
	}
};

namespace VulkanBindless
{
	static constexpr uint32 MaxUniformBuffersPerStage = 16;
	static constexpr uint32 MaxUniformBuffersTotal = 32;  // Guaranteed to be min 32 by Vulkan 1.4

	enum EDescriptorSets
	{
		BindlessSamplerSet = 0,
		BindlessResourceSet = 1,

		BindlessStorageBufferSet = BindlessResourceSet,
		BindlessUniformBufferSet = BindlessResourceSet,

		BindlessStorageImageSet = BindlessResourceSet,
		BindlessSampledImageSet = BindlessResourceSet,

		BindlessStorageTexelBufferSet = BindlessResourceSet,
		BindlessUniformTexelBufferSet = BindlessResourceSet,

		BindlessAccelerationStructureSet = BindlessResourceSet,

		// Number of sets reserved for samplers/resources
		NumBindlessSets,

		// Index of the descriptor set used for single use ub (like globals)
		BindlessSingleUseUniformBufferSet = NumBindlessSets,

		// Total number of descriptor sets used in a bindless pipeline
		MaxNumSets = NumBindlessSets + 1
	};

	inline int32 GetOffsetForFrequency(EShaderFrequency Frequency)
	{
		// Stages that run alone or that are likely to contain more UBs start at 0
		// The second stage follows starting at +MaxUniformBuffersPerStage
		// The third stage returns -1 to signal descriptors are allocated from the end 
		// The sum of second and third stage must never go beyond MaxUniformBuffersPerStage

		switch (Frequency)
		{
		case SF_Pixel:			return 0;

		case SF_Vertex:			return MaxUniformBuffersPerStage;
		case SF_Geometry:		return -1;

		case SF_Mesh:			return MaxUniformBuffersPerStage;
		case SF_Amplification:	return -1;

		case SF_Compute:		return 0;

		case SF_RayGen:			
		case SF_RayMiss:
		case SF_RayHitGroup:
		case SF_RayCallable:	return 0;
		default:
			checkf(false, TEXT("Invalid shader frequency %d"), (int32)Frequency);
			break;
		}

		return 0;
	}

};

DECLARE_LOG_CATEGORY_EXTERN(LogVulkan, Display, All);

template< class T >
static inline void ZeroVulkanStruct(T& Struct, int32 VkStructureType)
{
	static_assert(!TIsPointer<T>::Value, "Don't use a pointer!");
	static_assert(STRUCT_OFFSET(T, sType) == 0, "Assumes sType is the first member in the Vulkan type!");
	static_assert(sizeof(T::sType) == sizeof(int32), "Assumed sType is compatible with int32!");
	// Horrible way to coerce the compiler to not have to know what T::sType is so we can have this header not have to include vulkan.h
	(int32&)Struct.sType = VkStructureType;
	FMemory::Memzero(((uint8*)&Struct) + sizeof(VkStructureType), sizeof(T) - sizeof(VkStructureType));
}

template <typename ExistingChainType, typename NewStructType>
static inline void AddToPNext(ExistingChainType& Existing, NewStructType& Added)
{
	Added.pNext = (void*)Existing.pNext;
	Existing.pNext = (void*)&Added;
}
