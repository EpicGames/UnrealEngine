// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanConfiguration.h: Control compilation of the runtime RHI.
=============================================================================*/

// Compiled with 1.2.141.2

#pragma once

#include "HAL/LowLevelMemTracker.h"
#include "VulkanCommon.h"
#include "VulkanPlatform.h"

struct VkAllocationCallbacks;

// API version we want to target.
#ifndef UE_VK_API_VERSION
	#define UE_VK_API_VERSION									VK_API_VERSION_1_1
#endif

// by default, we enable debugging in Development builds, unless the platform says not to
#ifndef VULKAN_SHOULD_DEBUG_IN_DEVELOPMENT
	#define VULKAN_SHOULD_DEBUG_IN_DEVELOPMENT					1
#endif

#ifndef VULKAN_HAS_DEBUGGING_ENABLED
	#define VULKAN_HAS_DEBUGGING_ENABLED						(!IS_PROGRAM && (UE_BUILD_DEBUG || (UE_BUILD_DEVELOPMENT && VULKAN_SHOULD_DEBUG_IN_DEVELOPMENT)))
#endif

// default value of r.Vulkan.EnableValidation
// 0 - disable validation layers
// 1 - enable errors
// 2 - enable errors & warnings
// 3 - enable errors, warnings & performance warnings
// 4 - enable errors, warnings, performance & information messages
// 5 - enable all messages
#ifndef VULKAN_VALIDATION_DEFAULT_VALUE
	#define VULKAN_VALIDATION_DEFAULT_VALUE						(UE_BUILD_DEBUG ? 2 : 0)
#endif

#ifndef VULKAN_SHOULD_ENABLE_DRAW_MARKERS
	#define VULKAN_SHOULD_ENABLE_DRAW_MARKERS					0
#endif

// Enables logging wrappers per Vulkan call
#ifndef VULKAN_ENABLE_DUMP_LAYER
	#define VULKAN_ENABLE_DUMP_LAYER							0
#endif

#define VULKAN_ENABLE_DRAW_MARKERS								VULKAN_SHOULD_ENABLE_DRAW_MARKERS

#ifndef VULKAN_ENABLE_IMAGE_TRACKING_LAYER
	#define VULKAN_ENABLE_IMAGE_TRACKING_LAYER					0
#endif

#ifndef VULKAN_ENABLE_BUFFER_TRACKING_LAYER
	#define VULKAN_ENABLE_BUFFER_TRACKING_LAYER					0
#endif

#define VULKAN_ENABLE_TRACKING_LAYER							(VULKAN_ENABLE_BUFFER_TRACKING_LAYER || VULKAN_ENABLE_IMAGE_TRACKING_LAYER)
#define VULKAN_ENABLE_WRAP_LAYER								(VULKAN_ENABLE_DUMP_LAYER || VULKAN_ENABLE_TRACKING_LAYER)

#define VULKAN_HASH_POOLS_WITH_TYPES_USAGE_ID					1

#define VULKAN_SINGLE_ALLOCATION_PER_RESOURCE					0

#ifndef VULKAN_SHOULD_USE_LLM
	#define VULKAN_SHOULD_USE_LLM								0
#endif

#ifndef VULKAN_USE_LLM
	#define VULKAN_USE_LLM										((ENABLE_LOW_LEVEL_MEM_TRACKER) && VULKAN_SHOULD_USE_LLM)
#endif

#ifndef VULKAN_CUSTOM_MEMORY_MANAGER_ENABLED
	#define VULKAN_CUSTOM_MEMORY_MANAGER_ENABLED				VULKAN_USE_LLM
#endif

#ifndef VULKAN_SHOULD_USE_COMMANDWRAPPERS
	#define VULKAN_SHOULD_USE_COMMANDWRAPPERS					VULKAN_ENABLE_WRAP_LAYER
#endif

#ifndef VULKAN_COMMANDWRAPPERS_ENABLE
	#define VULKAN_COMMANDWRAPPERS_ENABLE						VULKAN_SHOULD_USE_COMMANDWRAPPERS
#endif

#ifndef VULKAN_USE_IMAGE_ACQUIRE_FENCES
	#define VULKAN_USE_IMAGE_ACQUIRE_FENCES						1
#endif

#define VULKAN_ENABLE_AGGRESSIVE_STATS							0

#define VULKAN_REUSE_FENCES										1

#ifndef VULKAN_QUERY_CALLSTACK
	#define VULKAN_QUERY_CALLSTACK								0
#endif

#ifndef VULKAN_ENABLE_DESKTOP_HMD_SUPPORT
	#define VULKAN_ENABLE_DESKTOP_HMD_SUPPORT					0
#endif

#ifndef VULKAN_SIGNAL_UNIMPLEMENTED
	#define VULKAN_SIGNAL_UNIMPLEMENTED()
#endif

#ifndef VULKAN_ENABLE_LRU_CACHE
	#define VULKAN_ENABLE_LRU_CACHE								0
#endif

#ifndef VULKAN_SUPPORTS_VALIDATION_CACHE
	#define VULKAN_SUPPORTS_VALIDATION_CACHE					1
#endif

#ifndef VULKAN_HAS_VALIDATION_FEATURES
	#define VULKAN_HAS_VALIDATION_FEATURES						1
#endif

#ifndef VULKAN_SUPPORTS_DEDICATED_ALLOCATION
	#define VULKAN_SUPPORTS_DEDICATED_ALLOCATION				1
#endif

#ifndef VULKAN_SUPPORTS_GOOGLE_DISPLAY_TIMING
	#define VULKAN_SUPPORTS_GOOGLE_DISPLAY_TIMING				0
#endif

#ifndef VULKAN_USE_CREATE_ANDROID_SURFACE
	#define VULKAN_USE_CREATE_ANDROID_SURFACE					0
#endif

#ifndef VULKAN_USE_CREATE_WIN32_SURFACE
	#define VULKAN_USE_CREATE_WIN32_SURFACE						0
#endif

#ifndef VULKAN_USE_DIFFERENT_POOL_CMDBUFFERS
	#define VULKAN_USE_DIFFERENT_POOL_CMDBUFFERS				1
#endif

#ifndef VULKAN_DELETE_STALE_CMDBUFFERS
	#define VULKAN_DELETE_STALE_CMDBUFFERS						1
#endif

#ifndef VULKAN_SUPPORTS_SCALAR_BLOCK_LAYOUT
	#define VULKAN_SUPPORTS_SCALAR_BLOCK_LAYOUT					1
#endif


#ifndef VULKAN_SUPPORTS_MEMORY_BUDGET
	#define VULKAN_SUPPORTS_MEMORY_BUDGET						1
#endif

#ifndef VULKAN_SUPPORTS_MEMORY_PRIORITY
	#define VULKAN_SUPPORTS_MEMORY_PRIORITY						1
#endif

#ifndef VULKAN_SUPPORTS_DRIVER_PROPERTIES
	#define VULKAN_SUPPORTS_DRIVER_PROPERTIES					1
#endif

#ifndef VULKAN_SUPPORTS_QCOM_RENDERPASS_SHADER_RESOLVE
	#define VULKAN_SUPPORTS_QCOM_RENDERPASS_SHADER_RESOLVE		1
#endif

#ifndef VULKAN_SUPPORTS_FULLSCREEN_EXCLUSIVE
	#define VULKAN_SUPPORTS_FULLSCREEN_EXCLUSIVE				0
#endif

#ifndef VULKAN_SUPPORTS_TEXTURE_COMPRESSION_ASTC_HDR
	#define VULKAN_SUPPORTS_TEXTURE_COMPRESSION_ASTC_HDR		1
#endif

#ifndef VULKAN_SUPPORTS_ANDROID_EXTERNAL_MEMORY_ANDROID_HARDWARE_BUFFER
	#define VULKAN_SUPPORTS_ANDROID_EXTERNAL_MEMORY_ANDROID_HARDWARE_BUFFER	1
#endif

#ifndef VULKAN_SUPPORTS_RENDERPASS2
	#define VULKAN_SUPPORTS_RENDERPASS2							1
#endif

#ifndef VULKAN_SUPPORTS_ASTC_DECODE_MODE
	#define VULKAN_SUPPORTS_ASTC_DECODE_MODE					1
#endif

#ifndef VULKAN_SUPPORTS_SHADER_VIEWPORT_INDEX_LAYER
	#define VULKAN_SUPPORTS_SHADER_VIEWPORT_INDEX_LAYER			1
#endif

#ifndef VULKAN_SUPPORTS_DESCRIPTOR_INDEXING
	#define VULKAN_SUPPORTS_DESCRIPTOR_INDEXING					1
#endif

#ifndef VULKAN_OBJECT_TRACKING 
#define VULKAN_OBJECT_TRACKING 0 //Track objects created and memory used. use r.vulkan.dumpmemory to dump to console
#endif

VULKANRHI_API DECLARE_LOG_CATEGORY_EXTERN(LogVulkanRHI, Log, All);

#if VULKAN_CUSTOM_MEMORY_MANAGER_ENABLED
	#define VULKAN_CPU_ALLOCATOR								VulkanRHI::GetMemoryAllocator(nullptr)
#else
	#define VULKAN_CPU_ALLOCATOR								nullptr
#endif

#ifndef VULKAN_PURGE_SHADER_MODULES
	#define VULKAN_PURGE_SHADER_MODULES							0
#endif

#ifndef VULKAN_SUPPORTS_TRANSIENT_RESOURCE_ALLOCATOR
	#define VULKAN_SUPPORTS_TRANSIENT_RESOURCE_ALLOCATOR		1
#endif

#ifndef VK_TYPE_TO_STRING
#	define VK_TYPE_TO_STRING(Type, Value) *FString::Printf(TEXT("%u"), (uint32)Value)
#endif
#ifndef VK_FLAGS_TO_STRING
#	define VK_FLAGS_TO_STRING(Type, Value) *FString::Printf(TEXT("%u"), (uint32)Value)
#endif

#ifndef VULKAN_USE_DEBUG_NAMES
#	define VULKAN_USE_DEBUG_NAMES								1
#endif

namespace VulkanRHI
{
	static inline const VkAllocationCallbacks* GetMemoryAllocator(const VkAllocationCallbacks* Allocator)
	{
#if VULKAN_CUSTOM_MEMORY_MANAGER_ENABLED
		extern VkAllocationCallbacks GAllocationCallbacks;
		return Allocator ? Allocator : &GAllocationCallbacks;
#else
		return Allocator;
#endif
	}
}
