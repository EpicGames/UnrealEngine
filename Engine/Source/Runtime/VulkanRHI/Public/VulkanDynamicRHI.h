// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanDynamicRHI.h: Public Vulkan RHI definitions.
=============================================================================*/

#pragma once

#include "Misc/CoreMiscDefines.h"
UE_DEPRECATED_HEADER(5.6, "VulkanDynamicRHI.h in the Public folder is deprecated. Please switch to using IVulkanDynamicRHI.h")

#include "IVulkanDynamicRHI.h"
