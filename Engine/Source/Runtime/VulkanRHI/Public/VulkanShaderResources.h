// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanResources.h: Vulkan resource RHI definitions.
=============================================================================*/

#pragma once

#include "BoundShaderStateCache.h"
#include "CrossCompilerCommon.h"
#include "Hash/ShaderHash.h"
#include "VulkanCommon.h"
#include "VulkanThirdParty.h"


// Vulkan ParameterMap:
// Buffer Index = EBufferIndex
// Base Offset = Index into the subtype
// Size = Ignored for non-globals
struct FVulkanShaderHeader
{
	// Includes all bindings, the index in this array is the binding slot
	struct FBindingInfo
	{
		// VkDescriptorType
		uint32					DescriptorType;
#if VULKAN_ENABLE_BINDING_DEBUG_NAMES
		FString					DebugName;
#endif
	};
	TArray<FBindingInfo>		Bindings;

	// FBindingInfo with type VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER have a corresponding entry in this table (at the same index)
	struct FUniformBufferInfo
	{
		uint32					LayoutHash;
		uint8					bHasResources;
		uint8					BindlessCBIndex;
	};
	TArray<FUniformBufferInfo>	UniformBufferInfos;

	// The order of this enum should always match the strings in VulkanBackend.cpp (VULKAN_SUBPASS_FETCH)
	enum class EAttachmentType : uint8
	{
		Depth,
		Color0,
		Color1,
		Color2,
		Color3,
		Color4,
		Color5,
		Color6,
		Color7,

		Count,
	};

	// Used to determine the EAttachmentType of a FBindingInfo with type VK_DESCRIPTOR_TYPE_INPUT_ATTACHMENT
	struct FInputAttachmentInfo
	{
		uint8					BindingIndex;
		EAttachmentType			Type;
	};
	TArray<FInputAttachmentInfo> InputAttachmentInfos;

	// Types of Global Samplers (see Common.ush for types)
	// Must match GetGlobalSamplerType() in SpirVShaderCompiler.inl and
	// and declarations in VulkanCommon.ush
	enum class EGlobalSamplerType : uint8
	{
		PointClampedSampler,
		PointWrappedSampler,
		BilinearClampedSampler,
		BilinearWrappedSampler,
		TrilinearClampedSampler,
		TrilinearWrappedSampler,

		Count,
		Invalid,
	};
	struct FGlobalSamplerInfo
	{
		uint8					BindingIndex;
		EGlobalSamplerType		Type;
	};
	TArray<FGlobalSamplerInfo>  GlobalSamplerInfos;

	// The number of uniform buffers containing constants and requiring bindings
	// Uniform buffers beyond this index do not have bindings (resource only UB)
	uint32						NumBoundUniformBuffers = 0;

	// Size of the uniform buffer containing packed globals
	// If present (not zero), it will always be at binding 0 of the stage
	uint32						PackedGlobalsSize = 0;

	// Mask of input attachments being used (the index of the bit corresponds to EAttachmentType value)
	uint32						InputAttachmentsMask = 0;

	// Mostly relevant for Vertex Shaders
	uint32						InOutMask;

	// Relevant for Ray Tracing Shaders
	uint32						RayTracingPayloadType = 0;
	uint32						RayTracingPayloadSize = 0;

	// Specialization constant info
	FSpecializationConstantData	SpecializationConstantData;

	FShaderHash					CodeHash;
	uint32						SpirvCRC = 0;
	uint8						WaveSize = 0;

	// For RayHitGroup shaders
	enum class ERayHitGroupEntrypoint : uint8
	{
		NotPresent = 0,

		// Hit group types are all stored in a single spirv blob
		// and each have different entry point names
		// NOTE: Not used yet because of compiler issues
		CommonBlob,

		// Hit group types are each stored in a different spirv blob
		// to circumvent DXC compilation issues
		SeparateBlob
	};
	ERayHitGroupEntrypoint				RayGroupAnyHit = ERayHitGroupEntrypoint::NotPresent;
	ERayHitGroupEntrypoint				RayGroupIntersection = ERayHitGroupEntrypoint::NotPresent;

	FString								DebugName;

	FVulkanShaderHeader() = default;
	enum EInit
	{
		EZero
	};
	FVulkanShaderHeader(EInit)
		: InOutMask(0)
	{
	}
};

inline FArchive& operator<<(FArchive& Ar, FVulkanShaderHeader::FBindingInfo& BindingInfo)
{
	Ar << BindingInfo.DescriptorType;
#if VULKAN_ENABLE_BINDING_DEBUG_NAMES
	Ar << BindingInfo.DebugName;
#endif
	return Ar;
}

inline FArchive& operator<<(FArchive& Ar, FVulkanShaderHeader::FUniformBufferInfo& Info)
{
	Ar << Info.LayoutHash;
	Ar << Info.bHasResources;
	Ar << Info.BindlessCBIndex;
	return Ar;
}

inline FArchive& operator<<(FArchive& Ar, FVulkanShaderHeader::FInputAttachmentInfo& Info)
{
	Ar << Info.BindingIndex;
	Ar << Info.Type;
	return Ar;
}

inline FArchive& operator<<(FArchive& Ar, FVulkanShaderHeader::FGlobalSamplerInfo& Info)
{
	Ar << Info.BindingIndex;
	Ar << Info.Type;
	return Ar;
}

inline FArchive& operator<<(FArchive& Ar, FVulkanShaderHeader& Header)
{
	Ar << Header.Bindings;
	Ar << Header.UniformBufferInfos;
	Ar << Header.InputAttachmentInfos;
	Ar << Header.GlobalSamplerInfos;
	Ar << Header.NumBoundUniformBuffers;
	Ar << Header.PackedGlobalsSize;
	Ar << Header.InputAttachmentsMask;
	Ar << Header.InOutMask;
	Ar << Header.RayTracingPayloadType;
	Ar << Header.RayTracingPayloadSize;
	Ar << Header.SpecializationConstantData;
	Ar << Header.CodeHash;
	Ar << Header.SpirvCRC;
	Ar << Header.WaveSize;
	Ar << Header.RayGroupAnyHit;
	Ar << Header.RayGroupIntersection;
	Ar << Header.DebugName;
	return Ar;
}
