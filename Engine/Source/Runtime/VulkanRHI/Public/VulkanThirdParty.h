// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if defined(VK_USE_PLATFORM_WIN32_KHR)
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
	#include <windows.h>
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#if PLATFORM_MAC
// Horrible hack as we need the enum available but the Vulkan headers do not compile on Mac
typedef enum VkDescriptorType {
	VK_DESCRIPTOR_TYPE_SAMPLER = 0,
	VK_DESCRIPTOR_TYPE_COMBINED_IMAGE_SAMPLER = 1,
	VK_DESCRIPTOR_TYPE_SAMPLED_IMAGE = 2,
	VK_DESCRIPTOR_TYPE_STORAGE_IMAGE = 3,
	VK_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER = 4,
	VK_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER = 5,
	VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER = 6,
	VK_DESCRIPTOR_TYPE_STORAGE_BUFFER = 7,
	VK_DESCRIPTOR_TYPE_UNIFORM_BUFFER_DYNAMIC = 8,
	VK_DESCRIPTOR_TYPE_STORAGE_BUFFER_DYNAMIC = 9,
	VK_DESCRIPTOR_TYPE_INPUT_ATTACHMENT = 10,
	VK_DESCRIPTOR_TYPE_INLINE_UNIFORM_BLOCK = 1000138000,
	VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR = 1000150000,
	VK_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_NV = 1000165000,
	VK_DESCRIPTOR_TYPE_SAMPLE_WEIGHT_IMAGE_QCOM = 1000440000,
	VK_DESCRIPTOR_TYPE_BLOCK_MATCH_IMAGE_QCOM = 1000440001,
	VK_DESCRIPTOR_TYPE_MUTABLE_EXT = 1000351000,
	VK_DESCRIPTOR_TYPE_INLINE_UNIFORM_BLOCK_EXT = VK_DESCRIPTOR_TYPE_INLINE_UNIFORM_BLOCK,
	VK_DESCRIPTOR_TYPE_MUTABLE_VALVE = VK_DESCRIPTOR_TYPE_MUTABLE_EXT,
	VK_DESCRIPTOR_TYPE_MAX_ENUM = 0x7FFFFFFF
} VkDescriptorType;
#else

THIRD_PARTY_INCLUDES_START
	#define VK_NO_PROTOTYPES
	#include <vulkan.h>
THIRD_PARTY_INCLUDES_END

#if VK_HEADER_VERSION >= 141
	//workaround for removed defines in sdk 141
	#define VK_DESCRIPTOR_TYPE_BEGIN_RANGE (VK_DESCRIPTOR_TYPE_SAMPLER)
	#define VK_DESCRIPTOR_TYPE_END_RANGE (VK_DESCRIPTOR_TYPE_INPUT_ATTACHMENT)
	#define VK_DESCRIPTOR_TYPE_RANGE_SIZE (VK_DESCRIPTOR_TYPE_INPUT_ATTACHMENT - VK_DESCRIPTOR_TYPE_SAMPLER + 1)
	#define VK_IMAGE_VIEW_TYPE_RANGE_SIZE (VK_IMAGE_VIEW_TYPE_CUBE_ARRAY - VK_IMAGE_VIEW_TYPE_1D + 1)
	#define VK_FORMAT_RANGE_SIZE (VK_FORMAT_ASTC_12x12_SRGB_BLOCK - VK_FORMAT_UNDEFINED + 1)
#endif

#endif

#if PLATFORM_ANDROID
#include <android/hardware_buffer.h>
#endif