// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanUtil.h: Vulkan Utility definitions.
=============================================================================*/

#pragma once

#include "HAL/Platform.h"
#include "Serialization/MemoryWriter.h"
#include "Templates/UniquePtr.h"
#include "VulkanThirdParty.h"

class FVulkanDevice;
class FVulkanTexture;

namespace VulkanRHI
{
	/**
	 * Checks that the given result isn't a failure.  If it is, the application exits with an appropriate error message.
	 * @param	Result - The result code to check
	 * @param	Code - The code which yielded the result.
	 * @param	VkFunction - Tested function name.
	 * @param	Filename - The filename of the source file containing Code.
	 * @param	Line - The line number of Code within Filename.
	 */
	extern VULKANRHI_API void VerifyVulkanResult(VkResult Result, const ANSICHAR* VkFuntion, const ANSICHAR* Filename, uint32 Line);

	VkBuffer CreateBuffer(FVulkanDevice* InDevice, VkDeviceSize Size, VkBufferUsageFlags BufferUsageFlags, VkMemoryRequirements& OutMemoryRequirements);
}

#define VERIFYVULKANRESULT(VkFunction)				{ const VkResult ScopedResult = VkFunction; if (ScopedResult != VK_SUCCESS) { VulkanRHI::VerifyVulkanResult(ScopedResult, #VkFunction, __FILE__, __LINE__); }}
#define VERIFYVULKANRESULT_EXPANDED(VkFunction)		{ const VkResult ScopedResult = VkFunction; if (ScopedResult < VK_SUCCESS) { VulkanRHI::VerifyVulkanResult(ScopedResult, #VkFunction, __FILE__, __LINE__); }}

void SetVulkanResourceName(FVulkanDevice* Device, FVulkanTexture* Texture, const TCHAR* Name);

template<typename T>
inline bool CopyAndReturnNotEqual(T& A, T B)
{
	const bool bOut = A != B;
	A = B;
	return bOut;
}

template <int Version>
class TDataKeyBase;

template <>
class TDataKeyBase<0>
{
protected:
	template <class DataReceiver>
	void GetData(DataReceiver&& ReceiveData)
	{
		TArray<uint8> TempData;
		ReceiveData(TempData);
	}

	void SetData(const void* InData, uint32 InSize) {}

	void CopyDataDeep(TDataKeyBase& Result) const {}
	void CopyDataShallow(TDataKeyBase& Result) const {}
	bool IsDataEquals(const TDataKeyBase& Other) const { return true; }

protected:
	uint32 Hash = 0;
};

template <>
class TDataKeyBase<1>
{
protected:
	template <class DataReceiver>
	void GetData(DataReceiver&& ReceiveData)
	{
		EnsureDataStorage();
		ReceiveData(*Data);
	}

	void SetData(const void* InData, uint32 InSize)
	{
		EnsureDataStorage();
		Data->SetNum(InSize);
		FMemory::Memcpy(Data->GetData(), InData, InSize);
	}

	void CopyDataDeep(TDataKeyBase& Result) const
	{
		check(Data);
		Result.DataStorage = MakeUnique<TArray<uint8>>(*Data);
		Result.Data = Result.DataStorage.Get();
	}

	void CopyDataShallow(TDataKeyBase& Result) const
	{
		check(Data);
		Result.Data = Data;
	}

	bool IsDataEquals(const TDataKeyBase& Other) const
	{
		check(Data && Other.Data);
		check(Data->Num() == Other.Data->Num());
		check(FMemory::Memcmp(Data->GetData(), Other.Data->GetData(), Data->Num()) == 0);
		return true;
	}
public:
	TArray<uint8>& GetDataRef()
	{
		return *Data;
	}

private:
	void EnsureDataStorage()
	{
		if (!DataStorage)
		{
			DataStorage = MakeUnique<TArray<uint8>>();
			Data = DataStorage.Get();
		}
	}

protected:
	uint32 Hash = 0;
	TArray<uint8> *Data = nullptr;
private:
	TUniquePtr<TArray<uint8>> DataStorage;
};

template <>
class TDataKeyBase<2> : public TDataKeyBase<1>
{
protected:
	bool IsDataEquals(const TDataKeyBase& Other) const
	{
		check(Data && Other.Data);
		return ((Data->Num() == Other.Data->Num()) &&
			(FMemory::Memcmp(Data->GetData(), Other.Data->GetData(), Data->Num()) == 0));
	}
};

template <class Derived, bool AlwaysCompareData = false>
class TDataKey : public TDataKeyBase<AlwaysCompareData ? 2 : (DO_CHECK != 0)>
{
public:
	template <class ArchiveWriter>
	void GenerateFromArchive(ArchiveWriter&& WriteToArchive, int32 DataReserve = 0)
	{
		this->GetData([&](TArray<uint8>& InData)
		{
			FMemoryWriter Ar(InData);

			InData.Reset(DataReserve);
			WriteToArchive(Ar);

			this->Hash = FCrc::MemCrc32(InData.GetData(), InData.Num());
		});
	}

	template <class ObjectType>
	void GenerateFromObject(const ObjectType& Object)
	{
		GenerateFromData(&Object, sizeof(Object));
	}

	void GenerateFromData(const void* InData, uint32 InSize)
	{
		this->SetData(InData, InSize);
		this->Hash = FCrc::MemCrc32(InData, InSize);
	}

	uint32 GetHash() const
	{
		return this->Hash;
	}

	Derived CopyDeep() const
	{
		Derived Result;
		Result.Hash = this->Hash;
		this->CopyDataDeep(Result);
		return Result;
	}

	Derived CopyShallow() const
	{
		Derived Result;
		Result.Hash = this->Hash;
		this->CopyDataShallow(Result);
		return Result;
	}

	friend uint32 GetTypeHash(const Derived& Key)
	{
		return Key.Hash;
	}

	friend bool operator==(const Derived& A, const Derived& B)
	{
		return ((A.Hash == B.Hash) && A.IsDataEquals(B));
	}
};
