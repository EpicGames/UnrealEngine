// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildBase;
using UnrealBuildTool;

[SupportedPlatformGroups("Desktop", "Android")]
public class VulkanRHI : ModuleRules
{
	public VulkanRHI(ReadOnlyTargetRules Target) : base(Target)
	{
		IWYUSupport = IWYUSupport.None;
		bLegalToDistributeObjectCode = true;
		bBuildLocallyWithSNDBS = false; // VulkanPlatform.h

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "AMD_AGS");
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			PublicDefinitions.Add("VK_USE_PLATFORM_WIN32_KHR=1");
			PublicDefinitions.Add("VK_USE_PLATFORM_WIN32_KHX=1");
		}
		else if (Target.Platform.IsInGroup(UnrealPlatformGroup.Android))
		{
			PublicDefinitions.Add("VK_USE_PLATFORM_ANDROID_KHR=1");
		}


		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core", 
				"CoreUObject",
				"ApplicationCore",
				"Engine", 
				"RHI",
				"RHICore",
				"RenderCore", 
				"HeadMountedDisplay",
				"PreLoadScreen",
				"BuildSettings",
				"TraceLog",
			}
		);

		if (Target.Platform == UnrealTargetPlatform.Android)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"Launch"
				}
			);

			// for Swappy
			PublicDefinitions.Add("USE_ANDROID_VULKAN_SWAPPY=1");
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
				"Launch",
				"GoogleGameSDK"
				}
			);
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) || Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "NVAftermath");
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) || Target.Platform == UnrealTargetPlatform.Android
			|| Target.IsInPlatformGroup(UnrealPlatformGroup.Unix) || Target.Platform == UnrealTargetPlatform.Mac)
		{
            AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan");
			PublicIncludePathModuleNames.Add("Vulkan");
        }

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PrivateDependencyModuleNames.Add("ApplicationCore");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "SDL3");

			// Always add the SDK's loader in the ThirdParty binaries, it might be used if the local system's loader is not supported
			RuntimeDependencies.Add(Path.Combine(Unreal.EngineDirectory.ToString(), "Binaries/ThirdParty/Vulkan/Linux/libvulkan.so"), StagedFileType.NonUFS);
		}
		else
		{
			PrecompileForTargets = PrecompileTargetsType.None;
		}
	}
}

